// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Animation/AnimatedAttributeManager.h"

////////////////////////////////////////////////////////////////////////////////
// TAnimatedAttributeBase
////////////////////////////////////////////////////////////////////////////////

TAnimatedAttributeBase::TAnimatedAttributeBase()
	: bIsRegistered(false)
{
}

TAnimatedAttributeBase::~TAnimatedAttributeBase()
{
}

void TAnimatedAttributeBase::Register()
{
	if(!bIsRegistered)
	{
		FAnimatedAttributeManager::Get().RegisterAttribute(AsShared());
		bIsRegistered = true;
	}
}

void TAnimatedAttributeBase::Unregister()
{
	if(bIsRegistered)
	{
		FAnimatedAttributeManager::Get().UnregisterAttribute(AsShared());
		bIsRegistered = false;
	}
}

////////////////////////////////////////////////////////////////////////////////
// FAnimatedAttributeManager
////////////////////////////////////////////////////////////////////////////////

FAnimatedAttributeManager::FAnimatedAttributeManager()
{
}

FAnimatedAttributeManager::~FAnimatedAttributeManager()
{
}

void FAnimatedAttributeManager::Tick(float InDeltaTime)
{
	// remove state attributes
	Attributes.RemoveAll([](const TWeakPtr<TAnimatedAttributeBase>& Attribute) {
		return !Attribute.IsValid();
	});

	// at this point we expect the remaining attributes to be valid
	for(const TWeakPtr<TAnimatedAttributeBase>& Attribute : Attributes)
	{
		Attribute.Pin()->Tick(InDeltaTime);
	}
}

void FAnimatedAttributeManager::SetupTick()
{
	if(FSlateApplication::IsInitialized() && !TickHandle.IsValid())
	{
		const FSlateApplication::FSlateTickEvent::FDelegate TickDelegate =
			FSlateApplication::FSlateTickEvent::FDelegate::CreateRaw(this, &FAnimatedAttributeManager::Tick);
		TickHandle = FSlateApplication::Get().OnPreTick().Add(TickDelegate);
	}
}

void FAnimatedAttributeManager::TeardownTick()
{
	if(TickHandle.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnPreTick().Remove(TickHandle);
	}
	TickHandle.Reset();
}

void FAnimatedAttributeManager::RegisterAttribute(const TSharedRef<TAnimatedAttributeBase>& InAttribute)
{
	Attributes.Add(InAttribute.ToWeakPtr());
	SetupTick();
}

void FAnimatedAttributeManager::UnregisterAttribute(const TSharedRef<TAnimatedAttributeBase>& InAttribute)
{
	Attributes.Remove(InAttribute.ToWeakPtr());
}

FAnimatedAttributeManager& FAnimatedAttributeManager::Get()
{
	static FAnimatedAttributeManager Manager;
	return Manager;
}
