// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Animation/AttributeInterpolator.h"
#include "AHEasing/easing.h"

float EaseInterpolatorRatio(EEasingInterpolatorType InEasingType, float InRatio)
{
	switch(InEasingType)
	{
		case EEasingInterpolatorType::Linear:
		{
			return FMath::Clamp<float>(InRatio, 0, 1);
		}
		case EEasingInterpolatorType::QuadraticEaseIn:
		{
			return QuadraticEaseIn(InRatio);
		}
		case EEasingInterpolatorType::QuadraticEaseOut:
		{
			return QuadraticEaseOut(InRatio);
		}
		case EEasingInterpolatorType::QuadraticEaseInOut:
		{
			return QuadraticEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::CubicEaseIn:
		{
			return CubicEaseIn(InRatio);
		}
		case EEasingInterpolatorType::CubicEaseOut:
		{
			return CubicEaseOut(InRatio);
		}
		case EEasingInterpolatorType::CubicEaseInOut:
		{
			return CubicEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::QuarticEaseIn:
		{
			return QuarticEaseIn(InRatio);
		}
		case EEasingInterpolatorType::QuarticEaseOut:
		{
			return QuarticEaseOut(InRatio);
		}
		case EEasingInterpolatorType::QuarticEaseInOut:
		{
			return QuarticEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::QuinticEaseIn:
		{
			return QuinticEaseIn(InRatio);
		}
		case EEasingInterpolatorType::QuinticEaseOut:
		{
			return QuinticEaseOut(InRatio);
		}
		case EEasingInterpolatorType::QuinticEaseInOut:
		{
			return QuinticEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::SineEaseIn:
		{
			return SineEaseIn(InRatio);
		}
		case EEasingInterpolatorType::SineEaseOut:
		{
			return SineEaseOut(InRatio);
		}
		case EEasingInterpolatorType::SineEaseInOut:
		{
			return SineEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::CircularEaseIn:
		{
			return CircularEaseIn(InRatio);
		}
		case EEasingInterpolatorType::CircularEaseOut:
		{
			return CircularEaseOut(InRatio);
		}
		case EEasingInterpolatorType::CircularEaseInOut:
		{
			return CircularEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::ExponentialEaseIn:
		{
			return ExponentialEaseIn(InRatio);
		}
		case EEasingInterpolatorType::ExponentialEaseOut:
		{
			return ExponentialEaseOut(InRatio);
		}
		case EEasingInterpolatorType::ExponentialEaseInOut:
		{
			return ExponentialEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::ElasticEaseIn:
		{
			return ElasticEaseIn(InRatio);
		}
		case EEasingInterpolatorType::ElasticEaseOut:
		{
			return ElasticEaseOut(InRatio);
		}
		case EEasingInterpolatorType::ElasticEaseInOut:
		{
			return ElasticEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::BackEaseIn:
		{
			return BackEaseIn(InRatio);
		}
		case EEasingInterpolatorType::BackEaseOut:
		{
			return BackEaseOut(InRatio);
		}
		case EEasingInterpolatorType::BackEaseInOut:
		{
			return BackEaseInOut(InRatio);
		}
		case EEasingInterpolatorType::BounceEaseIn:
		{
			return BounceEaseIn(InRatio);
		}
		case EEasingInterpolatorType::BounceEaseOut:
		{
			return BounceEaseOut(InRatio);
		}
		case EEasingInterpolatorType::BounceEaseInOut:
		{
			return BounceEaseInOut(InRatio);
		}
	}

	return 0.f;
}