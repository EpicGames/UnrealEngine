// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/AnalogCursor.h"
#include "InputCoreTypes.h"
#include "Input/Events.h"
#include "Widgets/SWidget.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"


FAnalogCursor::FAnalogCursor()
: CurrentSpeed(FVector2D::ZeroVector)
, CurrentOffset(FVector2D::ZeroVector)
, Acceleration(1000.0f)
, MaxSpeed(1500.0f)
, StickySlowdown(0.5f)
, DeadZone(0.1f)
, Mode(AnalogCursorMode::Accelerated)
{
	AnalogValues[ static_cast< uint8 >( EAnalogStick::Left ) ] = FVector2D::ZeroVector;
	AnalogValues[ static_cast< uint8 >( EAnalogStick::Right ) ] = FVector2D::ZeroVector;
}

FAnalogCursor::FAnalogCursor(const FAnalogCursor&) = default;
FAnalogCursor::~FAnalogCursor() = default;

void FAnalogCursor::Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor>)
{
	if (TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(GetOwnerUserIndex()))
	{
		const FVector2D NewPosition = CalculateTickedCursorPosition(DeltaTime, SlateApp, SlateUser);

		// update the cursor position
		UpdateCursorPosition(SlateApp, SlateUser.ToSharedRef(), NewPosition);
	}
}

bool FAnalogCursor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	if (IsRelevantInput(InKeyEvent))
	{
		FKey Key = InKeyEvent.GetKey();
		// Consume the sticks input so it doesn't effect other things
		if (Key == EKeys::Gamepad_LeftStick_Right ||
			Key == EKeys::Gamepad_LeftStick_Left ||
			Key == EKeys::Gamepad_LeftStick_Up ||
			Key == EKeys::Gamepad_LeftStick_Down)
		{
			return true;
		}

		// Bottom face button is a click
		if (Key == EKeys::Virtual_Gamepad_Accept.GetVirtualKey())
		{
			if (!InKeyEvent.IsRepeat())
			{
				if (TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(InKeyEvent))
				{
					const bool bIsPrimaryUser = FSlateApplication::CursorUserIndex == SlateUser->GetUserIndex();
					FPointerEvent MouseEvent(
						SlateUser->GetUserIndex(),
						FSlateApplication::CursorPointerIndex,
						SlateUser->GetCursorPosition(),
						SlateUser->GetPreviousCursorPosition(),
						bIsPrimaryUser ? SlateApp.GetPressedMouseButtons() : FTouchKeySet::EmptySet,
						EKeys::LeftMouseButton,
						0,
						bIsPrimaryUser ? SlateApp.GetModifierKeys() : FModifierKeysState()
					);

					TSharedPtr<FGenericWindow> GenWindow;
					return SlateApp.ProcessMouseButtonDownEvent(GenWindow, MouseEvent);
				}
			}

			return true;
		}
	}

	return false;
}

bool FAnalogCursor::HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	if (IsRelevantInput(InKeyEvent))
	{
		FKey Key = InKeyEvent.GetKey();

		// Consume the sticks input so it doesn't effect other things
		if (Key == EKeys::Gamepad_LeftStick_Right ||
			Key == EKeys::Gamepad_LeftStick_Left ||
			Key == EKeys::Gamepad_LeftStick_Up ||
			Key == EKeys::Gamepad_LeftStick_Down)
		{
			return true;
		}

		// Bottom face button is a click
		if (Key == EKeys::Virtual_Gamepad_Accept.GetVirtualKey() && !InKeyEvent.IsRepeat())
		{
			if (TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(InKeyEvent))
			{
				const bool bIsPrimaryUser = FSlateApplication::CursorUserIndex == SlateUser->GetUserIndex();

				FPointerEvent MouseEvent(
					SlateUser->GetUserIndex(),
					FSlateApplication::CursorPointerIndex,
					SlateUser->GetCursorPosition(),
					SlateUser->GetPreviousCursorPosition(),
					bIsPrimaryUser ? SlateApp.GetPressedMouseButtons() : FTouchKeySet::EmptySet,
					EKeys::LeftMouseButton,
					0,
					bIsPrimaryUser ? SlateApp.GetModifierKeys() : FModifierKeysState()
				);

				return SlateApp.ProcessMouseButtonUpEvent(MouseEvent);
			}
		}
	}
	return false;
}

bool FAnalogCursor::HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent)
{
	if (IsRelevantInput(InAnalogInputEvent))
	{
		FKey Key = InAnalogInputEvent.GetKey();
		float AnalogValue = InAnalogInputEvent.GetAnalogValue();

		if (Key == EKeys::Gamepad_LeftX)
		{
			FVector2D& Value = GetAnalogValue(EAnalogStick::Left);
			Value.X = AnalogValue;
		}
		else if (Key == EKeys::Gamepad_LeftY)
		{
			FVector2D& Value = GetAnalogValue(EAnalogStick::Left);
			Value.Y = -AnalogValue;
		}
		else if (Key == EKeys::Gamepad_RightX)
		{
			FVector2D& Value = GetAnalogValue(EAnalogStick::Right);
			Value.X = AnalogValue;
		}
		else if (Key == EKeys::Gamepad_RightY)
		{
			FVector2D& Value = GetAnalogValue(EAnalogStick::Right);
			Value.Y = -AnalogValue;
		}
		else
		{
			return false;
		}
	}
	return true;
}

bool FAnalogCursor::HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	return false;
}

void FAnalogCursor::SetAcceleration(float NewAcceleration)
{
	Acceleration = NewAcceleration;
}

void FAnalogCursor::SetMaxSpeed(float NewMaxSpeed)
{
	MaxSpeed = NewMaxSpeed;
}

void FAnalogCursor::SetStickySlowdown(float NewStickySlowdown)
{
	StickySlowdown = NewStickySlowdown;
}

void FAnalogCursor::SetDeadZone(float NewDeadZone)
{
	DeadZone = NewDeadZone;
}

void FAnalogCursor::SetMode(AnalogCursorMode::Type NewMode)
{
	Mode = NewMode;

	CurrentSpeed = FVector2D::ZeroVector;
}

bool FAnalogCursor::IsRelevantInput(const FInputEvent& InputEvent) const
{
	return GetOwnerUserIndex() == InputEvent.GetUserIndex();
}

bool FAnalogCursor::IsRelevantInput(const FKeyEvent& KeyEvent) const
{
	return IsRelevantInput(static_cast<FInputEvent>(KeyEvent));
}

bool FAnalogCursor::IsRelevantInput(const FAnalogInputEvent& AnalogInputEvent) const
{
	return IsRelevantInput(static_cast<FInputEvent>(AnalogInputEvent));
}

bool FAnalogCursor::IsRelevantInput(const FPointerEvent& MouseEvent) const
{
	return IsRelevantInput(static_cast<FInputEvent>(MouseEvent));
}

void FAnalogCursor::ClearAnalogValues()
{
	AnalogValues[static_cast<uint8>(EAnalogStick::Left)] = FVector2D::ZeroVector;
	AnalogValues[static_cast<uint8>(EAnalogStick::Right)] = FVector2D::ZeroVector;
}

void FAnalogCursor::UpdateCursorPosition(FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor, const FVector2D& NewPosition, bool bForce)
{
	//grab the old position
	const FVector2D OldPosition = Cursor->GetPosition();

	//make sure we are actually moving
	int32 NewIntPosX = NewPosition.X;
	int32 NewIntPosY = NewPosition.Y;
	int32 OldIntPosX = OldPosition.X;
	int32 OldIntPosY = OldPosition.Y;
	if (bForce || OldIntPosX != NewIntPosX || OldIntPosY != NewIntPosY)
	{
		//put the cursor in the correct spot
		Cursor->SetPosition(NewIntPosX, NewIntPosY);

		// Since the cursor may have been locked and its location clamped, get the actual new position
		const FVector2D UpdatedPosition = Cursor->GetPosition();
		if (TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(GetOwnerUserIndex()))
		{
			//create a new mouse event
			const bool bIsPrimaryUser = FSlateApplication::CursorUserIndex == SlateUser->GetUserIndex();
			FPointerEvent MouseEvent(
				SlateApp.CursorPointerIndex,
				UpdatedPosition,
				OldPosition,
				bIsPrimaryUser ? SlateApp.GetPressedMouseButtons() : FTouchKeySet::EmptySet,
				EKeys::Invalid,
				0,
				bIsPrimaryUser ? SlateApp.GetModifierKeys() : FModifierKeysState()
			);
			//process the event
			SlateApp.ProcessMouseMoveEvent(MouseEvent);
		}
	}
}

void FAnalogCursor::UpdateCursorPosition(FSlateApplication& SlateApp, TSharedRef<FSlateUser> SlateUser, const FVector2D& NewPosition, bool bForce)
{
	//grab the old position
	const FVector2D OldPosition = SlateUser->GetCursorPosition();

	//make sure we are actually moving
	int32 NewIntPosX = NewPosition.X;
	int32 NewIntPosY = NewPosition.Y;
	int32 OldIntPosX = OldPosition.X;
	int32 OldIntPosY = OldPosition.Y;
	if (bForce || OldIntPosX != NewIntPosX || OldIntPosY != NewIntPosY)
	{
		//put the cursor in the correct spot
		SlateUser->SetCursorPosition(NewIntPosX, NewIntPosY);
	
		// Since the cursor may have been locked and its location clamped, get the actual new position
		const FVector2D UpdatedPosition = SlateUser->GetCursorPosition();
		//create a new mouse event
		const bool bIsPrimaryUser = FSlateApplication::CursorUserIndex == SlateUser->GetUserIndex();
		FPointerEvent MouseEvent(
			SlateUser->GetUserIndex(),
			FSlateApplication::CursorPointerIndex,
			UpdatedPosition,
			OldPosition,
			bIsPrimaryUser ? SlateApp.GetPressedMouseButtons() : FTouchKeySet::EmptySet,
			EKeys::Invalid,
			0,
			bIsPrimaryUser ? SlateApp.GetModifierKeys() : FModifierKeysState()
		);

		//process the event
		SlateApp.ProcessMouseMoveEvent(MouseEvent);
	}
}

FVector2D FAnalogCursor::CalculateTickedCursorPosition(const float DeltaTime, FSlateApplication& SlateApp, TSharedPtr<FSlateUser> SlateUser) 
{
	const FVector2D OldPosition = SlateUser->GetCursorPosition();

	float SpeedMult = 1.0f; // Used to do a speed multiplication before adding the delta to the position to make widgets sticky
	FVector2D AdjAnalogVals = GetAnalogValue(EAnalogStick::Left); // A copy of the analog values so I can modify them based being over a widget, not currently doing this

	// Adjust analog values according to dead zone
	const float AnalogValsSize = AdjAnalogVals.Size();

	if (AnalogValsSize > 0.0f)
	{
		const float TargetSize = FMath::Max(AnalogValsSize - DeadZone, 0.0f) / (1.0f - DeadZone);
		AdjAnalogVals /= AnalogValsSize;
		AdjAnalogVals *= TargetSize;
	}


	// Check if there is a sticky widget beneath the cursor
	FWidgetPath WidgetPath = SlateApp.LocateWindowUnderMouse(OldPosition, SlateApp.GetInteractiveTopLevelWindows(), false, SlateUser->GetUserIndex());
	if (WidgetPath.IsValid())
	{
		const FArrangedChildren::FArrangedWidgetArray& AllArrangedWidgets = WidgetPath.Widgets.GetInternalArray();
		for (const FArrangedWidget& ArrangedWidget : AllArrangedWidgets)
		{
			TSharedRef<SWidget> Widget = ArrangedWidget.Widget;
			if (Widget->IsInteractable())
			{
				SpeedMult = StickySlowdown;
				//FVector2D Adjustment = WidgetsAndCursors.Last().Geometry.Position - OldPosition; // example of calculating distance from cursor to widget center
				break;
			}
		}
	}

	switch (Mode)
	{
	case AnalogCursorMode::Accelerated:
	{
		// Generate Min and Max for X to clamp the speed, this gives us instant direction change when crossing the axis
		float CurrentMinSpeedX = 0.0f;
		float CurrentMaxSpeedX = 0.0f;
		if (AdjAnalogVals.X > 0.0f)
		{
			CurrentMaxSpeedX = AdjAnalogVals.X * MaxSpeed;
		}
		else
		{
			CurrentMinSpeedX = AdjAnalogVals.X * MaxSpeed;
		}

		// Generate Min and Max for Y to clamp the speed, this gives us instant direction change when crossing the axis
		float CurrentMinSpeedY = 0.0f;
		float CurrentMaxSpeedY = 0.0f;
		if (AdjAnalogVals.Y > 0.0f)
		{
			CurrentMaxSpeedY = AdjAnalogVals.Y * MaxSpeed;
		}
		else
		{
			CurrentMinSpeedY = AdjAnalogVals.Y * MaxSpeed;
		}

		// Cubic acceleration curve
		FVector2D ExpAcceleration = AdjAnalogVals * AdjAnalogVals * AdjAnalogVals * Acceleration;
		// Preserve direction (if we use a squared equation above)
		//ExpAcceleration.X *= FMath::Sign(AnalogValues.X);
		//ExpAcceleration.Y *= FMath::Sign(AnalogValues.Y);

		CurrentSpeed += ExpAcceleration * DeltaTime;

		CurrentSpeed.X = FMath::Clamp(CurrentSpeed.X, CurrentMinSpeedX, CurrentMaxSpeedX);
		CurrentSpeed.Y = FMath::Clamp(CurrentSpeed.Y, CurrentMinSpeedY, CurrentMaxSpeedY);

		break;
	}

	case AnalogCursorMode::Direct:

		CurrentSpeed = AdjAnalogVals * MaxSpeed;

		break;
	}

	CurrentOffset += CurrentSpeed * DeltaTime * SpeedMult;
	const FVector2D NewPosition = OldPosition + CurrentOffset;

	// save the remaining sub-pixel offset 
	CurrentOffset.X = FGenericPlatformMath::Frac(NewPosition.X);
	CurrentOffset.Y = FGenericPlatformMath::Frac(NewPosition.Y);

	return NewPosition;
}

