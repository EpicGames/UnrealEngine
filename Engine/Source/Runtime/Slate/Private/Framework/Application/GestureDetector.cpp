// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/GestureDetector.h"

double FGestureDetector::LongPressSeconds = 0.800f;
float FGestureDetector::LongPressAllowedMovement = 0;

bool FGestureDetector::IsGestureSupported(EGestureEvent Gesture)
{
	switch ( Gesture )
	{
	case EGestureEvent::LongPress:
		return true;
	default:
		return false;
	}
}

void FGestureDetector::OnTouchStarted(int32 TouchIndex, const FVector2D& Location)
{
	if ( TouchIndex < EKeys::NUM_TOUCH_KEYS )
	{
		LongPressTrack[TouchIndex].Reset();
		LongPressTrack[TouchIndex].Time = FPlatformTime::Seconds();
		LongPressTrack[TouchIndex].Location = Location;
	}
}

void FGestureDetector::OnTouchEnded(int32 TouchIndex, const FVector2D& Location)
{
	if ( TouchIndex < EKeys::NUM_TOUCH_KEYS )
	{
		LongPressTrack[TouchIndex].Reset();
	}
}

void FGestureDetector::OnTouchMoved(int32 TouchIndex, const FVector2D& Location)
{
	if ( TouchIndex < EKeys::NUM_TOUCH_KEYS )
	{
		FVector2D Delta = ( Location - LongPressTrack[TouchIndex].Location );
		if ( Delta.Size() >= FGestureDetector::LongPressAllowedMovement )
		{
			LongPressTrack[TouchIndex].Reset();
		}
	}
}

void FGestureDetector::GenerateGestures(FGenericApplicationMessageHandler& MessageHandler, const TBitArray<FDefaultBitArrayAllocator>& EnabledGestures)
{
	if ( EnabledGestures[(uint8)EGestureEvent::LongPress] )
	{
		const double CurrentTime = FPlatformTime::Seconds();

		for ( int32 TouchIndex = 0; TouchIndex < EKeys::NUM_TOUCH_KEYS; TouchIndex++ )
		{
			if ( LongPressTrack[TouchIndex].Time.IsSet() )
			{
				const double DeltaTime = CurrentTime - LongPressTrack[TouchIndex].Time.GetValue();
				if ( DeltaTime >= FGestureDetector::LongPressSeconds )
				{
					MessageHandler.OnTouchGesture(EGestureEvent::LongPress, FVector2D(0, 0), 0, false);
					LongPressTrack[TouchIndex].Reset();
				}
			}
		}
	}
}