// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/IPlatformTextField.h"
#include "Features/IModularFeatures.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/Platform.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

bool IPlatformTextField::ShouldUseVirtualKeyboardAutocorrect(TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget)
{
	bool bUseAutocorrect = false;

	if (TextEntryWidget.IsValid() && TextEntryWidget->GetVirtualKeyboardOptions().bEnableAutocorrect)
	{
		switch (TextEntryWidget->GetVirtualKeyboardType())
		{
		case EKeyboardType::Keyboard_Password:
			{
				// Never use autocorrect for password
				bUseAutocorrect = false;
			}
			break;

		case EKeyboardType::Keyboard_Email:
		case EKeyboardType::Keyboard_Web:
		case EKeyboardType::Keyboard_Number:
		case EKeyboardType::Keyboard_AlphaNumeric:
		case EKeyboardType::Keyboard_Default:
		default:
			{
				// Check to see if autocorrect is turned on in the input settings
				bUseAutocorrect = GConfig->GetBool(TEXT("/Script/Engine.InputSettings"), TEXT("bUseAutocorrect"), bUseAutocorrect, GInputIni);
			}
			break;
		}

		if (bUseAutocorrect)
		{
			static FString PlatformVersion;

			if (PlatformVersion.IsEmpty())
			{
				FString OSVersion;
				FString OSSubVersion;

				FPlatformMisc::GetOSVersions(OSVersion, OSSubVersion);

				PlatformVersion = OSVersion + " " + OSSubVersion;
			}

			// Match the current platform version (such as "iOS 11.2") against the list of excluded OS versions. If the platform version
			// starts with an excluded version, disable autocorrect (this is to allow for "iOS 11" to disable autocorrect on all iOS 11 devices,
			// or for "iOS 11.2.2" to only exclude that version).
			TArray<FString> ExcludedOSVersions;
			GConfig->GetArray(TEXT("/Script/Engine.InputSettings"), TEXT("ExcludedAutocorrectOS"), ExcludedOSVersions, GInputIni);

			for (const FString& ExcludedVersion : ExcludedOSVersions)
			{
				if (PlatformVersion.StartsWith(ExcludedVersion))
				{
					bUseAutocorrect = false;
					break;
				}
			}

			// Get the current culture that the game is running in, and check against INI settings to see if autocorrect should be
			// disabled for that culture. E.g., specifying a culture such as "en" in the INI will disable autocorrect for all English 
			// cultures, but specifying "en-CA" will disable autocorrect only for Canadian English.
			if (bUseAutocorrect && FInternationalization::IsAvailable())
			{
				FInternationalization& I18N = FInternationalization::Get();
				TArray<FString> PrioritizedCultureNames = I18N.GetPrioritizedCultureNames(I18N.GetCurrentCulture()->GetName());

				TArray<FString> ExcludedCultures;
				GConfig->GetArray(TEXT("/Script/Engine.InputSettings"), TEXT("ExcludedAutocorrectCultures"), ExcludedCultures, GInputIni);

				for (const FString& CultureName : PrioritizedCultureNames)
				{
					if (ExcludedCultures.Contains(CultureName))
					{
						bUseAutocorrect = false;
						break;
					}
				}
			}
		}
	}

	return bUseAutocorrect;
}



FName IPlatformTextFieldFactory::FeatureName = FName("PlatformTextField");

TUniquePtr<IPlatformTextField> IPlatformTextFieldFactory::TryCreateInstance()
{
	TUniquePtr<IPlatformTextField> Result;

	TArray<IPlatformTextFieldFactory*> TextFieldFactories = IModularFeatures::Get().GetModularFeatureImplementations<IPlatformTextFieldFactory>(IPlatformTextFieldFactory::FeatureName);
	for (IPlatformTextFieldFactory* TextFieldFactory : TextFieldFactories)
	{
		Result = TextFieldFactory->CreateInstance();
		if (Result.IsValid())
		{
			break;
		}
	}

	return MoveTemp(Result);
}
