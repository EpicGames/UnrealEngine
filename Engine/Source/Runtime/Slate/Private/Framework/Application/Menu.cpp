// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/Menu.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"

FMenuBase::FMenuBase(TSharedRef<SWidget> InContent, const bool bCollapsedByParent)
	: Content(InContent)
	, bDismissing(false)
	, bIsCollapsedByParent(bCollapsedByParent)
{
}

FMenuInWindow::FMenuInWindow(TSharedRef<SWindow> InWindow, TSharedRef<SWidget> InContent, const bool bIsCollapsedByParent)
	: FMenuBase(InContent, bIsCollapsedByParent)
	, Window(InWindow)
{
}

TSharedPtr<SWindow> FMenuInWindow::GetParentWindow() const
{
	// Return the menu's window
	return Window.Pin();
}

void FMenuInWindow::Dismiss()
{
	if (!bDismissing)
	{
		bDismissing = true;
		OnMenuDismissed.Broadcast(AsShared());

		// Close the window
		// Will cause the window destroy code to call back into the stack to clean up
		TSharedPtr<SWindow> WindowPinned = Window.Pin();
		if (WindowPinned.IsValid())
		{
			WindowPinned->RequestDestroyWindow();
		}
	}
}


FMenuInPopup::FMenuInPopup(TSharedRef<SWidget> InContent, const bool bIsCollapsedByParent)
	: FMenuBase(InContent, bIsCollapsedByParent)
{
}

TSharedPtr<SWindow> FMenuInPopup::GetParentWindow() const
{
	// Return the menu's window
	return FSlateApplication::Get().GetVisibleMenuWindow();
}

void FMenuInPopup::Dismiss()
{
	if (!bDismissing)
	{
		bDismissing = true;
		OnMenuDismissed.Broadcast(AsShared());
	}
}


FMenuInHostWidget::FMenuInHostWidget(TSharedRef<IMenuHost> InHost, const TSharedRef<SWidget>& InContent, const bool bIsCollapsedByParent)
	: FMenuBase(InContent, bIsCollapsedByParent)
	, MenuHost(InHost)
{
}

TSharedPtr<SWindow> FMenuInHostWidget::GetParentWindow() const
{
	// Return the menu's window
	TSharedPtr<IMenuHost> HostPinned = MenuHost.Pin();
	if (HostPinned.IsValid())
	{
		return HostPinned->GetMenuWindow();
	}
	return TSharedPtr<SWindow>();
}

void FMenuInHostWidget::Dismiss()
{
	if (!bDismissing)
	{
		bDismissing = true;
		TSharedPtr<IMenuHost> HostPinned = MenuHost.Pin();
		if (HostPinned.IsValid())
		{
			HostPinned->OnMenuDismissed();
		}
		OnMenuDismissed.Broadcast(AsShared());
	}
}

bool FMenuInHostWidget::UsingApplicationMenuStack() const
{
	TSharedPtr<IMenuHost> HostPinned = MenuHost.Pin();
	if (HostPinned.IsValid())
	{
		return HostPinned->UsingApplicationMenuStack();
	}
	return true;
}
