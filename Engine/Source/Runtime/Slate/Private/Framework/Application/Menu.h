// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/PopupMethodReply.h"
#include "Framework/Application/IMenu.h"

class SWidget;
class SWindow;

/**
 * Represents the base class of popup menus.
 */
class FMenuBase : public IMenu, public TSharedFromThis<FMenuBase>
{
public:
	virtual FOnMenuDismissed& GetOnMenuDismissed() override { return OnMenuDismissed; }
	virtual TSharedPtr<SWidget> GetContent() const override { return Content; }
	bool IsCollapsedByParent() const { return bIsCollapsedByParent; }
	virtual bool UsingApplicationMenuStack() const override { return true; }

protected:
	FMenuBase(TSharedRef<SWidget> InContent, const bool bCollapsedByParent);

	FOnMenuDismissed OnMenuDismissed;
	TSharedRef<SWidget> Content;
	bool bDismissing;
	bool bIsCollapsedByParent;
};

/**
 * Represents a popup menu that is shown in its own SWindow.
 */
class FMenuInWindow : public FMenuBase
{
public:
	FMenuInWindow(TSharedRef<SWindow> InWindow, TSharedRef<SWidget> InContent, const bool bIsCollapsedByParent);
	virtual ~FMenuInWindow() = default;

	virtual EPopupMethod GetPopupMethod() const override { return EPopupMethod::CreateNewWindow; }
	virtual TSharedPtr<SWindow> GetParentWindow() const override;
	virtual TSharedPtr<SWindow> GetOwnedWindow() const override { return GetParentWindow(); }
	virtual void Dismiss() override;

private:
	TWeakPtr<SWindow> Window;
};

/**
 * Represents a popup menu that is shown in the same window as the widget that summons it.
 */
class FMenuInPopup : public FMenuBase
{
public:
	FMenuInPopup(TSharedRef<SWidget> InContent, const bool bIsCollapsedByParent);
	virtual ~FMenuInPopup() = default;

	virtual EPopupMethod GetPopupMethod() const { return EPopupMethod::UseCurrentWindow; }
	virtual TSharedPtr<SWindow> GetParentWindow() const;
	virtual TSharedPtr<SWindow> GetOwnedWindow() const { return TSharedPtr<SWindow>(); }
	virtual void Dismiss() override;
};

/**
* Represents a popup menu that is shown in a host widget (such as a menu anchor).
*/
class FMenuInHostWidget : public FMenuBase
{
public:
	FMenuInHostWidget(TSharedRef<IMenuHost> InHost, const TSharedRef<SWidget>& InContent, const bool bIsCollapsedByParent);
	virtual ~FMenuInHostWidget() = default;

	virtual EPopupMethod GetPopupMethod() const { return EPopupMethod::UseCurrentWindow; }
	virtual TSharedPtr<SWindow> GetParentWindow() const;
	virtual TSharedPtr<SWindow> GetOwnedWindow() const { return TSharedPtr<SWindow>(); }
	virtual void Dismiss() override;
	virtual bool UsingApplicationMenuStack() const override;

private:
	TWeakPtr<IMenuHost> MenuHost;
};
