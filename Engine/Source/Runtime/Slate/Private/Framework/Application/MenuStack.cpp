// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/MenuStack.h"
#include "Layout/LayoutUtils.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SPopup.h"
#include "Framework/Application/Menu.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "MenuStack"

namespace FMenuStackDefs
{
	/** Maximum size of menus as a fraction of the work area height */
	const float MaxMenuScreenHeightFraction = 0.8f;
	const float AnimationDuration = 0.15f;
}

#if WITH_SLATE_DEBUGGING
FAutoConsoleVariable CVarSlateDismissMenuStacksOnFocusLost(
	TEXT("Slate.DismissMenuStacksOnFocusLost"),
	true,
	TEXT("Should open menus disappear when whey lose focus?")
);
#endif

/** Overlay widget class used to hold menu contents and display them on top of the current window */
class SMenuPanel : public SOverlay
{
public:
	SLATE_BEGIN_ARGS(SMenuPanel)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SOverlay::Construct(SOverlay::FArguments());
	}

	void PushMenu(TSharedRef<FMenuBase> InMenu, const FVector2f& InLocation)
	{
		check(InMenu->GetContent().IsValid());

		TSharedPtr<SWindow> ParentWindow = InMenu->GetParentWindow();
		check(ParentWindow.IsValid());

		// Transform InLocation into a position local to this panel (assumes the panel is in an overlay that covers the whole of the panel window) 
		FVector2f PanelInScreen = ParentWindow->GetRectInScreen().GetTopLeft();
		FVector2f PanelInWindow = ParentWindow->GetLocalToScreenTransform().Inverse().TransformPoint(PanelInScreen);
		FVector2f LocationInWindow = ParentWindow->GetLocalToScreenTransform().Inverse().TransformPoint(InLocation);
		FVector2f LocationInPanel = LocationInWindow - PanelInWindow;

		// Add the new menu into a slot on this panel and set the padding so that its position is correct
		AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(LocationInPanel.X, LocationInPanel.Y, 0, 0)
		[
			InMenu->GetContent().ToSharedRef()
		];

		// Make sure that the menu will remove itself from the panel when dismissed
		InMenu->GetOnMenuDismissed().AddSP(this, &SMenuPanel::OnMenuClosed);
	}

	void OnMenuClosed(TSharedRef<IMenu> InMenu)
	{
		RemoveSlot(InMenu->GetContent().ToSharedRef());
	}
};


namespace MenuStackInternal
{
	/** Widget that wraps any menu created in FMenuStack to provide default key handling, focus tracking and helps us spot menus in widget paths */
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnKeyDown, FKey)
	DECLARE_DELEGATE_OneParam(FOnMenuLostFocus, const FWidgetPath&)
	class SMenuContentWrapper : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SMenuContentWrapper) 
				: _MenuContent()
				, _OnKeyDown()
				, _OptionalMinMenuWidth()
				, _OptionalMinMenuHeight()
				, _bShowBackground(true)
			{}

			SLATE_DEFAULT_SLOT(FArguments, MenuContent)
			SLATE_EVENT(FOnKeyDown, OnKeyDown)
			SLATE_EVENT(FOnMenuLostFocus, OnMenuLostFocus)
			SLATE_ARGUMENT(FOptionalSize, OptionalMinMenuWidth)
			SLATE_ARGUMENT(FOptionalSize, OptionalMinMenuHeight)
			SLATE_ARGUMENT(bool, bShowBackground)
		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs)
		{
			// The visibility of the content wrapper should match that of the provided content
			SetVisibility(AccessWidgetVisibilityAttribute(InArgs._MenuContent.Widget));

			OnKeyDownDelegate = InArgs._OnKeyDown;
			OnMenuLostFocus = InArgs._OnMenuLostFocus;

			TSharedPtr<SWidget> ChildContent;
			if (InArgs._bShowBackground)
			{
				// Always add a background to the menu. This includes a small outline around the background to distinguish open menus from each other
				ChildContent = SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("Menu.Background"))
					]
					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetOptionalBrush("Menu.Outline", nullptr))
					]
					+ SOverlay::Slot()
					[
						SNew(SBorder)
						.Padding(0.f)
						.BorderImage(FStyleDefaults::GetNoBrush())
						.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
						[
							InArgs._MenuContent.Widget
						]
					];
			}
			else
			{
				ChildContent = SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SBorder)
						.Padding(0.f)
						.BorderImage(FStyleDefaults::GetNoBrush())
						.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
						[
							InArgs._MenuContent.Widget
						]
					];
			}

			ChildSlot
				[
					SNew(SBox)
					.MinDesiredWidth(InArgs._OptionalMinMenuWidth)
					.MaxDesiredHeight(InArgs._OptionalMinMenuHeight)
					[
						ChildContent.ToSharedRef()
					]
			];
		}

		virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override
		{
			// if focus changed and this menu content had focus (or one of its children did) then inform the stack via the OnMenuLostFocus event
			if (OnMenuLostFocus.IsBound() && PreviousFocusPath.ContainsWidget(this))
			{
				return OnMenuLostFocus.Execute(NewWidgetPath);
			}
		}

	private:

		/** This widget must support keyboard focus */
		virtual bool SupportsKeyboardFocus() const override
		{
			return true;
		}

		virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
		{
			if (OnKeyDownDelegate.IsBound())
			{
				return OnKeyDownDelegate.Execute(InKeyEvent.GetKey());
			}

			return FReply::Unhandled();
		}

		/** Delegate to forward keys down events on the menu */
		FOnKeyDown OnKeyDownDelegate;

		/** Delegate to inform the stack that a menu has lost focus and might need to be closed */
		FOnMenuLostFocus OnMenuLostFocus;
	};


	/** Global handler used to handle key presses on popup menus */
	FReply OnMenuKeyDown(const FKey Key)
	{
		if (Key == EKeys::Escape)
		{
			FSlateApplication::Get().DismissAllMenus();
			return FReply::Handled().ClearUserFocus();
		}

		return FReply::Unhandled();
	}

	FSimpleDelegate MenuStackPushDebuggingInfo;
}	// anon namespace

TSharedRef<IMenu> FMenuStack::Push(const FWidgetPath& InOwnerPath, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize, TOptional<EPopupMethod> InMethod, const bool bIsCollapsedByParent, const bool bEnablePerPixelTransparency, const int32 FocusUserIndex)
{
	// We want to ensure that when the window is restored, we restore the current keyboard focus, 
	// but only if it is valid, otherwise we could end up clearing a previously valid path.
	TSharedPtr<SWidget> FocusedWidget = (FocusUserIndex == INDEX_NONE) ? FSlateApplication::Get().GetKeyboardFocusedWidget() : FSlateApplication::Get().GetUserFocusedWidget(FocusUserIndex);
	if (FocusedWidget.IsValid())
	{
		InOwnerPath.GetWindow()->SetWidgetToFocusOnActivate(FocusedWidget);
	}

	FSlateRect Anchor(SummonLocation, SummonLocation + SummonLocationSize);
	TSharedPtr<IMenu> ParentMenu;

	if (HasMenus())
	{
		// Find a menu in the stack in InOwnerPath to determine the level to insert this 
		ParentMenu = FindMenuInWidgetPath(InOwnerPath);
		check(HostWindow.IsValid());
	}

	if (!ParentMenu.IsValid())
	{
		// pushing a new root menu (leave ParentMenu invalid)

		// The active method is determined when a new root menu is pushed
		ActiveMethod = InMethod.IsSet() ? FPopupMethodReply::UseMethod(InMethod.GetValue()) : QueryPopupMethod(InOwnerPath);

		// The host window is determined when a new root menu is pushed
		// This must be set prior to PushInternal below, as it will be referenced if the menu being created is a new root menu.
		SetHostPath(InOwnerPath);
	}

	MenuStackInternal::MenuStackPushDebuggingInfo.Unbind();

#if WITH_EDITOR
	// If there is an error, this delegate will be called to print out additional details
	if (InOwnerPath.Widgets.Num() > 0)
	{
		// Copy data in case this array is modified
		TArray<FName> DebugSourceFilenames;
		DebugSourceFilenames.Reserve(InOwnerPath.Widgets.Num());
		for (int32 i=0; i < InOwnerPath.Widgets.Num(); ++i)
		{
			DebugSourceFilenames.Add(InOwnerPath.Widgets[i].Widget->GetCreatedInLocation());
		}

		MenuStackInternal::MenuStackPushDebuggingInfo.BindLambda([DebugSourceFilenames]()
		{
			for (const FName SourceFilename : DebugSourceFilenames)
			{
				UE_LOGF(LogSlate, Warning, "  WidgetSourceFile: %ls", *SourceFilename.ToString());
			}
		});
	}
#endif

	TGuardValue<bool> Guard(bHostWindowGuard, true);
	return PushInternal(ParentMenu, InContent, Anchor, TransitionEffect, bFocusImmediately, ActiveMethod.GetShouldThrottle(), bIsCollapsedByParent, bEnablePerPixelTransparency, FocusUserIndex);
}

TSharedRef<IMenu> FMenuStack::Push(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize, const bool bIsCollapsedByParent, const bool bEnablePerPixelTransparency, const int32 FocusUserIndex)
{
	check(Stack.Contains(InParentMenu));
	check(HostWindow.IsValid());

	FSlateRect Anchor(SummonLocation, SummonLocation + SummonLocationSize);

	return PushInternal(InParentMenu, InContent, Anchor, TransitionEffect, bFocusImmediately, EShouldThrottle::Yes, bIsCollapsedByParent, bEnablePerPixelTransparency, FocusUserIndex);
}

TSharedRef<IMenu> FMenuStack::PushHosted(const FWidgetPath& InOwnerPath, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent)
{
	TSharedPtr<IMenu> ParentMenu;

	if (HasMenus())
	{
		// Find a menu in the stack in InOwnerPath to determine the level to insert this 
		ParentMenu = FindMenuInWidgetPath(InOwnerPath);
		check(HostWindow.IsValid());
	}

	if (!ParentMenu.IsValid())
	{
		// pushing a new root menu (leave ParentMenu invalid)

		// The active method is determined when a new root menu is pushed and hosted menus are always UseCurrentWindow
		ActiveMethod = FPopupMethodReply::UseMethod(EPopupMethod::UseCurrentWindow);

		// The host window is determined when a new root menu is pushed
		SetHostPath(InOwnerPath);
	}

	return PushHosted(ParentMenu, InMenuHost, InContent, OutWrappedContent, TransitionEffect, ShouldThrottle);
}

TSharedRef<IMenu> FMenuStack::PushHosted(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent)
{
	check(HostWindow.IsValid());

	// Create a FMenuInHostWidget
	TSharedRef<SWidget> WrappedContent = WrapContent(InContent, FOptionalSize(), FOptionalSize(), InMenuHost->bShowMenuBackground);
	TSharedRef<FMenuInHostWidget> OutMenu = MakeShareable(new FMenuInHostWidget(InMenuHost, WrappedContent, bIsCollapsedByParent));
	PendingNewMenu = OutMenu;

	// Set the returned content - this must be drawn by the hosting widget until the menu gets dismissed and calls IMenuHost::OnMenuDismissed on its host
	OutWrappedContent = WrappedContent;

	// Register to get a callback when it's dismissed - to fixup stack
	OutMenu->GetOnMenuDismissed().AddRaw(this, &FMenuStack::OnMenuDestroyed);

	PostPush(InParentMenu, OutMenu, ShouldThrottle);

	PendingNewMenu.Reset();

	return OutMenu;
}

TSharedRef<IMenu> FMenuStack::PushInternal(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<SWidget>& InContent, FSlateRect Anchor, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent, const bool bEnablePerPixelTransparency, const int32 FocusUserIndex)
{
	FPrePushArgs PrePushArgs;
	PrePushArgs.Content = InContent;
	PrePushArgs.Anchor = Anchor;
	PrePushArgs.TransitionEffect = TransitionEffect;
	PrePushArgs.FocusUserIndex = FocusUserIndex;
	PrePushArgs.bFocusImmediately = bFocusImmediately;
	PrePushArgs.bIsCollapsedByParent = bIsCollapsedByParent;

	// Pre-push stage
	//   Determines correct layout
	//   Wraps content
	//   Other common setup steps needed by all (non-hosted) menus
	const FPrePushResults PrePushResults = PrePush(PrePushArgs);

	// Menu object creation stage
	TSharedRef<FMenuBase> OutMenu = ActiveMethod.GetPopupMethod() == EPopupMethod::CreateNewWindow
		? PushNewWindow(InParentMenu, PrePushResults, bEnablePerPixelTransparency, FocusUserIndex)
		: PushPopup(InParentMenu, PrePushResults, FocusUserIndex);

	// Post-push stage
	//   Updates the stack and content map member variables
	const bool bInInsertAfterDismiss = ActiveMethod.GetPopupMethod() == EPopupMethod::CreateNewWindow;
	PostPush(InParentMenu, OutMenu, ShouldThrottle, bInInsertAfterDismiss);

	PendingNewMenu.Reset();

	return OutMenu;
}

FMenuStack::FPrePushResults FMenuStack::PrePush(const FPrePushArgs& InArgs)
{
	FPrePushResults OutResults;
	OutResults.bIsCollapsedByParent = InArgs.bIsCollapsedByParent;
	OutResults.bFocusImmediately = InArgs.bFocusImmediately;
	if (InArgs.bFocusImmediately)
	{
		OutResults.WidgetToFocus = InArgs.Content;
	}

	// Only enable window position/size transitions if we're running at a decent frame rate
	OutResults.bAllowAnimations = FSlateApplication::Get().AreMenuAnimationsEnabled() && FSlateApplication::Get().IsRunningAtTargetFrameRate();

	// Calc the max height available on screen for the menu
	float MaxHeight;
	const float ApplicationScale = FSlateApplication::Get().GetApplicationScale() * HostWindow->GetNativeWindow()->GetDPIScaleFactor();
	if (ActiveMethod.GetPopupMethod() == EPopupMethod::CreateNewWindow)
	{
		FSlateRect WorkArea = FSlateApplication::Get().GetWorkArea(InArgs.Anchor);
		MaxHeight = FMenuStackDefs::MaxMenuScreenHeightFraction * WorkArea.GetSize().Y / ApplicationScale;
	}
	else
	{
		MaxHeight = FMenuStackDefs::MaxMenuScreenHeightFraction * HostWindow->GetClientSizeInScreen().Y / ApplicationScale;
	}

	bool bAnchorSetsMinWidth = InArgs.TransitionEffect.SlideDirection == FPopupTransitionEffect::ComboButton;

	// Wrap menu content in a box needed for various sizing and tracking purposes
	FOptionalSize OptionalMinWidth = bAnchorSetsMinWidth ? InArgs.Anchor.GetSize().X : FOptionalSize();
	FOptionalSize OptionalMinHeight = MaxHeight;

	// Wrap content in an SPopup before the rest of the wrapping process - should make menus appear on top using deferred presentation
	TSharedRef<SWidget> TempContent = SNew(SPopup)[InArgs.Content.ToSharedRef()];

	OutResults.WrappedContent = WrapContent(TempContent, OptionalMinWidth, OptionalMinHeight);

	OutResults.WrappedContent->SlatePrepass(ApplicationScale);
	// @todo slate: Doesn't take into account potential window border size
	OutResults.ExpectedSize = OutResults.WrappedContent->GetDesiredSize() * ApplicationScale;

	EOrientation Orientation = (InArgs.TransitionEffect.SlideDirection == FPopupTransitionEffect::SubMenu) ? Orient_Horizontal : Orient_Vertical;

	// Calculate the correct position for the menu based on the popup method
	if (ActiveMethod.GetPopupMethod() == EPopupMethod::CreateNewWindow)
	{
		// already handled
		const bool bAutoAdjustForDPIScale = false;
		// Places the menu's window in the work area
		OutResults.AnimStartLocation = OutResults.AnimFinalLocation = FSlateApplication::Get().CalculatePopupWindowPosition(InArgs.Anchor, OutResults.ExpectedSize, bAutoAdjustForDPIScale, FVector2f::ZeroVector, Orientation);
	}
	else
	{
		// Places the menu's content in the host window
		const FVector2f ProposedPlacement(
			Orientation == Orient_Horizontal ? InArgs.Anchor.Right : InArgs.Anchor.Left,
			Orientation == Orient_Horizontal ? InArgs.Anchor.Top : InArgs.Anchor.Bottom);

		OutResults.AnimStartLocation = OutResults.AnimFinalLocation =
			ComputePopupFitInRect(InArgs.Anchor, FSlateRect(ProposedPlacement, ProposedPlacement + OutResults.ExpectedSize), Orientation, HostWindow->GetClientRectInScreen());
	}

	// Start the pop-up menu at an offset location, then animate it to its target location over time
	// @todo: Anims aren't supported or attempted - this is legacy code left in in case we reinstate menu anims
	if (OutResults.bAllowAnimations)
	{
		const bool bSummonRight = OutResults.AnimFinalLocation.X >= OutResults.AnimStartLocation.X;
		const bool bSummonBelow = OutResults.AnimFinalLocation.Y >= OutResults.AnimStartLocation.Y;
		const int32 SummonDirectionX = bSummonRight ? 1 : -1;
		const int32 SummonDirectionY = bSummonBelow ? 1 : -1;

		switch (InArgs.TransitionEffect.SlideDirection)
		{
		case FPopupTransitionEffect::None:
			// No sliding
			break;

		case FPopupTransitionEffect::ComboButton:
			OutResults.AnimStartLocation.Y = FMath::Max(OutResults.AnimStartLocation.Y + 30.0f * SummonDirectionY, 0.0f);
			break;

		case FPopupTransitionEffect::TopMenu:
			OutResults.AnimStartLocation.Y = FMath::Max(OutResults.AnimStartLocation.Y + 60.0f * SummonDirectionY, 0.0f);
			break;

		case FPopupTransitionEffect::SubMenu:
			OutResults.AnimStartLocation.X += 60.0f * SummonDirectionX;
			break;

		case FPopupTransitionEffect::TypeInPopup:
			OutResults.AnimStartLocation.Y = FMath::Max(OutResults.AnimStartLocation.Y + 30.0f * SummonDirectionY, 0.0f);
			break;

		case FPopupTransitionEffect::ContextMenu:
			OutResults.AnimStartLocation.X += 30.0f * SummonDirectionX;
			OutResults.AnimStartLocation.Y += 50.0f * SummonDirectionY;
			break;
		}
	}

	// Release the mouse so that context can be properly restored upon closing menus.  See CL 1411833 before changing this.
	if (InArgs.bFocusImmediately)
	{
		if (InArgs.FocusUserIndex == INDEX_NONE)
		{
			FSlateApplication::Get().ReleaseAllPointerCapture();
		}
		else
		{
			FSlateApplication::Get().ReleaseAllPointerCapture(InArgs.FocusUserIndex);
		}
	}

	return OutResults;
}

TSharedRef<FMenuBase> FMenuStack::PushNewWindow(TSharedPtr<IMenu> InParentMenu, const FPrePushResults& InPrePushResults, const bool bEnablePerPixelTransparency, const int32 FocusUserIndex)
{
	check(ActiveMethod.GetPopupMethod() == EPopupMethod::CreateNewWindow);

	// Start pop-up windows out transparent, then fade them in over time
#if ALPHA_BLENDED_WINDOWS
	const EWindowTransparency Transparency(bEnablePerPixelTransparency ? EWindowTransparency::PerPixel : InPrePushResults.bAllowAnimations ? EWindowTransparency::PerWindow : EWindowTransparency::None);
#else
	const EWindowTransparency Transparency(InPrePushResults.bAllowAnimations ? EWindowTransparency::PerWindow : EWindowTransparency::None);
#endif

	const float InitialWindowOpacity = InPrePushResults.bAllowAnimations ? 0.0f : 1.0f;
	const float TargetWindowOpacity = 1.0f;

	// Create a new window for the menu
	TSharedRef<SWindow> NewMenuWindow = SNew(SWindow)
		.Type(EWindowType::Menu)
		.IsPopupWindow(true)
		.SizingRule(ESizingRule::Autosized)
		.ScreenPosition(InPrePushResults.AnimStartLocation)
		.AutoCenter(EAutoCenter::None)
		.ClientSize(InPrePushResults.ExpectedSize)
		.AdjustInitialSizeAndPositionForDPIScale(false)
		.InitialOpacity(InitialWindowOpacity)
		.SupportsTransparency(Transparency)
		.FocusWhenFirstShown(InPrePushResults.bFocusImmediately)
		.FocusUserIndex(FocusUserIndex)
		.ActivationPolicy(InPrePushResults.bFocusImmediately ? EWindowActivationPolicy::Always : EWindowActivationPolicy::Never)
		[
			InPrePushResults.WrappedContent.ToSharedRef()
		];

	PendingNewWindow = NewMenuWindow;

	if (InPrePushResults.bFocusImmediately && InPrePushResults.WidgetToFocus.IsValid())
	{
		// Focus the unwrapped content rather than just the window
		NewMenuWindow->SetWidgetToFocusOnActivate(InPrePushResults.WidgetToFocus);
	}

	TSharedRef<FMenuInWindow> Menu = MakeShareable(new FMenuInWindow(NewMenuWindow, InPrePushResults.WrappedContent.ToSharedRef(), InPrePushResults.bIsCollapsedByParent));
	PendingNewMenu = Menu;

	TSharedPtr<SWindow> ParentWindow;
	if (InParentMenu.IsValid())
	{
		ParentWindow = InParentMenu->GetParentWindow();
	}
	else
	{
		ParentWindow = HostWindow;
	}

	FSlateApplication::Get().AddWindowAsNativeChild(NewMenuWindow, ParentWindow.ToSharedRef(), true);

	// Kick off the intro animation!
	// @todo: Anims aren't supported or attempted - this is legacy code left in in case we reinstate menu anims
	if (InPrePushResults.bAllowAnimations)
	{
		FCurveSequence Sequence;
		Sequence.AddCurve(0, FMenuStackDefs::AnimationDuration, ECurveEaseFunction::CubicOut);
		NewMenuWindow->MorphToPosition(Sequence, TargetWindowOpacity, InPrePushResults.AnimFinalLocation);
	}

	PendingNewWindow.Reset();

	return Menu;
}

TSharedRef<FMenuBase> FMenuStack::PushPopup(TSharedPtr<IMenu> InParentMenu, const FPrePushResults& InPrePushResults, const int32 FocusUserIndex)
{
	check(ActiveMethod.GetPopupMethod() == EPopupMethod::UseCurrentWindow);

	// Create a FMenuInPopup
	check(InPrePushResults.WrappedContent.IsValid());
	TSharedRef<FMenuInPopup> Menu = MakeShareable(new FMenuInPopup(InPrePushResults.WrappedContent.ToSharedRef(), InPrePushResults.bIsCollapsedByParent));
	PendingNewMenu = Menu;

	// Register to get callback when it's dismissed - to fixup stack
	Menu->GetOnMenuDismissed().AddRaw(this, &FMenuStack::OnMenuDestroyed);

	// Add it to a slot on the menus panel widget
	HostWindowPopupPanel->PushMenu(Menu, InPrePushResults.AnimFinalLocation);

	if (InPrePushResults.bFocusImmediately && InPrePushResults.WidgetToFocus.IsValid())
	{
		if (FocusUserIndex == INDEX_NONE)
		{
			FSlateApplication::Get().SetKeyboardFocus(InPrePushResults.WidgetToFocus, EFocusCause::SetDirectly);
		}
		else
		{
			FSlateApplication::Get().SetUserFocus(FocusUserIndex, InPrePushResults.WidgetToFocus, EFocusCause::SetDirectly);
		}
	}

	return Menu;
}

void FMenuStack::PostPush(TSharedPtr<IMenu> InParentMenu, TSharedRef<FMenuBase> InMenu, EShouldThrottle ShouldThrottle, bool bInInsertAfterDismiss)
{
	// Determine at which index we insert this new menu in the stack
	int32 InsertIndex = 0;
	if (InParentMenu.IsValid())
	{
		int32 ParentIndex = Stack.IndexOfByKey(InParentMenu);
		check(ParentIndex != INDEX_NONE);

		InsertIndex = ParentIndex + 1;
	}

	// Do original behavior of insert before dismiss
	// Note: This will often crash because DismissFrom() may trigger FMenuStack::OnWindowActivated() then DismissAll() that empties Stack
	int32 RemovingAtIndex = InsertIndex;
	if (!bInInsertAfterDismiss)
	{
		Stack.Insert(InMenu, InsertIndex);
		CachedContentMap.Add(InMenu->GetContent(), InMenu);
		RemovingAtIndex = InsertIndex + 1;
	}

	// Dismiss menus after the insert point
	if (Stack.Num() > RemovingAtIndex)
	{
		// Note: DismissFrom() may trigger FMenuStack::OnWindowActivated() then DismissAll() that empties Stack
		DismissFrom(Stack[RemovingAtIndex]);

		// tidy the stack data now (it will happen via callbacks from the dismissed menus but that might be delayed)
		for (int32 StackIndex = Stack.Num() - 1; StackIndex >= RemovingAtIndex; --StackIndex)
		{
			CachedContentMap.Remove(Stack[StackIndex]->GetContent());
			Stack.RemoveAt(StackIndex);
		}
	}

	// Note: DismissFrom() above may trigger FMenuStack::OnWindowActivated() then DismissAll() that empties Stack
	// Insert menu after the dismiss when possible to avoid menu being deleted
	if (bInInsertAfterDismiss)
	{
		if (InsertIndex != Stack.Num())
		{
			MenuStackInternal::MenuStackPushDebuggingInfo.ExecuteIfBound();

			// A menu was unexpectedly removed or added
			UE_LOGF(LogSlate, Warning, "A menu was unexpectedly removed or added. InsertIndex:%d, Stack.Num:%d, InParentMenu.IsValid:%d", InsertIndex, Stack.Num(), InParentMenu.IsValid());
		}

		Stack.Add(InMenu);
		CachedContentMap.Add(InMenu->GetContent(), InMenu);
	}

	// When a new menu is pushed, if we are not already in responsive mode for Slate UI, enter it now
	// to ensure the menu is responsive in low FPS situations
	if (!ThrottleHandle.IsValid() && ShouldThrottle == EShouldThrottle::Yes)
	{
		ThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();
	}
}

FPopupMethodReply FMenuStack::QueryPopupMethod(const FWidgetPath& PathToQuery)
{
	for (int32 WidgetIndex = PathToQuery.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		FPopupMethodReply PopupMethod = PathToQuery.Widgets[WidgetIndex].Widget->OnQueryPopupMethod();
		if (PopupMethod.IsEventHandled())
		{
			return PopupMethod;
		}
	}

	return FPopupMethodReply::UseMethod(EPopupMethod::CreateNewWindow);
}

void FMenuStack::DismissFrom(const TSharedPtr<IMenu>& InFromMenu)
{
	int32 Index = Stack.IndexOfByKey(InFromMenu);
	if (Index != INDEX_NONE)
	{
		DismissInternal(Index);
	}
}

void FMenuStack::DismissAll()
{
	const int32 TopLevel = 0;
	DismissInternal(TopLevel);
}

void FMenuStack::DismissInternal(int32 FirstStackIndexToRemove)
{
	// Dismiss the stack in reverse order so we destroy children before parents (causes focusing issues if done the other way around)
	for ( int32 StackIndex = Stack.Num() - 1; StackIndex >= FirstStackIndexToRemove; --StackIndex )
	{
		if ( Stack.IsValidIndex(StackIndex) )
		{
			Stack[StackIndex]->Dismiss();
		}
	}
}

void FMenuStack::SetHostPath(const FWidgetPath& InOwnerPath)
{
	if (bHostWindowGuard)
	{
		return;
	}

	if ( HostPopupLayer.IsValid() )
	{
		if ( !InOwnerPath.ContainsWidget(&HostPopupLayer->GetHost().Get()) )
		{
			HostPopupLayer->Remove();
			HostPopupLayer.Reset();
			HostWindowPopupPanel.Reset();
		}
	}

	HostWindow = InOwnerPath.IsValid() ? InOwnerPath.GetWindow() : TSharedPtr<SWindow>();

	HostWidget = InOwnerPath.IsValid() ? InOwnerPath.GetLastWidget() : TWeakPtr<SWidget>();

	if ( HostWindow.IsValid() && !HostWindowPopupPanel.IsValid() )
	{
		TSharedRef<SMenuPanel> NewHostWindowPopupPanel = SNew(SMenuPanel);
		for ( int i = InOwnerPath.Widgets.Num() - 1; i >= 0; i-- )
		{
			const TSharedRef<SWidget>& CurrentWidget = InOwnerPath.Widgets[i].Widget;
			HostPopupLayer = CurrentWidget->OnVisualizePopup(NewHostWindowPopupPanel);
			if ( HostPopupLayer.IsValid() )
			{
				HostWindowPopupPanel = NewHostWindowPopupPanel;
				break;
			}
		}
	}
}

void FMenuStack::OnMenuDestroyed(TSharedRef<IMenu> InMenu)
{
	int32 Index = Stack.IndexOfByKey(InMenu);
	if (Index != INDEX_NONE)
	{
		// Dismiss menus below this one
		for (int32 StackIndex = Stack.Num() - 1; StackIndex > Index; --StackIndex)
		{
			Stack[StackIndex]->Dismiss();	// this will cause OnMenuDestroyed() to re-enter
		}

		if (MenuDestroyedEvent.IsBound())
		{
			MenuDestroyedEvent.Broadcast(InMenu);
		}

		// Clean up the stack and content map arrays
		for (int32 StackIndex = Stack.Num() - 1; StackIndex >= Index; --StackIndex)
		{
			CachedContentMap.Remove(Stack[StackIndex]->GetContent());
			Stack.RemoveAt(StackIndex);
		}

		// Leave responsive mode once the last menu closes
		if (Stack.Num() == 0)
		{
			if (ThrottleHandle.IsValid())
			{
				FSlateThrottleManager::Get().LeaveResponsiveMode(ThrottleHandle);
			}

			SetHostPath(FWidgetPath());
		}
	}
}

void FMenuStack::OnMenuContentLostFocus(const FWidgetPath& InFocussedPath)
{
	// In UseCurrentWindow mode we must look for focus moving from menus
	// Window activation messages will make menus collapse when in CreateNewWindow mode
	// However, we cannot rely on window activation messages because they might not be generated on Mac.
	// So, always do this focus/collapse code, even in CreateNewWindow mode.
	if (HasMenus() && !PendingNewMenu.IsValid())
	{
		// If focus is switching determine which of our menus it is in, if any
		TSharedPtr<IMenu> FocussedMenu = FindMenuInWidgetPath(InFocussedPath);

		if (FocussedMenu.IsValid())
		{
			// dismiss menus below FocussedMenu
			int32 FocussedIndex = Stack.IndexOfByKey(FocussedMenu);
			check(FocussedIndex != INDEX_NONE);

			for (int32 DismissIndex = FocussedIndex + 1; DismissIndex < Stack.Num(); DismissIndex++)
			{
				if (Stack[DismissIndex]->IsCollapsedByParent())
				{
					DismissFrom(Stack[DismissIndex]);
					break;
				}
			}
		}
		else
		{
#if WITH_SLATE_DEBUGGING
			if (!CVarSlateDismissMenuStacksOnFocusLost->GetBool())
			{
				return;
			}
#endif

			// Focus has moved outside all menus - collapse the stack
			DismissAll();
		}
	}
}

TSharedRef<SWidget> FMenuStack::WrapContent(TSharedRef<SWidget> InContent, FOptionalSize OptionalMinWidth, FOptionalSize OptionalMinHeight, bool bShouldShowBackground)
{
	// Wrap menu content in a box that limits its maximum height
	// and in a SMenuContentWrapper that handles key presses and focus changes.
	return SNew(MenuStackInternal::SMenuContentWrapper)
		.OnKeyDown_Static(&MenuStackInternal::OnMenuKeyDown)
		.OnMenuLostFocus_Raw(this, &FMenuStack::OnMenuContentLostFocus)
		.OptionalMinMenuWidth(OptionalMinWidth)
		.OptionalMinMenuHeight(OptionalMinHeight)
		.bShowBackground(bShouldShowBackground)
		.MenuContent()
		[
				InContent
		];
}

TSharedPtr<IMenu> FMenuStack::FindMenuInWidgetPath(const FWidgetPath& PathToQuery) const
{
	for (int32 PathIndex = PathToQuery.Widgets.Num() - 1; PathIndex >= 0; --PathIndex)
	{
		TSharedPtr<const SWidget> Widget = PathToQuery.Widgets[PathIndex].Widget;
		const TSharedPtr<FMenuBase>* FoundMenu = CachedContentMap.Find(Widget);
		if (FoundMenu != nullptr)
		{
			return *FoundMenu;
		}
	}
	return TSharedPtr<IMenu>();
}

void FMenuStack::OnWindowDestroyed(TSharedRef<SWindow> InWindow)
{
	if (HostWindow == InWindow)
	{
		// If the host window is destroyed, collapse the whole stack and reset all state
		Stack.Empty();
		CachedContentMap.Empty();

		SetHostPath(FWidgetPath());
	}
	else
	{
		// A window was requested to be destroyed, so make sure it's not in the menu stack to avoid it
		// becoming a parent to a freshly-created window!
		TSharedPtr<IMenu> Menu = FindMenuFromWindow(InWindow);

		if (Menu.IsValid())
		{
			OnMenuDestroyed(Menu.ToSharedRef());
		}
	}
}

void FMenuStack::OnWindowActivated( TSharedRef<SWindow> ActivatedWindow )
{
	if (ActivatedWindow != PendingNewWindow && HasMenus() && !FSlateApplication::Get().IsWindowHousingInteractiveTooltip(ActivatedWindow))
	{
		TWeakPtr<IMenu> ActivatedMenu = FindMenuFromWindow(ActivatedWindow);

		if (ActivatedMenu.IsValid())
		{
			// Dismiss menus below ActivatedMenu
			int32 ActivatedIndex = Stack.IndexOfByKey(ActivatedMenu);
			check(ActivatedIndex != INDEX_NONE);

			for (int32 DismissIndex = ActivatedIndex + 1; DismissIndex < Stack.Num(); DismissIndex++)
			{
				if (Stack[DismissIndex]->IsCollapsedByParent())
				{
					DismissFrom(Stack[DismissIndex]);
					break;
				}
			}
		}
		else
		{
#if WITH_SLATE_DEBUGGING
			if (!CVarSlateDismissMenuStacksOnFocusLost->GetBool())
			{
				return;
			}
#endif

			// Activated a window that isn't a menu - collapse the stack
			DismissAll();
		}
	}
}

TSharedPtr<IMenu> FMenuStack::FindMenuFromWindow(TSharedRef<SWindow> InWindow) const
{
	const TSharedPtr<FMenuBase>* FoundMenu = Stack.FindByPredicate([InWindow](TSharedPtr<FMenuBase> Menu) { return Menu->GetOwnedWindow() == InWindow; });
	if (FoundMenu != nullptr)
	{
		return *FoundMenu;
	}
	return TSharedPtr<IMenu>();
}

bool FMenuStack::GetToolTipForceFieldRect(const TSharedRef<IMenu>& InMenu, const FWidgetPath& InPathContainingMenu, FSlateRect& OutSlateRect) const
{
	bool bWasSolutionFound = false;
	OutSlateRect = FSlateRect(0, 0, 0, 0);

	int32 StackLevel = Stack.IndexOfByKey(InMenu);

	if (StackLevel != INDEX_NONE)
	{
		for (int32 StackLevelIndex = StackLevel + 1; StackLevelIndex < Stack.Num(); ++StackLevelIndex)
		{
			TSharedPtr<SWidget> MenuContent = Stack[StackLevelIndex]->GetContent();
			if (MenuContent.IsValid())
			{
				FWidgetPath WidgetPath = InPathContainingMenu.GetPathDownTo(MenuContent.ToSharedRef());
				if (!WidgetPath.IsValid())
				{
					FSlateApplication::Get().GeneratePathToWidgetUnchecked(MenuContent.ToSharedRef(), WidgetPath);
				}
				if (WidgetPath.IsValid())
				{
					const FGeometry& ContentGeometry = WidgetPath.Widgets.Last().Geometry;
					// No first time: Expand
					if (bWasSolutionFound)
					{
						OutSlateRect = OutSlateRect.Expand(ContentGeometry.GetLayoutBoundingRect());
					}
					// First time: assign it
					// Otherwise, it would assume that the point [0,0,0,0] is part of the final rectangle, which is not the case in multiple scenarios.
					// E.g., if the monitor where Slate is running is not the main one or if the Slate window is restored to the right half of the monitor.
					else
					{
						OutSlateRect = ContentGeometry.GetLayoutBoundingRect();
						bWasSolutionFound = true;
					}
				}
			}
		}
	}
	return bWasSolutionFound;
}

bool FMenuStack::HasMenus() const
{
	return Stack.Num() > 0;
}

bool FMenuStack::HasOpenSubMenus(TSharedPtr<IMenu> InMenu) const
{
	int32 StackIndex = Stack.IndexOfByKey(InMenu);
	return StackIndex != INDEX_NONE && StackIndex < Stack.Num() - 1;
}

TSharedPtr<SWindow> FMenuStack::GetHostWindow() const
{
	return HostWindow;
}

TSharedPtr<SWidget> FMenuStack::GetHostWidget() const
{
	return HostWidget.Pin();
}

#undef LOCTEXT_NAMESPACE
