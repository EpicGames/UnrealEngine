// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/NavigationConfig.h"
#include "Types/SlateEnums.h"
#include "Input/Events.h"
#include "Misc/App.h"

FNavigationConfig::FNavigationConfig()
	: bTabNavigation(true)
	, bKeyNavigation(true)
	, bAnalogNavigation(true)
	, bIgnoreModifiersForNavigationActions(true)
	, AnalogNavigationHorizontalThreshold(0.50f)
	, AnalogNavigationVerticalThreshold(0.50f)
{
	AnalogHorizontalKey = EKeys::Gamepad_LeftX;
	AnalogVerticalKey = EKeys::Gamepad_LeftY;

	KeyEventRules.Emplace(EKeys::Left, EUINavigation::Left);
	KeyEventRules.Emplace(EKeys::Gamepad_DPad_Left, EUINavigation::Left);

	KeyEventRules.Emplace(EKeys::Right, EUINavigation::Right);
	KeyEventRules.Emplace(EKeys::Gamepad_DPad_Right, EUINavigation::Right);

	KeyEventRules.Emplace(EKeys::Up, EUINavigation::Up);
	KeyEventRules.Emplace(EKeys::Gamepad_DPad_Up, EUINavigation::Up);

	KeyEventRules.Emplace(EKeys::Down, EUINavigation::Down);
	KeyEventRules.Emplace(EKeys::Gamepad_DPad_Down, EUINavigation::Down);

	// By default, enter, space, and gamepad accept are all counted as accept
	KeyActionRules.Emplace(EKeys::Enter, EUINavigationAction::Accept);
	KeyActionRules.Emplace(EKeys::SpaceBar, EUINavigationAction::Accept);
	KeyActionRules.Emplace(EKeys::Virtual_Gamepad_Accept.GetVirtualKey(), EUINavigationAction::Accept);

	// By default, escape and gamepad back count as leaving current scope
	KeyActionRules.Emplace(EKeys::Escape, EUINavigationAction::Back);
	KeyActionRules.Emplace(EKeys::Virtual_Gamepad_Back.GetVirtualKey(), EUINavigationAction::Back);
}

FNavigationConfig::~FNavigationConfig()
{
}

void FNavigationConfig::OnRegister()
{
	UserNavigationState.Reset();
}

void FNavigationConfig::OnUnregister()
{
}

void FNavigationConfig::OnUserRemoved(int32 UserIndex)
{
	UserNavigationState.Remove(UserIndex);
}

EUINavigation FNavigationConfig::GetNavigationDirectionFromKey(const FKeyEvent& InKeyEvent) const
{
	if (const EUINavigation* Rule = KeyEventRules.Find(InKeyEvent.GetKey()))
	{
		if (bKeyNavigation)
		{
			return *Rule;
		}
	}
	else if (bTabNavigation && InKeyEvent.GetKey() == EKeys::Tab )
	{
		//@TODO: Really these uses of input should be at a lower priority, only occurring if nothing else handled them
		// For now this code prevents consuming them when some modifiers are held down, allowing some limited binding
		const bool bAllowEatingKeyEvents = !InKeyEvent.IsControlDown() && !InKeyEvent.IsAltDown() && !InKeyEvent.IsCommandDown();

		if ( bAllowEatingKeyEvents )
		{
			return ( InKeyEvent.IsShiftDown() ) ? EUINavigation::Previous : EUINavigation::Next;
		}
	}

	return EUINavigation::Invalid;
}

EUINavigation FNavigationConfig::GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent)
{
	if (bAnalogNavigation)
	{
		const EUINavigation DesiredNavigation = GetNavigationDirectionFromAnalogInternal(InAnalogEvent);
		if (DesiredNavigation != EUINavigation::Invalid)
		{
			FUserNavigationState& UserState = UserNavigationState.FindOrAdd(InAnalogEvent.GetUserIndex());
			FAnalogNavigationState& AnalogState = UserState.AnalogNavigationState.FindOrAdd(FAnalogNavigationKey(InAnalogEvent.GetKey(), DesiredNavigation));

			const float RepeatRate = GetRepeatRateForPressure(FMath::Abs(InAnalogEvent.GetAnalogValue()), FMath::Max(AnalogState.Repeats - 1, 0));
			const double CurrentTime = FApp::GetCurrentTime();
			if (CurrentTime - AnalogState.LastNavigationTime > RepeatRate)
			{
				AnalogState.LastNavigationTime = CurrentTime;
				AnalogState.Repeats++;
				return DesiredNavigation;
			}
		}
	}

	return EUINavigation::Invalid;
}

EUINavigation FNavigationConfig::GetNavigationDirectionFromAnalogInternal(const FAnalogInputEvent& InAnalogEvent)
{
	if (bAnalogNavigation)
	{	
		FUserNavigationState& UserState = UserNavigationState.FindOrAdd(InAnalogEvent.GetUserIndex());
	
		const FKey& AnalogKey   = InAnalogEvent.GetKey();
		const float AnalogValue = InAnalogEvent.GetAnalogValue();
	
		if (IsAnalogHorizontalKey(AnalogKey))
		{
			if (AnalogValue < -AnalogNavigationHorizontalThreshold)
			{
				return EUINavigation::Left;
			}
			else if (AnalogValue > AnalogNavigationHorizontalThreshold)
			{
				return EUINavigation::Right;
			}
			else
			{
				UserState.AnalogNavigationState.Add(FAnalogNavigationKey(AnalogKey,EUINavigation::Left), FAnalogNavigationState());
				UserState.AnalogNavigationState.Add(FAnalogNavigationKey(AnalogKey, EUINavigation::Right), FAnalogNavigationState());
			}
		}
		else if (IsAnalogVerticalKey(AnalogKey))
		{
			if (AnalogValue > AnalogNavigationVerticalThreshold)
			{
				return EUINavigation::Up;
			}
			else if (AnalogValue < -AnalogNavigationVerticalThreshold)
			{
				return EUINavigation::Down;
			}
			else
			{
				UserState.AnalogNavigationState.Add(FAnalogNavigationKey(AnalogKey, EUINavigation::Up), FAnalogNavigationState());
                UserState.AnalogNavigationState.Add(FAnalogNavigationKey(AnalogKey, EUINavigation::Down), FAnalogNavigationState());
			}
		}
	}

	return EUINavigation::Invalid;
}

float FNavigationConfig::GetRepeatRateForPressure(float InPressure, int32 InRepeats) const
{
	const float RepeatRate = (InRepeats == 0) ? 0.5f : 0.25f;
	if (InPressure > 0.90f)
	{
		return RepeatRate * 0.5f;
	}

	return RepeatRate;
}

EUINavigationAction FNavigationConfig::GetNavigationActionFromKey(const FKeyEvent& InKeyEvent) const
{
	const bool bModifierHeld = InKeyEvent.IsControlDown() || InKeyEvent.IsAltDown() || InKeyEvent.IsCommandDown() || InKeyEvent.IsShiftDown();
	if (bIgnoreModifiersForNavigationActions || !bModifierHeld)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Call raw key version for back compatibility, subclasses should override this function
		return GetNavigationActionForKey(InKeyEvent.GetKey());
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	}
	return EUINavigationAction::Invalid;
}

EUINavigationAction FNavigationConfig::GetNavigationActionForKey(const FKey& InKey) const
{
	if (const EUINavigationAction* Action = KeyActionRules.Find(InKey))
	{
		return *Action;
	}
	return EUINavigationAction::Invalid;
}

FString FNavigationConfig::ToString() const
{
	TStringBuilder<1024> Builder;

	Builder.Appendf(TEXT("bTabNavigation: %u\n"), bTabNavigation);
	Builder.Appendf(TEXT("bKeyNavigation: %u\n"), bKeyNavigation);
	Builder.Appendf(TEXT("bAnalogNavigation: %u\n"), bAnalogNavigation);
	Builder.Appendf(TEXT("AnalogNavigationHorizontalThreshold: %f\n"), AnalogNavigationHorizontalThreshold);
	Builder.Appendf(TEXT("AnalogNavigationVerticalThreshold: %f\n"), AnalogNavigationVerticalThreshold);
	Builder.Appendf(TEXT("AnalogHorizontalKey: %s\n"), *AnalogHorizontalKey.ToString());
	Builder.Appendf(TEXT("AnalogVerticalKey: %s\n"), *AnalogVerticalKey.ToString());

	Builder.Appendf(TEXT("KeyEventRules: \n"));
	for (TPair<FKey, EUINavigation> KeyEventRule : KeyEventRules)
	{
		Builder.Appendf(TEXT("Pair: %s -> %s\n"), *KeyEventRule.Key.ToString(), *UEnum::GetValueAsString(KeyEventRule.Value));
	}

	return Builder.ToString();
}

bool FNavigationConfig::IsAnalogEventBeyondNavigationThreshold(const FAnalogInputEvent& InAnalogEvent) const
{
	if (bAnalogNavigation)
	{
		const FKey& AnalogKey = InAnalogEvent.GetKey();
		const float AbsAnalogValue = FMath::Abs(InAnalogEvent.GetAnalogValue());
	 
	 	return (IsAnalogHorizontalKey(AnalogKey) && AbsAnalogValue > AnalogNavigationHorizontalThreshold)
			|| (IsAnalogVerticalKey(AnalogKey) 	 && AbsAnalogValue > AnalogNavigationVerticalThreshold);
	}

	return false;
}

FTwinStickNavigationConfig::FTwinStickNavigationConfig()
{
	bTabNavigation = false;

	KeyEventRules =
	{
		{EKeys::Gamepad_DPad_Left, EUINavigation::Left},
		{EKeys::Gamepad_DPad_Right, EUINavigation::Right},
		{EKeys::Gamepad_DPad_Up, EUINavigation::Up},
		{EKeys::Gamepad_DPad_Down, EUINavigation::Down}
	};
}

bool FTwinStickNavigationConfig::IsAnalogHorizontalKey(const FKey& InKey) const
{
	return InKey == EKeys::Gamepad_LeftX || InKey == EKeys::Gamepad_RightX;
}

bool FTwinStickNavigationConfig::IsAnalogVerticalKey(const FKey& InKey) const
{
	return InKey == EKeys::Gamepad_LeftY || InKey == EKeys::Gamepad_RightY;
}
