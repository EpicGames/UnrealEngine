// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/SWindowTitleBar.h"

void SWindowTitleBar::Construct( const FArguments& InArgs, const TSharedRef<SWindow>& InWindow, const TSharedPtr<SWidget>& InCenterContent, EHorizontalAlignment InCenterContentAlignment )
{
	SetCanTick(false);

	OwnerWindowPtr = InWindow;
	Style = InArgs._Style;
	ShowAppIcon = InArgs._ShowAppIcon;
	Title = InArgs._Title;
	CloseButtonToolTipText = InArgs._CloseButtonToolTipText;

	WindowMenuSlot = nullptr;

	if (!Title.IsSet() && !Title.IsBound())
	{
		Title = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SWindowTitleBar::HandleWindowTitleText));
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(0.0f)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.BorderImage(this, &SWindowTitleBar::GetWindowTitlebackgroundImage)
		[
			SNew(SOverlay)
			.Visibility(EVisibility::SelfHitTestInvisible)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(this, &SWindowTitleBar::GetWindowFlashVisibility)
				.Image(&Style->FlashTitleBrush )
				.ColorAndOpacity(this, &SWindowTitleBar::GetWindowTitleAreaColor)
			]
			+ SOverlay::Slot()
			[
				MakeTitleBarContent(InCenterContent, InCenterContentAlignment)
			]
		]
	];
}

void SWindowTitleBar::Flash()
{
	TitleFlashSequence = FCurveSequence(0, SWindowTitleBarDefs::WindowFlashDuration, ECurveEaseFunction::Linear);
	TitleFlashSequence.Play(this->AsShared());
}

void SWindowTitleBar::UpdateWindowMenu(TSharedPtr<SWidget> MenuContent)
{
	if (!WindowMenuSlot)
	{
		return;
	}

	if(MenuContent.IsValid() && bAllowMenuBar)
	{
		(*WindowMenuSlot)
		[
			MenuContent.ToSharedRef()
		];
	}
	else
	{
		(*WindowMenuSlot)
		[
			SNullWidget::NullWidget
		];
	}
}

void SWindowTitleBar::UpdateBackgroundContent(TSharedPtr<SWidget> BackgroundContent)
{
	if (BackgroundContent.IsValid())
	{
		if (RightSideContentSlot != nullptr)
		{
			(*RightSideContentSlot)
			[
				BackgroundContent.ToSharedRef()
			];
		}
	}
	else
	{
		if (RightSideContentSlot != nullptr)
		{
			(*RightSideContentSlot)
			[
				SNullWidget::NullWidget
			];
		}
	}
}

void SWindowTitleBar::SetAllowMenuBar(bool bInAllowMenuBar)
{
	if (!FSlateApplicationBase::IsInitialized())
	{
		return;
	}
	bAllowMenuBar = bInAllowMenuBar;

	if (AppIconWidget)
	{
		if (bInAllowMenuBar)
		{
			AppIconWidget->SetIcon(FSlateApplicationBase::Get().GetAppIcon(), FAppStyle::Get().GetMargin("AppIconPadding", nullptr, FMargin(0)));
		}
		else
		{
			AppIconWidget->SetIcon(FSlateApplicationBase::Get().GetAppIconSmall(), FAppStyle::Get().GetMargin("AppIconPadding.Small", nullptr, FMargin(0)));

		}
	}
}

void SWindowTitleBar::MakeTitleBarContentWidgets( TSharedPtr< SWidget >& OutLeftContent, TSharedPtr< SWidget >& OutRightContent )
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (!OwnerWindow.IsValid())
	{
		return;
	}

	const bool bHasWindowButtons = OwnerWindow->HasCloseBox() || OwnerWindow->HasMinimizeBox() || OwnerWindow->HasMaximizeBox();

// We don't need these on the Mac where the system adds the "traffic light" buttons (close, minimize, and maximize).
#if !PLATFORM_MAC
	if (bHasWindowButtons)
	{
		MinimizeButton = SNew(SButton)
				.IsFocusable(false)
				.IsEnabled(OwnerWindow->HasMinimizeBox())
				.ContentPadding(0.f)
				.OnClicked(this, &SWindowTitleBar::MinimizeButton_OnClicked)
				.Cursor(EMouseCursor::Default)
				.ButtonStyle(FAppStyle::Get(), "Window.MinMaxRestoreButtonHover")
				.AddMetaData(FDriverMetaData::Id("launcher-minimizeWindowButton"))
				[
					SNew(SImage)
						.Image(this, &SWindowTitleBar::GetMinimizeImage)
						.ColorAndOpacity(this, &SWindowTitleBar::GetWindowTitleContentColor)
						.AccessibleText(NSLOCTEXT("WindowTitleBar", "Minimize", "Minimize"))
						.ToolTipText(NSLOCTEXT("WindowTitleBar", "Minimize", "Minimize"))
				];

		MaximizeRestoreButton = SNew(SButton)
				.IsFocusable(false)
				.IsEnabled(OwnerWindow->HasMaximizeBox())
				.ContentPadding(0.0f)
				.OnClicked(this, &SWindowTitleBar::MaximizeRestoreButton_OnClicked)
				.Cursor(EMouseCursor::Default)
				.ButtonStyle(FAppStyle::Get(), "Window.MinMaxRestoreButtonHover")
				.AddMetaData(FDriverMetaData::Id("launcher-maximizeRestoreWindowButton"))
				[
					SNew(SImage)
						.Image(this, &SWindowTitleBar::GetMaximizeRestoreImage)
						.ColorAndOpacity(this, &SWindowTitleBar::GetWindowTitleContentColor)
						.AccessibleText(this, &SWindowTitleBar::GetMaximizeButtonToolTip)
						.ToolTipText(this, &SWindowTitleBar::GetMaximizeButtonToolTip)
				];

		CloseButton = SNew(SButton)
				.IsFocusable(false)
				.IsEnabled(OwnerWindow->HasCloseBox())
				.ContentPadding(0.0f)
				.OnClicked(this, &SWindowTitleBar::CloseButton_OnClicked)
				.Cursor(EMouseCursor::Default)
				.ButtonStyle(FAppStyle::Get(), "Window.CloseButtonHover")
				.AddMetaData(FDriverMetaData::Id("launcher-closeWindowButton"))
				[
					SNew(SImage)
						.Image(this, &SWindowTitleBar::GetCloseImage)
						.ColorAndOpacity(this, &SWindowTitleBar::GetWindowTitleContentColor)
						.AccessibleText(CloseButtonToolTipText)
						.ToolTipText(CloseButtonToolTipText)
				];
	}
#endif //!PLATFORM_MAC

	if (ShowAppIcon && bHasWindowButtons)
	{
		OutLeftContent = 
			SNew(SHorizontalBox)
#if PLATFORM_MAC
			// The Mac has traffic-light buttons to the left, so we need some empty space under them (but no icon).
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				// This leaves spaces for the macOS "traffic light" close/minimize/maximize buttons. Without this space, the traffic lights would render on top of the main menu on the Mac.
				SNew(SSpacer)
				.Size(FVector2D(64, 10))
			]
#else
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(FMargin(0, 0, 0, 4.0f))
			[
				SAssignNew(AppIconWidget, SAppIconWidget)
				.IconColorAndOpacity(this, &SWindowTitleBar::GetWindowTitleContentColor)
			]
#endif //PLATFORM_MAC
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Top)
			.FillWidth(1)
#if PLATFORM_MAC
			// Align the main menu with the macOS "traffic light" buttons (close, minimize, and maximize).
			.Padding(FMargin(0, 3.0f, 0, 0))
#endif // PLATFORM_MAC
			.Expose(WindowMenuSlot);

		// Default everything to use the small icon unless specifically set to use the large icon.
		SetAllowMenuBar(false);
	}
	else
	{
		WindowMenuSlot = nullptr;

		OutLeftContent = SNew(SSpacer);
	}

	if (bHasWindowButtons)
	{
		OutRightContent = SNew(SBox)
			.Visibility(EVisibility::SelfHitTestInvisible)
			.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
			[
				SNew(SHorizontalBox)
// We don't need to add these as macOS draws them for us.
#if !PLATFORM_MAC
				// Minimize
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					MinimizeButton.ToSharedRef()
				]

				// Maximize/Restore
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					MaximizeRestoreButton.ToSharedRef()
				]

				// Close button
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					CloseButton.ToSharedRef()
				]
#endif //!PLATFORM_MAC
			];
	}
	else
	{
		RightSideContentSlot = nullptr;

		OutRightContent = SNew(SSpacer);
	}
}

TSharedRef<SWidget> SWindowTitleBar::MakeTitleBarContent( TSharedPtr<SWidget> CenterContent, EHorizontalAlignment CenterContentAlignment )
{
	TSharedPtr<SWidget> LeftContent;
	TSharedPtr<SWidget> RightContent;

	MakeTitleBarContentWidgets(LeftContent, RightContent);

	// Extension content lives in its own FillWidth slot so it clips before
	// the window buttons (AutoWidth, right) and the app icon + menus (AutoWidth, left) do.
	TSharedPtr<SWidget> ExtensionContent = SNew(SBox)
		.Visibility(EVisibility::SelfHitTestInvisible)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Expose(RightSideContentSlot)
		];

	// create window title if no content was provided
	if (!CenterContent.IsValid())
	{
		CenterContent = SNew(SBox)
			.HAlign(HAlign_Center)
			.Visibility(EVisibility::SelfHitTestInvisible)
			.Padding(FMargin(5.0f, 2.0f, 5.0f, 2.0f))
			[
				// NOTE: We bind the window's title text to our window's GetTitle method, so that if the
				//       title is changed later, the text will always be visually up to date
				SNew(STextBlock)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.TextStyle(&Style->TitleTextStyle)
					.Text(Title)
			];
	}

	// Adjust the center content alignment if needed. Windows without any title bar buttons look better if the title is centered.
	if (LeftContent == SNullWidget::NullWidget && RightContent == SNullWidget::NullWidget && CenterContentAlignment == EHorizontalAlignment::HAlign_Left)
	{
		CenterContentAlignment = EHorizontalAlignment::HAlign_Center;
	}

	// calculate content dimensions
	LeftContent->SlatePrepass();
	ExtensionContent->SlatePrepass();
	RightContent->SlatePrepass();

	FVector2D LeftSize = LeftContent->GetDesiredSize();
	FVector2D ExtensionSize = ExtensionContent->GetDesiredSize();
	FVector2D ButtonsSize = RightContent->GetDesiredSize();
	FVector2D RightSize = FVector2D(ExtensionSize.X + ButtonsSize.X, FMath::Max(ExtensionSize.Y, ButtonsSize.Y));

	if (CenterContentAlignment == HAlign_Center)
	{
		LeftSize = FVector2D::Max(LeftSize, RightSize);
		RightSize = LeftSize;
	}

	float SpacerHeight = (float)FMath::Max(LeftSize.Y, RightSize.Y);

	// create title bar
	return 
		SAssignNew(TitleArea, SBox)
		.Visibility(EVisibility::SelfHitTestInvisible)
		[
			SNew(SOverlay)
			.Visibility(EVisibility::SelfHitTestInvisible)
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				.Visibility(EVisibility::SelfHitTestInvisible)
						
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					SNew(SSpacer)
					.Size(FVector2D(LeftSize.X, SpacerHeight))
				]

				+ SHorizontalBox::Slot()
				.HAlign(CenterContentAlignment)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					CenterContent.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				[
					SNew(SSpacer)
					.Size(FVector2D(RightSize.X, SpacerHeight))
				]
			]
			// Foreground: nested layout with explicit clip priority.
			// 1. RightContent (outer AutoWidth): never clips.
			// 2. ExtensionContent (inner FillWidth + ClipToBounds): clips first.
			// 3. LeftContent (inner AutoWidth, clipped by outer SBox): clips last.
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				.Visibility(EVisibility::SelfHitTestInvisible)

				// Outer fill slot: absorbs all available space left of the buttons.
				// ClipToBounds ensures LeftContent cannot bleed into the button area.
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Top)
				[
					SNew(SBox)
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						SNew(SHorizontalBox)

						// LeftContent: AutoWidth inside the clipped container.
						// Only clips once the ExtensionContent area is fully gone.
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							LeftContent.ToSharedRef()
						]

						// ExtensionContent: FillWidth so it clips before LeftContent is affected.
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Top)
						[
							SNew(SBox)
							.Clipping(EWidgetClipping::ClipToBounds)
							[
								ExtensionContent.ToSharedRef()
							]
						]
					]
				]

				// RightContent: AutoWidth so it never clips.
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				[
					RightContent.ToSharedRef()
				]
			]
		];
}

FReply SWindowTitleBar::CloseButton_OnClicked()
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (OwnerWindow.IsValid())
	{
		OwnerWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

const FSlateBrush* SWindowTitleBar::GetCloseImage() const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (!OwnerWindow.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();

	if (CloseButton->IsPressed())
	{
		return &Style->CloseButtonStyle.Pressed;
	}

	if (CloseButton->IsHovered())
	{
		return &Style->CloseButtonStyle.Hovered;
	}

	return &Style->CloseButtonStyle.Normal;
}

FReply SWindowTitleBar::MaximizeRestoreButton_OnClicked( )
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (OwnerWindow.IsValid())
	{
		TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();

		if (NativeWindow.IsValid())
		{
			if (NativeWindow->GetWindowMode() == EWindowMode::WindowedFullscreen)
			{
				OwnerWindow->SetWindowMode(EWindowMode::Windowed);
			}
			else if (NativeWindow->IsMaximized())
			{
				NativeWindow->Restore();
			}
			else
			{
				NativeWindow->Maximize();
			}
		}
	}

	return FReply::Handled();
}

const FSlateBrush* SWindowTitleBar::GetMaximizeRestoreImage( ) const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (!OwnerWindow.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();

	if (!OwnerWindow->HasMaximizeBox())
	{
		return &Style->MaximizeButtonStyle.Disabled;
	}
	
	if (NativeWindow.IsValid())
	{
		if (NativeWindow->GetWindowMode() == EWindowMode::WindowedFullscreen)
		{
			if (MaximizeRestoreButton->IsPressed())
			{
				return &Style->ExitFullscreenButtonStyle.Pressed;
			}
			if (MaximizeRestoreButton->IsHovered())
			{
				return &Style->ExitFullscreenButtonStyle.Hovered;
			}
			return &Style->ExitFullscreenButtonStyle.Normal;
		}
		
		if (NativeWindow->IsMaximized())
		{
			if (MaximizeRestoreButton->IsPressed())
			{
				return &Style->RestoreButtonStyle.Pressed;
			}
			if (MaximizeRestoreButton->IsHovered())
			{
				return &Style->RestoreButtonStyle.Hovered;
			}
			return &Style->RestoreButtonStyle.Normal;
		}
	}

	if (MaximizeRestoreButton->IsPressed())
	{
		return &Style->MaximizeButtonStyle.Pressed;
	}
	if (MaximizeRestoreButton->IsHovered())
	{
		return &Style->MaximizeButtonStyle.Hovered;
	}
	return &Style->MaximizeButtonStyle.Normal;
}

FReply SWindowTitleBar::MinimizeButton_OnClicked( )
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (OwnerWindow.IsValid())
	{
		TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();

		if (NativeWindow.IsValid())
		{
			NativeWindow->Minimize();
		}
	}

	return FReply::Handled();
}

const FSlateBrush* SWindowTitleBar::GetMinimizeImage( ) const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (!OwnerWindow.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();

	if (!OwnerWindow->HasMinimizeBox())
	{
		return &Style->MinimizeButtonStyle.Disabled;
	}
	else if (MinimizeButton->IsPressed())
	{
		return &Style->MinimizeButtonStyle.Pressed;
	}
	else if (MinimizeButton->IsHovered())
	{
		return &Style->MinimizeButtonStyle.Hovered;
	}
	else
	{
		return &Style->MinimizeButtonStyle.Normal;
	}
}

const FSlateBrush* SWindowTitleBar::GetWindowTitlebackgroundImage( ) const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (!OwnerWindow.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();
	const bool bIsActive = NativeWindow.IsValid() && NativeWindow->IsForegroundWindow();

	return bIsActive ? &Style->ActiveTitleBrush : &Style->InactiveTitleBrush;
}

EVisibility SWindowTitleBar::GetWindowFlashVisibility( ) const
{
	return TitleFlashSequence.IsPlaying() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;
}
	
FSlateColor SWindowTitleBar::GetWindowTitleAreaColor( ) const
{	
	// Color of the white flash in the title area
	float Flash = GetFlashValue();
	float Alpha = Flash * 0.4f;

	FLinearColor Color = FLinearColor::White;
	Color.A = Alpha;

	return Color;
}

FText SWindowTitleBar::HandleWindowTitleText( ) const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (!OwnerWindow.IsValid())
	{
		return FText::GetEmpty();
	}

	return OwnerWindow->GetTitle();
}

FText SWindowTitleBar::GetMaximizeButtonToolTip() const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();

	if (OwnerWindow.IsValid())
	{
		TSharedPtr<FGenericWindow> NativeWindow = OwnerWindow->GetNativeWindow();
		if (NativeWindow.IsValid())
		{
			if (NativeWindow->IsMaximized())
			{
				return NSLOCTEXT("WindowTitleBar", "MaximizeRestoreDown", "Restore Down");
			}
			if (NativeWindow->GetWindowMode() == EWindowMode::WindowedFullscreen)
			{
				return NSLOCTEXT("WindowTitleBar", "MaximizeExitWindowedFullscreen", "Exit Full Screen");
			}
		}
	}

	return NSLOCTEXT("WindowTitleBar", "Maximize", "Maximize");
}