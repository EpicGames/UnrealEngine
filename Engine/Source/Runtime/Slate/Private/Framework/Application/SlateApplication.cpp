// Copyright Epic Games, Inc. All Rights Reserved.


#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/TimeGuard.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "InputCoreModule.h"
#include "Input/NavigationMetadata.h"
#include "Layout/LayoutUtils.h"
#include "Sound/ISlateSoundDevice.h"
#include "Sound/NullSlateSoundDevice.h"
#include "Framework/Text/PlatformTextField.h"
#include "Framework/Application/NavigationConfig.h"
#include "Widgets/SWeakWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SDeferredToolTip.h"
#include "Widgets/SViewport.h"
#include "Framework/Application/SWindowTitleBar.h"
#include "Input/Events.h"
#include "Input/HittestGrid.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformStackWalk.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#if PLATFORM_MAC || PLATFORM_IOS
#include "Apple/AppleTagRect.h"
#endif

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

#include "Framework/Application/IWidgetReflector.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Notifications/SlateAsyncTaskNotificationImpl.h"
#include "Framework/Application/IInputProcessor.h"
#include "GenericPlatform/ITextInputMethodSystem.h"
#include "Framework/Docking/TabCommands.h"
#include "Framework/Docking/TabManager.h"
#include "Math/UnitConversion.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/StallDetector.h"
#include "Types/ReflectionMetadata.h"
#include "Trace/SlateMemoryTags.h"
#include "Trace/SlateTrace.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateApplication)

#ifndef SLATE_HAS_WIDGET_REFLECTOR
	#define SLATE_HAS_WIDGET_REFLECTOR !(UE_BUILD_TEST || UE_BUILD_SHIPPING) && PLATFORM_DESKTOP
#endif

#if PLATFORM_MICROSOFT
#include "Microsoft/WindowsHWrapper.h"
#endif
#include "Debugging/SlateDebugging.h"
#include "Styling/StyleColors.h"


CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);

//////////////////////////////////////////////////////////////////////////

#if WITH_SLATE_DEBUGGING
bool GSlateVerifyParentChildrenRelationship = false;
static FAutoConsoleVariableRef CVarSlateVerifyParentChildrenRelationship(
	TEXT("Slate.VerifyParentChildrenRelationship"),
	GSlateVerifyParentChildrenRelationship,
	TEXT("Every tick, verify that a widget has only one parent.")
);

bool GSlateVerifyWidgetLayerId = false;
static FAutoConsoleVariableRef CVarSlateVerifyWidgetLayerId(
	TEXT("Slate.VerifyWidgetLayerId"),
	GSlateVerifyWidgetLayerId,
	TEXT("Every tick, verify that widgets have a LayerId range that fits with their siblings and their parent.")
);
namespace UE::Slate::Private
{
	void VerifyParentChildrenRelationship(const TSharedRef<SWindow>& WindowToDraw);
	void VerifyWidgetLayerId(const TSharedRef<SWindow>& WindowToDraw);
}

bool GSlateTraceNavigationConfig = false;
static FAutoConsoleVariableRef CVarSlateTraceNavigationConfig(
	TEXT("Slate.Debug.TraceNavigationConfig"),
	GSlateTraceNavigationConfig,
	TEXT("True enables tracing of navigation config & callstack to log.")
);
#endif //WITH_SLATE_DEBUGGING

bool GSlateInputMotionFiresUserInteractionEvents = true;
static FAutoConsoleVariableRef CVarSlateInputMotionFiresUserInteractionEvents(
	TEXT("Slate.Input.MotionFiresUserInteractionEvents"),
	GSlateInputMotionFiresUserInteractionEvents,
	TEXT("If this is false, LastUserInteractionTimeUpdateEvent events won't be fired based on motion input, and LastInteractionTime won't be updated\n")
	TEXT("Some motion devices report small tiny changes constantly without filtering, so motion input is unhelpful for determining user activity"));

static bool GSlateInputPointerUpFiresPointerMoveForDragDrop = true;
static FAutoConsoleVariableRef CVarSlateInputPointerUpFiresPointerMoveForDragDrop(
	TEXT("Slate.Input.PointerUpFiresPointerMoveForDragDrop"),
	GSlateInputPointerUpFiresPointerMoveForDragDrop,
	TEXT("When true, a synthetic pointer move event is fired from pointer up to ensure drag events are called if necessary on any widgets before OnDrop."));

//////////////////////////////////////////////////////////////////////////

bool GSlateEnableGamepadEditorNavigation = true;
static FAutoConsoleVariableRef CVarSlateEnableGamepadEditorNavigation(
	TEXT("Slate.EnableGamepadEditorNavigation"),
	GSlateEnableGamepadEditorNavigation,
	TEXT("True implies we allow gamepad navigation outside of the game viewport.")
);

static bool GSlateUseFixedDeltaTime = false;
static FAutoConsoleVariableRef CVarSlateUseFixedDeltaTime(
	TEXT("Slate.UseFixedDeltaTime"),
	GSlateUseFixedDeltaTime,
	TEXT("True means we use a constant delta time on every widget tick.")
);

static bool GSlateSkipWidgetDrawingInHeadlessMode = true;
static FAutoConsoleVariableRef CVarSlateSkipWidgetDrawingInHeadlessMode(
	TEXT("Slate.SkipWidgetDrawingInHeadlessMode"),
	GSlateSkipWidgetDrawingInHeadlessMode,
	TEXT("Skip drawing the widgets when running without rendering (e.g. -nullrhi). On by default, disable if there's non-visual logic in widget's Tick function (as this also skips ticking them).")
);

//////////////////////////////////////////////////////////////////////////

/** 
 * The cursor given to any additional input-providing SlateUsers beyond the first one (since the first one owns the platform cursor)
 * Or, if we're on a platform without a cursor, User 0 gets one of these too
 */
class FFauxSlateCursor : public ICursor
{
public:
	FFauxSlateCursor() 
	{
		// We don't support any concept of invalid or unset position for pointer events.
		// To avoid collisions with fullscreen or windows or multi-monitor setups with monitors left of primary,
		// we initialize the faux position to something that shouldn't generate overlap with any widgets.
		CurrentPosition = FVector2D(std::numeric_limits<int32>::min(), std::numeric_limits<int32>::min());
	}
	virtual ~FFauxSlateCursor() {}
	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override {}
	
	virtual FVector2D GetPosition() const override { return CurrentPosition; }
	virtual EMouseCursor::Type GetType() const override { return CurrentType; }
	
	virtual void SetType(const EMouseCursor::Type InNewCursor) override { CurrentType = InNewCursor; }
	virtual void Show(bool bShow) override { bIsVisible = bShow; }

	virtual void SetPosition(const int32 X, const int32 Y) override
	{
		FVector2D NewPosition(X, Y);
		UpdateCursorClipping(NewPosition);

		CurrentPosition = NewPosition;
	}

	virtual void GetSize(int32& Width, int32& Height) const override
	{
		Width = Height = 32;
	}

	virtual void Lock(const RECT* const Bounds) override
	{
		if (Bounds)
		{
			CursorClipRect.Min.X = Bounds->left;
			CursorClipRect.Min.Y = Bounds->top;
			CursorClipRect.Max.X = Bounds->right - 1;
			CursorClipRect.Max.Y = Bounds->bottom - 1;
		}
		else
		{
			//@todo DanH: Do we want to enforce a limit to the bounds of the screen like the console cursors?
			CursorClipRect = FIntRect();
		}

		FVector2D Position = GetPosition();
		if (UpdateCursorClipping(Position))
		{
			SetPosition(Position.X, Position.Y);
		}
	}

private:
	bool UpdateCursorClipping(FVector2D& CursorPosition)
	{
		bool bAdjusted = false;

		if (CursorClipRect.Area() > 0)
		{
			if (CursorPosition.X < CursorClipRect.Min.X)
			{
				CursorPosition.X = CursorClipRect.Min.X;
				bAdjusted = true;
			}
			else if (CursorPosition.X > CursorClipRect.Max.X)
			{
				CursorPosition.X = CursorClipRect.Max.X;
				bAdjusted = true;
			}

			if (CursorPosition.Y < CursorClipRect.Min.Y)
			{
				CursorPosition.Y = CursorClipRect.Min.Y;
				bAdjusted = true;
			}
			else if (CursorPosition.Y > CursorClipRect.Max.Y)
			{
				CursorPosition.Y = CursorClipRect.Max.Y;
				bAdjusted = true;
			}
		}

		return bAdjusted;
	}

	bool bIsVisible = false;
	EMouseCursor::Type CurrentType = EMouseCursor::None;
	FVector2D CurrentPosition = FVector2D::ZeroVector;
	FIntRect CursorClipRect;
};

//////////////////////////////////////////////////////////////////////////

class FEventRouter
{

// @todo slate : Remove remaining [&]-style mass captures.
// @todo slate : Eliminate all ad-hoc uses of SetEventPath()

public:

	class FDirectPolicy
	{
	public:
		FDirectPolicy( const FWidgetAndPointer& InTarget, const FWidgetPath& InRoutingPath )
		: bEventSent(false)
		, RoutingPath(InRoutingPath)
			, WidgetsUnderCursor(&RoutingPath)
			, Target(InTarget)
		{
		}

		FDirectPolicy(const FWidgetAndPointer& InTarget, const FWidgetPath& InRoutingPath, const FWidgetPath* InWidgetsUnderCursor)
			: bEventSent(false)
			, RoutingPath(InRoutingPath)
			, WidgetsUnderCursor(InWidgetsUnderCursor)
		, Target(InTarget)
		{
		}

		static FName Name;

		bool ShouldKeepGoing() const
		{
			return !bEventSent;
		}

		void Next()
		{
			bEventSent = true;
		}

		FWidgetAndPointer GetWidget() const
		{
			return Target;
		}

		const FWidgetPath& GetRoutingPath() const
		{
			return RoutingPath;
		}

		const FWidgetPath* GetWidgetsUnderCursor() const
		{
			return WidgetsUnderCursor;
		}

	private:
		bool bEventSent;
		const FWidgetPath& RoutingPath;
		const FWidgetPath* WidgetsUnderCursor;
		const FWidgetAndPointer& Target;
	};

	class FToLeafmostPolicy
	{
	public:
		FToLeafmostPolicy( const FWidgetPath& InRoutingPath )
		: bEventSent(false)
		, RoutingPath(InRoutingPath)
		{
		}

		static FName Name;

		bool ShouldKeepGoing() const
		{
			return !bEventSent && RoutingPath.Widgets.Num() > 0;
		}

		void Next()
		{
			bEventSent = true;
		}

		FWidgetAndPointer GetWidget() const
		{
			const int32 WidgetIndex = RoutingPath.Widgets.Num()-1;
			return FWidgetAndPointer(RoutingPath.Widgets[WidgetIndex], RoutingPath.GetVirtualPointerPosition(WidgetIndex));
		}

		const FWidgetPath& GetRoutingPath() const
		{
			return RoutingPath;
		}

		const FWidgetPath* GetWidgetsUnderCursor() const
		{
			return &RoutingPath;
		}

	private:
		bool bEventSent;
		const FWidgetPath& RoutingPath;
	};

	class FTunnelPolicy
	{
	public:
		FTunnelPolicy( const FWidgetPath& InRoutingPath )
		: WidgetIndex(0)
		, RoutingPath(InRoutingPath)
		{
		}

		static FName Name;

		bool ShouldKeepGoing() const
		{
			return WidgetIndex < RoutingPath.Widgets.Num();
		}

		void Next()
		{
			++WidgetIndex;
		}

		FWidgetAndPointer GetWidget() const
		{
			return FWidgetAndPointer(RoutingPath.Widgets[WidgetIndex], RoutingPath.GetVirtualPointerPosition(WidgetIndex));
		}
		
		const FWidgetPath& GetRoutingPath() const
		{
			return RoutingPath;
		}

		const FWidgetPath* GetWidgetsUnderCursor() const
		{
			return &RoutingPath;
		}

	private:
		int32 WidgetIndex;
		const FWidgetPath& RoutingPath;
	};

	class FBubblePolicy
	{
	public:
		FBubblePolicy( const FWidgetPath& InRoutingPath )
		: WidgetIndex( InRoutingPath.Widgets.Num()-1 )
		, RoutingPath (InRoutingPath)
		{
		}

		static FName Name;

		bool ShouldKeepGoing() const
		{
			return WidgetIndex >= 0;
		}

		void Next()
		{
			--WidgetIndex;
		}

		FWidgetAndPointer GetWidget() const
		{
			return FWidgetAndPointer(RoutingPath.Widgets[WidgetIndex], RoutingPath.GetVirtualPointerPosition(WidgetIndex));
		}

		const FWidgetPath& GetRoutingPath() const
		{
			return RoutingPath;
		}

		const FWidgetPath* GetWidgetsUnderCursor() const
		{
			return &RoutingPath;
		}
	
	private:
		int32 WidgetIndex;
		const FWidgetPath& RoutingPath;
	};

	/**
	 * Route an event along a focus path (as opposed to PointerPath)
	 *
	 * Focus paths are used focus devices.(e.g. Keyboard or Game Pads)
	 * Focus paths change when the user navigates focus (e.g. Tab or
	 * Shift Tab, clicks on a focusable widget, or navigation with keyboard/game pad.)
	 */
	template< typename RoutingPolicyType, typename FuncType, typename EventType >
	static FReply RouteAlongFocusPath( FSlateApplication* ThisApplication, RoutingPolicyType RoutingPolicy, EventType KeyEventCopy, const FuncType& Lambda, ESlateDebuggingInputEvent DebuggingInputEvent)
	{
		return Route<FReply>(ThisApplication, RoutingPolicy, KeyEventCopy, Lambda, DebuggingInputEvent);
	}

	/**
	 * Route an event based on the Routing Policy.
	 */
	template< typename ReplyType, typename RoutingPolicyType, typename EventType, typename FuncType >
	static ReplyType Route( FSlateApplication* ThisApplication, RoutingPolicyType RoutingPolicy, EventType EventCopy, const FuncType& Lambda, ESlateDebuggingInputEvent DebuggingInputEvent)
	{
		ReplyType Reply = ReplyType::Unhandled();
		const FWidgetPath& RoutingPath = RoutingPolicy.GetRoutingPath();
		const FWidgetPath* WidgetsUnderCursor = RoutingPolicy.GetWidgetsUnderCursor();
		
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::FScopeRouteInputEvent Scope(DebuggingInputEvent, RoutingPolicyType::Name);
#endif

		EventCopy.SetEventPath( RoutingPath );

		for (; !Reply.IsEventHandled() && RoutingPolicy.ShouldKeepGoing(); RoutingPolicy.Next())
		{
			const FWidgetAndPointer& ArrangedWidget = RoutingPolicy.GetWidget();

			if constexpr (Translate<EventType>::TranslationNeeded())
			{
				const EventType TranslatedEvent = Translate<EventType>::PointerEvent(ArrangedWidget, EventCopy);
				Reply = Lambda(ArrangedWidget, TranslatedEvent).SetHandler(ArrangedWidget.Widget);
				ProcessReply(ThisApplication, RoutingPath, Reply, WidgetsUnderCursor, &TranslatedEvent);
			}
			else
			{
				Reply = Lambda(ArrangedWidget, EventCopy).SetHandler(ArrangedWidget.Widget);
				ProcessReply(ThisApplication, RoutingPath, Reply, WidgetsUnderCursor, &EventCopy);
			}
		}

		return Reply;
	}

	static void ProcessReply( FSlateApplication* Application, const FWidgetPath& RoutingPath, const FNoReply& Reply, const FWidgetPath* WidgetsUnderCursor, const FInputEvent* )
	{
	}

	static void ProcessReply( FSlateApplication* Application, const FWidgetPath& RoutingPath, const FCursorReply& Reply, const FWidgetPath* WidgetsUnderCursor, const FInputEvent* )
	{
	}

	static void ProcessReply( FSlateApplication* Application, const FWidgetPath& RoutingPath, const FReply& Reply, const FWidgetPath* WidgetsUnderCursor, const FInputEvent* PointerEvent )
	{
		Application->ProcessReply(RoutingPath, Reply, WidgetsUnderCursor, nullptr, PointerEvent->GetUserIndex());
	}

	static void ProcessReply( FSlateApplication* Application, const FWidgetPath& RoutingPath, const FReply& Reply, const FWidgetPath* WidgetsUnderCursor, const FPointerEvent* PointerEvent )
	{
		Application->ProcessReply(RoutingPath, Reply, WidgetsUnderCursor, PointerEvent, PointerEvent->GetUserIndex());
	}

	template<typename EventType>
	struct Translate
	{
		static constexpr bool TranslationNeeded() { return false; }
		static EventType PointerEvent( const FWidgetAndPointer& InPosition, const EventType& InEvent )
		{
			// Most events do not do any coordinate translation.
			return InEvent;
		}
	};

};

FName FEventRouter::FDirectPolicy::Name = "Direct";
FName FEventRouter::FToLeafmostPolicy::Name = "ToLeafmost";
FName FEventRouter::FTunnelPolicy::Name = "Tunnel";
FName FEventRouter::FBubblePolicy::Name = "Bubble";

template<>
struct FEventRouter::Translate<FPointerEvent>
{
	static constexpr bool TranslationNeeded() { return true; }
	static  FPointerEvent PointerEvent( const FWidgetAndPointer& InPosition, const FPointerEvent& InEvent )
	{
		// Pointer events are translated into the virtual window space. For 3D Widget Components this means
		if ( !InPosition.GetPointerPosition().IsSet() )
		{
			return InEvent;
		}
		else
		{
			return FPointerEvent::MakeTranslatedEvent<FPointerEvent>( InEvent, InPosition.GetPointerPosition().GetValue() );
		}
	}
};

//////////////////////////////////////////////////////////////////////////

struct FScopedNavigationTransition
{
	FScopedNavigationTransition(EUINavigation NavigationType, TSharedPtr<FSlateUser> User)
		: NavigationType(NavigationType)
		, User(User)
	{
		OldFocusedWidget = GetFocusedWidget();
	}

	~FScopedNavigationTransition()
	{
		NewFocusedWidget = GetFocusedWidget();

		ExecuteNavigationTransition(OldFocusedWidget, ENavigationTransitionDirection::Outgoing);
		ExecuteNavigationTransition(NewFocusedWidget, ENavigationTransitionDirection::Incoming);

		// Clear any pending navigation context - valid only for this navigation event.
		if (User.IsValid())
		{
			FSlateApplication::Get().ClearPendingNavigationContext(User->GetUserIndex());
		}
	}

	TWeakPtr<SWidget> GetFocusedWidget()
	{
		if (User.IsValid() && User->GetWeakFocusPath().IsValid())
		{
			return User->GetWeakFocusPath().GetLastWidget().Pin();
		}
		return nullptr;
	}

	void ExecuteNavigationTransition(TWeakPtr<SWidget> Widget, const ENavigationTransitionDirection& Direction)
	{
		// No transition occurred.
		if (OldFocusedWidget == NewFocusedWidget)
		{
			return;
		}
		// Notify widget hierarchy of navigation event
		while(Widget.IsValid())
		{
			TSharedPtr<SWidget> PinnedWidget = Widget.Pin();
			if (TSharedPtr<FNavigationTransitionMetadata> Metadata = PinnedWidget->GetMetaData<FNavigationTransitionMetadata>())
			{
				const FNavigationTransition Transition(NavigationType, OldFocusedWidget, NewFocusedWidget, Direction);
				Metadata->OnNavigationTransition.ExecuteIfBound(Transition);
			}
			Widget = PinnedWidget->GetParentWidget();
		}
	}

	EUINavigation NavigationType;
	TSharedPtr<FSlateUser> User;
	TWeakPtr<SWidget> OldFocusedWidget; 
	TWeakPtr<SWidget> NewFocusedWidget; 
};


DECLARE_CYCLE_STAT( TEXT("Message Tick Time"), STAT_SlateMessageTick, STATGROUP_Slate );
DECLARE_CYCLE_STAT( TEXT("Slate App Input"), STAT_SlateApplicationInput, STATGROUP_Slate );
DECLARE_CYCLE_STAT( TEXT("Total Slate Tick Time"), STAT_SlateTickTime, STATGROUP_Slate );
DECLARE_CYCLE_STAT( TEXT("Draw Window And Children Time"), STAT_SlateDrawWindowTime, STATGROUP_Slate );
DECLARE_CYCLE_STAT( TEXT("TickRegisteredWidgets"), STAT_SlateTickRegisteredWidgets, STATGROUP_Slate );
DECLARE_CYCLE_STAT( TEXT("PreTickEvent"), STAT_SlatePreTickEvent, STATGROUP_Slate );

DECLARE_CYCLE_STAT(TEXT("ShowVirtualKeyboard"), STAT_ShowVirtualKeyboard, STATGROUP_Slate);

DECLARE_CYCLE_STAT(TEXT("ProcessKeyDown"), STAT_ProcessKeyDown, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessKeyUp"), STAT_ProcessKeyUp, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessKeyChar"), STAT_ProcessKeyChar, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessKeyChar (route focus)"), STAT_ProcessKeyChar_RouteAlongFocusPath, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessKeyChar (call OnKeyChar)"), STAT_ProcessKeyChar_Call_OnKeyChar, STATGROUP_Slate);

DECLARE_CYCLE_STAT(TEXT("ProcessAnalogInput"), STAT_ProcessAnalogInput, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessMouseButtonDown"), STAT_ProcessMouseButtonDown, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessMouseButtonDoubleClick"), STAT_ProcessMouseButtonDoubleClick, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessMouseButtonUp"), STAT_ProcessMouseButtonUp, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessMouseWheelGesture"), STAT_ProcessMouseWheelGesture, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("ProcessMouseMove"), STAT_ProcessMouseMove, STATGROUP_Slate);


namespace SlateDefs
{
	// How far tool tips should be offset from the mouse cursor position, in pixels
	static const FVector2f ToolTipOffsetFromMouse( 12.0f, 8.0f );

	// How far tool tips should be pushed out from a force field border, in pixels
	static const FVector2f ToolTipOffsetFromForceField( 4.0f, 3.0f );

	// Empty set of Touch Key
	static TSet<FKey> EmptyTouchKeySet;
}


/** True if we should allow throttling based on mouse movement activity.  int32 instead of bool only for console variable system. */
TAutoConsoleVariable<int32> ThrottleWhenMouseIsMoving( 
	TEXT( "Slate.ThrottleWhenMouseIsMoving" ),
	false,
	TEXT( "Whether to attempt to increase UI responsiveness based on mouse cursor movement." ) );

/** Minimum sustained average frame rate required before we consider the editor to be "responsive" for a smooth UI experience */
TAutoConsoleVariable<int32> TargetFrameRateForResponsiveness(
	TEXT( "Slate.TargetFrameRateForResponsiveness" ),
	35,	// Frames per second
	TEXT( "Minimum sustained average frame rate required before we consider the editor to be \"responsive\" for a smooth UI experience" ) );

/** Whether Slate should go to sleep when there are no active timers and the user is idle */
TAutoConsoleVariable<int32> AllowSlateToSleep(
	TEXT("Slate.AllowSlateToSleep"),
	GIsEditor, // Default to on for editor and standalone programs, off for games
	TEXT("Whether Slate should go to sleep when there are no active timers and the user is idle"));

/** The amount of time that must pass without any user action before Slate is put to sleep (provided that there are no active timers). */
TAutoConsoleVariable<float> SleepBufferPostInput(
	TEXT("Slate.SleepBufferPostInput"),
	0.0f,
	TEXT("The amount of time that must pass without any user action before Slate is put to sleep (provided that there are no active timers)."));

static bool bRequireFocusForGamepadInput = false;
FAutoConsoleVariableRef CVarRequireFocusForGamepadInput(
	TEXT("Slate.RequireFocusForGamepadInput"),
	bRequireFocusForGamepadInput,
	TEXT("Whether gamepad input should be ignored by the engine if the application is not currently active")
);

static bool TransformFullscreenMouseInput = true;
FAutoConsoleVariableRef CVarSlateTransformFullscreenMouseInput(
	TEXT("Slate.Transform.FullscreenMouseInput"),
	TransformFullscreenMouseInput,
	TEXT("Set true to transform mouse input to account for viewport stretching at fullscreen resolutions not natively supported by the monitor.")
);

#if PLATFORM_UI_NEEDS_TOOLTIPS
static bool bEnableTooltips = true;
#else
static bool bEnableTooltips = false;
#endif
FAutoConsoleVariableRef CVarEnableTooltips(
	TEXT("Slate.EnableTooltips"),
	bEnableTooltips,
	TEXT("Whether to allow tooltips to spawn at all."));

#if !UE_BUILD_SHIPPING

static void HandleGlobalInvalidateCVarTriggered(const TArray<FString>& Args)
{
	FSlateApplication::Get().InvalidateAllWidgets(true);
}

static FAutoConsoleCommand GlobalInvalidateCommand(
	TEXT("Slate.TriggerInvalidate"),
	TEXT("Triggers a global invalidate of all widgets"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&HandleGlobalInvalidateCVarTriggered)
);
#endif

//////////////////////////////////////////////////////////////////////////

FDelegateHandle FPopupSupport::RegisterClickNotification( const TSharedRef<SWidget>& NotifyWhenClickedOutsideMe, const FOnClickedOutside& InNotification )
{
	// If the subscriber or a zone object is destroyed, the subscription is
	// no longer active. Clean it up here so that consumers of this API have an
	// easy time with resource management.
	struct { void operator()( TArray<FClickSubscriber>& Notifications ) {
		for ( int32 SubscriberIndex=0; SubscriberIndex < Notifications.Num(); )
		{
			if ( !Notifications[SubscriberIndex].ShouldKeep() )
			{
				Notifications.RemoveAtSwap(SubscriberIndex);
			}
			else
			{
				SubscriberIndex++;
			}		
		}
	}} ClearOutStaleNotifications;
	
	ClearOutStaleNotifications( ClickZoneNotifications );

	// Add a new notification.
	ClickZoneNotifications.Add( FClickSubscriber( NotifyWhenClickedOutsideMe, InNotification ) );

	return ClickZoneNotifications.Last().Notification.GetHandle();
}

void FPopupSupport::UnregisterClickNotification( FDelegateHandle Handle )
{
	for (int32 SubscriptionIndex=0; SubscriptionIndex < ClickZoneNotifications.Num();)
	{
		if (ClickZoneNotifications[SubscriptionIndex].Notification.GetHandle() == Handle)
		{
			ClickZoneNotifications.RemoveAtSwap(SubscriptionIndex);
		}
		else
		{
			SubscriptionIndex++;
		}
	}	
}

void FPopupSupport::SendNotifications( const FWidgetPath& WidgetsUnderCursor )
{
	struct FArrangedWidgetMatcher
	{
		FArrangedWidgetMatcher( const TSharedRef<SWidget>& InWidgetToMatch )
		: WidgetToMatch( InWidgetToMatch )
		{}

		bool operator()(const FArrangedWidget& Candidate) const
		{
			return WidgetToMatch == Candidate.Widget;
		}

		const TSharedRef<SWidget>& WidgetToMatch;
	};

	// For each subscription, if the widget in question is not being clicked, send the notification.
	// i.e. Notifications are saying "some widget outside you was clicked".
	for (int32 SubscriberIndex=0; SubscriberIndex < ClickZoneNotifications.Num(); ++SubscriberIndex)
	{
		FClickSubscriber& Subscriber = ClickZoneNotifications[SubscriberIndex];
		if (Subscriber.DetectClicksOutsideMe.IsValid())
		{
			// Did we click outside the region in this subscription? If so send the notification.
			FArrangedWidgetMatcher Matcher(Subscriber.DetectClicksOutsideMe.Pin().ToSharedRef());
			const bool bClickedOutsideOfWidget = WidgetsUnderCursor.Widgets.GetInternalArray().IndexOfByPredicate(Matcher) == INDEX_NONE;
			if ( bClickedOutsideOfWidget )
			{
				Subscriber.Notification.ExecuteIfBound();
			}
		}
	}
}

void FSlateApplication::SetPlatformApplication(const TSharedRef<class GenericApplication>& InPlatformApplication)
{
	PlatformApplication->SetMessageHandler(MakeShareable(new FGenericApplicationMessageHandler()));
#if WITH_ACCESSIBILITY
	PlatformApplication->SetAccessibleMessageHandler(MakeShareable(new FGenericAccessibleMessageHandler()));
#endif

	PlatformApplication = InPlatformApplication;
	PlatformApplication->SetMessageHandler(CurrentApplication.ToSharedRef());
#if WITH_ACCESSIBILITY
	PlatformApplication->SetAccessibleMessageHandler(CurrentApplication->GetAccessibleMessageHandler());
#endif
}

void FSlateApplication::OverridePlatformApplication(TSharedPtr<class GenericApplication> InPlatformApplication)
{
	PlatformApplication = InPlatformApplication;

	if (PlatformApplication && PlatformApplication->Cursor)
	{
		// Update the Slate user who was using the platform application cursor so
		// they are now using the overridden platform application cursor.
		GetCursorUser()->OverrideCursor(PlatformApplication->Cursor);
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "OverridePlatformApplication: Cannot override cursor with invalid value");
	}
}

void FSlateApplication::Create()
{
	GSlateFastWidgetPath = GIsEditor ? false : true;

	Create(MakeShareable(FPlatformApplicationMisc::CreateApplication()));
}

TSharedRef<FSlateApplication> FSlateApplication::Create(const TSharedRef<class GenericApplication>& InPlatformApplication)
{
	EKeys::Initialize();

	InitializeCoreStyle();

	
	// Note: Important to establish the static PlatformApplication property first, as the FSlateApplication ctor relies on it
	PlatformApplication = InPlatformApplication;

	CurrentApplication = MakeShareable( new FSlateApplication() );
	CurrentBaseApplication = CurrentApplication;

	UE_TRACE_SLATE_APPLICATION_REGISTER_TRACE_EVENTS(*CurrentApplication);

	PlatformApplication->SetMessageHandler( CurrentApplication.ToSharedRef() );
#if WITH_ACCESSIBILITY
	PlatformApplication->SetAccessibleMessageHandler(CurrentApplication->GetAccessibleMessageHandler());
#endif

	// The grid needs to know the size and coordinate system of the desktop.
	// Some monitor setups have a primary monitor on the right and below the
	// left one, so the leftmost upper right monitor can be something like (-1280, -200)Synt
	{
		// Get an initial value for the VirtualDesktop geometry
		CurrentApplication->VirtualDesktopRect = []()
		{
			FDisplayMetrics DisplayMetrics;
			FSlateApplicationBase::Get().GetDisplayMetrics(DisplayMetrics);
			const FPlatformRect& VirtualDisplayRect = DisplayMetrics.VirtualDisplayRect;
			return FSlateRect(VirtualDisplayRect.Left, VirtualDisplayRect.Top, VirtualDisplayRect.Right, VirtualDisplayRect.Bottom);
		}();

		// Sign up for updates from the OS. Polling this every frame is too expensive on at least some OSs.
		PlatformApplication->OnDisplayMetricsChanged().AddSP(CurrentApplication.ToSharedRef(), &FSlateApplication::OnVirtualDesktopSizeChanged);
	}

	FAsyncTaskNotificationFactory::Get().RegisterFactory(TEXT("Slate"), []() -> FAsyncTaskNotificationFactory::FImplPointerType { return MakeShared<FSlateAsyncTaskNotificationImpl>(); });

	return CurrentApplication.ToSharedRef();
}

void FSlateApplication::Shutdown(bool bShutdownPlatform)
{
	if (FSlateApplication::IsInitialized())
	{
		CurrentApplication->OnPreShutdown().Broadcast();

		FAsyncTaskNotificationFactory::Get().UnregisterFactory(TEXT("Slate"));
		
		PlatformApplication->SetMessageHandler(MakeShareable(new FGenericApplicationMessageHandler()));
#if WITH_ACCESSIBILITY
		PlatformApplication->SetAccessibleMessageHandler(MakeShareable(new FGenericAccessibleMessageHandler()));
#endif

		CurrentApplication->OnShutdown();
		CurrentApplication->DestroyRenderer();
		CurrentApplication->Renderer.Reset();

		if (bShutdownPlatform)
		{
			PlatformApplication->DestroyApplication();
		}

		PlatformApplication.Reset();
		CurrentApplication.Reset();
		CurrentBaseApplication.Reset();
	}
}

TSharedPtr<FSlateApplication> FSlateApplication::CurrentApplication = nullptr;
double FSlateApplication::FixedDeltaTime = 1 / 60.0;

FSlateApplication::FSlateApplication()
	: bAppIsActive(true)
	, bSlateWindowActive(true)
	, Scale( 1.0f )
	, DragTriggerDistance( 0 )
	, CursorRadius( 0.0f )
	, DrawId(0)
	, LastUserInteractionTime( 0.0 )
	, LastUserInteractionTimeForThrottling( 0.0 )
	, LastMouseMoveTime( 0.0 )
	, SlateSoundDevice( MakeShareable(new FNullSlateSoundDevice()) )
	, CurrentTime( FPlatformTime::Seconds() )
	, LastTickTime( 0.0 )
	, AverageDeltaTime( 1.0f / 30.0f )	// Prime the running average with a typical frame rate so it doesn't have to spin up from zero
	, AverageDeltaTimeForResponsiveness( 1.0f / 30.0f )
	, OnExitRequested()
	, NumExternalModalWindowsActive( 0 )
	, RootStyleNode(nullptr)
	, bRequestLeaveDebugMode( false )
	, bLeaveDebugForSingleStep( false )
	, bIsExternalUIOpened( false )
	, bIsFakingTouch(FParse::Param(FCommandLine::Get(), TEXT("simmobile")) || FParse::Param(FCommandLine::Get(), TEXT("faketouches")))
	, bIsGameFakingTouch( false )
	, bIsFakingTouched( false )
	, bAllowFakingTouch( true )
	, bHandleDeviceInputWhenApplicationNotActive(false)
	, bTouchFallbackToMouse( true )
	, bSoftwareCursorAvailable( false )	
	, bMenuAnimationsEnabled( false )
	, AppIcon(nullptr)
	, VirtualDesktopRect( 0,0,0,0 )
	, NavigationConfig(MakeShared<FNavigationConfig>())
#if WITH_EDITOR
	, EditorNavigationConfig(MakeShared<FNavigationConfig>())
#endif
	, SimulateGestures(false, (int32)EGestureEvent::Count)
	, ProcessingInput(0)
	, InputManager(MakeShared<FSlateDefaultInputMapping>())
{
#if WITH_UNREAL_DEVELOPER_TOOLS
	FModuleManager::Get().LoadModule(TEXT("Settings"));
#endif

	// If we are embedded inside another app then we never need to be "active"
	bAppIsActive = !GUELibraryOverrideSettings.bIsEmbedded;

	SetupPhysicalSensitivities();

	if (GConfig)
	{
		GConfig->GetBool(TEXT("MobileSlateUI"), TEXT("bTouchFallbackToMouse"), bTouchFallbackToMouse, GEngineIni);
		GConfig->GetBool(TEXT("CursorControl"), TEXT("bAllowSoftwareCursor"), bSoftwareCursorAvailable, GEngineIni);
	}

	bRenderOffScreen = FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen"));

	// causes InputCore to initialize, even if statically linked
	FInputCoreModule& InputCore = FModuleManager::LoadModuleChecked<FInputCoreModule>(TEXT("InputCore"));

	FGenericCommands::Register();
	FTabCommands::Register();

	NormalExecutionGetter.BindRaw( this, &FSlateApplication::IsNormalExecution );

	// Add the standard 'default' user (there's always guaranteed to be at least one)
	// The default cursor platform user id the primary platform user
	ensure(SlateAppPrimaryPlatformUser.IsValid() && SlateAppPrimaryPlatformUser == IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser());
	RegisterNewUser(SlateAppPrimaryPlatformUser);

	NavigationConfig->OnRegister();
#if WITH_EDITOR
	EditorNavigationConfig->bIgnoreModifiersForNavigationActions = false;
	EditorNavigationConfig->OnRegister();
#endif

	SimulateGestures[(int32)EGestureEvent::LongPress] = true;

#if WITH_EDITOR
	FCoreDelegates::OnSafeFrameChangedEvent.AddRaw(this, &FSlateApplication::SwapSafeZoneTypes);
	OnDebugSafeZoneChanged.AddRaw(this, &FSlateApplication::UpdateCustomSafeZone);

	MenuStack.OnMenuDestroyedEvent().AddRaw(this, &FSlateApplication::OnMenuDestroyed);
#endif

	IConsoleVariable* CVarGlobalInvalidation = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.EnableGlobalInvalidation"));
	if (CVarGlobalInvalidation)
	{
		CVarGlobalInvalidation->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Variable)
		{
			UE_TRACE_SLATE_BOOKMARK(TEXT("GlobalInvalidationChanged"));
			OnGlobalInvalidationToggledEvent.Broadcast(GSlateEnableGlobalInvalidation);
		}));
	}
}

FSlateApplication::~FSlateApplication()
{
	FTabCommands::Unregister();
	FGenericCommands::Unregister();

#if WITH_EDITOR
	OnDebugSafeZoneChanged.RemoveAll(this);
	MenuStack.OnMenuDestroyedEvent().RemoveAll(this);
#endif

	IConsoleVariable* CVarGlobalInvalidation = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.EnableGlobalInvalidation"));
	if (CVarGlobalInvalidation)
	{
		CVarGlobalInvalidation->SetOnChangedCallback(FConsoleVariableDelegate());
	}

}

void FSlateApplication::SetupPhysicalSensitivities()
{
	const float DragTriggerDistanceInInches = FUnitConversion::Convert(1.0f, EUnit::Millimeters, EUnit::Inches);
	FPlatformApplicationMisc::ConvertInchesToPixels(DragTriggerDistanceInInches, DragTriggerDistance);

	// TODO Rather than allow people to request the DragTriggerDistance directly, we should
	// probably store a drag trigger distance for touch and mouse, and force users to pass
	// the pointer event they're checking for, and if that pointer event is touch, use touch
	// and if not touch, return mouse.
#if PLATFORM_DESKTOP
	DragTriggerDistance = FMath::Max(DragTriggerDistance, 5.0f);
#else
	DragTriggerDistance = FMath::Max(DragTriggerDistance, 10.0f);
#endif

	FGestureDetector::LongPressAllowedMovement = DragTriggerDistance;
}

void FSlateApplication::InitHighDPI(const bool bForceEnable)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("EnableHighDPIAwareness"));

	if (CVar)
	{
		bool bRequestEnableHighDPI = true;
		if (GIsEditor)
		{
			GConfig->GetBool(TEXT("HDPI"), TEXT("EnableHighDPIAwareness"), bRequestEnableHighDPI, GEditorSettingsIni);
		}
		else
		{
			GConfig->GetBool(TEXT("/Script/Engine.UserInterfaceSettings"), TEXT("bAllowHighDPIInGameMode"), bRequestEnableHighDPI, GEngineIni);
		}

		bool bEnableHighDPI = bRequestEnableHighDPI && !FParse::Param(FCommandLine::Get(), TEXT("nohighdpi"));
		bEnableHighDPI |= bForceEnable;

		// Set the cvar here for other systems that need it.
		CVar->Set(bEnableHighDPI);

		// High DPI must be enabled before any windows are shown.
		if (bEnableHighDPI)
		{
			FPlatformApplicationMisc::SetHighDPIMode();

			// If the Slate application already exists, its display metrics cache was
			// populated before DPI awareness was set and needs to be updated.
			if (FSlateApplication::IsInitialized())
			{
				FDisplayMetrics DisplayMetrics;
				FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
				FSlateApplication::Get().OnVirtualDesktopSizeChanged(DisplayMetrics);
			}
		}
	}
}

const FStyleNode* FSlateApplication::GetRootStyle() const
{
	return RootStyleNode;
}

bool FSlateApplication::InitializeRenderer( TSharedRef<FSlateRenderer> InRenderer, bool bQuietMode )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSlateApplication::InitializeRenderer);
	Renderer = InRenderer;
	bool bResult = Renderer->Initialize();
	if (!bResult && !bQuietMode)
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *NSLOCTEXT("SlateD3DRenderer", "ProblemWithGraphicsCard", "There is a problem with your graphics card. Please ensure your card meets the minimum system requirements and that you have the latest drivers installed.").ToString(), *NSLOCTEXT("SlateD3DRenderer", "UnsupportedVideoCardErrorTitle", "Unsupported Graphics Card").ToString());
	}
	return bResult;
}

void FSlateApplication::InitializeSound( const TSharedRef<ISlateSoundDevice>& InSlateSoundDevice )
{
	SlateSoundDevice = InSlateSoundDevice;
}

void FSlateApplication::DestroyRenderer()
{
	if( Renderer.IsValid() )
	{
		Renderer->Destroy();
	}
}

/**
 * Called when the user closes the outermost frame (i.e. quitting the app). Uses standard UE global variable
 * so normal UE applications work as expected
 */
static void OnRequestExit()
{
	RequestEngineExit(TEXT("Normal Slate Window Closed"));
}

void FSlateApplication::PlaySound( const FSlateSound& SoundToPlay, int32 UserIndex ) const
{
	SlateSoundDevice->PlaySound(SoundToPlay, UserIndex);
}

float FSlateApplication::GetSoundDuration(const FSlateSound& Sound) const
{
	return SlateSoundDevice->GetSoundDuration(Sound);
}

UE::Slate::FDeprecateVector2DResult FSlateApplication::GetCursorPos() const
{
	return GetCursorUser()->GetCursorPosition();
}

UE::Slate::FDeprecateVector2DResult FSlateApplication::GetLastCursorPos() const
{
	return GetCursorUser()->GetPreviousCursorPosition();
}

void FSlateApplication::SetCursorPos(const FVector2D& MouseCoordinate)
{
	GetCursorUser()->SetCursorPosition(UE::Slate::CastToVector2f(MouseCoordinate));
	if (PlatformApplication->Cursor.IsValid())
	{
		LastPlatformCursorPosition = UE::Slate::CastToVector2f(PlatformApplication->Cursor->GetPosition());
	}
}

void FSlateApplication::OverridePlatformTextField(TUniquePtr<IPlatformTextField> PlatformTextField)
{
	SlateTextField = MoveTemp(PlatformTextField);
}

void FSlateApplication::UsePlatformCursorForCursorUser(bool bUsePlatformCursor)
{
	if (TSharedPtr<FSlateUser> SlateUser = GetUser(CursorUserIndex))
	{
		const bool bIsUsingPlatformCursor = SlateUser->GetCursor() == PlatformApplication->Cursor;

		if (bIsUsingPlatformCursor != bUsePlatformCursor)
		{
			if (PlatformApplication && PlatformApplication->Cursor)
			{
				PlatformMouseMovementEvents = 0;
				SlateUser->OverrideCursor(bUsePlatformCursor ? PlatformApplication->Cursor : MakeShared<FFauxSlateCursor>());
				UE_LOGF(LogSlate, Log, "User[%d] UsePlatformCursorForCursorUser(%ls)", SlateUser->GetUserIndex(), bUsePlatformCursor ? TEXT("true") : TEXT("false"));
			}
		}
	}
}

void FSlateApplication::SetPlatformCursorVisibility(bool bNewVisibility)
{
	if (PlatformApplication && PlatformApplication->Cursor)
	{
		PlatformApplication->Cursor->SetType(bNewVisibility ? EMouseCursor::Default : EMouseCursor::None);
	}
}

FWidgetPath FSlateApplication::LocateWindowUnderMouse( UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate, const TArray< TSharedRef< SWindow > >& Windows, bool bIgnoreEnabledStatus, int32 UserIndex)
{
	// First, give the OS a chance to tell us which window to use, in case a child window is not guaranteed to stay on top of its parent window
	TSharedPtr<FGenericWindow> NativeWindowUnderMouse = PlatformApplication->GetWindowUnderCursor();
	if (NativeWindowUnderMouse.IsValid())
	{
		TSharedPtr<SWindow> Window = FSlateWindowHelper::FindWindowByPlatformWindow(Windows, NativeWindowUnderMouse.ToSharedRef());
		if (Window.IsValid())
		{
			return LocateWidgetInWindow(ScreenspaceMouseCoordinate, Window.ToSharedRef(), bIgnoreEnabledStatus, UserIndex);
		}
	}

	bool bPrevWindowWasModal = false;

	for (int32 WindowIndex = Windows.Num() - 1; WindowIndex >= 0; --WindowIndex)
	{ 
		const TSharedRef<SWindow>& Window = Windows[WindowIndex];

		if ( !Window->IsVisible() || Window->IsWindowMinimized())
		{
			continue;
		}
		
		// Hittest the window's children first.
		FWidgetPath ResultingPath = LocateWindowUnderMouse(ScreenspaceMouseCoordinate, Window->GetChildWindows(), bIgnoreEnabledStatus, UserIndex);
		if (ResultingPath.IsValid())
		{
			return ResultingPath;
		}

		// If none of the children were hit, hittest the parent.

		// Only accept input if the current window accepts input and the current window is not under a modal window or an interactive tooltip

		if (!bPrevWindowWasModal)
		{
			FWidgetPath PathToLocatedWidget = LocateWidgetInWindow(ScreenspaceMouseCoordinate, Window, bIgnoreEnabledStatus, UserIndex);
			if (PathToLocatedWidget.IsValid())
			{
				return PathToLocatedWidget;
			}
		}
	}
	
	return FWidgetPath();
}

bool FSlateApplication::IsWindowHousingInteractiveTooltip(const TSharedRef<const SWindow>& WindowToTest) const
{
	for (TSharedPtr<const FSlateUser> User : Users)
	{
		if (User && User->IsWindowHousingInteractiveTooltip(WindowToTest))
		{
			return true;
		}
	}
	return false;
}

void FSlateApplication::DrawWindows()
{
	SCOPED_NAMED_EVENT_TEXT("Slate::DrawWindows", FColor::Magenta);
	PrivateDrawWindows();
}

struct FDrawWindowArgs
{
	FDrawWindowArgs( FSlateDrawBuffer& InDrawBuffer, const FWidgetPath& InWidgetsToVisualizeUnderCursor )
		: OutDrawBuffer( InDrawBuffer )
		, WidgetsToVisualizeUnderCursor(InWidgetsToVisualizeUnderCursor)
	{}

	FSlateDrawBuffer& OutDrawBuffer;
	const FWidgetPath& WidgetsToVisualizeUnderCursor;
};


void FSlateApplication::DrawWindowAndChildren( const TSharedRef<SWindow>& WindowToDraw, FDrawWindowArgs& DrawWindowArgs )
{
#if WITH_SLATE_DEBUGGING
	if (GSlateVerifyParentChildrenRelationship)
	{
		UE::Slate::Private::VerifyParentChildrenRelationship(WindowToDraw);
	}
#endif

	// Skip Draw if we are debugging that window.
	if (TSharedPtr<SWindow> CurrentDebuggingWindowPinned = CurrentDebuggingWindow.Pin())
	{
		if (CurrentDebuggingWindowPinned == WindowToDraw)
		{
			return;
		}
	}

	// On Mac, where child windows can be on screen even if their parent is hidden or minimized, we want to always draw child windows.
	// On other platforms we set bDrawChildWindows to true only if we draw the current window.
	bool bDrawChildWindows = PLATFORM_MAC;

	// Only draw visible windows or in off-screen rendering mode
	if (bRenderOffScreen || (WindowToDraw->IsVisible() && (!WindowToDraw->IsWindowMinimized() || FApp::UseVRFocus())) )
	{
		TGuardValue TmpContext(CurrentDebugContextWidget, TWeakPtr<SWidget>(WindowToDraw));
		// Switch to the appropriate world for drawing
		FScopedSwitchWorldHack SwitchWorld( WindowToDraw );

		// Draw Prep
		FSlateWindowElementList& WindowElementList = DrawWindowArgs.OutDrawBuffer.AddWindowElementList(WindowToDraw);

		// Drawing is done in window space, so null out the positions and keep the size.
		int32 MaxLayerId = 0;
		{
			{
				SCOPED_NAMED_EVENT_TEXT("Slate::DrawWindow", FColor::Magenta);

#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BeginWindow.Broadcast(WindowElementList);
#endif
				MaxLayerId = WindowToDraw->PaintWindow(
					GetCurrentTime(),
					GetDeltaTime(),
					WindowElementList,
					FWidgetStyle(),
					WindowToDraw->IsEnabled());

#if WITH_SLATE_DEBUGGING
				FSlateDebugging::EndWindow.Broadcast(WindowElementList);
#endif
			}

			// Draw windowless drag drop operations
			ForEachUser([&WindowToDraw, &WindowElementList, &MaxLayerId](FSlateUser& User) {
				User.DrawWindowlessDragDropContent(WindowToDraw, WindowElementList, MaxLayerId);
			});

			// Draw Software Cursors
			ForEachUser([&WindowToDraw, &WindowElementList, &MaxLayerId](FSlateUser& User) {
				User.DrawCursor(WindowToDraw, WindowElementList, MaxLayerId);
			});
		}

#if SLATE_HAS_WIDGET_REFLECTOR

		// The widget reflector may want to paint some additional stuff as part of the Widget introspection that it performs.
		// For example: it may draw layout rectangles for hovered widgets.
		const bool bVisualizeLayoutUnderCursor = DrawWindowArgs.WidgetsToVisualizeUnderCursor.IsValid();
		const bool bCapturingFromThisWindow = bVisualizeLayoutUnderCursor && DrawWindowArgs.WidgetsToVisualizeUnderCursor.TopLevelWindow == WindowToDraw;
		TSharedPtr<IWidgetReflector> WidgetReflector = WidgetReflectorPtr.Pin();
		if ( bCapturingFromThisWindow || (WidgetReflector.IsValid() && WidgetReflector->ReflectorNeedsToDrawIn(WindowToDraw)) )
		{
			MaxLayerId = WidgetReflector->Visualize( DrawWindowArgs.WidgetsToVisualizeUnderCursor, WindowElementList, MaxLayerId );
		}

#endif

#if WITH_SLATE_DEBUGGING
		if (GSlateVerifyWidgetLayerId)
		{
			UE::Slate::Private::VerifyWidgetLayerId(WindowToDraw);
		}
#endif

		// This window is visible, so draw its child windows as well
		bDrawChildWindows = true;
	}

	if (bDrawChildWindows)
	{
		// Draw the child windows
		const TArray< TSharedRef<SWindow> > WindowChildren = WindowToDraw->GetChildWindows();
		for (int32 ChildIndex=0; ChildIndex < WindowChildren.Num(); ++ChildIndex)
		{
			DrawWindowAndChildren( WindowChildren[ChildIndex], DrawWindowArgs );
		}
	}
}

static bool DoAnyWindowDescendantsNeedPrepass(TSharedRef<SWindow> WindowToPrepass)
{
	TArray<TSharedRef<SWindow>>& ChildWindows = WindowToPrepass->GetChildWindows();
	if (ChildWindows.IsEmpty())
	{
		return false;
	}

	const TSharedRef<SWindow>& ChildWindow = ChildWindows[0];
	if (ChildWindow->IsVisible() && !ChildWindow->IsWindowMinimized())
	{
		return true;
	}
	else
	{
		return DoAnyWindowDescendantsNeedPrepass(ChildWindow);
	}
}

static void PrepassWindowAndChildren(TSharedRef<SWindow> WindowToPrepass, const TSharedPtr<SWindow>& DebuggingWindow, TWeakPtr<SWidget>& CurrentContext)
{
	// Skip Prepass if we are debugging that window or if we are on the server.
	if (IsRunningDedicatedServer() || WindowToPrepass == DebuggingWindow)
	{
		return;
	}

	const bool bIsWindowVisible = WindowToPrepass->IsVisible() && !WindowToPrepass->IsWindowMinimized();

	if (bIsWindowVisible || DoAnyWindowDescendantsNeedPrepass(WindowToPrepass))
	{
		TGuardValue TmpContext(CurrentContext, TWeakPtr<SWidget>(WindowToPrepass));
		FScopedSwitchWorldHack SwitchWorld(WindowToPrepass);
		
		{
			WindowToPrepass->ProcessWindowInvalidation();
			WindowToPrepass->SlatePrepass(FSlateApplication::Get().GetApplicationScale() * WindowToPrepass->GetNativeWindow()->GetDPIScaleFactor());
		}

		if ( bIsWindowVisible && WindowToPrepass->IsAutosized() )
		{
			WindowToPrepass->Resize(WindowToPrepass->GetDesiredSizeDesktopPixels());
		}

		// Note: Iterate over copy since num children can change during resize above.
		TArray<TSharedRef<SWindow>, FConcurrentLinearArrayAllocator> ChildWindows(WindowToPrepass->GetChildWindows());
		for (const TSharedRef<SWindow>& ChildWindow : ChildWindows)
		{
			PrepassWindowAndChildren(ChildWindow, DebuggingWindow, CurrentContext);
		}
	}
}

void FSlateApplication::DrawPrepass( TSharedPtr<SWindow> DrawOnlyThisWindow )
{
	SCOPED_NAMED_EVENT_TEXT("Slate::Prepass", FColor::Magenta);
	CSV_SCOPED_TIMING_STAT(Slate, DrawPrePass);

	TSharedPtr<SWindow> CurrentDebuggingWindowPinned = CurrentDebuggingWindow.Pin();

	if (TSharedPtr<SWindow> ActiveModalWindow = GetActiveModalWindow())
	{
		PrepassWindowAndChildren(ActiveModalWindow.ToSharedRef(), CurrentDebuggingWindowPinned, CurrentDebugContextWidget);

		for (TArray< TSharedRef<SWindow> >::TConstIterator CurrentWindowIt(SlateWindows); CurrentWindowIt; ++CurrentWindowIt)
		{
			const TSharedRef<SWindow>& CurrentWindow = *CurrentWindowIt;
			if (CurrentWindow->IsTopmostWindow())
			{
				PrepassWindowAndChildren(CurrentWindow, CurrentDebuggingWindowPinned, CurrentDebugContextWidget);
			}
		}

		TArray< TSharedRef<SWindow> > NotificationWindows;
		FSlateNotificationManager::Get().GetWindows(NotificationWindows);
		for (auto CurrentWindowIt(NotificationWindows.CreateIterator()); CurrentWindowIt; ++CurrentWindowIt)
		{
			PrepassWindowAndChildren(*CurrentWindowIt, CurrentDebuggingWindowPinned, CurrentDebugContextWidget);
		}
	}
	else if (DrawOnlyThisWindow.IsValid())
	{
		PrepassWindowAndChildren(DrawOnlyThisWindow.ToSharedRef(), CurrentDebuggingWindowPinned, CurrentDebugContextWidget);
	}
	else
	{
		// Draw all windows
		for (const TSharedRef<SWindow>& CurrentWindow : SlateWindows)
		{
			PrepassWindowAndChildren(CurrentWindow, CurrentDebuggingWindowPinned, CurrentDebugContextWidget);
		}
	}
}

TArray<SWindow*> GatherAllDescendants(const TArray< TSharedRef<SWindow> >& InWindowList)
{
	TArray<SWindow*> GatheredDescendants;
	GatheredDescendants.Reserve(InWindowList.Num());
	for (const TSharedRef<SWindow>& Window : InWindowList)
	{
		GatheredDescendants.Add(&Window.Get());
	}

	for (const TSharedRef<SWindow>& SomeWindow : InWindowList)
	{
		GatheredDescendants.Append(GatherAllDescendants(SomeWindow->GetChildWindows()));
	}

	return GatheredDescendants;
}

void FSlateApplication::PrivateDrawWindows( TSharedPtr<SWindow> DrawOnlyThisWindow )
{
	if (GSlateSkipWidgetDrawingInHeadlessMode && !FApp::CanEverRender())
	{
		// early out, as window "drawing" can take 1-2ms of a -nullrhi PC game
		return;
	}

	check(Renderer.IsValid());

	// Grab a scope lock around access to the resource proxy map, just to ensure we never cross over
	// with the loading thread.
	FScopeLock ScopeLock(Renderer->GetResourceCriticalSection());

	FWidgetPath WidgetsToVisualizeUnderCursor;
	
#if SLATE_HAS_WIDGET_REFLECTOR
	// Is user expecting visual feedback from the Widget Reflector?
	if (WidgetReflectorPtr.IsValid() && WidgetReflectorPtr.Pin()->IsVisualizingLayoutUnderCursor())
	{
		WidgetsToVisualizeUnderCursor = GetCursorUser()->GetLastWidgetsUnderCursor().ToWidgetPath();
	}
#endif

	// Prepass the window
	DrawPrepass( DrawOnlyThisWindow );

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::ResetInvalidationPanelCount();
#endif

	{
		FSlateRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *Renderer };
		FDrawWindowArgs DrawWindowArgs( ScopedDrawBuffer.GetDrawBuffer(), WidgetsToVisualizeUnderCursor);
		ensureMsgf(DrawWindowArgs.OutDrawBuffer.IsLocked(), TEXT("The buffer should be lock by GetDrawBuffer."));

		{
			SCOPE_CYCLE_COUNTER( STAT_SlateDrawWindowTime );

			TSharedPtr<SWindow> ActiveModalWindow = GetActiveModalWindow(); 

			if (ActiveModalWindow.IsValid())
			{
				DrawWindowAndChildren( ActiveModalWindow.ToSharedRef(), DrawWindowArgs );

				for( TArray< TSharedRef<SWindow> >::TConstIterator CurrentWindowIt( SlateWindows ); CurrentWindowIt; ++CurrentWindowIt )
				{
					const TSharedRef<SWindow>& CurrentWindow = *CurrentWindowIt;
					if ( CurrentWindow->GetType() == EWindowType::ToolTip )
					{
						DrawWindowAndChildren(CurrentWindow, DrawWindowArgs);
					}
				}

				TArray< TSharedRef<SWindow> > NotificationWindows;
				FSlateNotificationManager::Get().GetWindows(NotificationWindows);
				for( auto CurrentWindowIt( NotificationWindows.CreateIterator() ); CurrentWindowIt; ++CurrentWindowIt )
				{
					DrawWindowAndChildren(*CurrentWindowIt, DrawWindowArgs);
				}	
			}
			else if( DrawOnlyThisWindow.IsValid() )
			{
				DrawWindowAndChildren( DrawOnlyThisWindow.ToSharedRef(), DrawWindowArgs );
			}
			else
			{
				// Draw all windows
				// Use of an old-style iterator is intentional here, as SlateWindows 
				// array may be mutated by user logic in draw calls. The iterator 
				// prevents us from reading off the end and only keeps an index 
				// internally:
				for( TArray< TSharedRef<SWindow> >::TConstIterator CurrentWindowIt( SlateWindows ); CurrentWindowIt; ++CurrentWindowIt )
				{
					TSharedRef<SWindow> CurrentWindow = *CurrentWindowIt;
					// Only draw visible windows or in off-screen rendering mode
					if (bRenderOffScreen || CurrentWindow->IsVisible() )
					{
						DrawWindowAndChildren( CurrentWindow, DrawWindowArgs );
					}
				}
			}
		}

		// This is potentially dangerous on the movie playback thread that slate sometimes runs on
		if(!IsInSlateThread())
		{
			// Some windows may have been destroyed/removed.
			// Do not attempt to draw any windows that have been removed.
			TArray<SWindow*> AllWindows = GatherAllDescendants(SlateWindows);
			DrawWindowArgs.OutDrawBuffer.RemoveUnusedWindowElement(AllWindows);
		}

		Renderer->DrawWindows( DrawWindowArgs.OutDrawBuffer );
	}
}

void FSlateApplication::PollGameDeviceState()
{
	if( ActiveModalWindows.Num() == 0 && !GIntraFrameDebuggingGameThread && (!bRequireFocusForGamepadInput || IsActive()))
	{
		// Don't poll when a modal window open or intra frame debugging is happening
		PlatformApplication->PollGameDeviceState( GetDeltaTime() );
	}
}

void FSlateApplication::FinishedInputThisFrame()
{
	const float DeltaTime = GetDeltaTime();

	PlatformApplication->FinishedInputThisFrame();
	
	// Any preprocessors are given a chance to process accumulated values (or do whatever other tick things they want)
	// after we've finished processing all of the input for the frame
	if (PlatformApplication->Cursor.IsValid())
	{
		InputPreProcessors.Tick(DeltaTime, *this, PlatformApplication->Cursor.ToSharedRef());
	}

	// Any widgets that may have received pointer input events are given a chance to process accumulated values.
	ForEachUser([](FSlateUser& User) {
		if (User.HasAnyCapture())
		{
			for (const TSharedRef<SWidget>& Captor : User.GetCaptorWidgets())
			{
				Captor->OnFinishedPointerInput();
			}
		}
		else
		{
			for (const auto& IndexPathPair : User.GetWidgetsUnderPointerLastEventByIndex())
			{
				for (const TWeakPtr<SWidget>& WidgetPtr : IndexPathPair.Value.Widgets)
				{
					if (TSharedPtr<SWidget> Widget = WidgetPtr.Pin())
					{
						Widget->OnFinishedPointerInput();
					}
					else
					{
						break;
					}
				}
			}
		}
	});
	
	// Any widgets that may have received key events are given a chance to process accumulated values.
	ForEachUser([] (FSlateUser& User) {
		const FWeakWidgetPath& WidgetPath = User.GetWeakFocusPath();
		for ( const TWeakPtr<SWidget>& WidgetPtr : WidgetPath.Widgets )
		{
			if (TSharedPtr<SWidget> Widget = WidgetPtr.Pin())
			{
				Widget->OnFinishedKeyInput();
			}
			else
			{
				break;
			}
		}
	});

	ForEachUser([] (FSlateUser& User) { User.FinishFrame(); });
}

static const TCHAR* LexToString(ESlateTickType TickType)
{
	switch (TickType)
	{
	case ESlateTickType::Time:
		return TEXT("Time");
	case ESlateTickType::PlatformAndInput:
		return TEXT("Platform and Input");
	case ESlateTickType::Widgets:
		return TEXT("Widgets");
	case ESlateTickType::TimeAndWidgets:
		return TEXT("Time and Widgets");
	case ESlateTickType::All:
	default:
		return TEXT("All");
	}
}

void FSlateApplication::Tick(ESlateTickType TickType)
{
	LLM_SCOPE_BYTAG(UI_Slate);

	SCOPE_TIME_GUARD(TEXT("FSlateApplication::Tick"));
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UI);

	// It is not valid to tick Slate on any other thread but the game thread unless we are only updating time
	check(IsInGameThread() || TickType == ESlateTickType::Time);

	FScopeLock SlateTickAccess(&SlateTickCriticalSection);

	TGuardValue<bool> IsTickingGuard(bIsTicking, true);

	SCOPED_NAMED_EVENT_F(TEXT("Slate::Tick (%s)"), FColor::Magenta, LexToString(TickType));

	{
		SCOPE_CYCLE_COUNTER(STAT_SlateTickTime);

		float DeltaTime = GetDeltaTime();

		// IMPORTANT
		// Do not add code to these different if-statements, if you need to add additional logic to
		// ticking the platform, do it inside of TickPlatform, for example.  These functions are sometimes
		// called directly inside of Slate Application, so unless they're embedded in those calls, they wont
		// get run.

		if (EnumHasAnyFlags(TickType, ESlateTickType::PlatformAndInput))
		{
			TickPlatform(DeltaTime);
		}

		if (EnumHasAnyFlags(TickType, ESlateTickType::Time))
		{
			TickTime();
		}

		if (GSlateUseFixedDeltaTime)
		{
			DeltaTime = GetFixedDeltaTime();

		}

		if (EnumHasAnyFlags(TickType, ESlateTickType::Widgets))
		{
			TickAndDrawWidgets(DeltaTime);
		}
	}
}

bool FSlateApplication::IsTicking() const
{
	return bIsTicking;
}

void FSlateApplication::TickTime()
{
	LastTickTime = CurrentTime;
	CurrentTime = FPlatformTime::Seconds();

	// Handle large quantums
	const double MaxQuantumBeforeClamp = 1.0 / 8.0;		// 8 FPS
	if (GetDeltaTime() > MaxQuantumBeforeClamp)
	{
		LastTickTime = CurrentTime - MaxQuantumBeforeClamp;
	}
}

void FSlateApplication::TickPlatform(float DeltaTime)
{
	SCOPED_NAMED_EVENT_TEXT("Slate::TickPlatform", FColor::Magenta);
	CSV_SCOPED_TIMING_STAT(Slate, TickPlatform);

#if WITH_ACCESSIBILITY
	{
		// We ensure to only call this in TickType::All to avoid the movie thread also calling this unnecessarily 
		GetAccessibleMessageHandler()->ProcessAccessibleTasks();
	}
#endif

	{
		SCOPE_CYCLE_COUNTER(STAT_SlateMessageTick);

		// We need to pump messages here so that slate can receive input.  
		if ( ( ActiveModalWindows.Num() > 0 ) || GIntraFrameDebuggingGameThread )
		{
			// We only need to pump messages for slate when a modal window or blocking mode is active is up because normally message pumping is handled in FEngineLoop::Tick
			PlatformApplication->PumpMessages(DeltaTime);

			if ( FCoreDelegates::StarvedGameLoop.IsBound() )
			{
				FCoreDelegates::StarvedGameLoop.Execute();
			}
		}

		PlatformApplication->Tick(DeltaTime);
		PlatformApplication->ProcessDeferredEvents(DeltaTime);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_SlateApplicationInput);

		const bool bCanSpawnNewTooltip = PlatformApplication->IsCursorDirectlyOverSlateWindow();
		ForEachUser([this, bCanSpawnNewTooltip](FSlateUser& User) {
			User.UpdateCursor();
			User.UpdateTooltip(MenuStack, bCanSpawnNewTooltip);
		});

		bool bSynthesizedCursorMoveThisFrame = false;
		ForEachUser([&bSynthesizedCursorMoveThisFrame](FSlateUser& User) {
			bSynthesizedCursorMoveThisFrame |= User.SynthesizeCursorMoveIfNeeded();
		});
		bSynthesizedCursorMove = bSynthesizedCursorMoveThisFrame;

		// Generate any simulated gestures that we've detected.
		// Skip the software gesture detector when native platform gesture recognizers are active.
		if (!FGenericPlatformApplicationMisc::AreNativeGesturesEnabled())
		{
			ForEachUser([this](FSlateUser& User) {
				User.GetGestureDetector().GenerateGestures(*this, SimulateGestures);
			});
		}
	}
}

void FSlateApplication::TickAndDrawWidgets(float DeltaTime)
{
	if (Renderer.IsValid())
	{
		// Release any temporary material or texture resources we may have cached and are reporting to prevent
		// GC on those resources.  We don't need to force it, we just need to let the ones used last frame to
		// be queued up to be released.
		Renderer->ReleaseAccessedResources(/* Flush State */ false);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_SlatePreTickEvent);
		PreTickEvent.Broadcast(DeltaTime);
	}

	// Update average time between ticks.  This is used to monitor how responsive the application "feels".
	// Note that we calculate this before we apply the max quantum clamping below, because we want to store
	// the actual frame rate, even if it is very low.
	{
		// Scalar percent of new delta time that contributes to running average.  Use a lower value to add more smoothing
		// to the average frame rate.  A value of 1.0 will disable smoothing.
		const float RunningAverageScale = 0.1f;

		AverageDeltaTime = AverageDeltaTime * ( 1.0f - RunningAverageScale ) + GetDeltaTime() * RunningAverageScale;

		// Don't update average delta time if we're in an exceptional situation, such as when throttling mode
		// is active, because the measured tick time will not be representative of the application's performance.
		// In these cases, the cached average delta time from before the throttle activated will be used until
		// throttling has finished.
		if( FSlateThrottleManager::Get().IsAllowingExpensiveTasks() )
		{
			// Clamp to avoid including huge hitchy frames in our average
			const float ClampedDeltaTime = FMath::Clamp( GetDeltaTime(), 0.0f, 1.0f );
			AverageDeltaTimeForResponsiveness = AverageDeltaTimeForResponsiveness * ( 1.0f - RunningAverageScale ) + ClampedDeltaTime * RunningAverageScale;
		}
	}

	{	
		// Update auto-throttling based on elapsed time since user interaction
		ThrottleApplicationBasedOnMouseMovement();

		TSharedPtr<SWindow> ActiveModalWindow = GetActiveModalWindow();

		const float SleepThreshold = SleepBufferPostInput.GetValueOnGameThread();
		const double TimeSinceInput = LastTickTime - LastUserInteractionTime;
		const double TimeSinceMouseMove = LastTickTime - LastMouseMoveTime;
	
		const bool bIsUserIdle = (TimeSinceInput > SleepThreshold) && (TimeSinceMouseMove > SleepThreshold);
		UpdateAnyActiveTimersArePending();

		// skip tick/draw if we are idle and there are no active timers registered that we need to drive slate for.
		// This effectively means the slate application is totally idle and we don't need to update the UI.
		// This relies on Widgets properly registering for Active timer when they need something to happen even
		// when the user is not providing any input (ie, animations, viewport rendering, async polling, etc).
		bIsSlateAsleep = true;
		if	(!AllowSlateToSleep.GetValueOnGameThread() || bAnyActiveTimersPending || !bIsUserIdle || bSynthesizedCursorMove || FApp::UseVRFocus())
		{
			if (!bSynthesizedCursorMove)
			{
				ForEachUser([](FSlateUser& User) { User.QueueSyntheticCursorMove(); });
			}

			bIsSlateAsleep = false; // if we get here, then Slate is not sleeping

			// Update any notifications - this needs to be done after windows have updated themselves 
			// (so they know their size)
			{
				FSlateNotificationManager::Get().Tick();
			}

			// Draw all windows
			DrawWindows();

#if WITH_ACCESSIBILITY
			AccessibleMessageHandler->Tick();
#endif
		}
	}

	{
		// SCOPE_CYCLE_COUNTER(STAT_SlatePostTickEvent);
		PostTickEvent.Broadcast(DeltaTime);
	}

#if WITH_ACCESSIBILITY
	{
		// we call this again to improve the responsiveness of accessibility navigation and announcements 
		GetAccessibleMessageHandler()->ProcessAccessibleTasks();
	}
#endif

	++DrawId;

	UE_TRACE_SLATE_APPLICATION_TICK_AND_DRAW_WIDGETS(GetDeltaTime());
}

void FSlateApplication::PumpMessages()
{
	PlatformApplication->PumpMessages( GetDeltaTime() );
}

void FSlateApplication::ThrottleApplicationBasedOnMouseMovement()
{
	bool bShouldThrottle = false;
	if( ThrottleWhenMouseIsMoving.GetValueOnGameThread() )	// Interpreted as bool here
	{
		// We only want to engage the throttle for a short amount of time after the mouse stops moving
		const float TimeToThrottleAfterMouseStops = 0.1f;

		// After a key or mouse button is pressed, we'll leave the throttle disengaged for awhile so the
		// user can use the keys to navigate in a viewport, for example.
		const double MinTimeSinceButtonPressToThrottle = 1.0;

		// Use a small movement threshold to avoid engaging the throttle when the user bumps the mouse
		const float MinMouseMovePixelsBeforeThrottle = 2.0f;

		const FVector2f& CursorPos = GetCursorPos();
		static FVector2f LastCursorPos = GetCursorPos();
		//static double LastMouseMoveTime = FPlatformTime::Seconds();
		static bool bIsMouseMoving = false;
		if( CursorPos != LastCursorPos )
		{
			// Did the cursor move far enough that we care?
			if( bIsMouseMoving || ( CursorPos - LastCursorPos ).SizeSquared() >= MinMouseMovePixelsBeforeThrottle * MinMouseMovePixelsBeforeThrottle )
			{
				bIsMouseMoving = true;
				LastMouseMoveTime = this->GetCurrentTime();
				LastCursorPos = CursorPos;
			}
		}

		const double TimeSinceLastUserInteraction = CurrentTime - LastUserInteractionTimeForThrottling;
		const double TimeSinceLastMouseMove = CurrentTime - LastMouseMoveTime;
		if( TimeSinceLastMouseMove < TimeToThrottleAfterMouseStops )
		{
			// Only throttle if a Slate window is currently active.
			if( this->GetActiveTopLevelWindow().IsValid() )
			{
				// Only throttle if the user hasn't pressed a button in awhile
				if( TimeSinceLastUserInteraction > MinTimeSinceButtonPressToThrottle )
				{
					// If a widget has the mouse captured, then we won't bother throttling
					if( !HasAnyMouseCaptor() )
					{
						// If there is no Slate window under the mouse, then we won't engage throttling
						if( LocateWindowUnderMouse( GetCursorPos(), GetInteractiveTopLevelWindows() ).IsValid() )
						{
							bShouldThrottle = true;
						}
					}
				}
			}
		}
		else
		{
			// Mouse hasn't moved in a bit, so reset our movement state
			bIsMouseMoving = false;
			LastCursorPos = CursorPos;
		}
	}

	if( bShouldThrottle )
	{
		if( !UserInteractionResponsivnessThrottle.IsValid() )
		{
			// Engage throttling
			UserInteractionResponsivnessThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
		}
	}
	else
	{
		if( UserInteractionResponsivnessThrottle.IsValid() )
		{
			// Disengage throttling
			FSlateThrottleManager::Get().LeaveResponsiveMode( UserInteractionResponsivnessThrottle );
		}
	}
}

FWidgetPath FSlateApplication::LocateWidgetInWindow(UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate, const TSharedRef<SWindow>& Window, bool bIgnoreEnabledStatus, int32 UserIndex) const
{
	const bool bAcceptsInput = Window->IsVisible() && (Window->AcceptsInput() || IsWindowHousingInteractiveTooltip(Window));
	if (bAcceptsInput && Window->IsScreenspaceMouseWithin(ScreenspaceMouseCoordinate))
	{
		FVector2f CursorPosition = ScreenspaceMouseCoordinate;

		if (TransformFullscreenMouseInput && !GIsEditor && Window->GetWindowMode() == EWindowMode::Fullscreen)
		{
			// Screen space mapping scales everything. When window resolution doesn't match platform resolution, 
			// this causes offset cursor hit-tests in fullscreen. Correct in slate since we are first window-aware slate processor.
			const FVector2f WindowSize = Window->GetSizeInScreen();
			const FVector2f DisplaySize = CachedDisplayMetrics.MonitorInfo.IsEmpty()
				? FVector2f(CachedDisplayMetrics.PrimaryDisplayWidth, CachedDisplayMetrics.PrimaryDisplayHeight)
				: Window->GetFullScreenInfo().GetSize2f();

			const FVector2f PositionInScreen = Window->GetPositionInScreen();
			CursorPosition -= PositionInScreen;
			CursorPosition *= WindowSize / DisplaySize;
			CursorPosition += PositionInScreen;
		}

		TArray<FWidgetAndPointer> WidgetsAndCursors = Window->GetHittestGrid().GetBubblePath(CursorPosition, GetCursorRadius(), bIgnoreEnabledStatus, UserIndex);
		return FWidgetPath(MoveTemp(WidgetsAndCursors));
	}
	else
	{
		return FWidgetPath();
	}
}


TSharedRef<SWindow> FSlateApplication::AddWindow( TSharedRef<SWindow> InSlateWindow, const bool bShowImmediately )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSlateApplication::AddWindow);
	// Add the Slate window to the Slate application's top-level window array.  Note that neither the Slate window
	// or the native window are ready to be used yet, however we need to make sure they're in the Slate window
	// array so that we can properly respond to OS window messages as soon as they're sent.  For example, a window
	// activation message may be sent by the OS as soon as the window is shown (in the Init function), and if we
	// don't add the Slate window to our window list, we wouldn't be able to route that message to the window.

	FSlateWindowHelper::ArrangeWindowToFront(SlateWindows, InSlateWindow);
	TSharedRef<FGenericWindow> NewWindow = MakeWindow( InSlateWindow, bShowImmediately );

	if( bShowImmediately )
	{
		InSlateWindow->ShowWindow();

		//@todo Slate: Potentially dangerous and annoying if all slate windows that are created steal focus.
		if( InSlateWindow->SupportsKeyboardFocus() && InSlateWindow->IsFocusedInitially() )
		{
			InSlateWindow->GetNativeWindow()->SetWindowFocus();
		}
	}

	return InSlateWindow;
}

TSharedRef< FGenericWindow > FSlateApplication::MakeWindow( TSharedRef<SWindow> InSlateWindow, const bool bShowImmediately )
{
	// When rendering off-screen without the null platform, don't render to screen. Create a dummy generic window instead
	if (bRenderOffScreen && !FNullPlatformApplicationMisc::IsUsingNullApplication())
	{
		TSharedRef< FGenericWindow > NewWindow = MakeShareable(new FGenericWindow());
		InSlateWindow->SetNativeWindow(NewWindow);
		FSlateApplicationBase::Get().GetRenderer()->CreateViewport(InSlateWindow);
		return NewWindow;
	}

	TSharedPtr<FGenericWindow> NativeParent = nullptr;
	TSharedPtr<SWindow> ParentWindow = InSlateWindow->GetParentWindow();
	if ( ParentWindow.IsValid() )
	{
		NativeParent = ParentWindow->GetNativeWindow();
	}

	TSharedRef< FGenericWindowDefinition > Definition = MakeShareable( new FGenericWindowDefinition() );

	Definition->Type = InSlateWindow->GetType();

	const FVector2f Size = InSlateWindow->GetInitialDesiredSizeInScreen();
	Definition->WidthDesiredOnScreen = Size.X;
	Definition->HeightDesiredOnScreen = Size.Y;

	const FVector2f Position = InSlateWindow->GetInitialDesiredPositionInScreen();
	Definition->XDesiredPositionOnScreen = Position.X;
	Definition->YDesiredPositionOnScreen = Position.Y;

	Definition->HasOSWindowBorder = InSlateWindow->HasOSWindowBorder();
	Definition->TransparencySupport = InSlateWindow->GetTransparencySupport();
	Definition->AppearsInTaskbar = InSlateWindow->AppearsInTaskbar();
	Definition->IsTopmostWindow = InSlateWindow->IsTopmostWindow();
	Definition->AcceptsInput = InSlateWindow->AcceptsInput();
	Definition->ActivationPolicy = InSlateWindow->ActivationPolicy();
	Definition->FocusWhenFirstShown = InSlateWindow->IsFocusedInitially();

	Definition->HasCloseButton = InSlateWindow->HasCloseBox();
	Definition->SupportsMinimize = InSlateWindow->HasMinimizeBox();
	Definition->SupportsMaximize = InSlateWindow->HasMaximizeBox();

	Definition->IsModalWindow = InSlateWindow->IsModalWindow();
	Definition->IsRegularWindow = InSlateWindow->IsRegularWindow();
	Definition->HasSizingFrame = InSlateWindow->HasSizingFrame();
	Definition->SizeWillChangeOften = InSlateWindow->SizeWillChangeOften();
	Definition->ShouldPreserveAspectRatio = InSlateWindow->ShouldPreserveAspectRatio();
	Definition->ExpectedMaxWidth = InSlateWindow->GetExpectedMaxWidth();
	Definition->ExpectedMaxHeight = InSlateWindow->GetExpectedMaxHeight();

	Definition->Title = InSlateWindow->GetTitle().ToString();
	Definition->Opacity = InSlateWindow->GetOpacity();
	Definition->CornerRadius = InSlateWindow->GetCornerRadius();

	Definition->SizeLimits = InSlateWindow->GetSizeLimits();

	Definition->bManualDPI = InSlateWindow->IsManualManageDPIChanges();

	TSharedRef< FGenericWindow > NewWindow = PlatformApplication->MakeWindow();

	if (LIKELY(FApp::CanEverRender()))
	{
		InSlateWindow->SetNativeWindow(NewWindow);

		InSlateWindow->SetCachedScreenPosition( Position );
		InSlateWindow->SetCachedSize( Size );

		PlatformApplication->InitializeWindow( NewWindow, Definition, NativeParent, bShowImmediately );

		ITextInputMethodSystem* const TextInputMethodSystem = PlatformApplication->GetTextInputMethodSystem();
		if ( TextInputMethodSystem )
		{
			TextInputMethodSystem->ApplyDefaults( NewWindow );
		}
	}
	else
	{
		InSlateWindow->SetNativeWindow(MakeShareable(new FGenericWindow()));
	}

	return NewWindow;
}

bool FSlateApplication::CanAddModalWindow() const
{
	// A modal window cannot be opened until the renderer has been created.
	return CanDisplayWindows();
}

bool FSlateApplication::CanDisplayWindows() const
{
	// The renderer must be created and global shaders be available
	return Renderer.IsValid() && Renderer->AreShadersInitialized();
}

#if WITH_EDITOR
static bool IsFocusInViewport(const TSet<TWeakPtr<SViewport>> Viewports, const FWeakWidgetPath& FocusPath)
{
	if (Viewports.Num() > 0)
	{
		for (const TWeakPtr<SWidget>& FocusWidget : FocusPath.Widgets)
		{
			for (const TWeakPtr<SViewport>& Viewport : Viewports)
			{
				if (FocusWidget == Viewport)
				{
					return true;
				}
			}
		}
	}
	return false;
}
#endif

EUINavigation FSlateApplication::GetNavigationDirectionFromKey(const FKeyEvent& InKeyEvent) const
{
	TSharedRef<FNavigationConfig> RelevantNavConfig = GetRelevantNavConfig(InKeyEvent.GetUserIndex());
	return RelevantNavConfig->GetNavigationDirectionFromKey(InKeyEvent);
}

EUINavigation FSlateApplication::GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent)
{
	TSharedRef<FNavigationConfig> RelevantNavConfig = GetRelevantNavConfig(InAnalogEvent.GetUserIndex());
	return RelevantNavConfig->GetNavigationDirectionFromAnalog(InAnalogEvent);
}

EUINavigationAction FSlateApplication::GetNavigationActionFromKey(const FKeyEvent& InKeyEvent) const
{
	TSharedRef<FNavigationConfig> RelevantNavConfig = GetRelevantNavConfig(InKeyEvent.GetUserIndex());
	return RelevantNavConfig->GetNavigationActionFromKey(InKeyEvent);
}

EUINavigationAction FSlateApplication::GetNavigationActionForKey(const FKey& InKey) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Not enough info to pick the best config, so this can only use the default
	return NavigationConfig->GetNavigationActionForKey(InKey);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSlateApplication::AddModalWindow( TSharedRef<SWindow> InSlateWindow, const TSharedPtr<const SWidget> InParentWidget, bool bSlowTaskWindow )
{
	if( !CanAddModalWindow() )
	{
		// Bail out.  The incoming window will never be added, and no native window will be created.
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSlateApplication::AddModalWindow);

	if( GIsRunningUnattendedScript && !bSlowTaskWindow )
	{
		UE_LOGF(LogSlate, Warning, "A modal window tried to take control while running in unattended script mode. The window was canceled.");
		if (FPlatformMisc::IsDebuggerPresent())
		{
			UE_DEBUG_BREAK();
		}
		else
		{
			FDebug::DumpStackTraceToLog(ELogVerbosity::Error);
		}
		return;
	}

#if WITH_EDITOR
	FCoreDelegates::PreSlateModalWithContext.Broadcast(
		FCoreDelegates::FModalWindowContext{ 
			.bIsSlowTaskWindow = bSlowTaskWindow,
			.WindowIdentifier = (std::intptr_t)(&InSlateWindow.Get()) });
PRAGMA_DISABLE_DEPRECATION_WARNINGS
    FCoreDelegates::PreSlateModal.Broadcast();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
    // Push the active modal window onto the stack.
	ActiveModalWindows.AddUnique( InSlateWindow );

	// Close all the open tooltips when a new window is opened.  
	// Tooltips from non-modal windows can be dangerous and re-enter into code that shouldn't execute in a modal state.
	ForEachUser([](FSlateUser& User) {
		User.CloseTooltip();
	});

	// Set the modal flag on the window
	InSlateWindow->SetAsModalWindow();
	
	// Make sure we aren't in the middle of using a slate draw buffer
	Renderer->FlushCommands();

	// In slow task windows, depending on the frequency with which the window is updated, it could be quite some time 
	// before the window is ticked (and drawn) so we hide the window by default and the slow task window will show it when needed
	const bool bShowWindow = !bSlowTaskWindow;

	// Create the new window
	// Note: generally a modal window should not be added without a parent but 
	// due to this being called from wxWidget editors, this is not always possible
	if( InParentWidget.IsValid() )
	{
		// Find the window of the parent widget
		FWidgetPath WidgetPath;
		if (GeneratePathToWidgetUnchecked( InParentWidget.ToSharedRef(), WidgetPath ))
		{
			AddWindowAsNativeChild( InSlateWindow, WidgetPath.GetWindow(), bShowWindow );
		}
		else
		{
			UE_LOGF(LogSlate, Warning, "Modal Window fail to open as a native child. The path to the parent widget (%ls) could not be found", *InParentWidget->ToString());
			AddWindow( InSlateWindow, bShowWindow );
		}
	}
	else
	{
		AddWindow( InSlateWindow, bShowWindow );
	}

	if ( ActiveModalWindows.Num() == 1 )
	{
		// Signal that a slate modal window has opened so external windows may be disabled as well
		ModalWindowStackStartedDelegate.ExecuteIfBound();
	}

	// Release mouse capture here in case the new modal window has been added in one of the mouse button
	// event callbacks. Otherwise it will be unresponsive until the next mouse up event.
	ReleaseAllPointerCapture();

	// Clear the cached pressed mouse buttons, in case a new modal window has been added between the mouse down and mouse up of another window.
	PressedMouseButtons.Empty();

	// Also force the platform capture off as the call to ReleaseMouseCapture() above still relies on mouse up messages to clear the capture
	PlatformApplication->SetCapture( nullptr );

	// Disable high precision mouse mode when a modal window is added.  On some OS'es even when a window is diabled, raw input is sent to it.
	PlatformApplication->SetHighPrecisionMouseMode( false, nullptr );

	// Block on all modal windows unless its a slow task.  In that case the game thread is allowed to run.
	if( !bSlowTaskWindow )
	{
		// Time blocked in this scope shouldn't count against detection of stalls
		SCOPE_STALL_DETECTOR_PAUSE();

		// Show the cursor if it was previously hidden so users can interact with the window
		if ( PlatformApplication->Cursor.IsValid() )
		{
			PlatformApplication->Cursor->Show( true );
		}
		
		// Since the engine tick is paused here we have to end any outstanding frames.
		// That will put us in a clean state for the Slate Tick loop below.
		Renderer->EndFrame();

		//Throttle loop data
		double LastLoopTime = FPlatformTime::Seconds();
		const double MinThrottlePeriod = (1.0 / 60.0); //Throttle the loop to a maximum of 60Hz

		// Tick slate from here in the event that we should not return until the modal window is closed.
		while( InSlateWindow == GetActiveModalWindow() )
		{
			//Throttle the loop
			const double CurrentLoopTime = FPlatformTime::Seconds();
			const float SleepTime = static_cast<float>(MinThrottlePeriod - (CurrentLoopTime-LastLoopTime));
			LastLoopTime = CurrentLoopTime;
			if (SleepTime > 0.0f)
			{
				// Sleep a bit to not eat up all CPU time
				FPlatformProcess::Sleep(SleepTime);
			}

			const float DeltaTime = GetDeltaTime();

			// Tick any other systems that need to update during modal dialogs
			ModalLoopTickEvent.Broadcast(DeltaTime);

			{
				SCOPE_CYCLE_COUNTER(STAT_SlateTickTime);

				// Tick and pump messages for the platform.
				TickPlatform(DeltaTime);

				// It's possible that during ticking the platform we'll find out the modal dialog was closed.
				// in which case we need to abort the current flow.
				if ( InSlateWindow != GetActiveModalWindow() )
				{
					break;
				}

				// Advance time for the application
				TickTime();

				// Tick and render Slate
				TickAndDrawWidgets(DeltaTime);
				
				Renderer->EndFrame();
			}

			// Synchronize the game thread and the render thread so that the render thread doesn't get too far behind.
			Renderer->Sync();
		}
	}
}

void FSlateApplication::SetModalWindowStackStartedDelegate(FModalWindowStackStarted StackStartedDelegate)
{
	ModalWindowStackStartedDelegate = StackStartedDelegate;
}

void FSlateApplication::SetModalWindowStackEndedDelegate(FModalWindowStackEnded StackEndedDelegate)
{
	ModalWindowStackEndedDelegate = StackEndedDelegate;
}

TSharedRef<SWindow> FSlateApplication::AddWindowAsNativeChild( TSharedRef<SWindow> InSlateWindow, TSharedRef<SWindow> InParentWindow, const bool bShowImmediately )
{
	// @VREDITOR HACK
	// Parent window must already have been added
	//checkSlow(FSlateWindowHelper::ContainsWindow(SlateWindows, InParentWindow));

	// Add the Slate window to the Slate application's top-level window array.  Note that neither the Slate window
	// or the native window are ready to be used yet, however we need to make sure they're in the Slate window
	// array so that we can properly respond to OS window messages as soon as they're sent.  For example, a window
	// activation message may be sent by the OS as soon as the window is shown (in the Init function), and if we
	// don't add the Slate window to our window list, we wouldn't be able to route that message to the window.
	InParentWindow->AddChildWindow( InSlateWindow );

	// Only make native generic windows if the parent has one. Nullrhi makes only generic windows, whose handles are always null
	if ( InParentWindow->GetNativeWindow()->GetOSWindowHandle() || !FApp::CanEverRender() || bRenderOffScreen )
	{
		TSharedRef<FGenericWindow> NewWindow = MakeWindow(InSlateWindow, bShowImmediately);

		if ( bShowImmediately )
		{
			InSlateWindow->ShowWindow();

			//@todo Slate: Potentially dangerous and annoying if all slate windows that are created steal focus.
			if ( InSlateWindow->SupportsKeyboardFocus() && InSlateWindow->IsFocusedInitially() )
			{
				InSlateWindow->GetNativeWindow()->SetWindowFocus();
			}
		}
	}

	return InSlateWindow;
}

TSharedPtr<IMenu> FSlateApplication::PushMenu(const TSharedRef<SWidget>& InParentWidget, const FWidgetPath& InOwnerPath, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize, TOptional<EPopupMethod> Method, const bool bIsCollapsedByParent, const int32 FocusUserIndex)
{
	constexpr bool bEnablePerPixelTransparency = false;
	// Caller supplied a valid path? Pass it to the menu stack.
	if (InOwnerPath.IsValid())
	{
		return MenuStack.Push(InOwnerPath, InContent, SummonLocation, TransitionEffect, bFocusImmediately, SummonLocationSize, Method, bIsCollapsedByParent, bEnablePerPixelTransparency, FocusUserIndex);
	}

	// If the caller doesn't specify a valid event path we'll generate one from InParentWidget
	FWidgetPath WidgetPath;
	if (GeneratePathToWidgetUnchecked(InParentWidget, WidgetPath))
	{
		return MenuStack.Push(WidgetPath, InContent, SummonLocation, TransitionEffect, bFocusImmediately, SummonLocationSize, Method, bIsCollapsedByParent, bEnablePerPixelTransparency, FocusUserIndex);
	}

	UE_LOGF(LogSlate, Warning, "Menu could not be pushed.  A path to the parent widget(%ls) could not be found", *InParentWidget->ToString());
	return TSharedPtr<IMenu>();
}

TSharedPtr<IMenu> FSlateApplication::PushMenu(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize, const bool bIsCollapsedByParent, const int32 FocusUserIndex)
{
	constexpr bool bEnablePerPixelTransparency = false;
	return MenuStack.Push(InParentMenu, InContent, SummonLocation, TransitionEffect, bFocusImmediately, SummonLocationSize, bIsCollapsedByParent, bEnablePerPixelTransparency, FocusUserIndex);
}

TSharedPtr<IMenu> FSlateApplication::PushHostedMenu(const TSharedRef<SWidget>& InParentWidget, const FWidgetPath& InOwnerPath, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent)
{
	// Caller supplied a valid path? Pass it to the menu stack.
	if (InOwnerPath.IsValid())
	{
		return MenuStack.PushHosted(InOwnerPath, InMenuHost, InContent, OutWrappedContent, TransitionEffect, ShouldThrottle, bIsCollapsedByParent);
	}

	// If the caller doesn't specify a valid event path we'll generate one from InParentWidget
	FWidgetPath WidgetPath;
	if (GeneratePathToWidgetUnchecked(InParentWidget, WidgetPath))
	{
		return MenuStack.PushHosted(WidgetPath, InMenuHost, InContent, OutWrappedContent, TransitionEffect, ShouldThrottle, bIsCollapsedByParent);
	}

	return TSharedPtr<IMenu>();
}

TSharedPtr<IMenu> FSlateApplication::PushHostedMenu(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent)
{
	return MenuStack.PushHosted(InParentMenu, InMenuHost, InContent, OutWrappedContent, TransitionEffect, ShouldThrottle, bIsCollapsedByParent);
}

bool FSlateApplication::HasOpenSubMenus(TSharedPtr<IMenu> InMenu) const
{
	return MenuStack.HasOpenSubMenus(InMenu);
}

bool FSlateApplication::AnyMenusVisible() const
{
	return MenuStack.HasMenus();
}

TSharedPtr<IMenu> FSlateApplication::FindMenuInWidgetPath(const FWidgetPath& InWidgetPath) const
{
	return MenuStack.FindMenuInWidgetPath(InWidgetPath);
}

TSharedPtr<SWindow> FSlateApplication::GetVisibleMenuWindow() const
{
	return MenuStack.GetHostWindow();
}

TSharedPtr<SWidget> FSlateApplication::GetMenuHostWidget() const
{
	return MenuStack.GetHostWidget();
}

void FSlateApplication::DismissAllMenus()
{
	MenuStack.DismissAll();
}

void FSlateApplication::DismissMenu(const TSharedPtr<IMenu>& InFromMenu)
{
	MenuStack.DismissFrom(InFromMenu);
}

void FSlateApplication::DismissMenuByWidget(const TSharedRef<SWidget>& InWidgetInMenu)
{
	FWidgetPath WidgetPath;
	if (GeneratePathToWidgetUnchecked(InWidgetInMenu, WidgetPath))
	{
		TSharedPtr<IMenu> Menu = MenuStack.FindMenuInWidgetPath(WidgetPath);
		if (Menu.IsValid())
		{
			MenuStack.DismissFrom(Menu);
		}
	}
}

void FSlateApplication::RequestDestroyWindow( TSharedRef<SWindow> InWindowToDestroy )
{
	ForEachUser([&InWindowToDestroy] (FSlateUser& User) {
		User.NotifyWindowDestroyed(InWindowToDestroy);
	});

	// Logging to track down window shutdown issues with movie loading threads. Too spammy in editor builds with all the windows
#if !WITH_EDITOR
	UE_LOGF(LogSlate, Log, "Request Window '%ls' being destroyed", *InWindowToDestroy->GetTitle().ToString() );
#endif
	struct local
	{
		static void Helper( const TSharedRef<SWindow> WindowToDestroy, TArray< TSharedRef<SWindow> >& OutWindowDestroyQueue)
		{
			/** @return the list of this window's child windows */
			TArray< TSharedRef<SWindow> >& ChildWindows = WindowToDestroy->GetChildWindows();

			// Children need to be destroyed first. 
			if( ChildWindows.Num() > 0 )
			{
				for( int32 ChildIndex = 0; ChildIndex < ChildWindows.Num(); ++ChildIndex )
				{	
					// Recursively request that the window is destroyed which will also queue any children of children etc...
					Helper( ChildWindows[ ChildIndex ], OutWindowDestroyQueue );
				}
			}

			OutWindowDestroyQueue.AddUnique( WindowToDestroy );
		}
	};

	local::Helper( InWindowToDestroy, WindowDestroyQueue );

	DestroyWindowsImmediately();
}

void FSlateApplication::DestroyWindowImmediately( TSharedRef<SWindow> WindowToDestroy ) 
{
	// Request that the window and its children are destroyed
	RequestDestroyWindow( WindowToDestroy );

	DestroyWindowsImmediately();
}


void FSlateApplication::ExternalModalStart()
{
	if( NumExternalModalWindowsActive++ == 0 )
	{
		// Close all open menus.
		DismissAllMenus();

		// Close tool-tips
		ForEachUser([](FSlateUser& User) {
			User.CloseTooltip();
		});

		// Tick and render Slate so that it can destroy any menu windows if necessary before we disable.
		Tick();
		Renderer->Sync();

		if( ActiveModalWindows.Num() > 0 )
		{
			// There are still modal windows so only enable the new active modal window.
			GetActiveModalWindow()->EnableWindow( false );
		}
		else
		{
			// We are creating a modal window so all other windows need to be disabled.
			for( TArray< TSharedRef<SWindow> >::TIterator CurrentWindowIt( SlateWindows ); CurrentWindowIt; ++CurrentWindowIt )
			{
				TSharedRef<SWindow> CurrentWindow = ( *CurrentWindowIt );
				CurrentWindow->EnableWindow( false );
			}
		}
	}
}


void FSlateApplication::ExternalModalStop()
{
	check(NumExternalModalWindowsActive > 0);
	if( --NumExternalModalWindowsActive == 0 )
	{
		if( ActiveModalWindows.Num() > 0 )
		{
			// There are still modal windows so only enable the new active modal window.
			GetActiveModalWindow()->EnableWindow( true );
		}
		else
		{
			// We are creating a modal window so all other windows need to be disabled.
			for( TArray< TSharedRef<SWindow> >::TIterator CurrentWindowIt( SlateWindows ); CurrentWindowIt; ++CurrentWindowIt )
			{
				TSharedRef<SWindow> CurrentWindow = ( *CurrentWindowIt );
				CurrentWindow->EnableWindow( true );
			}
		}
	}
}

void FSlateApplication::RegisterGameViewport( TSharedRef<SViewport> InViewport )
{
	RegisterViewport(InViewport);

#if WITH_EDITOR
	AllGameViewports.Add(InViewport);
#endif
	
	if (GameViewportWidget != InViewport)
	{
		InViewport->SetActive(true);
		GameViewportWidget = InViewport;
	}
	
	ActivateGameViewport();
}

void FSlateApplication::RegisterViewport(TSharedRef<SViewport> InViewport)
{
	TSharedPtr<SWindow> ParentWindow = FindWidgetWindow(InViewport);
	if (ParentWindow.IsValid())
	{
		TWeakPtr<ISlateViewport> SlateViewport = InViewport->GetViewportInterface();
		if (ensure(SlateViewport.IsValid()))
		{
			ParentWindow->SetViewport(SlateViewport.Pin().ToSharedRef());
		}
	}
}

void FSlateApplication::UnregisterGameViewport()
{
	ResetToDefaultPointerInputSettings();

	bIsFakingTouched = false;
	bIsGameFakingTouch = false;

#if WITH_EDITOR
	AllGameViewports.Empty();
#endif

	if (GameViewportWidget.IsValid())
	{
		GameViewportWidget.Pin()->SetActive(false);
	}
	GameViewportWidget.Reset();
}

void FSlateApplication::RegisterVirtualWindow(TSharedRef<SWindow> InWindow)
{
	SlateVirtualWindows.AddUnique(InWindow);
}

void FSlateApplication::UnregisterVirtualWindow(TSharedRef<SWindow> InWindow)
{
	SlateVirtualWindows.Remove(InWindow);
}

void FSlateApplication::FlushRenderState()
{
	InvalidateAllWidgets(true);

	if ( Renderer.IsValid() )
	{
		// Release any temporary material or texture resources we may have cached and are reporting to prevent
		// GC on those resources.  If the game viewport is being unregistered, we need to flush these resources
		// to allow for them to be GC'ed.
		Renderer->ReleaseAccessedResources(/* Flush State */ true);
	}
}

TSharedPtr<SViewport> FSlateApplication::GetGameViewport() const
{
	return GameViewportWidget.Pin();
}

int32 FSlateApplication::GetUserIndexForMouse() const
{
	return InputManager->GetUserIndexForMouse();
}

int32 FSlateApplication::GetUserIndexForKeyboard() const
{
	return InputManager->GetUserIndexForKeyboard();
}

FInputDeviceId FSlateApplication::GetInputDeviceIdForMouse() const
{
	return InputManager->GetInputDeviceIdForMouse();
}

FInputDeviceId FSlateApplication::GetInputDeviceIdForKeyboard() const
{
	return InputManager->GetInputDeviceIdForKeyboard();
}

TOptional<int32> FSlateApplication::GetUserIndexForController(int32 ControllerId, FKey InKey) const
{
	return InputManager->GetUserIndexForController(ControllerId, InKey);
}

int32 FSlateApplication::GetUserIndexForController(int32 ControllerId) const
{
	return InputManager->GetUserIndexForController(ControllerId);
}

TOptional<int32> FSlateApplication::GetUserIndexForInputDevice(FInputDeviceId InputDeviceId) const
{
	return InputManager->GetUserIndexForInputDevice(InputDeviceId);
}

TOptional<int32> FSlateApplication::GetUserIndexForPlatformUser(FPlatformUserId PlatformUser) const
{
	return InputManager->GetUserIndexForPlatformUser(PlatformUser);
}

void FSlateApplication::SetInputManager(TSharedRef<ISlateInputManager> InInputManager)
{
	//@todo DanH/NickD: Should we be worried at all about potential invalidation of SlateUsers we've created but that no longer occupy an index that can be referenced?
	InputManager = InInputManager;
}

void FSlateApplication::SetUserFocusToGameViewport(uint32 UserIndex, EFocusCause ReasonFocusIsChanging /* = EFocusCause::SetDirectly*/)
{
	TSharedPtr<SViewport> CurrentGameViewportWidget = GameViewportWidget.Pin();
	if (CurrentGameViewportWidget.IsValid())
	{
		SetUserFocus(UserIndex, CurrentGameViewportWidget, ReasonFocusIsChanging);
	}
}

void FSlateApplication::SetAllUserFocusToGameViewport(EFocusCause ReasonFocusIsChanging /* = EFocusCause::SetDirectly*/)
{
	TSharedPtr< SViewport > CurrentGameViewportWidget = GameViewportWidget.Pin();

	if (CurrentGameViewportWidget.IsValid())
	{
		FWidgetPath PathToWidget;
		FSlateWindowHelper::FindPathToWidget(SlateWindows, CurrentGameViewportWidget.ToSharedRef(), /*OUT*/ PathToWidget);

		SetAllUserFocus(PathToWidget, ReasonFocusIsChanging);
	}
}

void FSlateApplication::ActivateGameViewport()
{
	// Only focus the window if the application is active, if not the application activation sequence will take care of it.
	if (bAppIsActive && GameViewportWidget.IsValid())
	{
		TSharedRef<SViewport> GameViewportWidgetRef = GameViewportWidget.Pin().ToSharedRef();
		
		FWidgetPath PathToViewport;
		// If we cannot find the window it could have been destroyed.
		if (FSlateWindowHelper::FindPathToWidget(SlateWindows, GameViewportWidgetRef, PathToViewport, EVisibility::All))
		{
			TSharedRef<SWindow> Window = PathToViewport.GetWindow();

			// Set keyboard focus on the actual OS window for the top level Slate window in the viewport path
			// This is needed because some OS messages are only sent to the window with keyboard focus
			// Slate will translate the message and send it to the actual widget with focus.
			// Without this we don't get WM_KEYDOWN or WM_CHAR messages in play in viewport sessions.
			Window->GetNativeWindow()->SetWindowFocus();

			// Activate the viewport and process the reply 
			FWindowActivateEvent ActivateEvent(FWindowActivateEvent::EA_Activate, Window);
			FReply ViewportActivatedReply = GameViewportWidgetRef->OnViewportActivated(ActivateEvent);
			if (ViewportActivatedReply.IsEventHandled())
			{
				ProcessReply(PathToViewport, ViewportActivatedReply, nullptr, nullptr);
			}
		}
	}
}

bool FSlateApplication::GetTransformFullscreenMouseInput() const
{
	return TransformFullscreenMouseInput;
}

#if WITH_SLATE_DEBUGGING
void FSlateApplication::TryDumpNavigationConfig(TSharedPtr<FNavigationConfig> InNavigationConfig) const
{
	if (GSlateTraceNavigationConfig && InNavigationConfig)
	{
		UE_LOGF(LogSlate, Log, "Navigation Config Change:\n%ls", *InNavigationConfig->ToString());

		const uint32 DumpCallstackSize = 65535;
		ANSICHAR DumpCallstack[DumpCallstackSize] = { 0 };
		FString ScriptStack = FFrame::GetScriptCallstack(true /* bReturnEmpty */);
		FPlatformStackWalk::StackWalkAndDump(DumpCallstack, DumpCallstackSize, 0);
		UE_LOGF(LogSlate, Log, "--- Navigation Config Changing Callstack ---");
		UE_LOGF(LogSlate, Log, "Script Stack:\n%ls", *ScriptStack);
		UE_LOGF(LogSlate, Log, "Callstack:\n%ls", ANSI_TO_TCHAR(DumpCallstack));
	}
}
#endif // WITH_SLATE_DEBUGGING

bool FSlateApplication::SetUserFocus(uint32 UserIndex, const TSharedPtr<SWidget>& WidgetToFocus, EFocusCause ReasonFocusIsChanging /* = EFocusCause::SetDirectly*/)
{
	TSharedPtr<FSlateUser> CurrentUser = GetUser(UserIndex);
	if (ensureMsgf(WidgetToFocus.IsValid(), TEXT("Attempting to focus an invalid widget. If your intent is to clear focus use ClearUserFocus()")) && CurrentUser)
	{
		FWidgetPath PathToWidget;
		const bool bFound = FSlateWindowHelper::FindPathToWidget(SlateWindows, WidgetToFocus.ToSharedRef(), /*OUT*/ PathToWidget);
		if (bFound)
		{
			return SetUserFocus(*CurrentUser, PathToWidget, ReasonFocusIsChanging);
		}
		else
		{
			const bool bFoundVirtual = FSlateWindowHelper::FindPathToWidget(SlateVirtualWindows, WidgetToFocus.ToSharedRef(), /*OUT*/ PathToWidget);
			if (bFoundVirtual)
			{
				return SetUserFocus(*CurrentUser, PathToWidget, ReasonFocusIsChanging);
			}
		}
	}

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastWarning(NSLOCTEXT("SlateDebugging", "SetUserFocusFailed", "Attempting to focus a widget that isn't in the tree and visible. If your intent is to clear focus use ClearUserFocus()"), WidgetToFocus);
#endif

	return false;
}

void FSlateApplication::SetAllUserFocus(const TSharedPtr<SWidget>& WidgetToFocus, EFocusCause ReasonFocusIsChanging /*= EFocusCause::SetDirectly*/)
{
	const bool bValidWidget = WidgetToFocus.IsValid();
	ensureMsgf(bValidWidget, TEXT("Attempting to focus an invalid widget. If your intent is to clear focus use ClearAllUserFocus()"));
	if (bValidWidget)
	{
		FWidgetPath PathToWidget;
		const bool bFound = FSlateWindowHelper::FindPathToWidget(SlateWindows, WidgetToFocus.ToSharedRef(), /*OUT*/ PathToWidget);
		if (bFound)
		{
			SetAllUserFocus(PathToWidget, ReasonFocusIsChanging);
		}
		else
		{
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastWarning(NSLOCTEXT("SlateDebugging", "SetUserFocusFailedAll", "Attempting to focus a widget that isn't in the tree and visible. If your intent is to clear focus use ClearUserFocus()"), WidgetToFocus);
#endif
		}
	}
}

TSharedPtr<SWidget> FSlateApplication::GetUserFocusedWidget(uint32 UserIndex) const
{
	TSharedPtr<const FSlateUser> User = GetUser(UserIndex);
	return User ? User->GetFocusedWidget() : TSharedPtr<SWidget>();
}

void FSlateApplication::ClearUserFocus(uint32 UserIndex, EFocusCause ReasonFocusIsChanging /* = EFocusCause::SetDirectly*/)
{
	SetUserFocus(UserIndex, FWidgetPath(), ReasonFocusIsChanging);
}

void FSlateApplication::ClearAllUserFocus(EFocusCause ReasonFocusIsChanging /*= EFocusCause::SetDirectly*/)
{
	SetAllUserFocus(FWidgetPath(), ReasonFocusIsChanging);
}

void FSlateApplication::SetPendingNavigationContext(uint32 UserIndex, TSharedPtr<ISlateMetaData> InContext)
{
	PendingNavigationContexts.Add(UserIndex, MoveTemp(InContext));
}

void FSlateApplication::ClearPendingNavigationContext(uint32 UserIndex)
{
	PendingNavigationContexts.Remove(UserIndex);
}

bool FSlateApplication::SetKeyboardFocus(const TSharedPtr< SWidget >& OptionalWidgetToFocus, EFocusCause ReasonFocusIsChanging /* = EFocusCause::SetDirectly*/)
{
	return SetUserFocus(GetUserIndexForKeyboard(), OptionalWidgetToFocus, ReasonFocusIsChanging);
}

void FSlateApplication::ClearKeyboardFocus(const EFocusCause ReasonFocusIsChanging)
{
	SetUserFocus(GetUserIndexForKeyboard(), FWidgetPath(), ReasonFocusIsChanging);
}

TSharedPtr<SWidget> FSlateApplication::GetCurrentDebugContextWidget() const
{
	return CurrentDebugContextWidget.Pin();
}

void FSlateApplication::ResetToDefaultInputSettings()
{
	ProcessReply(FWidgetPath(), FReply::Handled().ClearUserFocus(true), nullptr, nullptr);
	ResetToDefaultPointerInputSettings();
}

void FSlateApplication::ResetToDefaultPointerInputSettings()
{
	//@todo DanH: Leaving unchanged during the FSlateUser updates, but this seems to be overkill to loop through every single captor
	//		and process a ReleaseCapture reply for each. Can't we just ReleaseAllCapture() on every user? (We'd still need to update each user's cursor type)
	//ReleaseMouseCapture();
	ForEachUser([this](FSlateUser& User) {
		for (const FWidgetPath& CaptorPath : User.GetCaptorPaths())
		{
			ProcessReply(CaptorPath, FReply::Handled().ReleaseMouseCapture(), nullptr, nullptr);
		}
		User.GetCursor()->SetType(EMouseCursor::Default);
	});

	ProcessReply(FWidgetPath(), FReply::Handled().ReleaseMouseLock(), nullptr, nullptr);
}

void* FSlateApplication::GetMouseCaptureWindow() const
{
	return PlatformApplication->GetCapture();
}

void FSlateApplication::ReleaseMouseCapture()
{
	ReleaseAllPointerCapture();
}

void FSlateApplication::ReleaseAllPointerCapture()
{
	ForEachUser([](FSlateUser& User) { User.ReleaseAllCapture(); });
}

void FSlateApplication::ReleaseAllPointerCapture(int32 UserIndex)
{
	if (TSharedPtr<FSlateUser> User = GetUser(UserIndex))
	{
		User->ReleaseAllCapture();
	}
}

void FSlateApplication::ReleaseMouseCaptureForUser(int32 UserIndex)
{
	ReleaseAllPointerCapture(UserIndex);
}

FDelegateHandle FSlateApplication::RegisterOnWindowActionNotification(const FOnWindowAction& Notification)
{
	OnWindowActionNotifications.Add(Notification);
	return OnWindowActionNotifications.Last().GetHandle();
}

void FSlateApplication::UnregisterOnWindowActionNotification(FDelegateHandle Handle)
{
	for (int32 Index = 0; Index < OnWindowActionNotifications.Num();)
	{
		if (OnWindowActionNotifications[Index].GetHandle() == Handle)
		{
			OnWindowActionNotifications.RemoveAtSwap(Index);
		}
		else
		{
			Index++;
		}
	}
}

TSharedPtr<SWindow> FSlateApplication::FindBestParentWindowForDialogs(const TSharedPtr<SWidget>& InWidget, const ESlateParentWindowSearchMethod InParentWindowSearchMethod)
{
	TSharedPtr<SWindow> ParentWindow = ( InWidget.IsValid() ) ? FindWidgetWindow(InWidget.ToSharedRef()) : TSharedPtr<SWindow>();

	if ( !ParentWindow.IsValid() )
	{
		// First check the active top level window.
		TSharedPtr<SWindow> ActiveTopWindow = GetActiveTopLevelWindow();
		if ( ActiveTopWindow.IsValid() && ActiveTopWindow->IsRegularWindow() && InParentWindowSearchMethod == ESlateParentWindowSearchMethod::ActiveWindow )
		{
			ParentWindow = ActiveTopWindow;
		}
		else
		{
			// If the active top level window isn't a good host, lets just try and find the first
			// reasonable window we can host new dialogs off of.
			for ( TSharedPtr<SWindow> SlateWindow : SlateWindows )
			{
				if ( SlateWindow->IsVisible() && SlateWindow->IsRegularWindow() )
				{
					ParentWindow = SlateWindow;
					break;
				}
			}
		}
	}

	return ParentWindow;
}

const void* FSlateApplication::FindBestParentWindowHandleForDialogs(const TSharedPtr<SWidget>& InWidget, const ESlateParentWindowSearchMethod InParentWindowSearchMethod)
{
	TSharedPtr<SWindow> ParentWindow = FindBestParentWindowForDialogs(InWidget, InParentWindowSearchMethod);

	const void* ParentWindowWindowHandle = nullptr;
	if ( ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid() )
	{
		ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
	}

	return ParentWindowWindowHandle;
}

uint32 FSlateApplication::GetCurrentDrawId() const
{
	return DrawId;
}

const TSet<FKey>& FSlateApplication::GetPressedMouseButtons() const
{
	return PressedMouseButtons;
}

TSharedPtr<SWindow> FSlateApplication::GetActiveTopLevelWindow() const
{
	return ActiveTopLevelWindow.Pin();
}

TSharedPtr<SWindow> FSlateApplication::GetActiveTopLevelRegularWindow() const
{
	TSharedPtr<SWindow> ActiveWindow = ActiveTopLevelWindow.Pin();
	while (ActiveWindow && !ActiveWindow->IsRegularWindow())
	{
		ActiveWindow = ActiveWindow->GetParentWindow();
	}

	return ActiveWindow;
}

TSharedPtr<SWindow> FSlateApplication::GetActiveModalWindow() const
{
	return (ActiveModalWindows.Num() > 0) ? ActiveModalWindows.Last() : nullptr;
}

bool FSlateApplication::SetKeyboardFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause /*= EFocusCause::SetDirectly*/)
{
	return SetUserFocus(GetUserIndexForKeyboard(), InFocusPath, InCause);
}

bool FSlateApplication::SetUserFocus(const uint32 InUserIndex, const FWidgetPath& InFocusPath, const EFocusCause InCause)
{
	TSharedPtr<FSlateUser> User = GetUser(InUserIndex);
	return User && SetUserFocus(*User, InFocusPath, InCause);
}

bool FSlateApplication::SetUserFocusAllowingDescendantFocus(const uint32 InUserIndex, const FWidgetPath& InFocusPath, const EFocusCause InCause)
{
	if (TSharedPtr<FSlateUser> User = GetUser(InUserIndex))
	{
		const TSharedRef<SWidget>& FocusWidget = InFocusPath.Widgets.Last().Widget;
		if (User->GetWeakFocusPath().ContainsWidget(&FocusWidget.Get()))
		{
			// A descendant is already focused
			return true;
		}

		return SetUserFocus(*User, InFocusPath, InCause);
	}

	return false;
}

bool FSlateApplication::SetUserFocus(FSlateUser& User, const FWidgetPath& InFocusPath, const EFocusCause InCause)
{
	if (InFocusPath.IsValid())
	{
		TSharedRef<SWindow> Window = InFocusPath.GetWindow();

		// Prevent interactions with tooltips from disrupting the current focus state and closing open menus.
		if (IsWindowHousingInteractiveTooltip(Window))
		{
			return false;
		}

		if (ActiveModalWindows.Num() != 0 && !(Window->IsDescendantOf(GetActiveModalWindow()) || ActiveModalWindows.Top() == Window))
		{
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastWarning(NSLOCTEXT("SlateDebugging", "SetUserFocusInvalidWindowFailed", "Ignoring SetUserFocus because it's not an active modal Window"), Window);
#endif
			UE_LOGF(LogSlate, Warning, "Ignoring SetUserFocus because it's not an active modal Window (user %i not set to %ls.", User.GetUserIndex(), *InFocusPath.GetLastWidget()->ToString());
			return false;
		}
	}

	TSharedPtr<IWidgetReflector> WidgetReflector = WidgetReflectorPtr.Pin();
	const bool bReflectorShowingFocus = WidgetReflector.IsValid() && WidgetReflector->IsShowingFocus();

	// Get the old Widget information
	const FWeakWidgetPath OldFocusedWidgetPath = User.GetWeakFocusPath();
	TSharedPtr<SWidget> OldFocusedWidget = OldFocusedWidgetPath.IsValid() ? OldFocusedWidgetPath.GetLastWidget().Pin() : TSharedPtr< SWidget >();

	// Get the new widget information by finding the first widget in the path that supports focus
	FWidgetPath NewFocusedWidgetPath;
	TSharedPtr<SWidget> NewFocusedWidget;

	if (InFocusPath.IsValid())
	{
		//UE_LOGF(LogSlate, Warning, "Focus for user %i seeking focus path:\n%ls", InUserIndex, *InFocusPath.ToString());

		for (int32 WidgetIndex = InFocusPath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
		{
			const FArrangedWidget& WidgetToFocus = InFocusPath.Widgets[WidgetIndex];

			// Does this widget support keyboard focus?  If so, then we'll go ahead and set it!
			if (WidgetToFocus.Widget->SupportsKeyboardFocus())
			{
				// Is we aren't changing focus then simply return
				if (WidgetToFocus.Widget == OldFocusedWidget)
				{
					//UE_LOGF(LogSlate, Warning, "--Focus Has Not Changed--");
					return false;
				}
				NewFocusedWidget = WidgetToFocus.Widget;
				NewFocusedWidgetPath = InFocusPath.GetPathDownTo(NewFocusedWidget.ToSharedRef());
				break;
			}
		}
	}

	User.IncrementFocusVersion();
	int32 CurrentFocusVersion = User.GetFocusVersion();

	FFocusEvent FocusEvent(InCause, User.GetUserIndex());

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastFocusChanging(FocusEvent, OldFocusedWidgetPath, OldFocusedWidget, NewFocusedWidgetPath, NewFocusedWidget);
#endif
	FocusChangingDelegate.Broadcast(FocusEvent, OldFocusedWidgetPath, OldFocusedWidget, NewFocusedWidgetPath, NewFocusedWidget);

	// Notify widgets in the old focus path that focus is changing
	if (OldFocusedWidgetPath.IsValid())
	{
		FScopedSwitchWorldHack SwitchWorld(OldFocusedWidgetPath.Window.Pin());

		for (int32 ChildIndex = 0; ChildIndex < OldFocusedWidgetPath.Widgets.Num(); ++ChildIndex)
		{
			TSharedPtr<SWidget> SomeWidget = OldFocusedWidgetPath.Widgets[ChildIndex].Pin();
			if (SomeWidget.IsValid())
			{
				SomeWidget->OnFocusChanging(OldFocusedWidgetPath, NewFocusedWidgetPath, FocusEvent);

				// If focus setting is interrupted, stop what we're doing, as someone has already changed the focus path.
				if ( CurrentFocusVersion != User.GetFocusVersion())
				{
					return false;
				}
			}
		}
	}

	// Notify widgets in the new focus path that focus is changing
	if (NewFocusedWidgetPath.IsValid())
	{
		FScopedSwitchWorldHack SwitchWorld(NewFocusedWidgetPath.GetWindow());

		for (int32 ChildIndex = 0; ChildIndex < NewFocusedWidgetPath.Widgets.Num(); ++ChildIndex)
		{
			TSharedPtr<SWidget> SomeWidget = NewFocusedWidgetPath.Widgets[ChildIndex].Widget;
			if (SomeWidget.IsValid())
			{
				SomeWidget->OnFocusChanging(OldFocusedWidgetPath, NewFocusedWidgetPath, FocusEvent);

				// If focus setting is interrupted, stop what we're doing, as someone has already changed the focus path.
				if ( CurrentFocusVersion != User.GetFocusVersion())
				{
					return false;
				}
			}
		}
	}

	//UE_LOGF(LogSlate, Warning, "Focus for user %i set to %ls.", User.GetUserIndex(), NewFocusedWidget.IsValid() ? *NewFocusedWidget->ToString() : TEXT("Invalid"));

	// Figure out if we should show focus for this focus entry
	bool ShowFocus = false;
	if (NewFocusedWidgetPath.IsValid())
	{
		ShowFocus = InCause == EFocusCause::Navigation;
		for (int32 WidgetIndex = NewFocusedWidgetPath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
		{
			TOptional<bool> QueryShowFocus = NewFocusedWidgetPath.Widgets[WidgetIndex].Widget->OnQueryShowFocus(InCause);
			if ( QueryShowFocus.IsSet())
			{
				ShowFocus = QueryShowFocus.GetValue();
				break;
			}
		}
	}

	// Store a weak widget path to the widget that's taking focus
	User.SetFocusPath(NewFocusedWidgetPath, InCause, ShowFocus);

	// Let the old widget know that it lost keyboard focus
	if (OldFocusedWidget.IsValid())
	{
		// Switch worlds for widgets in the old path
		FScopedSwitchWorldHack SwitchWorld(OldFocusedWidgetPath.Window.Pin());

#if PLATFORM_UI_NEEDS_FOCUS_OUTLINES
		if (OldFocusedWidget->CanSupportFocus() && OldFocusedWidget->SupportsKeyboardFocus())
		{
			OldFocusedWidget->Invalidate(EInvalidateWidgetReason::Paint);
		}
#endif
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastFocusLost(FocusEvent, OldFocusedWidgetPath, OldFocusedWidget, NewFocusedWidgetPath, NewFocusedWidget);
#endif

		// Let previously-focused widget know that it's losing focus
		OldFocusedWidget->OnFocusLost(FocusEvent);
#if WITH_ACCESSIBILITY
		GetAccessibleMessageHandler()->OnWidgetEventRaised(FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(OldFocusedWidget.ToSharedRef(), EAccessibleEvent::FocusChange, true, false, User.GetUserIndex()));
#endif
	}

#if SLATE_HAS_WIDGET_REFLECTOR
	if (bReflectorShowingFocus)
	{
		WidgetReflector->SetWidgetsToVisualize(NewFocusedWidgetPath);
	}
#endif

	// Let the new widget know that it's received keyboard focus
	if (NewFocusedWidget.IsValid())
	{
		TSharedPtr<SWindow> FocusedWindow = NewFocusedWidgetPath.GetWindow();

		// Switch worlds for widgets in the new path
		FScopedSwitchWorldHack SwitchWorld(FocusedWindow);

		// Set ActiveTopLevelWindow to the newly focused window
		ActiveTopLevelWindow = FocusedWindow;
		
		const FArrangedWidget& WidgetToFocus = NewFocusedWidgetPath.Widgets.Last();

#if PLATFORM_UI_NEEDS_FOCUS_OUTLINES
		if (NewFocusedWidget->CanSupportFocus() && NewFocusedWidget->SupportsKeyboardFocus())
		{
			NewFocusedWidget->Invalidate(EInvalidateWidgetReason::Paint);
		}
#endif
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastFocusReceived(FocusEvent, OldFocusedWidgetPath, OldFocusedWidget, NewFocusedWidgetPath, NewFocusedWidget);
#endif

		FReply Reply = NewFocusedWidget->OnFocusReceived(WidgetToFocus.Geometry, FocusEvent);
		if (Reply.IsEventHandled())
		{
			ProcessReply(InFocusPath, Reply, nullptr, nullptr, User.GetUserIndex());
		}

		GetRelevantNavConfig(User.GetUserIndex())->OnNavigationChangedFocus(OldFocusedWidget, NewFocusedWidget, FocusEvent);

#if WITH_ACCESSIBILITY
		GetAccessibleMessageHandler()->OnWidgetEventRaised(FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(NewFocusedWidget.ToSharedRef(), EAccessibleEvent::FocusChange, false, true, User.GetUserIndex()));
#endif
	}

	return true;
}


void FSlateApplication::SetAllUserFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause)
{
	ForEachUser([&] (FSlateUser& User) {
		SetUserFocus(User, InFocusPath, InCause);
	});

	// cache the focus path so it can be applied to any new users
	if (InFocusPath.IsValid())
	{
		LastAllUsersFocusWidget = InFocusPath.GetLastWidget();
	}
	else
	{
		LastAllUsersFocusWidget.Reset();
	}
	LastAllUsersFocusCause = InCause;
}

void FSlateApplication::SetAllUserFocusAllowingDescendantFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause)
{
	const TSharedRef<SWidget>& FocusWidget = InFocusPath.Widgets.Last().Widget;

	ForEachUser([&] (FSlateUser& User) {
		if (!User.GetWeakFocusPath().ContainsWidget(&FocusWidget.Get()))
		{
			SetUserFocus(User, InFocusPath, InCause);
		}
	});

	// cache the focus path so it can be applied to any new users
	LastAllUsersFocusWidget = FocusWidget;
	LastAllUsersFocusCause = InCause;
}

FModifierKeysState FSlateApplication::GetModifierKeys() const
{
	return PlatformApplication->GetModifierKeys();
}


void FSlateApplication::OnShutdown()
{
	CloseAllWindowsImmediately();
}

void FSlateApplication::CloseAllWindowsImmediately()
{
	ForEachUser([](FSlateUser& User) { 
		User.ResetDragDropContent();
		User.ResetTooltipWindow();
	});

	// Destroy all top level windows.  
	// ::RequestDestroyWindow will remove the window from the TArray SlateWindows so we
	// should iterate over it backwards to make sure that the WindowIndex is still correct.
	for (int32 WindowIndex = SlateWindows.Num() -1; WindowIndex >= 0; --WindowIndex)
	{
		// This will also request that all children of each window be destroyed
		RequestDestroyWindow(SlateWindows[WindowIndex]);
	}

	DestroyWindowsImmediately();
}

void FSlateApplication::DestroyWindowsImmediately()
{
	// Destroy any windows that were queued for deletion.

	// Thomas.Sarkanen: I've changed this from a for() to a while() loop so that it is now valid to call RequestDestroyWindow()
	// in the callstack of another call to RequestDestroyWindow(). Previously this would cause a stack overflow, as the
	// WindowDestroyQueue would be continually added to each time the for() loop ran.
	while ( WindowDestroyQueue.Num() > 0 )
	{
		TSharedRef<SWindow> CurrentWindow = WindowDestroyQueue[0];
		WindowDestroyQueue.Remove(CurrentWindow);
		if( ActiveModalWindows.Num() > 0 && ActiveModalWindows.Contains( CurrentWindow ) )
		{
			ActiveModalWindows.Remove( CurrentWindow );

			if( ActiveModalWindows.Num() > 0 )
			{
				// There are still modal windows so only enable the new active modal window.
				GetActiveModalWindow()->EnableWindow( true );
			}
			else
			{
				//  There are no modal windows so renable all slate windows
				for ( TArray< TSharedRef<SWindow> >::TConstIterator SlateWindowIter( SlateWindows ); SlateWindowIter; ++SlateWindowIter )
				{
					// All other windows need to be re-enabled BEFORE a modal window is destroyed or focus will not be set correctly
					(*SlateWindowIter)->EnableWindow( true );
				}

				// Signal that all slate modal windows are closed
				ModalWindowStackEndedDelegate.ExecuteIfBound();
			}
		}

		// Any window being destroyed should be removed from the menu stack if its in it
		MenuStack.OnWindowDestroyed(CurrentWindow);

		// Perform actual cleanup of the window
		PrivateDestroyWindow( CurrentWindow );
	}

	WindowDestroyQueue.Empty();
}


void FSlateApplication::SetExitRequestedHandler( const FSimpleDelegate& OnExitRequestedHandler )
{
	OnExitRequested = OnExitRequestedHandler;
}


bool FSlateApplication::GeneratePathToWidgetUnchecked( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter ) const
{
	if ( !FSlateWindowHelper::FindPathToWidget(SlateWindows, InWidget, OutWidgetPath, VisibilityFilter) )
	{
		return FSlateWindowHelper::FindPathToWidget(SlateVirtualWindows, InWidget, OutWidgetPath, VisibilityFilter);
	}

	return true;
}


void FSlateApplication::GeneratePathToWidgetChecked( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter ) const
{
	if ( !FSlateWindowHelper::FindPathToWidget(SlateWindows, InWidget, OutWidgetPath, VisibilityFilter) )
	{
		const bool bWasFound = FSlateWindowHelper::FindPathToWidget(SlateVirtualWindows, InWidget, OutWidgetPath, VisibilityFilter);
		check(bWasFound);
	}
}


TSharedPtr<SWindow> FSlateApplication::FindWidgetWindow( TSharedRef<const SWidget> InWidget ) const
{
	TSharedPtr<SWidget> TestWidget = ConstCastSharedRef<SWidget>(InWidget);
	while (TestWidget.IsValid())
	{
		if (TestWidget->Advanced_IsWindow())
		{
			return StaticCastSharedPtr<SWindow>(TestWidget);
		}

		TestWidget = TestWidget->GetParentWidget();
	};

	return nullptr;
}


TSharedPtr<SWindow> FSlateApplication::FindWidgetWindow( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath ) const
{
	// If the user wants a widget path back populate it instead
	if ( !FSlateWindowHelper::FindPathToWidget(SlateWindows, InWidget, OutWidgetPath, EVisibility::All) )
	{
		if ( !FSlateWindowHelper::FindPathToWidget(SlateVirtualWindows, InWidget, OutWidgetPath, EVisibility::All) )
		{
			return nullptr;
		}
	}

	return OutWidgetPath.TopLevelWindow;
}

void FSlateApplication::ProcessExternalReply(const FWidgetPath& CurrentEventPath, const FReply TheReply, const int32 UserIndex, const int32 PointerIndex)
{
	const int32 ValidatedUserIndex = (UserIndex >= 0) ?  UserIndex : 0;
	if (PointerIndex == CursorPointerIndex)
	{
		TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(ValidatedUserIndex);
		const bool bIsPrimaryUser = ValidatedUserIndex == CursorUserIndex;

		FPointerEvent MouseEvent(
			ValidatedUserIndex,
			PointerIndex,
			SlateUser->GetCursorPosition(),
			SlateUser->GetPreviousCursorPosition(),
			bIsPrimaryUser ? PressedMouseButtons : SlateDefs::EmptyTouchKeySet,
			EKeys::Invalid,
			0,
			bIsPrimaryUser ? PlatformApplication->GetModifierKeys() : FModifierKeysState()
		);

		FWidgetPath PathToWidget;
		const FWidgetPath* PathToWidgetPtr = nullptr;
		FWeakWidgetPath LastWidgetsUnderCursor = SlateUser->GetLastWidgetsUnderCursor();
		if (LastWidgetsUnderCursor.IsValid())
		{
			PathToWidget = LastWidgetsUnderCursor.ToWidgetPath();
			PathToWidgetPtr = &PathToWidget;
		}

		ProcessReply(CurrentEventPath, TheReply, PathToWidgetPtr, &MouseEvent, ValidatedUserIndex);
	}
	else
	{
		ProcessReply(CurrentEventPath, TheReply, nullptr, nullptr, ValidatedUserIndex);
	}
}

void FSlateApplication::ProcessReply( const FWidgetPath& CurrentEventPath, const FReply& TheReply, const FWidgetPath* WidgetsUnderMouse, const FPointerEvent* InMouseEvent, const uint32 UserIndex )
{
	const TSharedPtr<FDragDropOperation> ReplyDragDropContent = TheReply.GetDragDropContent();
	const bool bStartingDragDrop = ReplyDragDropContent.IsValid() && WidgetsUnderMouse && WidgetsUnderMouse->IsValid();
	const bool bIsVirtualInteraction = CurrentEventPath.IsValid() ? CurrentEventPath.GetWindow()->IsVirtualWindow() : false;

	// Release mouse capture if requested or if we are starting a drag and drop.
	// Make sure to only clobber WidgetsUnderCursor if we actually had a mouse capture.
	uint32 PointerIndex = InMouseEvent != nullptr ? InMouseEvent->GetPointerIndex() : CursorPointerIndex;

	TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(UserIndex);
 	if (SlateUser->HasCapture(PointerIndex) && (TheReply.ShouldReleaseMouse() || bStartingDragDrop))
	{
		SlateUser->ReleaseCapture(PointerIndex);
	}

	// Clear focus is requested.
	if (TheReply.ShouldReleaseUserFocus())
	{
		if (TheReply.AffectsAllUsers())
		{
			ClearAllUserFocus(TheReply.GetFocusCause());
		}
		else
		{
			SlateUser->ClearFocus(TheReply.GetFocusCause());
		}
	}

	if ((bDisableLastDragOnDragEnter && bStartingDragDrop && SlateUser->IsDragDropping()) || TheReply.ShouldEndDragDrop())
	{
		SlateUser->CancelDragDrop();
	}

	if (bStartingDragDrop)
	{
		checkf(!SlateUser->IsDragDropping(), TEXT("Drag and Drop already in progress!"));
		check(TheReply.IsEventHandled());
		check(WidgetsUnderMouse);
		check(InMouseEvent);

		FPointerEvent TransformedPointerEvent = TransformPointerEvent(*InMouseEvent, WidgetsUnderMouse->GetWindow());

		SlateUser->SetDragDropContent(ReplyDragDropContent.ToSharedRef());
		
		const FWeakWidgetPath LastWidgetsUnderCursor = SlateUser->GetLastWidgetsUnderPointer(PointerIndex);

		// We have entered drag and drop mode.
		// LastWidgetsUnderCursor.ToWidgetPath(), CurrentEventPath, and
		// *WidgetsUnderMouse should all be the same except during mouse move/
		// drag detected events, in which case, the differences are as
		// follows:
		//
		// A) LastWidgetsUnderCursor.ToWidgetPath() - The path to the widget
		//     that the mouse cursor was over on the previous frame/mouse
		//     event.
		// B) CurrentEventPath - For the DragDetected event, this is the path
		//     to the widget on which the drag was started. Since the drag
		//     operation does not activate until the mouse has been dragged
		//     a short distance, this means that this can be different from
		//     both the widget path that the cursor is currently on and the
		//     widget path that the cursor was on for the previous event.
		// C) *WidgetsUnderMouse - The widget that the mouse is currently
		//     over.
		//
		// To process the beginning of the drag operation, widgets previously
		// under the mouse cursor receive the OnMouseLeave notification,
		// regardless of whether the cursor is still over them or not.
		FEventRouter::Route<FNoReply>(this, FEventRouter::FBubblePolicy(LastWidgetsUnderCursor.ToWidgetPath()), TransformedPointerEvent, [](const FArrangedWidget& SomeWidget, const FPointerEvent& PointerEvent)
		{
			SomeWidget.Widget->OnMouseLeave( PointerEvent );
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::MouseLeave, &PointerEvent, SomeWidget.Widget);
#endif
			return FNoReply();
		}, ESlateDebuggingInputEvent::MouseLeave);

		// Then, the original widget started the drag receives OnDragEnter.
		FEventRouter::Route<FNoReply>(this, FEventRouter::FBubblePolicy(CurrentEventPath), FDragDropEvent( TransformedPointerEvent, ReplyDragDropContent ), [](const FArrangedWidget& SomeWidget, const FDragDropEvent& DragDropEvent )
		{
			SomeWidget.Widget->OnDragEnter( SomeWidget.Geometry, DragDropEvent );
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::DragEnter, &DragDropEvent, SomeWidget.Widget);
#endif
			return FNoReply();
		}, ESlateDebuggingInputEvent::DragEnter);

		// If the cursor is not currently over the widget on which the drag
		// operation started (which should only be the case due to cursor
		// movement), the remainder of events are handled in
		// RoutePointerMoveEvent(), which will immediately call OnDragLeave()
		// on the widget that started the drag followed by OnDragEnter() on
		// the current widget. Thus, using the letters above, and assuming
		// all of the widgets are different, the sequence should end up being:
		//
		// 1. B - OnDragDetected (processing reply with drag content)
		//   1.1. A - OnMouseLeave
		//   1.2. B - OnDragEnter
		// 2. B - OnDragLeave
		// 3. C - OnDragEnter
		// 4. C - OnDragOver
	}
	
	// Setting mouse capture, mouse position, and locking the mouse
	// are all operations that we shouldn't do if our application isn't Active (The OS ignores half of it, and we'd be in a half state)
	// We do allow the release of capture and lock when deactivated, this is innocuous of some platforms but required on others when 
	// the Application deactivated before the window. (Mac is an example of this)
	if (bHandleDeviceInputWhenApplicationNotActive || bAppIsActive || bIsVirtualInteraction)
	{
		TSharedPtr<SWidget> RequestedMouseCaptor = TheReply.GetMouseCaptor();

		// Do not capture the mouse if we are also starting a drag and drop.
		if (RequestedMouseCaptor.IsValid() && !bStartingDragDrop)
		{
			if (SlateUser->SetPointerCaptor(PointerIndex, RequestedMouseCaptor.ToSharedRef(), CurrentEventPath))
			{
				const FWeakWidgetPath LastWidgetsUnderCursor = SlateUser->GetLastWidgetsUnderPointer(PointerIndex);
				if (LastWidgetsUnderCursor.IsValid())
				{
					for (int32 WidgetIndex = LastWidgetsUnderCursor.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
					{
						TSharedPtr<SWidget> WidgetPreviouslyUnderCursor = LastWidgetsUnderCursor.Widgets[WidgetIndex].Pin();

						if (WidgetPreviouslyUnderCursor.IsValid())
						{
							if (WidgetPreviouslyUnderCursor != RequestedMouseCaptor)
							{
								// It's possible for mouse event to be null if we end up here from a keyboard event. If so, we should synthesize an event.
								// WidgetsUnderMouse can also be invalid if the mouse is not over a Slate widget.
								if (InMouseEvent && WidgetsUnderMouse && WidgetsUnderMouse->IsValid())
								{
									FPointerEvent TransformedPointerEvent = TransformPointerEvent(*InMouseEvent, WidgetsUnderMouse->GetWindow());

									// Note that the event's pointer position is not translated.
									WidgetPreviouslyUnderCursor->OnMouseLeave(TransformedPointerEvent);
								}
								else
								{
									const FPointerEvent& SimulatedPointer = FPointerEvent();
									WidgetPreviouslyUnderCursor->OnMouseLeave(SimulatedPointer);
								}
#if WITH_SLATE_DEBUGGING
								FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseLeave, InMouseEvent, WidgetPreviouslyUnderCursor);
#endif
							}
							else
							{
								// Done routing mouse leave
								break;
							}
						}
					}
					// Need to handle the case where the mouse has moved onto a new widget before the drag was detected by also calling MouseLeave on the newly hovered widgets
					if (WidgetsUnderMouse && WidgetsUnderMouse->IsValid() && LastWidgetsUnderCursor.Widgets.Last().Pin() != WidgetsUnderMouse->Widgets.Last().Widget)
					{
						for (int32 WidgetIndex = WidgetsUnderMouse->Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
						{
							TSharedPtr<SWidget> WidgetNowUnderCursor = WidgetsUnderMouse->Widgets[WidgetIndex].Widget;

							if (WidgetNowUnderCursor.IsValid())
							{
								if (WidgetNowUnderCursor != RequestedMouseCaptor && !LastWidgetsUnderCursor.ContainsWidget(WidgetNowUnderCursor.Get()))
								{
									// It's possible for mouse event to be null if we end up here from a keyboard event. If so, we should synthesize an event.
									if (InMouseEvent)
									{
										FPointerEvent TransformedPointerEvent = TransformPointerEvent(*InMouseEvent, WidgetsUnderMouse->GetWindow());

										// Note that the event's pointer position is not translated.
										WidgetNowUnderCursor->OnMouseLeave(TransformedPointerEvent);
									}
									else
									{
										const FPointerEvent& SimulatedPointer = FPointerEvent();
										WidgetNowUnderCursor->OnMouseLeave(SimulatedPointer);
									}
#if WITH_SLATE_DEBUGGING
									FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseLeave, InMouseEvent, WidgetNowUnderCursor);
#endif
								}
								else
								{
									// Done routing mouse leave
									break;
								}
							}
						}
					}
				}
			}
			else
			{
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastWarning(NSLOCTEXT("SlateDebugging", "FailedToCaptureMouse", "Failed To Mouse Capture"), RequestedMouseCaptor);
#endif
			}

			// When the cursor capture state changes we need to refresh cursor state.
			SlateUser->RequestCursorQuery();
		}

		if ( !bIsVirtualInteraction && CurrentEventPath.IsValid() && RequestedMouseCaptor.IsValid())
		{
			// If the mouse is being captured or released, toggle high precision raw input if requested by the reply.
			// Raw input is only used with mouse capture
			if (TheReply.ShouldUseHighPrecisionMouse() && !PlatformApplication->IsUsingHighPrecisionMouseMode())
			{
				const TSharedRef< SWindow> Window = CurrentEventPath.GetWindow();
				PlatformApplication->SetCapture(Window->GetNativeWindow());
				PlatformApplication->SetHighPrecisionMouseMode(true, Window->GetNativeWindow());

#if WITH_SLATE_DEBUGGING
				//TODO Capture
#endif

				// When the cursor capture state changes we need to refresh cursor state.
				SlateUser->RequestCursorQuery();
			}
		}

		TOptional<FIntPoint> RequestedMousePos = TheReply.GetRequestedMousePos();
		if (RequestedMousePos.IsSet())
		{
			SlateUser->SetCursorPosition(RequestedMousePos.GetValue());
		}

		if (TheReply.GetMouseLockWidget())
		{
			// The reply requested mouse lock so tell the native application to lock the mouse to the widget receiving the event
			SlateUser->LockCursor(TheReply.GetMouseLockWidget().ToSharedRef());
		}
	}

	// Releasing high precision mode.  @HACKISH We can only support high precision mode on true mouse hardware cursors
	// but if the user index isn't 0, there's no way it's the real mouse so we should ignore this if it's not user 0,
	// because that means it's a virtual controller.
	if ( UserIndex == CursorUserIndex && !bIsVirtualInteraction )
	{
		if ( CurrentEventPath.IsValid() && TheReply.ShouldReleaseMouse() && !TheReply.ShouldUseHighPrecisionMouse() )
		{
			if ( PlatformApplication->IsUsingHighPrecisionMouseMode() )
			{
				const TSharedRef< SWindow> Window = CurrentEventPath.GetWindow();
				PlatformApplication->SetHighPrecisionMouseMode(false, Window->GetNativeWindow());
				PlatformApplication->SetCapture(nullptr);

#if WITH_SLATE_DEBUGGING
				//TODO Release Capture
#endif

				// When the cursor capture state changes we need to refresh cursor state.
				SlateUser->RequestCursorQuery();
			}
		}
	}

	// Releasing Mouse Lock
	if (TheReply.ShouldReleaseMouseLock())
	{
		SlateUser->UnlockCursor();
	}
	
	// If we have a valid Navigation request attempt the navigation.
	if (TheReply.GetNavigationDestination().IsValid() || TheReply.GetNavigationType() != EUINavigation::Invalid)
	{
		FWidgetPath NavigationSource;
		if (TheReply.GetNavigationSource() == ENavigationSource::WidgetUnderCursor)
		{
			NavigationSource = *WidgetsUnderMouse;
		}
		else
		{
			NavigationSource = SlateUser->GetFocusPath().Get();
		}

		if (NavigationSource.IsValid())
		{
			if (!GSlateEnableGamepadEditorNavigation && TheReply.GetNavigationGenesis() == ENavigationGenesis::Controller && !NavigationSource.GetLastWidget()->GetPersistentState().IsProjectContent())
			{
				// Gamepad navigation while not in project content, do nothing as specified by GSlateEnableGamepadEditorNavigation
			}
			else if (TheReply.GetNavigationDestination().IsValid())
			{
				FScopedNavigationTransition ScopedNavigationTransition(TheReply.GetNavigationType(), SlateUser);

				const bool bAlwaysHandleNavigationAttempt = false;
				ExecuteNavigation(NavigationSource, TheReply.GetNavigationDestination(), UserIndex, bAlwaysHandleNavigationAttempt);
			}
			else
			{
				TSharedRef<SWindow> NavigationWindow = NavigationSource.GetDeepestWindow();

				FNavigationEvent NavigationEvent(PlatformApplication->GetModifierKeys(), UserIndex, TheReply.GetNavigationType(), TheReply.GetNavigationGenesis());

				FNavigationReply NavigationReply = FNavigationReply::Escape();

				for (int32 WidgetIndex = NavigationSource.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
				{
					FArrangedWidget& SomeWidgetGettingEvent = NavigationSource.Widgets[WidgetIndex];
					if (SomeWidgetGettingEvent.Widget->IsEnabled())
					{
						NavigationReply = SomeWidgetGettingEvent.Widget->OnNavigation(SomeWidgetGettingEvent.Geometry, NavigationEvent).SetHandler(SomeWidgetGettingEvent.Widget);
						if (NavigationReply.GetBoundaryRule() != EUINavigationRule::Escape || SomeWidgetGettingEvent.Widget == NavigationWindow || WidgetIndex == 0)
						{
							AttemptNavigation(NavigationSource, NavigationEvent, NavigationReply, SomeWidgetGettingEvent);
							break;
						}
					}
				}
			}
		}
	}

	if ( TheReply.GetDetectDragRequest().IsValid() )
	{
		checkSlow(InMouseEvent);

		FPointerEvent TransformedPointerEvent = TransformPointerEvent(*InMouseEvent, WidgetsUnderMouse->GetWindow());

		SlateUser->StartDragDetection(
			WidgetsUnderMouse->GetPathDownTo(TheReply.GetDetectDragRequest().ToSharedRef()),
			TransformedPointerEvent.GetPointerIndex(),
			TheReply.GetDetectDragRequestButton(),
			TransformedPointerEvent.GetScreenSpacePosition());
	}

	// Set focus if requested.
	TSharedPtr<SWidget> RequestedFocusRecepient = TheReply.GetUserFocusRecepient();
	if (TheReply.ShouldSetUserFocus() && RequestedFocusRecepient.IsValid())
	{
		if (TheReply.AffectsAllUsers())
		{
			SetAllUserFocus(RequestedFocusRecepient, TheReply.GetFocusCause());
		}
		else
		{
			SlateUser->SetFocus(RequestedFocusRecepient.ToSharedRef(), TheReply.GetFocusCause());
		}
	}
}

void FSlateApplication::SetLastUserInteractionTime(double InCurrentTime)
{
	if (LastUserInteractionTime != InCurrentTime)
	{
		LastUserInteractionTime = InCurrentTime;
		LastUserInteractionTimeUpdateEvent.Broadcast(LastUserInteractionTime);
	}
}

void FSlateApplication::QueryCursor()
{
	GetCursorUser()->QueryCursor();
}

void FSlateApplication::ProcessCursorReply(const FCursorReply& CursorReply)
{
	GetCursorUser()->ProcessCursorReply(CursorReply);
}

void FSlateApplication::SpawnToolTip(const TSharedRef<IToolTip>& InToolTip, const UE::Slate::FDeprecateVector2DParameter& InSpawnLocation)
{
	GetCursorUser()->ShowTooltip(InToolTip, InSpawnLocation);
}

void FSlateApplication::CloseToolTip()
{
	GetCursorUser()->CloseTooltip();
}

void FSlateApplication::UpdateToolTip(bool bAllowSpawningOfNewToolTips)
{
	GetCursorUser()->UpdateTooltip(MenuStack, bAllowSpawningOfNewToolTips);
}

TArray< TSharedRef<SWindow> > FSlateApplication::GetInteractiveTopLevelWindows()
{
	if (ActiveModalWindows.Num() > 0)
	{
		// If we have modal windows, only the topmost modal window and its children are interactive.
		TArray< TSharedRef<SWindow>, TInlineAllocator<1> > OutWindows;
		OutWindows.Add(ActiveModalWindows.Last().ToSharedRef());

		// If there is an interactive tooltip open from a modal window, include it too.
		for (int32 WindowIndex = SlateWindows.Num() - 1; WindowIndex >= 0; WindowIndex--)
		{
			TSharedRef<SWindow> CurrentWindow = SlateWindows[WindowIndex];
			if (GetCursorUser()->IsWindowHousingInteractiveTooltip(CurrentWindow))
			{
				OutWindows.Add(CurrentWindow);
			}
		}

		return TArray< TSharedRef<SWindow> >(OutWindows);
	}
	else
	{
		// No modal windows? All windows are interactive.
		return SlateWindows;
	}
}

void FSlateApplication::GetAllVisibleWindowsOrdered(TArray< TSharedRef<SWindow> >& OutWindows)
{
	for( TArray< TSharedRef<SWindow> >::TConstIterator CurrentWindowIt( SlateWindows ); CurrentWindowIt; ++CurrentWindowIt )
	{
		TSharedRef<SWindow> CurrentWindow = *CurrentWindowIt;
		if ( CurrentWindow->IsVisible() && !CurrentWindow->IsWindowMinimized() )
		{
			GetAllVisibleChildWindows(OutWindows, CurrentWindow);
		}
	}
}

void FSlateApplication::GetAllVisibleChildWindows(TArray< TSharedRef<SWindow> >& OutWindows, TSharedRef<SWindow> CurrentWindow)
{
	if ( CurrentWindow->IsVisible() && !CurrentWindow->IsWindowMinimized() )
	{
		OutWindows.Add(CurrentWindow);

		const TArray< TSharedRef<SWindow> >& WindowChildren = CurrentWindow->GetChildWindows();
		for (int32 ChildIndex=0; ChildIndex < WindowChildren.Num(); ++ChildIndex)
		{
			GetAllVisibleChildWindows( OutWindows, WindowChildren[ChildIndex] );
		}
	}
}

void FSlateApplication::EnterDebuggingMode()
{
	if (!IsInGameThread())
	{
		ensureMsgf(false, TEXT("Can only enter Debugging Mode while on the game thread."));
		return;
	}

	auto AddNotification = [Self=this](const FText& SubText)
	{
		if (TSharedPtr<SNotificationItem> MessagePinned = Self->DebuggingModeNotificationMessage.Pin())
		{
			static float DefaultDuration = FNotificationInfo{FText::GetEmpty()}.ExpireDuration;
			MessagePinned->SetSubText(SubText);
		}
		else
		{
			FNotificationInfo Info(NSLOCTEXT("EnterDebuggingMode", "FailTitle", "Debugging Mode Fail"));
			Info.SubText = SubText;
			Info.ExpireDuration = 15.f;
			Self->DebuggingModeNotificationMessage = FSlateNotificationManager::Get().AddNotification(Info);
		}
		UE_LOGF(LogSlate, Warning, "Enter Debugging Mode failed.");

		static volatile bool bDoDebugBreak = true;
		if (bDoDebugBreak)
		{
			//The application was not able to enter debugging mode.
			//The user will see a notification, but Blueprint breakpoint will not stop the execution.
			//We detected that the debugging mode would prevent the editor from ticking.
			//Entering debugging mode could make the application unresponsive or corrupt the application's message stack.
			// 
			//This can occur when you are debugging an editor blueprint that is executed inside a modal.
			//ie.
			// Engine::Tick
			//   RenamePrompt. ShowModal that blocks execution until the user presses a button.
			//    Inside the rename modal window, code executes that triggers a blueprint.
			// 
			//This can occur when you are debugging a Widget that is painting.
			//Game widgets are sadly included in the editor paint. It is related to how Slate rendering works.
			//ie.
			// World::Tick (no problem here)
			// SlateApplication::TickAndPaint
			//  SWindow::Paint // paint the editor
			//    SMyWidget::Paint (includes OnPaint, Tick, and other callbacks).
			UE_DEBUG_BREAK();
		}
	};

	if (GetActiveModalWindow().IsValid())
	{
		AddNotification(NSLOCTEXT("EnterDebuggingMode", "Fail_ModalWindow", "A modal window is open."));
		return;
	}

#if WITH_EDITOR
	// If we enter debugging mode to debug a widget in the main editor window,
	// the editor will become mostly unresponsive as it stops ticking the main editor window.
	// Otherwise allow debugging within tick.
	if (bIsTicking)
	{
		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		TSharedPtr<SWidget> ContextWidgetPinned = CurrentDebugContextWidget.Pin();
		TSharedPtr<SWindow> DebugWindow = ContextWidgetPinned.IsValid() ? FindWidgetWindow(ContextWidgetPinned.ToSharedRef()) : TSharedPtr<SWindow>();
		if (DebugWindow.IsValid() && DebugWindow == RootWindow)
		{
			AddNotification(INVTEXT("Tried to enter debugging mode inside Tick in the main editor window.\n\n"
				"Restart PIE using New Editor Window instead of Selected Viewport (or undock the viewport tab) and try again."));
			return;
		}
	}

	if (PreventDebuggingModeStack.Num() > 0)
	{
		AddNotification(PreventDebuggingModeStack.Last().Key);
		return;
	}
	FScopedPreventDebuggingMode Scope(NSLOCTEXT("EnterDebuggingMode", "AlreadyInDebuggingMode", "Already in debug mode."));
#endif

	bRequestLeaveDebugMode = false;

	// Note it is ok to hold a reference here as the game viewport should not be destroyed while in debugging mode
	TSharedPtr<SViewport> PreviousGameViewport;

	// Disable any game viewports while we are in debug mode so that mouse capture is released and the cursor is visible
	// We need to retain the keyboard input for debugging purposes, so this is called directly rather than calling UnregisterGameViewport which resets input.
	if (GameViewportWidget.IsValid())
	{
		PreviousGameViewport = GameViewportWidget.Pin();
		PreviousGameViewport->SetActive(false);
		GameViewportWidget.Reset();
	}

	// Find the SWindow that we should not tick while in DebuggingMode.
	ensureMsgf(!CurrentDebuggingWindow.IsValid(), TEXT("Reentry of EnterDebuggingMode with a valid Debugging Window is not supported"));
	CurrentDebuggingWindow.Reset();
	if (TSharedPtr<SWidget> CurrentDebugContextWidgetPinned = CurrentDebugContextWidget.Pin())
	{
		// Only prevent Paint if there is more than one window.
		//That is to prevent the user from getting stuck in the editor.
		if (SlateWindows.Num() > 0)
		{
			CurrentDebuggingWindow = FindWidgetWindow(CurrentDebugContextWidgetPinned.ToSharedRef());
		}
		else
		{
			UE_LOGF(LogSlate, Warning, "EnterDebuggingMode without blocking the window Paint. That may start a new Paint on the same Window while the previous Paint is not completed.");
		}
	}
	TGuardValue TmpContext(CurrentDebugContextWidget, TWeakPtr<SWidget>());

	Renderer->EndFrame();

	Renderer->FlushCommands();
	
	// We are about to start an in stack tick. Make sure the rendering thread isn't already behind
	Renderer->Sync();

#if WITH_EDITORONLY_DATA
	// Flag that we're about to enter the first frame of intra-frame debugging.
	GFirstFrameIntraFrameDebugging = true;
#endif	//WITH_EDITORONLY_DATA

	// Tick slate from here in the event that we should not return until the modal window is closed.
	while (!bRequestLeaveDebugMode)
	{
		// Tick and render Slate
		Tick();

		Renderer->EndFrame();
		
		Renderer->FlushCommands();
		
		// Synchronize the game thread and the render thread so that the render thread doesn't get too far behind.
		Renderer->Sync();

#if WITH_EDITORONLY_DATA
		// We are done with the first frame
		GFirstFrameIntraFrameDebugging = false;

		// If we are requesting leaving debugging mode, leave it now.
		GIntraFrameDebuggingGameThread = !bRequestLeaveDebugMode;
#endif	//WITH_EDITORONLY_DATA
	}
	
	bRequestLeaveDebugMode = false;
	
	CurrentDebuggingWindow.Reset();
	if ( PreviousGameViewport.IsValid() )
	{
		check(!GameViewportWidget.IsValid());

		// When in single step mode, register the game viewport so we can unregister it later
		// but do not do any of the other stuff like locking or capturing the mouse.
		if( bLeaveDebugForSingleStep )
		{
			GameViewportWidget = PreviousGameViewport;
		}
		else
		{
			// If we had a game viewport before debugging, re-register it now to capture the mouse and lock the cursor
			RegisterGameViewport( PreviousGameViewport.ToSharedRef() );
		}
	}

	bLeaveDebugForSingleStep = false;
}

void FSlateApplication::LeaveDebuggingMode( bool bLeavingForSingleStep )
{
	bRequestLeaveDebugMode = true;
	bLeaveDebugForSingleStep = bLeavingForSingleStep;
}

#if WITH_EDITOR
FSlateApplication::FScopedPreventDebuggingMode::FScopedPreventDebuggingMode(FText InReason)
{
	static int32 IdGenerator = 0;
	Id = ++IdGenerator;
	FSlateApplication::Get().PreventDebuggingModeStack.Emplace(MoveTemp(InReason), Id);
}

FSlateApplication::FScopedPreventDebuggingMode::~FScopedPreventDebuggingMode()
{
	int32 IndexToRemove = FSlateApplication::Get().PreventDebuggingModeStack.IndexOfByPredicate([this](const TPair<FText, int32>& Reference){ return Reference.Value == Id;});
	if (ensure(IndexToRemove != INDEX_NONE))
	{
		FSlateApplication::Get().PreventDebuggingModeStack.RemoveAtSwap(IndexToRemove);
	}
}
#endif

bool FSlateApplication::IsWindowInDestroyQueue(TSharedRef<SWindow> Window) const
{
	return WindowDestroyQueue.Contains(Window);
}

void FSlateApplication::SetUnhandledKeyDownEventHandler( const FOnKeyEvent& NewHandler )
{
	UnhandledKeyDownEventHandler = NewHandler;
}

void FSlateApplication::SetUnhandledKeyUpEventHandler(const FOnKeyEvent& NewHandler)
{
	UnhandledKeyUpEventHandler = NewHandler;
}

float FSlateApplication::GetDragTriggerDistance() const
{
	return DragTriggerDistance;
}

float FSlateApplication::GetDragTriggerDistanceSquared() const
{
	return DragTriggerDistance * DragTriggerDistance;
}

bool FSlateApplication::HasTraveledFarEnoughToTriggerDrag(const FPointerEvent& PointerEvent, const UE::Slate::FDeprecateVector2DParameter ScreenSpaceOrigin) const
{
	return ( PointerEvent.GetScreenSpacePosition() - ScreenSpaceOrigin ).SizeSquared() >= ( DragTriggerDistance * DragTriggerDistance );
}

bool FSlateApplication::HasTraveledFarEnoughToTriggerDrag(const FPointerEvent& PointerEvent, const UE::Slate::FDeprecateVector2DParameter ScreenSpaceOrigin, EOrientation Orientation) const
{
	if (Orientation == Orient_Horizontal)
	{
		return FMath::Abs(PointerEvent.GetScreenSpacePosition().X - ScreenSpaceOrigin.X) >= DragTriggerDistance;
	}
	else // Orientation == Orient_Vertical
	{
		return FMath::Abs(PointerEvent.GetScreenSpacePosition().Y - ScreenSpaceOrigin.Y) >= DragTriggerDistance;
	}
}

void FSlateApplication::SetDragTriggerDistance( float ScreenPixels )
{
	DragTriggerDistance = ScreenPixels;
}

void FSlateApplication::ToggleDisableLastDragOnDragEnter( bool bInDisableLastDragOnDragEnter )
{
	bDisableLastDragOnDragEnter = bInDisableLastDragOnDragEnter;
}

bool FSlateApplication::IsLastDragOnDragEnterDisabled() const
{
	return bDisableLastDragOnDragEnter;
}

bool FSlateApplication::RegisterInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor)
{
	return RegisterInputPreProcessor(InputProcessor, FInputPreprocessorRegistrationKey());
}

bool FSlateApplication::RegisterInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor, const int32 Index)
{
	return RegisterInputPreProcessor(InputProcessor, FInputPreprocessorRegistrationKey{ EInputPreProcessorType::Game, Index });
}

bool FSlateApplication::RegisterInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor, const EInputPreProcessorType Type)
{
	return RegisterInputPreProcessor(InputProcessor, FInputPreprocessorRegistrationKey{ Type, INDEX_NONE });
}

bool FSlateApplication::RegisterInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor, const FInputPreprocessorRegistrationKey& Info)
{
	bool bResult = false;
	if (InputProcessor.IsValid())
	{
		bResult = InputPreProcessors.Add(FInputPreprocessorRegistration{ Info, InputProcessor.ToSharedRef() });
	}
	return bResult;
}

void FSlateApplication::UnregisterInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor)
{
	InputPreProcessors.Remove(InputProcessor);
}

int32 FSlateApplication::FindInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor) const
{
	return InputPreProcessors.Find(InputProcessor, EInputPreProcessorType::Game);
}

int32 FSlateApplication::FindInputPreProcessor(TSharedPtr<IInputProcessor> InputProcessor, const EInputPreProcessorType& Type) const
{
	return InputPreProcessors.Find(InputProcessor, Type);
}

void FSlateApplication::SetCursorRadius(float NewRadius)
{
	CursorRadius = FMath::Max<float>(0.0f, NewRadius);
}

float FSlateApplication::GetCursorRadius() const
{
	return CursorRadius;
}

void FSlateApplication::SetAllowTooltips(bool bCanShow)
{
	bEnableTooltips = bCanShow;
}

bool FSlateApplication::GetAllowTooltips() const
{
	return bEnableTooltips;
}

void FSlateApplication::SetIsFakeTouch(bool bInIsFakingTouch)
{
	bIsFakingTouch = bInIsFakingTouch;
}

bool FSlateApplication::GetIsFakeTouch() const
{
	return bIsFakingTouch;
}

UE::Slate::FDeprecateVector2DResult FSlateApplication::CalculateTooltipWindowPosition( const FSlateRect& InAnchorRect, const UE::Slate::FDeprecateVector2DParameter& InSize, bool bAutoAdjustForDPIScale, EPopupCursorOverlapMode CursorOverlapMode) const
{
	// first use the CalculatePopupWindowPosition and if cursor is not inside it, proceed with it to avoid behavior change.
	FVector2f PopupPosition = CalculatePopupWindowPosition(InAnchorRect, InSize, bAutoAdjustForDPIScale, UE::Math::TVector2<float>::ZeroVector, Orient_Vertical, EPopupLayoutMode::ToolTip);
	FVector2f Cursor = GetCursorPos();
	if (CursorOverlapMode == EPopupCursorOverlapMode::AllowOverlap || (PopupPosition.X > Cursor.X || PopupPosition.X + InSize.X < Cursor.X ||
		PopupPosition.Y > Cursor.Y || PopupPosition.Y + InSize.Y < Cursor.Y))
	{
		return PopupPosition;
	}

	const FPlatformRect WorkAreaFinderRect (Cursor.X, Cursor.Y, Cursor.X + 1.0f, Cursor.Y + 1.0f);
	const FPlatformRect PlatformWorkArea = PlatformApplication->GetWorkArea(WorkAreaFinderRect);

	const FSlateRect WorkAreaRect( 
		PlatformWorkArea.Left, 
		PlatformWorkArea.Top, 
		PlatformWorkArea.Right, 
		PlatformWorkArea.Bottom);

	float DPIScale = 1.0f; 

	if (bAutoAdjustForDPIScale)
	{
		DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(Cursor.X, Cursor.Y);
	}

	// We want the Tooltip to appear in a 'comfortable' distance. The following vector: 'TooltipCursorOffset' 
	// is used to move away from the cursor tip position. If we wouldn't do this the Tooltip would directly
	// appear at the tip of the cursor. The coefficients 16 and 12 are estimated empirical.
	const FVector2f TooltipCursorOffset(16 * DPIScale, 12 * DPIScale);

	// Calculate the new position of the Tooltip by starting at the Top/Left corner.
	FVector2f ToolTipLocation = Cursor - TooltipCursorOffset - InSize;

	// Adjust the horizontal position so that it will be inside the work area.
	if ( ToolTipLocation.X < WorkAreaRect.Left )
	{
		ToolTipLocation.X += (InSize.X + 2.0 * TooltipCursorOffset.X);
	}

	// Adjust the vertical position so that it will be inside the work area.
	if ( ToolTipLocation.Y < WorkAreaRect.Top )
	{
		ToolTipLocation.Y += (InSize.Y + 2.0 * TooltipCursorOffset.Y);
	}

	return ToolTipLocation;
}

UE::Slate::FDeprecateVector2DResult FSlateApplication::CalculatePopupWindowPosition(const FSlateRect& InAnchor, const UE::Slate::FDeprecateVector2DParameter& InSize, bool bAutoAdjustForDPIScale, const UE::Slate::FDeprecateVector2DParameter& InProposedPlacement, const EOrientation Orientation, const EPopupLayoutMode LayoutMode) const
{
	FVector2D CalculatedPopUpWindowPosition(0.f, 0.f);

	float DPIScale = 1.0f;

	if (bAutoAdjustForDPIScale)
	{
		DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(InAnchor.Left, InAnchor.Top);
	}

	FVector2f AdjustedSize = InSize * DPIScale;

	FPlatformRect AnchorRect;
	AnchorRect.Left = InAnchor.Left;
	AnchorRect.Top = InAnchor.Top;
	AnchorRect.Right = InAnchor.Right;
	AnchorRect.Bottom = InAnchor.Bottom;

	EPopUpOrientation::Type PopUpOrientation = EPopUpOrientation::Horizontal;

	if (Orientation == EOrientation::Orient_Vertical)
	{
		PopUpOrientation = EPopUpOrientation::Vertical;
	}

	if (PlatformApplication->TryCalculatePopupWindowPosition(AnchorRect, FVector2D(AdjustedSize), FVector2D(InProposedPlacement), PopUpOrientation, /*OUT*/&CalculatedPopUpWindowPosition))
	{
		return UE::Slate::CastToVector2f(CalculatedPopUpWindowPosition / DPIScale);
	}
	else
	{
		// Calculate the rectangle around our work area
		// Use our own rect.  This window as probably doesn't have a size or position yet.
		// Use a size of 1 to get the closest monitor to the start point
		FPlatformRect WorkAreaFinderRect(AnchorRect);
		WorkAreaFinderRect.Right = AnchorRect.Left + 1;
		WorkAreaFinderRect.Bottom = AnchorRect.Top + 1;
		const FPlatformRect PlatformWorkArea = PlatformApplication->GetWorkArea(WorkAreaFinderRect);

		const FSlateRect WorkAreaRect(
			PlatformWorkArea.Left,
			PlatformWorkArea.Top,
			PlatformWorkArea.Right,
			PlatformWorkArea.Bottom);

		FVector2f ProposedPlacement = InProposedPlacement;

		if (ProposedPlacement.IsZero())
		{
			// Assume natural left-to-right, top-to-bottom flow; position popup below and to the right.
			ProposedPlacement = FVector2f(
				Orientation == Orient_Horizontal ? AnchorRect.Right : AnchorRect.Left,
				Orientation == Orient_Horizontal ? AnchorRect.Top : AnchorRect.Bottom);
		}

		return ComputePopupFitInRect(InAnchor, FSlateRect(ProposedPlacement, ProposedPlacement + AdjustedSize), Orientation, WorkAreaRect, /*bAllowFlip*/LayoutMode == EPopupLayoutMode::Menu) / DPIScale;
	}
}

bool FSlateApplication::IsRunningAtTargetFrameRate() const
{
	const float MinimumDeltaTime = 1.0f / TargetFrameRateForResponsiveness.GetValueOnGameThread();
	return ( AverageDeltaTimeForResponsiveness <= MinimumDeltaTime ) || !IsNormalExecution();
}


bool FSlateApplication::AreMenuAnimationsEnabled() const
{
	return bMenuAnimationsEnabled;
}


void FSlateApplication::EnableMenuAnimations( const bool bEnableAnimations )
{
	bMenuAnimationsEnabled = bEnableAnimations;
}


void FSlateApplication::SetAppIcon(const FSlateBrush* const InAppIcon)
{
	check(InAppIcon);
	AppIcon = InAppIcon;
}


const FSlateBrush* FSlateApplication::GetAppIcon() const
{
	static FName AppIconName("AppIcon");
	return FAppStyle::Get().GetBrush(AppIconName);
}

const FSlateBrush* FSlateApplication::GetAppIconSmall() const
{
	static FName AppIconName("AppIcon.Small");
	return FAppStyle::Get().GetBrush(AppIconName);
}

void FSlateApplication::ShowVirtualKeyboard( bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget )
{
	SCOPE_CYCLE_COUNTER(STAT_ShowVirtualKeyboard);

	if (!SlateTextField.IsValid())
	{
		SlateTextField = IPlatformTextFieldFactory::TryCreateInstance();
	}

	if (!SlateTextField.IsValid())
	{
		SlateTextField = MakeUnique<FPlatformTextField>();
	}

	SlateTextField->ShowVirtualKeyboard(bShow, UserIndex, TextEntryWidget);
}

bool FSlateApplication::AllowMoveCursor()
{
	if (!SlateTextField.IsValid())
	{
		SlateTextField = IPlatformTextFieldFactory::TryCreateInstance();
	}

	if (!SlateTextField.IsValid())
	{
		SlateTextField = MakeUnique<FPlatformTextField>();
	}

	return SlateTextField->AllowMoveCursor();
}

FSlateRect FSlateApplication::GetPreferredWorkArea() const
{
	if (TSharedPtr<const FSlateUser> KeyboardUser = GetUser(GetUserIndexForKeyboard()))
	{
		const FWeakWidgetPath& FocusedWidgetPath = KeyboardUser->GetWeakFocusPath();

		// First see if we have a focused widget
		if (FocusedWidgetPath.IsValid() && FocusedWidgetPath.Window.IsValid())
		{
			const FVector2f WindowPos = FocusedWidgetPath.Window.Pin()->GetPositionInScreen();
			const FVector2f WindowSize = FocusedWidgetPath.Window.Pin()->GetSizeInScreen();
			return GetWorkArea(FSlateRect(WindowPos.X, WindowPos.Y, WindowPos.X + WindowSize.X, WindowPos.Y + WindowSize.Y));
		}

		// no focus widget, so use cursor position if there are windows present in the work area
		const FVector2f CursorPos = KeyboardUser->GetCursorPosition();
		const FSlateRect WorkArea = GetWorkArea(FSlateRect(CursorPos.X, CursorPos.Y, CursorPos.X + 1.0f, CursorPos.Y + 1.0f));

		if (FSlateWindowHelper::CheckWorkAreaForWindows(SlateWindows, WorkArea))
		{
			return WorkArea;
		}

		// If we can't find a window where the cursor is at, try finding a main window.
		if (TSharedPtr<SWindow> ActiveTop = GetActiveTopLevelWindow())
		{
			// Use the current top level windows rect
			return GetWorkArea(ActiveTop->GetRectInScreen());
		}

		// If we can't find a top level window check for an active modal window
		if (TSharedPtr<SWindow> ActiveModal = GetActiveModalWindow())
		{
			// Use the current active modal windows rect
			return GetWorkArea(ActiveModal->GetRectInScreen());
		}
	}
	
	// Either there is no keyboard user or there are no windows on work area - fall back to primary display
	FDisplayMetrics DisplayMetrics;
	GetCachedDisplayMetrics(DisplayMetrics);

	const FPlatformRect& DisplayRect = DisplayMetrics.PrimaryDisplayWorkAreaRect;
	return FSlateRect((float)DisplayRect.Left, (float)DisplayRect.Top, (float)DisplayRect.Right, (float)DisplayRect.Bottom);
}

FSlateRect FSlateApplication::GetWorkArea( const FSlateRect& InRect ) const
{
	FPlatformRect InPlatformRect;
	InPlatformRect.Left = FMath::TruncToInt(InRect.Left);
	InPlatformRect.Top = FMath::TruncToInt(InRect.Top);
	InPlatformRect.Right = FMath::TruncToInt(InRect.Right);
	InPlatformRect.Bottom = FMath::TruncToInt(InRect.Bottom);

	const FPlatformRect OutPlatformRect = PlatformApplication->GetWorkArea( InPlatformRect );
	return FSlateRect( OutPlatformRect.Left, OutPlatformRect.Top, OutPlatformRect.Right, OutPlatformRect.Bottom );
}

bool FSlateApplication::SupportsSourceAccess() const
{
	if(QuerySourceCodeAccessDelegate.IsBound())
	{
		return QuerySourceCodeAccessDelegate.Execute();
	}
	return false;
}

void FSlateApplication::GotoLineInSource(const FString& FileName, int32 LineNumber) const
{
	if ( SupportsSourceAccess() )
	{
		if(SourceCodeAccessDelegate.IsBound())
		{
			SourceCodeAccessDelegate.Execute(FileName, LineNumber, 0);
		}
	}
}

void FSlateApplication::ForceRedrawWindow(const TSharedRef<SWindow>& InWindowToDraw)
{
	PrivateDrawWindows( InWindowToDraw );
}

bool FSlateApplication::TakeScreenshot(const TSharedRef<SWidget>& Widget, TArray<FColor>&OutColorData, FIntVector& OutSize)
{
	return TakeScreenshot(Widget, FIntRect(), OutColorData, OutSize);
}

bool FSlateApplication::TakeHDRScreenshot(const TSharedRef<SWidget>& Widget, TArray<FLinearColor>& OutColorData, FIntVector& OutSize)
{
	return TakeHDRScreenshot(Widget, FIntRect(), OutColorData, OutSize);
}

void TakeScreenshotCommon(const TSharedRef<SWidget>& Widget, const FIntRect& InnerWidgetArea, FIntRect& ScreenshotRect, 
						  SWindow* WidgetWindow)
{
	FWidgetPath WidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetChecked(Widget, WidgetPath);

	FArrangedWidget ArrangedWidget = WidgetPath.FindArrangedWidget(Widget).Get(FArrangedWidget::GetNullWidget());
	FVector2f Position = FVector2f(ArrangedWidget.Geometry.AbsolutePosition);
	FVector2f Size = ArrangedWidget.Geometry.GetDrawSize();
	FVector2f WindowPosition = WidgetWindow->GetPositionInScreen();

	ScreenshotRect = InnerWidgetArea.IsEmpty() ? FIntRect(0, 0, (int32)Size.X, (int32)Size.Y) : InnerWidgetArea;

	ScreenshotRect.Min.X += ( Position.X - WindowPosition.X );
	ScreenshotRect.Min.Y += ( Position.Y - WindowPosition.Y );
	ScreenshotRect.Max.X += ( Position.X - WindowPosition.X );
	ScreenshotRect.Max.Y += ( Position.Y - WindowPosition.Y );
	
	FSlateApplication::Get().OnPreScreenshot().Broadcast(Widget);
}

bool FSlateApplication::TakeScreenshot(const TSharedRef<SWidget>& Widget, const FIntRect& InnerWidgetArea, TArray<FColor>& OutColorData, FIntVector& OutSize)
{
	// We can't screenshot the widget unless there's a valid window handle to draw it in.
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(Widget);
	if (!WidgetWindow.IsValid())
	{
		return false;
	}

	TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

	FIntRect ScreenshotRect;
	TakeScreenshotCommon(Widget, InnerWidgetArea, ScreenshotRect, WidgetWindow.Get());

	Renderer->PrepareToTakeScreenshot(ScreenshotRect, &OutColorData, WidgetWindow.Get());
	PrivateDrawWindows(WidgetWindow);

	OutSize.X = ScreenshotRect.Size().X;
	OutSize.Y = ScreenshotRect.Size().Y;

	return (OutSize.X != 0 && OutSize.Y != 0 && OutColorData.Num() >= OutSize.X * OutSize.Y);
}

bool FSlateApplication::TakeHDRScreenshot(const TSharedRef<SWidget>& Widget, const FIntRect& InnerWidgetArea, TArray<FLinearColor>& OutColorData, FIntVector& OutSize)
{
	// We can't screenshot the widget unless there's a valid window handle to draw it in.
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(Widget);
	if (!WidgetWindow.IsValid())
	{
		return false;
	}

	TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

	FIntRect ScreenshotRect;
	TakeScreenshotCommon(Widget, InnerWidgetArea, ScreenshotRect, WidgetWindow.Get());

	Renderer->PrepareToTakeHDRScreenshot(ScreenshotRect, &OutColorData, WidgetWindow.Get());
	PrivateDrawWindows(WidgetWindow);

	OutSize.X = ScreenshotRect.Size().X;
	OutSize.Y = ScreenshotRect.Size().Y;

	return (OutSize.X != 0 && OutSize.Y != 0 && OutColorData.Num() >= OutSize.X * OutSize.Y);
}

TSharedPtr<FSlateUser> FSlateApplication::GetUser(FPlatformUserId PlatformUser)
{
	int32 InternalId = 0;
	if (PlatformUser.IsValid())
	{
		InternalId = PlatformUser.GetInternalId();
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "SlateApplication::GetUser called with an invalid platform user! Defaulting to 0");
	}
	return Users.IsValidIndex(InternalId) ? Users[InternalId] : nullptr;
}

TSharedPtr<FSlateUser> FSlateApplication::GetUserFromPlatformUser(FPlatformUserId PlatformUser)
{
	TOptional<int32> UserIndex = GetUserIndexForPlatformUser(PlatformUser);
	if (UserIndex.IsSet())
	{
		return GetUser(UserIndex.GetValue());
	}
	return nullptr;
}

TSharedPtr<const FSlateUser> FSlateApplication::GetUserFromPlatformUser(FPlatformUserId PlatformUser) const
{
	TOptional<int32> UserIndex = GetUserIndexForPlatformUser(PlatformUser);
	if (UserIndex.IsSet())
	{
		return GetUser(UserIndex.GetValue());
	}
	return nullptr;
}

TSharedRef<FSlateVirtualUserHandle> FSlateApplication::FindOrCreateVirtualUser(int32 VirtualUserIndex)
{
	// Ensure we have a large enough array to add the new virtual user
	if ( VirtualUserIndex >= VirtualUsers.Num() )
	{
		VirtualUsers.SetNum(VirtualUserIndex + 1);
	}

	TSharedPtr<FSlateVirtualUserHandle> VirtualUserHandle = VirtualUsers[VirtualUserIndex].Pin();
	if (!VirtualUserHandle.IsValid())
	{
		// Register at the next available user index (beyond those potentially occupied by real users)
		int32 NextVirtualUserIndex = SlateApplicationDefs::MaxHardwareUsers;
		while (GetUser(NextVirtualUserIndex))
		{
			NextVirtualUserIndex++;
		}

		TSharedRef<FSlateUser> NewUser = RegisterNewUser(NextVirtualUserIndex, true);

		// Make a virtual user handle that will unregister the virtual user upon destruction
		VirtualUserHandle = MakeShareable(new FSlateVirtualUserHandle(NewUser->GetUserIndex(), VirtualUserIndex));

		// Update the virtual user array, so we can get this user back later.
		VirtualUsers[VirtualUserIndex] = VirtualUserHandle;
	}

	return VirtualUserHandle.ToSharedRef();
}

TSharedRef<FSlateUser> FSlateApplication::GetOrCreateUser(int32 UserIndex)
{
	if (TSharedPtr<FSlateUser> FoundUser = GetUser(UserIndex))
	{
		return FoundUser.ToSharedRef();
	}
	return RegisterNewUser(UserIndex);
}

TSharedRef<FSlateUser> FSlateApplication::GetOrCreateUser(FPlatformUserId PlatformUserId)
{
	if (TSharedPtr<FSlateUser> FoundUser = GetUser(PlatformUserId))
	{
		return FoundUser.ToSharedRef();
	}
	return RegisterNewUser(PlatformUserId);
}

TSharedRef<FSlateUser> FSlateApplication::GetOrCreateUser(FInputDeviceId DeviceId)
{
	// Get a user based on the owning platform user of this input device
	return GetOrCreateUser(IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceId));
}

TSharedRef<FSlateUser> FSlateApplication::RegisterNewUser(int32 UserIndex, bool bIsVirtual)
{
	return RegisterNewUser(FGenericPlatformMisc::GetPlatformUserForUserIndex(UserIndex), bIsVirtual);
}

TSharedRef<FSlateUser> FSlateApplication::RegisterNewUser(FPlatformUserId PlatformUserId, bool bIsVirtual)
{
	int32 UserIndex = PlatformUserId.GetInternalId();
	
	// We tolerate no shenanigans with inappropriate arguments here
	// New users must be registered at a valid non-negative index that is not already occupied by another user
	check(UserIndex >= 0);
	check(!Users.IsValidIndex(UserIndex) || !Users[UserIndex]);

	TSharedPtr<ICursor> UserCursor;
	if (UserIndex == CursorUserIndex && PlatformApplication && PlatformApplication->Cursor)
	{
		check(!bIsVirtual);
		UserCursor = PlatformApplication->Cursor;
	}
	else if (!bIsVirtual)
	{
		// Real users all control a cursor, but there's only one platform cursor (and it *always* belongs to the 0th user)
		// Everyone else gets a faux cursor instead
		UserCursor = MakeShared<FFauxSlateCursor>();
	}

	TSharedRef<FSlateUser> NewUser = FSlateUser::Create(UserIndex, UserCursor);

	// Ensure we have a large enough array to add the new User
	if (UserIndex >= Users.Num())
	{
		Users.SetNum(UserIndex + 1);
	}

	Users[UserIndex] = NewUser;

	// Apply the last known "all users" focus widget path to this new user if they do not have a specific one
	if (!NewUser->HasValidFocusPath() && LastAllUsersFocusWidget.IsValid())
	{
		SetUserFocus(NewUser->GetUserIndex(), LastAllUsersFocusWidget.Pin(), LastAllUsersFocusCause);
	}

	UE_LOGF(LogSlate, Log, "Slate User Registered.  User Index %d, Is Virtual User: %d", UserIndex, bIsVirtual);
	UserRegisteredEvent.Broadcast(UserIndex);

	return NewUser;
}

void FSlateApplication::UnregisterUser(int32 UserIndex)
{
	if ( UserIndex < Users.Num() )
	{
		UE_LOGF(LogSlate, Log, "Slate User Unregistered.  User Index %d", UserIndex);

		ClearUserFocus(UserIndex, EFocusCause::SetDirectly);
		ClearPendingNavigationContext(UserIndex);
		Users[UserIndex].Reset();

		NavigationConfig->OnUserRemoved(UserIndex);
#if WITH_EDITOR
		EditorNavigationConfig->OnUserRemoved(UserIndex);
#endif
	}
}

void FSlateApplication::ForEachUser(TFunctionRef<void(FSlateUser&)> InPredicate, bool bIncludeVirtualUsers)
{
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		// Ignore virtual users unless told not to.
		if (User && (bIncludeVirtualUsers || !User->IsVirtualUser()))
		{
			InPredicate(*User);
		}
	}
}

void FSlateApplication::ForEachUser(TFunctionRef<void(FSlateUser*)> InPredicate, bool bIncludeVirtualUsers /*= false*/)
{
	ForEachUser([&InPredicate](FSlateUser& User)
		{
			InPredicate(&User);
		}, bIncludeVirtualUsers);
}

void FSlateApplication::SetFixedDeltaTime(double InSeconds)
{
	FixedDeltaTime = InSeconds;
}
/* FSlateApplicationBase interface
 *****************************************************************************/

UE::Slate::FDeprecateVector2DResult FSlateApplication::GetCursorSize( ) const
{
	if ( PlatformApplication->Cursor.IsValid() )
	{
		int32 X;
		int32 Y;
		PlatformApplication->Cursor->GetSize( X, Y );
		return FVector2f( static_cast<float>(X), static_cast<float>(Y) );
	}

	return FVector2f( 1.0f, 1.0f );
}

EVisibility FSlateApplication::GetSoftwareCursorVis( ) const
{
	const TSharedPtr<ICursor>& Cursor = PlatformApplication->Cursor;
	if (bSoftwareCursorAvailable && Cursor.IsValid() && Cursor->GetType() != EMouseCursor::None)
	{
		return EVisibility::HitTestInvisible;
	}
	return EVisibility::Hidden;
}

TSharedPtr<SWidget> FSlateApplication::GetKeyboardFocusedWidget() const
{
	TSharedPtr<const FSlateUser> KeyboardUser = GetUser(GetUserIndexForKeyboard());
	return KeyboardUser ? KeyboardUser->GetFocusedWidget() : nullptr;
}

TSharedPtr<SWidget> FSlateApplication::GetMouseCaptorImpl() const
{
	return GetCursorUser()->GetCursorCaptor();
}

bool FSlateApplication::HasAnyMouseCaptor() const
{
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		if (User && User->HasAnyCapture())
		{
			return true;
		}
	}
	return false;
}

bool FSlateApplication::HasUserMouseCapture(int32 UserIndex) const
{
	TSharedPtr<const FSlateUser> FoundUser = GetUser(UserIndex);
	return FoundUser && FoundUser->HasAnyCapture();
}

FPointerEvent FSlateApplication::TransformPointerEvent(const FPointerEvent& PointerEvent, const TSharedPtr<SWindow>& Window) const
{
	FPointerEvent TransformedPointerEvent = PointerEvent;
	if (Window)
	{
		if (TransformFullscreenMouseInput && !GIsEditor && Window->GetWindowMode() == EWindowMode::Fullscreen)
		{
			// Screen space mapping scales everything. When window resolution doesn't match platform resolution, 
			// this causes offset cursor hit-tests in fullscreen. Correct in slate since we are first window-aware slate processor.
			const FVector2f WindowSize = Window->GetSizeInScreen();
			const FVector2f DisplaySize = CachedDisplayMetrics.MonitorInfo.IsEmpty()
				? FVector2f(CachedDisplayMetrics.PrimaryDisplayWidth, CachedDisplayMetrics.PrimaryDisplayHeight)
				: Window->GetFullScreenInfo().GetSize2f();

			const FVector2f PositionInScreen = Window->GetPositionInScreen();
			const FVector2f Ratio = WindowSize / DisplaySize;
			FVector2f Position = PointerEvent.GetScreenSpacePosition();
			Position -= PositionInScreen;
			Position *= Ratio;
			Position += PositionInScreen;
			FVector2f LastPosition = PointerEvent.GetLastScreenSpacePosition();
			LastPosition -= PositionInScreen;
			LastPosition *= Ratio;
			LastPosition += PositionInScreen;
			TransformedPointerEvent = FPointerEvent(PointerEvent, Position, LastPosition);
		}
	}

	return TransformedPointerEvent;
}

bool FSlateApplication::DoesWidgetHaveMouseCaptureByUser(const TSharedPtr<const SWidget> Widget, int32 UserIndex, TOptional<int32> PointerIndex) const
{
	if (TSharedPtr<const FSlateUser> FoundUser = GetUser(UserIndex))
	{
		return PointerIndex.IsSet() ? FoundUser->DoesWidgetHaveCapture(Widget, PointerIndex.GetValue()) : FoundUser->DoesWidgetHaveAnyCapture(Widget);
	}
	return false;
}

bool FSlateApplication::DoesWidgetHaveMouseCapture(const TSharedPtr<const SWidget> Widget) const
{
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		if (User && User->DoesWidgetHaveAnyCapture(Widget))
		{
			return true;
		}
	}
	return false;
}

TOptional<EFocusCause> FSlateApplication::HasUserFocus(const TSharedPtr<const SWidget> Widget, int32 UserIndex) const
{
	TSharedPtr<const FSlateUser> FoundUser = GetUser(UserIndex);
	return FoundUser ? FoundUser->HasFocus(Widget) : TOptional<EFocusCause>();
}

TOptional<EFocusCause> FSlateApplication::HasAnyUserFocus(const TSharedPtr<const SWidget> Widget) const
{
	TOptional<EFocusCause> FocusCause;
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		FocusCause = User ? User->HasFocus(Widget) : TOptional<EFocusCause>();
		if (FocusCause.IsSet())
		{
			break;
		}
	}
	return FocusCause;
}

bool FSlateApplication::IsWidgetDirectlyHovered(const TSharedPtr<const SWidget> Widget) const
{
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		if (User && User->IsWidgetDirectlyUnderAnyPointer(Widget))
		{
			return true;
		}
	}
	return false;
}

bool FSlateApplication::ShowUserFocus(const TSharedPtr<const SWidget> Widget) const
{
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		if (User && User->ShouldShowFocus(Widget))
		{
			return true;
		}
	}
	return false;
}

TSharedRef<FNavigationConfig> FSlateApplication::GetRelevantNavConfig(int32 UserIndex) const
{
	TSharedPtr<FNavigationConfig> RelevantNavConfig = NavigationConfig;
	if (TSharedPtr<const FSlateUser> User = GetUser(UserIndex))
	{
#if WITH_EDITOR
		// Check if the focused widget is an editor widget or a PIE widget so we know which config to use.
		if (UserIndex == GetUserIndexForKeyboard() && !IsFocusInViewport(AllGameViewports, User->GetWeakFocusPath()))
		{
			RelevantNavConfig = EditorNavigationConfig;
		}
		else
#endif
		if (TSharedPtr<FNavigationConfig> UserNavConfig = User->GetUserNavigationConfig())
		{
			// Use the user's personal config if it has one assigned
			RelevantNavConfig = UserNavConfig;
		}
	}

	return RelevantNavConfig.ToSharedRef();
}

bool FSlateApplication::HasUserFocusedDescendants(const TSharedRef< const SWidget >& Widget, int32 UserIndex) const
{
	TSharedPtr<const FSlateUser> User = GetUser(UserIndex);
	return User && User->HasFocusedDescendants(Widget);
}

bool FSlateApplication::HasFocusedDescendants( const TSharedRef< const SWidget >& Widget ) const
{
	for (const TSharedPtr<FSlateUser>& User : Users)
	{
		if (User && User->HasFocusedDescendants(Widget))
		{
			return true;
		}
	}
	return false;
}

bool FSlateApplication::IsExternalUIOpened()
{
	return bIsExternalUIOpened;
}


TSharedRef<SImage> FSlateApplication::MakeImage( const TAttribute<const FSlateBrush*>& Image, const TAttribute<FSlateColor>& Color, const TAttribute<EVisibility>& Visibility ) const
{
	return SNew(SImage)
		.ColorAndOpacity(Color)
		.Image(Image)
		.Visibility(Visibility);
}


TSharedRef<SWidget> FSlateApplication::MakeWindowTitleBar(const FWindowTitleBarArgs& InArgs, TSharedPtr<IWindowTitleBar>& OutTitleBar) const
{
	TSharedRef<SWindowTitleBar> TitleBar = SNew(SWindowTitleBar, InArgs.Window, InArgs.CenterContent, InArgs.CenterContentAlignment)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.CloseButtonToolTipText(InArgs.CloseButtonToolTipText);

	OutTitleBar = TitleBar;

	return TitleBar;
}


TSharedRef<IToolTip> FSlateApplication::MakeToolTip(const TAttribute<FText>& ToolTipText)
{
	return MakeShared<SDeferredToolTipText>(ToolTipText);
}


TSharedRef<IToolTip> FSlateApplication::MakeToolTip( const FText& ToolTipText )
{
	return MakeShared<SDeferredToolTipText>(ToolTipText);
}

/* FGenericApplicationMessageHandler interface
 *****************************************************************************/

bool FSlateApplication::ShouldProcessUserInputMessages( const TSharedPtr< FGenericWindow >& PlatformWindow ) const
{
	TSharedPtr< SWindow > Window;
	if ( PlatformWindow.IsValid() )
	{
		Window = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, PlatformWindow.ToSharedRef() );
	}

	if (ActiveModalWindows.Num() == 0 ||
		(Window.IsValid() &&
		(Window->IsDescendantOf(GetActiveModalWindow()) || ActiveModalWindows.Top() == Window || IsWindowHousingInteractiveTooltip(Window.ToSharedRef()))))
	{
		return true;
	}
	return false;
}

bool FSlateApplication::OnKeyChar( const TCHAR Character, const bool IsRepeat )
{
	FCharacterEvent CharacterEvent( Character, PlatformApplication->GetModifierKeys(), 0, IsRepeat );
	return ProcessKeyCharEvent( CharacterEvent );
}

bool FSlateApplication::ProcessKeyCharEvent( const FCharacterEvent& InCharacterEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessKeyChar);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::KeyChar, InCharacterEvent);
#endif

	TScopeCounter<int32> BeginInput(ProcessingInput);

	FReply Reply = FReply::Unhandled();

	// NOTE: We intentionally don't reset LastUserInteractionTimeForThrottling here so that the UI can be responsive while typing

	// Bubble the key event
	TSharedRef<FSlateUser> User = GetOrCreateUser(InCharacterEvent);
	TSharedRef<FWidgetPath> EventPathRef = User->GetFocusPath();
	const FWidgetPath& EventPath = EventPathRef.Get();

	// Switch worlds for widgets in the current path
	FScopedSwitchWorldHack SwitchWorld(EventPath);

	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessKeyChar_RouteAlongFocusPath);
		Reply = FEventRouter::RouteAlongFocusPath(this, FEventRouter::FBubblePolicy(EventPath), InCharacterEvent, [](const FArrangedWidget& SomeWidgetGettingEvent, const FCharacterEvent& Event)
			{
				SCOPE_CYCLE_COUNTER(STAT_ProcessKeyChar_Call_OnKeyChar);

				if (SomeWidgetGettingEvent.Widget->IsEnabled())
				{
					const FReply TempReply = SomeWidgetGettingEvent.Widget->OnKeyChar(SomeWidgetGettingEvent.Geometry, Event);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::KeyChar, &Event, TempReply, SomeWidgetGettingEvent.Widget, Event.GetCharacter());
#endif
					return TempReply;
				}

				return FReply::Unhandled();
			}, ESlateDebuggingInputEvent::KeyChar);
	}

	return Reply.IsEventHandled();
}

bool FSlateApplication::OnKeyDown( const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat ) 
{
	FKey const Key = FInputKeyManager::Get().GetKeyFromCodes( KeyCode, CharacterCode );
	FKeyEvent KeyEvent(Key, PlatformApplication->GetModifierKeys(), GetUserIndexForKeyboard(), IsRepeat, CharacterCode, KeyCode);

	return ProcessKeyDownEvent( KeyEvent );
}

bool FSlateApplication::ProcessKeyDownEvent( const FKeyEvent& InKeyEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessKeyDown);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::KeyDown, InKeyEvent);
#endif

	TScopeCounter<int32> BeginInput(ProcessingInput);

	TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(InKeyEvent);

#if WITH_EDITOR
	//Send the key input to all pre input key down listener function
	if (OnApplicationPreInputKeyDownListenerEvent.IsBound())
	{
		OnApplicationPreInputKeyDownListenerEvent.Broadcast(InKeyEvent);
	}
#endif //WITH_EDITOR

	// Analog cursor gets first chance at the input
	if (InputPreProcessors.HandleKeyDownEvent(*this, InKeyEvent))
	{
		return true;
	}

	FReply Reply = FReply::Unhandled();

	SetLastUserInteractionTime(this->GetCurrentTime());
	
	
	if (SlateUser->IsDragDropping() && InKeyEvent.GetKey() == EKeys::Escape)
	{
		// Pressing ESC while drag and dropping terminates the drag drop.
		SlateUser->CancelDragDrop();
		Reply = FReply::Handled();
	}
	else
	{
		LastUserInteractionTimeForThrottling = LastUserInteractionTime;

#if SLATE_HAS_WIDGET_REFLECTOR
		// If we are inspecting, pressing ESC exits inspection mode.
		if ( InKeyEvent.GetKey() == EKeys::Escape )
		{
			TSharedPtr<IWidgetReflector> WidgetReflector = WidgetReflectorPtr.Pin();
			const bool bIsWidgetReflectorPicking = WidgetReflector.IsValid() && WidgetReflector->IsInPickingMode();
			if ( bIsWidgetReflectorPicking )
			{
					WidgetReflector->OnWidgetPicked();
					Reply = FReply::Handled();

					return Reply.IsEventHandled();
			}
		}
#endif
		// Bubble the keyboard event
		TSharedRef<FWidgetPath> EventPathRef = SlateUser->GetFocusPath();
		const FWidgetPath& EventPath = EventPathRef.Get();

		// Switch worlds for widgets in the current path
		FScopedSwitchWorldHack SwitchWorld(EventPath);

		// Tunnel the keyboard event
		Reply = FEventRouter::RouteAlongFocusPath(this, FEventRouter::FTunnelPolicy(EventPath), InKeyEvent, [] (const FArrangedWidget& CurrentWidget, const FKeyEvent& Event)
		{
			if (CurrentWidget.Widget->IsEnabled())
			{
				const FReply TempReply = CurrentWidget.Widget->OnPreviewKeyDown(CurrentWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::PreviewKeyDown, &Event, TempReply, CurrentWidget.Widget, Event.GetKey().GetFName());
#endif
				return TempReply;
			}
			else
			{
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::PreviewKeyDown, &Event, CurrentWidget.Widget);
#endif
			}
			return FReply::Unhandled();
		}, ESlateDebuggingInputEvent::PreviewKeyDown);

		// Send out key down events.
		if ( !Reply.IsEventHandled() )
		{
			Reply = FEventRouter::RouteAlongFocusPath(this, FEventRouter::FBubblePolicy(EventPath), InKeyEvent, [] (const FArrangedWidget& SomeWidgetGettingEvent, const FKeyEvent& Event)
			{
				if (SomeWidgetGettingEvent.Widget->IsEnabled())
				{
					const FReply TempReply = SomeWidgetGettingEvent.Widget->OnKeyDown(SomeWidgetGettingEvent.Geometry, Event);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::KeyDown, &Event, TempReply, SomeWidgetGettingEvent.Widget, Event.GetKey().GetFName());
#endif
					return TempReply;
				}
				else
				{
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::KeyDown, &Event, SomeWidgetGettingEvent.Widget);
#endif
				}

				return FReply::Unhandled();
			}, ESlateDebuggingInputEvent::KeyDown);
		}

		// If the key event was not processed by any widget...
		if ( !Reply.IsEventHandled() && UnhandledKeyDownEventHandler.IsBound() )
		{
			Reply = UnhandledKeyDownEventHandler.Execute(InKeyEvent);
		}
	}

	return Reply.IsEventHandled();
}

bool FSlateApplication::OnKeyUp( const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat )
{
	FKey const Key = FInputKeyManager::Get().GetKeyFromCodes( KeyCode, CharacterCode );
	FKeyEvent KeyEvent(Key, PlatformApplication->GetModifierKeys(), GetUserIndexForKeyboard(), IsRepeat, CharacterCode, KeyCode);

	return ProcessKeyUpEvent( KeyEvent );
}

bool FSlateApplication::ProcessKeyUpEvent( const FKeyEvent& InKeyEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessKeyUp);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::KeyUp, InKeyEvent);
#endif

	TScopeCounter<int32> BeginInput(ProcessingInput);

	// Analog cursor gets first chance at the input
	if (InputPreProcessors.HandleKeyUpEvent(*this, InKeyEvent))
	{
		return true;
	}

	FReply Reply = FReply::Unhandled();

	SetLastUserInteractionTime(this->GetCurrentTime());
	
	LastUserInteractionTimeForThrottling = LastUserInteractionTime;

	// Bubble the key event
	TSharedRef<FWidgetPath> EventPathRef = GetOrCreateUser(InKeyEvent)->GetFocusPath();
	const FWidgetPath& EventPath = EventPathRef.Get();

	// Switch worlds for widgets in the current path
	FScopedSwitchWorldHack SwitchWorld(EventPath);

	Reply = FEventRouter::RouteAlongFocusPath(this, FEventRouter::FBubblePolicy(EventPath), InKeyEvent, [](const FArrangedWidget& SomeWidgetGettingEvent, const FKeyEvent& Event)
		{
			if (SomeWidgetGettingEvent.Widget->IsEnabled())
			{
				const FReply TempReply = SomeWidgetGettingEvent.Widget->OnKeyUp(SomeWidgetGettingEvent.Geometry, Event);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::KeyUp, &Event, TempReply, SomeWidgetGettingEvent.Widget, Event.GetKey().ToString());
#endif
				return TempReply;
			}

			return FReply::Unhandled();
		}, ESlateDebuggingInputEvent::KeyUp);

		// If the key event was not processed by any widget...
		if (!Reply.IsEventHandled() && UnhandledKeyUpEventHandler.IsBound())
		{
			Reply = UnhandledKeyUpEventHandler.Execute(InKeyEvent);
		}

	return Reply.IsEventHandled();
}

void FSlateApplication::OnInputLanguageChanged()
{
	FInputKeyManager::Get().InitKeyMappings();
}

bool FSlateApplication::ProcessAnalogInputEvent(const FAnalogInputEvent& InAnalogInputEvent)
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessAnalogInput);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::AnalogInput, InAnalogInputEvent);
#endif

	TScopeCounter<int32> BeginInput(ProcessingInput);

	FReply Reply = FReply::Unhandled();

	// Analog cursor gets first chance at the input
	if (InputPreProcessors.HandleAnalogInputEvent(*this, InAnalogInputEvent))
	{
		Reply = FReply::Handled();
	}

	if (!Reply.IsEventHandled())
	{
		TSharedRef<FWidgetPath> EventPathRef = GetOrCreateUser(InAnalogInputEvent)->GetFocusPath();
		const FWidgetPath& EventPath = EventPathRef.Get();

		FAnalogInputEvent ModifiedEvent(InAnalogInputEvent);
		ModifiedEvent.SetEventPath(EventPath);

		// Switch worlds for widgets in the current path
		FScopedSwitchWorldHack SwitchWorld(EventPath);

		Reply = FEventRouter::RouteAlongFocusPath(this, FEventRouter::FBubblePolicy(EventPath), ModifiedEvent, [](const FArrangedWidget& SomeWidgetGettingEvent, const FAnalogInputEvent& Event)
			{
				if (SomeWidgetGettingEvent.Widget->IsEnabled())
				{
					const FReply TempReply = SomeWidgetGettingEvent.Widget->OnAnalogValueChanged(SomeWidgetGettingEvent.Geometry, Event);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::AnalogInput, &Event, TempReply, SomeWidgetGettingEvent.Widget, Event.GetKey().ToString());
#endif
					return TempReply;
				}

				return FReply::Unhandled();
			}, ESlateDebuggingInputEvent::AnalogInput);
	}

	// Ensure the analog input event exceeds the thresholds set in the navigation config before considering as interaction.
	const TSharedRef<FNavigationConfig> RelevantNavConfig = GetRelevantNavConfig(InAnalogInputEvent.GetUserIndex());
	if (RelevantNavConfig->IsAnalogEventBeyondNavigationThreshold(InAnalogInputEvent))
	{
		SetLastUserInteractionTime(this->GetCurrentTime());
		LastUserInteractionTimeForThrottling = LastUserInteractionTime;
	}

	return Reply.IsEventHandled();
}

FKey TranslateMouseButtonToKey( const EMouseButtons::Type Button )
{
	FKey Key = EKeys::Invalid;

	switch( Button )
	{
	case EMouseButtons::Left:
		Key = EKeys::LeftMouseButton;
		break;
	case EMouseButtons::Middle:
		Key = EKeys::MiddleMouseButton;
		break;
	case EMouseButtons::Right:
		Key = EKeys::RightMouseButton;
		break;
	case EMouseButtons::Thumb01:
		Key = EKeys::ThumbMouseButton;
		break;
	case EMouseButtons::Thumb02:
		Key = EKeys::ThumbMouseButton2;
		break;
	}

	return Key;
}

void FSlateApplication::SetGameIsFakingTouchEvents(const bool bIsFaking, FVector2D* CursorLocation)
{
	// note, this is usually guarded by FPlatformMisc::DesktopTouchScreen()
	// the only place this is not guarded is in FPIEPreviewDeviceModule::OnWindowReady()
	if ( bIsGameFakingTouch != bIsFaking )
	{
		if (bAllowFakingTouch && bIsFakingTouched && !bIsFaking && bIsGameFakingTouch && !bIsFakingTouch)
		{
			OnTouchEnded((CursorLocation ? *CursorLocation : PlatformApplication->Cursor->GetPosition()), 0, FSlateApplicationBase::SlateAppPrimaryPlatformUser, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());
		}

		bIsGameFakingTouch = bIsFaking;
	}
}

void FSlateApplication::SetGameAllowsFakingTouchEvents(const bool bAllowFaking)
{
	bAllowFakingTouch = bAllowFaking;
	if(!bAllowFaking && IsFakingTouchEvents())
	{
		SetGameIsFakingTouchEvents(bAllowFaking);
	}
}

bool FSlateApplication::IsFakingTouchEvents() const
{
	return bAllowFakingTouch && (bIsFakingTouch || bIsGameFakingTouch);
}

bool FSlateApplication::OnMouseDown(const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button)
{
	return OnMouseDown(PlatformWindow, Button, GetCursorPos());
}

bool FSlateApplication::OnMouseDown( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button, const FVector2D CursorPos )
{
	// convert a left mouse button click to touch event if we are faking it
	if (IsFakingTouchEvents() && Button == EMouseButtons::Left)
	{
		bIsFakingTouched = true;
		return OnTouchStarted( PlatformWindow, PlatformApplication->Cursor->GetPosition(), 1.0f, /* touch index */ 0, FSlateApplicationBase::SlateAppPrimaryPlatformUser, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice() );
	}

	FKey Key = TranslateMouseButtonToKey( Button );

	// Update PressedMouseButtons before constructing the event so that the value-copy
	// snapshot in FPointerEvent reflects the post-press state. The matching Add inside
	// ProcessMouseButtonDownEvent becomes a no-op for this path (TSet is idempotent)
	// but is still needed for touch events that call ProcessMouseButtonDownEvent directly.
	if (GetUserIndexForMouse() == CursorUserIndex)
	{
		PressedMouseButtons.Add(Key);
	}

	FPointerEvent MouseEvent(
		GetUserIndexForMouse(),
		CursorPointerIndex,
		CursorPos,
		GetLastCursorPos(),
		PressedMouseButtons,
		Key,
		0,
		PlatformApplication->GetModifierKeys()
		);

	return ProcessMouseButtonDownEvent( PlatformWindow, MouseEvent );
}

bool FSlateApplication::ProcessMouseButtonDownEvent( const TSharedPtr< FGenericWindow >& PlatformWindow, const FPointerEvent& MouseEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessMouseButtonDown);
	
	TScopeCounter<int32> BeginInput(ProcessingInput);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::MouseButtonDown, MouseEvent);
#endif

#if WITH_EDITOR
	//Send the key input to all pre input key down listener function
	if (OnApplicationMousePreInputButtonDownListenerEvent.IsBound())
	{
		OnApplicationMousePreInputButtonDownListenerEvent.Broadcast(MouseEvent);
	}
#endif //WITH_EDITOR

	SetLastUserInteractionTime(this->GetCurrentTime());
	LastUserInteractionTimeForThrottling = LastUserInteractionTime;
	
	if (PlatformWindow.IsValid())
	{
		PlatformApplication->SetCapture(PlatformWindow);
	}

	if (MouseEvent.GetUserIndex() == CursorUserIndex)
	{
		PressedMouseButtons.Add(MouseEvent.GetEffectingButton());
	}

	// Input preprocessors get the first chance at the input
	if (InputPreProcessors.HandleMouseButtonDownEvent(*this, MouseEvent))
	{
		return true;
	}

	bool bInGame = false;

	// Only process mouse down messages if we are not drag/dropping
	TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(MouseEvent);
	if (!SlateUser->IsDragDropping())
	{
		FReply Reply = FReply::Unhandled();
		if (SlateUser->HasCapture(MouseEvent.GetPointerIndex()))
		{
			FWidgetPath MouseCaptorPath = SlateUser->GetCaptorPath(MouseEvent.GetPointerIndex(), FWeakWidgetPath::EInterruptedPathHandling::Truncate, &MouseEvent);
			FArrangedWidget& MouseCaptorWidget = MouseCaptorPath.Widgets.Last();

			// Switch worlds widgets in the current path
			FScopedSwitchWorldHack SwitchWorld(MouseCaptorPath);
			bInGame = FApp::IsGame();

			FPointerEvent TransformedPointerEvent = TransformPointerEvent(MouseEvent, MouseCaptorPath.GetWindow());

			Reply = FEventRouter::Route<FReply>(this, FEventRouter::FToLeafmostPolicy(MouseCaptorPath), TransformedPointerEvent, [] (const FArrangedWidget& InMouseCaptorWidget, const FPointerEvent& Event)
			{
				const FReply TempReply = InMouseCaptorWidget.Widget->OnPreviewMouseButtonDown(InMouseCaptorWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::PreviewMouseButtonDown, &Event, TempReply, InMouseCaptorWidget.Widget);
#endif
				return TempReply;
			}, ESlateDebuggingInputEvent::PreviewMouseButtonDown);

			if ( !Reply.IsEventHandled() )
			{
				Reply = FEventRouter::Route<FReply>(this, FEventRouter::FToLeafmostPolicy(MouseCaptorPath), TransformedPointerEvent,
					[this] (const FArrangedWidget& InMouseCaptorWidget, const FPointerEvent& Event)
				{
					FReply TempReply = FReply::Unhandled();
					if ( Event.IsTouchEvent() )
					{
						TempReply = InMouseCaptorWidget.Widget->OnTouchStarted(InMouseCaptorWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchStart, &Event, TempReply, InMouseCaptorWidget.Widget);
#endif
					}
					if ( !Event.IsTouchEvent() || ( !TempReply.IsEventHandled() && this->bTouchFallbackToMouse ) )
					{
						TempReply = InMouseCaptorWidget.Widget->OnMouseButtonDown(InMouseCaptorWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseButtonDown, &Event, TempReply, InMouseCaptorWidget.Widget);
#endif
					}
					return TempReply;
				}, ESlateDebuggingInputEvent::MouseButtonDown);
			}
		}
		else
		{
			FWidgetPath WidgetsUnderCursor = LocateWindowUnderMouse( MouseEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, SlateUser->GetUserIndex());

			PopupSupport.SendNotifications( WidgetsUnderCursor );

			// Switch worlds widgets in the current path
			FScopedSwitchWorldHack SwitchWorld(WidgetsUnderCursor);
			bInGame = FApp::IsGame();

			//@todo NickD: Route API should be private; update Process methods to accept an FWidgetPath
			Reply = RoutePointerDownEvent(WidgetsUnderCursor, MouseEvent);
		}

		// See if expensive tasks should be throttled.  By default on mouse down expensive tasks are throttled
		// to ensure Slate responsiveness in low FPS situations
		if (Reply.IsEventHandled() && !bInGame && !MouseEvent.IsTouchEvent())
		{
			// Enter responsive mode if throttling should occur and its not already happening
			if( Reply.ShouldThrottle() && !MouseButtonDownResponsivnessThrottle.IsValid() )
			{
				MouseButtonDownResponsivnessThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
			}
			else if( !Reply.ShouldThrottle() && MouseButtonDownResponsivnessThrottle.IsValid() )
			{
				// Leave responsive mode if a widget chose not to throttle
				FSlateThrottleManager::Get().LeaveResponsiveMode( MouseButtonDownResponsivnessThrottle );
			}
		}
	}

	return true;
}

FReply FSlateApplication::RoutePointerDownEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent)
{
	TScopeCounter<int32> BeginInput(ProcessingInput);

	FPointerEvent TransformedPointerEvent = WidgetsUnderPointer.IsValid() ? TransformPointerEvent(PointerEvent, WidgetsUnderPointer.GetWindow()) : PointerEvent;
	
	TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(PointerEvent);
	SlateUser->UpdatePointerPosition(PointerEvent);

#if PLATFORM_MAC
	void* ActiveWindow = FMacPlatformApplicationMisc::GetActiveWindowPtr();
	const bool bNeedToActivateWindow = ( ActiveWindow == nullptr );
#else
	const bool bNeedToActivateWindow = false;
#endif

	const TSharedPtr<SWidget> PreviouslyFocusedWidget = GetKeyboardFocusedWidget();

	FReply Reply = FEventRouter::Route<FReply>( this, FEventRouter::FTunnelPolicy( WidgetsUnderPointer ), TransformedPointerEvent, []( const FArrangedWidget TargetWidget, const FPointerEvent& Event )
	{
		const FReply TempReply = TargetWidget.Widget->OnPreviewMouseButtonDown(TargetWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::PreviewMouseButtonDown, &Event, TempReply, TargetWidget.Widget);
#endif
		return TempReply;
	}, ESlateDebuggingInputEvent::PreviewMouseButtonDown);

	if( !Reply.IsEventHandled() )
	{
		Reply = FEventRouter::Route<FReply>( this, FEventRouter::FBubblePolicy( WidgetsUnderPointer ), TransformedPointerEvent, [this]( const FArrangedWidget TargetWidget, const FPointerEvent& Event )
		{
			FReply TempReply = FReply::Unhandled();
			if( !TempReply.IsEventHandled() )
			{
				if( Event.IsTouchEvent() )
				{
					TempReply = TargetWidget.Widget->OnTouchStarted( TargetWidget.Geometry, Event );
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchStart, &Event, TempReply, TargetWidget.Widget);
#endif
				}
				if( !Event.IsTouchEvent() || ( !TempReply.IsEventHandled() && this->bTouchFallbackToMouse ) )
				{
					TempReply = TargetWidget.Widget->OnMouseButtonDown( TargetWidget.Geometry, Event );
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseButtonDown, &Event, TempReply, TargetWidget.Widget);
#endif
				}
			}
			return TempReply;
		}, ESlateDebuggingInputEvent::MouseButtonDown);

		// When we perform a touch begin, we need to also send a mouse enter as if it were a cursor.
		if (PointerEvent.IsTouchEvent() && !IsFakingTouchEvents())
		{
			for (int32 WidgetIndex = WidgetsUnderPointer.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
			{
				const FArrangedWidget& TargetWidget = WidgetsUnderPointer.Widgets[WidgetIndex];

				TargetWidget.Widget->OnMouseEnter(TargetWidget.Geometry, TransformedPointerEvent);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseEnter, &TransformedPointerEvent, TargetWidget.Widget);
#endif
			}
		}
	}

#if PLATFORM_MAC
	const bool bIsDetectingLMBDrag = PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton && SlateUser->IsDetectingDrag(PointerEvent.GetPointerIndex());
#endif

	// If none of the widgets requested keyboard focus to be set (or set the keyboard focus explicitly), set it to the leaf-most widget under the mouse.
	// On Mac we prevent the OS from activating the window on mouse down, so we have full control and can activate only if there's nothing draggable under the mouse cursor.
	const bool bFocusChangedByEventHandler = PreviouslyFocusedWidget != GetKeyboardFocusedWidget();
	if( ( !bFocusChangedByEventHandler || bNeedToActivateWindow ) &&
		( !Reply.GetUserFocusRecepient().IsValid()
#if PLATFORM_MAC
			|| bIsDetectingLMBDrag
#endif
		)
	)
	{
		for ( int32 WidgetIndex = WidgetsUnderPointer.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex )
		{
			const FArrangedWidget& CurWidget = WidgetsUnderPointer.Widgets[WidgetIndex];
			if ( CurWidget.Widget->SupportsKeyboardFocus() )
			{
				FWidgetPath NewFocusedWidgetPath = WidgetsUnderPointer.GetPathDownTo(CurWidget.Widget);
				SetUserFocus(PointerEvent.GetUserIndex(), NewFocusedWidgetPath, EFocusCause::Mouse);
				break;
			}
		}

#if PLATFORM_MAC
		const bool bIsVirtualInteraction = WidgetsUnderPointer.TopLevelWindow.IsValid() ? WidgetsUnderPointer.TopLevelWindow->IsVirtualWindow() : false;
		if ( !bIsVirtualInteraction )
		{
			TSharedPtr<SWindow> TopLevelWindow = WidgetsUnderPointer.TopLevelWindow;
			if ( bNeedToActivateWindow || ( TopLevelWindow.IsValid() && TopLevelWindow->GetNativeWindow()->GetOSWindowHandle() != ActiveWindow ) )
			{
				// Clicking on a context menu should not activate anything
				// @todo: This needs to be updated when we have window type in SWindow and we no longer have to guess if WidgetsUnderCursor.TopLevelWindow is a menu
				const bool bIsContextMenu = TopLevelWindow.IsValid() && !TopLevelWindow->IsRegularWindow() && TopLevelWindow->HasMinimizeBox() && TopLevelWindow->HasMaximizeBox();
				if ( !bIsContextMenu && bIsDetectingLMBDrag && ActiveWindow == FMacPlatformApplicationMisc::GetActiveWindowPtr())
				{
					TMap<TSharedRef<FSlateUser>, TMap<uint32, FWeakWidgetPath>> AllPointerCaptors;
					ForEachUser([&AllPointerCaptors](FSlateUser& User) { AllPointerCaptors.Add(User.AsShared(), User.GetCaptorPathsByIndex()); });
					FPlatformApplicationMisc::ActivateApplication();
					if ( TopLevelWindow.IsValid() )
					{
						TopLevelWindow->BringToFront(true);
					}
					ForEachUser([&AllPointerCaptors](FSlateUser& User) { User.RestoreCaptorPathsByIndex(AllPointerCaptors.FindChecked(User.AsShared())); });
				}
			}
		}
#endif
	}

	return Reply;
}


FReply FSlateApplication::RoutePointerUpEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent)
{
	TScopeCounter<int32> BeginInput(ProcessingInput);

	FPointerEvent TransformedPointerEvent = WidgetsUnderPointer.IsValid() ? TransformPointerEvent(PointerEvent, WidgetsUnderPointer.GetWindow()) : PointerEvent;

	FReply Reply = FReply::Unhandled();
	TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(PointerEvent);
	const bool bIsDragDropping = SlateUser->IsDragDroppingAffected(PointerEvent);
	TSharedPtr<FDragDropOperation> LocalDragDropContent;
	
	bool bDropWasHandled = false;
	FWidgetPath LocalWidgetsUnderPointer = WidgetsUnderPointer;

	if (SlateUser->HasCapture(PointerEvent.GetPointerIndex()))
	{
		FWidgetPath MouseCaptorPath = SlateUser->GetCaptorPath(PointerEvent.GetPointerIndex(), FWeakWidgetPath::EInterruptedPathHandling::Truncate, &PointerEvent);
#if PLATFORM_MAC
		// Because of Deferred event on Mac, by the time this event is processed, the window with the mouse capture might have been dismissed before the mouse up is processed
		if ( MouseCaptorPath.Widgets.Num() > 0 ) 
#else
		if ( ensureMsgf(MouseCaptorPath.Widgets.Num() > 0, TEXT("A window had a widget with mouse capture. That entire window has been dismissed before the mouse up could be processed.")) )
#endif
		{
			// Switch worlds widgets in the current path
			FScopedSwitchWorldHack SwitchWorld( MouseCaptorPath );

			Reply =
				FEventRouter::Route<FReply>( this, FEventRouter::FToLeafmostPolicy(MouseCaptorPath), TransformedPointerEvent, [this]( const FArrangedWidget& TargetWidget, const FPointerEvent& Event )
				{
					FReply TempReply = FReply::Unhandled();
					if (Event.IsTouchEvent())
					{
						TempReply = TargetWidget.Widget->OnTouchEnded(TargetWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchEnd, &Event, TempReply, TargetWidget.Widget);
#endif
					}

					if (!Event.IsTouchEvent() || (!TempReply.IsEventHandled() && this->bTouchFallbackToMouse))
					{
						TempReply = TargetWidget.Widget->OnMouseButtonUp( TargetWidget.Geometry, Event );
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseButtonUp, &Event, TempReply, TargetWidget.Widget);
#endif
					}
					
					if ( Event.IsTouchEvent() && !IsFakingTouchEvents() )
					{
						// Generate a Leave event when a touch ends as well, since a touch can enter a widget and then end inside it
						TargetWidget.Widget->OnMouseLeave(Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseLeave, &Event, TargetWidget.Widget);
#endif
					}

					return TempReply;
				}, ESlateDebuggingInputEvent::MouseButtonUp);
		}
	}
	else
	{
		if (!LocalWidgetsUnderPointer.IsValid())
		{
			LocalWidgetsUnderPointer = LocateWindowUnderMouse(PointerEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, SlateUser->GetUserIndex());
		}
		
		// Switch worlds widgets in the current path
		FScopedSwitchWorldHack SwitchWorld(LocalWidgetsUnderPointer);

		if (bIsDragDropping)
		{
			// Route a synthetic pointer move event to ensure drag events ( e.g. OnDragLeave ) are called if necessary on any widgets before OnDrop
			if (GSlateInputPointerUpFiresPointerMoveForDragDrop)
			{
				const bool bIsSynthetic = true;
				RoutePointerMoveEvent(LocalWidgetsUnderPointer, PointerEvent, bIsSynthetic);
			}

			// Cache the drag drop content and reset the pointer in case OnMouseButtonUpMessage re-enters as a result of OnDrop
			// In such a case, we want the re-entrant call to skip any drag-drop stuff (otherwise we'd execute the drop action twice)
			LocalDragDropContent = SlateUser->GetDragDropContent();
			SlateUser->ResetDragDropContent();
		}

		Reply = FEventRouter::Route<FReply>(this, FEventRouter::FBubblePolicy(LocalWidgetsUnderPointer), TransformedPointerEvent, [&](const FArrangedWidget& CurWidget, const FPointerEvent& Event)
		{
			if (bIsDragDropping)
			{
				FDragDropEvent LocalDropEvent(Event, LocalDragDropContent);
				const FReply TempDropReply = CurWidget.Widget->OnDrop(CurWidget.Geometry, LocalDropEvent);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::DragDrop, &LocalDropEvent, TempDropReply, CurWidget.Widget);
#endif
				return TempDropReply;
			}

			FReply TempReply = FReply::Unhandled();

			if (Event.IsTouchEvent())
			{
				TempReply = CurWidget.Widget->OnTouchEnded(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchEnd, &Event, TempReply, CurWidget.Widget);
#endif
			}

			if (!Event.IsTouchEvent() || (!TempReply.IsEventHandled() && bTouchFallbackToMouse))
			{
				TempReply = CurWidget.Widget->OnMouseButtonUp(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseButtonUp, &Event, TempReply, CurWidget.Widget);
#endif
			}

			return TempReply;
		}, ESlateDebuggingInputEvent::MouseButtonUp);
	}

	SlateUser->NotifyPointerReleased(PointerEvent, LocalWidgetsUnderPointer, LocalDragDropContent, Reply.IsEventHandled());

#if PLATFORM_MAC
	// Make sure the application and its front window are activated if user wasn't drag & dropping between windows
	if (PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton && !bIsDragDropping)
	{
		TSharedPtr<SWindow> ActiveWindow = GetActiveTopLevelWindow();
		if (ActiveWindow.IsValid() && !ActiveWindow->GetNativeWindow()->IsForegroundWindow() && !ActiveWindow->GetNativeWindow()->IsMinimized())
		{
			FPlatformApplicationMisc::ActivateApplication();

			if (!ActiveWindow->IsVirtualWindow())
			{
				ActiveWindow->BringToFront(true);
			}
		}
		else if (FPlatformApplicationMisc::GetActiveWindowPtr() == nullptr)
		{
			FPlatformApplicationMisc::ActivateApplication();
		}
	}
#endif

	return Reply;
}

bool FSlateApplication::RoutePointerMoveEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent, bool bIsSynthetic)
{
	TScopeCounter<int32> BeginInput(ProcessingInput);

	bool bHandled = false;
	FWeakWidgetPath LastWidgetsUnderPointer;

	FPointerEvent TransformedPointerEvent = WidgetsUnderPointer.IsValid() ? TransformPointerEvent(PointerEvent, WidgetsUnderPointer.GetWindow()) : PointerEvent;

	TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(PointerEvent);
	SlateUser->NotifyPointerMoveBegin(PointerEvent);

	// Currently we support only one dragged widget at a time per user
	bool bShouldStartDetectingDrag = !SlateUser->IsDragDropping();

#if WITH_EDITOR
	//@TODO VREDITOR - Remove and move to interaction component
	if (bShouldStartDetectingDrag && OnDragDropCheckOverride.IsBound())
	{
		bShouldStartDetectingDrag = OnDragDropCheckOverride.Execute();
	}
#endif 

	if (!bIsSynthetic && bShouldStartDetectingDrag)
	{
		FWidgetPath DragDetectPath = SlateUser->DetectDrag(PointerEvent, GetDragTriggerDistance());
		if (DragDetectPath.IsValid())
		{
			FWidgetAndPointer DetectDragForMe = DragDetectPath.FindArrangedWidgetAndCursor(DragDetectPath.GetLastWidget()).Get(FWidgetAndPointer());

			// A drag has been triggered. The cursor exited some widgets as a result.
			// This assignment ensures that we will send OnLeave notifications to those widgets.
			LastWidgetsUnderPointer = DragDetectPath;

			// Switch worlds widgets in the current path
			FScopedSwitchWorldHack SwitchWorld(DragDetectPath);

			// Send an OnDragDetected to the widget that requested drag detection.
			FReply Reply = FEventRouter::Route<FReply>(this, FEventRouter::FDirectPolicy(DetectDragForMe, DragDetectPath, &WidgetsUnderPointer), PointerEvent, [](const FArrangedWidget& InDetectDragForMe, const FPointerEvent& TranslatedMouseEvent)
				{
					const FReply TempReply = InDetectDragForMe.Widget->OnDragDetected(InDetectDragForMe.Geometry, TranslatedMouseEvent);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::DragDetected, &TranslatedMouseEvent, TempReply, InDetectDragForMe.Widget);
#endif
					return TempReply;
				}, ESlateDebuggingInputEvent::DragDetected);
		}
	}

	// A drag was detected if the user is now executing a drag-drop action
	if (bShouldStartDetectingDrag && SlateUser->IsDragDropping())
	{
		// When a drag was detected, we pretend that the widgets under the mouse last time around.
		// We have set LastWidgetsUnderCursor accordingly when the drag was detected above.
	}
	else
	{
		LastWidgetsUnderPointer = SlateUser->GetWidgetsUnderPointerLastEventByIndex().FindRef(PointerEvent.GetPointerIndex());
	}

	FWidgetPath MouseCaptorPath = SlateUser->GetCaptorPath(PointerEvent.GetPointerIndex(), FWeakWidgetPath::EInterruptedPathHandling::ReturnInvalid, &PointerEvent);

	// Send out mouse leave events
	// If we are doing a drag and drop, we will send this event instead.
	TSharedPtr<FDragDropOperation> DragDropContent = SlateUser->GetDragDropContent();
	{
		FDragDropEvent DragDropEvent(PointerEvent, DragDropContent);
		// Switch worlds widgets in the current path
		FScopedSwitchWorldHack SwitchWorld(LastWidgetsUnderPointer.Window.Pin());

		for (int32 WidgetIndex = LastWidgetsUnderPointer.Widgets.Num()-1; WidgetIndex >=0; --WidgetIndex)
		{
			// Guards for cases where WidgetIndex can become invalid due to MouseMove being re-entrant.
			if (WidgetIndex >= LastWidgetsUnderPointer.Widgets.Num())
			{
				WidgetIndex = LastWidgetsUnderPointer.Widgets.Num() - 1;
			}

			if (WidgetIndex >= 0)
			{
				const TSharedPtr<SWidget>& SomeWidgetPreviouslyUnderCursor = LastWidgetsUnderPointer.Widgets[WidgetIndex].Pin();
				if (SomeWidgetPreviouslyUnderCursor.IsValid())
				{
					TOptional<FArrangedWidget> FoundWidget = WidgetsUnderPointer.FindArrangedWidget(SomeWidgetPreviouslyUnderCursor.ToSharedRef());
					const bool bWidgetNoLongerUnderMouse = !FoundWidget.IsSet();
					
					bool bWidgetUnderOtherUsersPointer = false;
					
					// Verify if the Widget is under another user pointer
					if (bWidgetNoLongerUnderMouse)
					{
						for (const TSharedPtr<FSlateUser>& user : Users)
						{
							if (user != nullptr && user->GetUserIndex() != PointerEvent.GetUserIndex())
							{
								if (user->IsWidgetDirectlyUnderAnyPointer(SomeWidgetPreviouslyUnderCursor))
								{
									bWidgetUnderOtherUsersPointer = true;

									break;
								}
							}
						}
					}

					// We consider the Widget is nolonger under a pointer if it's no longer under any pointers.
					if (bWidgetNoLongerUnderMouse && !bWidgetUnderOtherUsersPointer)
					{
						// Widget is no longer under cursor, so send a MouseLeave.
						// The widget might not even be in the hierarchy any more!
						// Thus, we cannot translate the PointerPosition into the appropriate space for this event.
						if (SlateUser->IsDragDroppingAffected(PointerEvent))
						{
							// Note that the event's pointer position is not translated.
							SomeWidgetPreviouslyUnderCursor->OnDragLeave(DragDropEvent);
#if WITH_SLATE_DEBUGGING
							FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::DragLeave, &DragDropEvent, SomeWidgetPreviouslyUnderCursor);
#endif
							// Reset the cursor override
							DragDropEvent.GetOperation()->SetCursorOverride(TOptional<EMouseCursor::Type>());
						}
						else
						{
							// Only fire mouse leave events for widgets inside the captor path, or whoever if there is no captor path.
							if (MouseCaptorPath.IsValid() == false || MouseCaptorPath.ContainsWidget(SomeWidgetPreviouslyUnderCursor.Get()))
							{
								// Note that the event's pointer position is not translated.
								SomeWidgetPreviouslyUnderCursor->OnMouseLeave(PointerEvent);
#if WITH_SLATE_DEBUGGING
								FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseLeave, &PointerEvent, SomeWidgetPreviouslyUnderCursor);
#endif
							}
						}
					}
				}
			}
		}
	}

	if (MouseCaptorPath.IsValid())
	{
		// Switch worlds widgets in the current path
		FScopedSwitchWorldHack SwitchWorld(MouseCaptorPath);

		FEventRouter::Route<FNoReply>(this, FEventRouter::FBubblePolicy(WidgetsUnderPointer), TransformedPointerEvent, [&MouseCaptorPath, &LastWidgetsUnderPointer](const FArrangedWidget& WidgetUnderCursor, const FPointerEvent& Event)
			{
				if (!LastWidgetsUnderPointer.ContainsWidget(WidgetUnderCursor.GetWidgetPtr()))
				{
					if (MouseCaptorPath.ContainsWidget(WidgetUnderCursor.GetWidgetPtr()))
					{
						WidgetUnderCursor.Widget->OnMouseEnter(WidgetUnderCursor.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::MouseEnter, &Event, WidgetUnderCursor.Widget);
#endif
					}
				}
				return FNoReply();
			}, ESlateDebuggingInputEvent::MouseEnter);

		FReply Reply = FEventRouter::Route<FReply>(this, FEventRouter::FToLeafmostPolicy(MouseCaptorPath), TransformedPointerEvent, [this, bIsSynthetic](const FArrangedWidget& MouseCaptorWidget, const FPointerEvent& Event)
			{
				FReply TempReply = FReply::Unhandled();

				bool bAllowMouseFallback = true;
				if (Event.IsTouchEvent())
				{
					if (Event.IsTouchForceChangedEvent())
					{
						TempReply = MouseCaptorWidget.Widget->OnTouchForceChanged(MouseCaptorWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchForceChanged, &Event, TempReply, MouseCaptorWidget.Widget);
#endif
						bAllowMouseFallback = false;
					}
					else if (Event.IsTouchFirstMoveEvent())
					{
						TempReply = MouseCaptorWidget.Widget->OnTouchFirstMove(MouseCaptorWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchFirstMove, &Event, TempReply, MouseCaptorWidget.Widget);
#endif
						bAllowMouseFallback = false;
					}
					else
					{
						TempReply = MouseCaptorWidget.Widget->OnTouchMoved(MouseCaptorWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchMoved, &Event, TempReply, MouseCaptorWidget.Widget);
#endif
					}
				}
				if ((!Event.IsTouchEvent() && bAllowMouseFallback) || (!TempReply.IsEventHandled() && this->bTouchFallbackToMouse))
				{
					// Only handle if not synthetic, else widgets with mouse capture can cause the mouse to move at app start.
					if (!bIsSynthetic)
					{
						TempReply = MouseCaptorWidget.Widget->OnMouseMove(MouseCaptorWidget.Geometry, Event);
					}
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseMove, &Event, TempReply, MouseCaptorWidget.Widget);
#endif
				}
				return TempReply;
			}, ESlateDebuggingInputEvent::MouseEnter);
		bHandled = Reply.IsEventHandled();
	}
	else
	{
		// Switch worlds widgets in the current path
		FScopedSwitchWorldHack SwitchWorld(WidgetsUnderPointer);

		const bool bIsDragDroppingAffected = SlateUser->IsDragDroppingAffected(PointerEvent);

		// Send out mouse enter events.
		if (bIsDragDroppingAffected)
		{
			FDragDropEvent DragDropEvent(PointerEvent, DragDropContent);
			FEventRouter::Route<FNoReply>(this, FEventRouter::FBubblePolicy(WidgetsUnderPointer), DragDropEvent, [&LastWidgetsUnderPointer](const FArrangedWidget& WidgetUnderCursor, const FDragDropEvent& InDragDropEvent)
			{
				if (!LastWidgetsUnderPointer.ContainsWidget(WidgetUnderCursor.GetWidgetPtr()))
				{
					WidgetUnderCursor.Widget->OnDragEnter(WidgetUnderCursor.Geometry, InDragDropEvent);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::DragEnter, &InDragDropEvent, WidgetUnderCursor.Widget);
#endif
				}
				return FNoReply();
			}, ESlateDebuggingInputEvent::DragEnter);
		}
		else
		{
			FEventRouter::Route<FNoReply>(this, FEventRouter::FBubblePolicy(WidgetsUnderPointer), TransformedPointerEvent, [&LastWidgetsUnderPointer](const FArrangedWidget& WidgetUnderCursor, const FPointerEvent& Event)
			{
				if (!LastWidgetsUnderPointer.ContainsWidget(WidgetUnderCursor.GetWidgetPtr()))
				{
					WidgetUnderCursor.Widget->OnMouseEnter(WidgetUnderCursor.Geometry, Event);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent::MouseEnter, &Event, WidgetUnderCursor.Widget);
#endif
				}
				return FNoReply();
			}, ESlateDebuggingInputEvent::MouseEnter);
		}

		// Bubble the MouseMove event.
		FReply Reply = FEventRouter::Route<FReply>(this, FEventRouter::FBubblePolicy(WidgetsUnderPointer), TransformedPointerEvent, [&](const FArrangedWidget& CurWidget, const FPointerEvent& Event)
		{
			FReply TempReply = FReply::Unhandled();

			if (bIsDragDroppingAffected)
			{
				FDragDropEvent DragDropEvent(Event, DragDropContent);
				TempReply = CurWidget.Widget->OnDragOver(CurWidget.Geometry, DragDropEvent);
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::DragOver, &DragDropEvent, TempReply, CurWidget.Widget);
#endif
			}
			else
			{
				bool bAllowMouseFallback = true;
				if (Event.IsTouchEvent())
				{
					if (Event.IsTouchForceChangedEvent())
					{
						TempReply = CurWidget.Widget->OnTouchForceChanged(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchForceChanged, &Event, TempReply, CurWidget.Widget);
#endif
						bAllowMouseFallback = false;
					}
					else if (Event.IsTouchFirstMoveEvent())
					{
						TempReply = CurWidget.Widget->OnTouchFirstMove(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchFirstMove, &Event, TempReply, CurWidget.Widget);
#endif
						bAllowMouseFallback = false;
					}
					else
					{
						TempReply = CurWidget.Widget->OnTouchMoved(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchMoved, &Event, TempReply, CurWidget.Widget);
#endif
					}
				}
				if (!TempReply.IsEventHandled() && bAllowMouseFallback)
				{
					TempReply = CurWidget.Widget->OnMouseMove(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseMove, &Event, TempReply, CurWidget.Widget);
#endif
				}
			}

			return TempReply;
		}, ESlateDebuggingInputEvent::MouseMove);

		bHandled = Reply.IsEventHandled();
	}

	SlateUser->NotifyPointerMoveComplete(PointerEvent, WidgetsUnderPointer);

	return bHandled;
}

bool FSlateApplication::OnMouseDoubleClick( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button )
{
	return OnMouseDoubleClick(PlatformWindow, Button, GetCursorPos());
}

bool FSlateApplication::OnMouseDoubleClick( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button, const FVector2D CursorPos )
{
	if (IsFakingTouchEvents())
	{
		bIsFakingTouched = true;
		return OnTouchStarted(PlatformWindow, PlatformApplication->Cursor->GetPosition(), 1.0f, /* touch index */ 0, FSlateApplicationBase::SlateAppPrimaryPlatformUser, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());
	}

	FKey Key = TranslateMouseButtonToKey( Button );

	// Update PressedMouseButtons before constructing the event — same reasoning as OnMouseDown.
	if (GetUserIndexForMouse() == CursorUserIndex)
	{
		PressedMouseButtons.Add(Key);
	}

	FPointerEvent MouseEvent(
		GetUserIndexForMouse(),
		CursorPointerIndex,
		CursorPos,
		GetLastCursorPos(),
		PressedMouseButtons,
		Key,
		0,
		PlatformApplication->GetModifierKeys()
		);

	return ProcessMouseButtonDoubleClickEvent( PlatformWindow, MouseEvent );
}

bool FSlateApplication::ProcessMouseButtonDoubleClickEvent( const TSharedPtr< FGenericWindow >& PlatformWindow, const FPointerEvent& InMouseEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessMouseButtonDoubleClick);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::MouseButtonDoubleClick, InMouseEvent);
#endif

	SetLastUserInteractionTime(this->GetCurrentTime());
	LastUserInteractionTimeForThrottling = LastUserInteractionTime;

	PlatformApplication->SetCapture( PlatformWindow );

	if (InMouseEvent.GetUserIndex() == CursorUserIndex)
	{
		PressedMouseButtons.Add( InMouseEvent.GetEffectingButton() );
	}

	// Input preprocessors get the first chance at the input
	if (InputPreProcessors.HandleMouseButtonDoubleClickEvent(*this, InMouseEvent))
	{
		return true;
	}

	if (GetOrCreateUser(InMouseEvent)->HasCapture(InMouseEvent.GetPointerIndex()))
	{
		// If a widget has mouse capture, we've opted to simply treat this event as a mouse down
		return ProcessMouseButtonDownEvent(PlatformWindow, InMouseEvent);
	}
	
	FWidgetPath WidgetsUnderCursor = LocateWindowUnderMouse(InMouseEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, InMouseEvent.GetUserIndex());

	FReply Reply = RoutePointerDoubleClickEvent( WidgetsUnderCursor, InMouseEvent );

	return Reply.IsEventHandled();
}


FReply FSlateApplication::RoutePointerDoubleClickEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent)
{
	TScopeCounter<int32> BeginInput(ProcessingInput);

	FReply Reply = FReply::Unhandled();

	// Switch worlds widgets in the current path
	FScopedSwitchWorldHack SwitchWorld( WidgetsUnderPointer );

	FPointerEvent TransformedPointerEvent = WidgetsUnderPointer.IsValid() ? TransformPointerEvent(PointerEvent, WidgetsUnderPointer.GetWindow()) : PointerEvent;

	Reply = FEventRouter::Route<FReply>( this, FEventRouter::FBubblePolicy( WidgetsUnderPointer ), TransformedPointerEvent, []( const FArrangedWidget& TargetWidget, const FPointerEvent& Event )
	{
		const FReply TempReply = TargetWidget.Widget->OnMouseButtonDoubleClick(TargetWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseButtonDoubleClick, &Event, TempReply, TargetWidget.Widget);
#endif
		return TempReply;
	}, ESlateDebuggingInputEvent::MouseButtonDoubleClick);

	return Reply;
}


bool FSlateApplication::OnMouseUp( const EMouseButtons::Type Button )
{
	return OnMouseUp(Button, GetCursorPos());
}

bool FSlateApplication::OnMouseUp( const EMouseButtons::Type Button, const FVector2D CursorPos )
{
	// convert left mouse click to touch event if we are faking it	
	if (IsFakingTouchEvents() && Button == EMouseButtons::Left)
	{
		bIsFakingTouched = false;
		return OnTouchEnded(PlatformApplication->Cursor->GetPosition(), 0, FSlateApplicationBase::SlateAppPrimaryPlatformUser, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());
	}

	FKey Key = TranslateMouseButtonToKey( Button );

	// Update PressedMouseButtons before constructing the event so the value-copy snapshot
	// reflects the post-release state (button is no longer down). The matching Remove inside
	// ProcessMouseButtonUpEvent becomes a no-op for this path but is still needed for touch
	// events routed via ProcessTouchEndedEvent -> ProcessMouseButtonUpEvent directly.
	if (GetUserIndexForMouse() == CursorUserIndex)
	{
		PressedMouseButtons.Remove(Key);
	}

	FPointerEvent MouseEvent(
		GetUserIndexForMouse(),
		CursorPointerIndex,
		CursorPos,
		GetLastCursorPos(),
		PressedMouseButtons,
		Key,
		0,
		PlatformApplication->GetModifierKeys()
		);

	return ProcessMouseButtonUpEvent( MouseEvent );
}

bool FSlateApplication::ProcessMouseButtonUpEvent( const FPointerEvent& MouseEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessMouseButtonUp);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::MouseButtonUp, MouseEvent);
#endif

	// If in responsive mode throttle, leave it on mouse up.  Release this before dispatching the event to prevent being stuck in this mode
	// until the next click if a modal dialog is opened.
	if (MouseButtonDownResponsivnessThrottle.IsValid())
	{
		FSlateThrottleManager::Get().LeaveResponsiveMode(MouseButtonDownResponsivnessThrottle);
	}

	SetLastUserInteractionTime(this->GetCurrentTime());
	LastUserInteractionTimeForThrottling = LastUserInteractionTime;

	const bool bIsCursorUser = MouseEvent.GetUserIndex() == CursorUserIndex;
	if (bIsCursorUser)
	{
		PressedMouseButtons.Remove(MouseEvent.GetEffectingButton());
	}

	// Input preprocessors get the first chance at the input
	if (InputPreProcessors.HandleMouseButtonUpEvent(*this, MouseEvent))
	{
		// If mouse up event is consumed by a preprocessor, associated mouse down event needs to be cleared as well. Otherwise, subsequent mouse down events get ignored until the first one is cleared.
		// This was only affecting the swipe detection on touch, we can remove this condition if we want the same fix for other platforms, but reducing the scope for now
		if (MouseEvent.IsTouchEvent())
		{
			TSharedRef<FSlateUser> SlateUser = GetOrCreateUser(MouseEvent);
			FWidgetPath WidgetsUnderPointer = LocateWindowUnderMouse(MouseEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, SlateUser->GetUserIndex());

			SlateUser->NotifyPointerReleased(MouseEvent, WidgetsUnderPointer, nullptr, true);
		}
		return true;
	}

	// An empty widget path is passed in.  As an optimization, one will be generated only if a captured mouse event isn't routed
	FWidgetPath EmptyPath;
	const bool bHandled = RoutePointerUpEvent( EmptyPath, MouseEvent ).IsEventHandled();

	if ( bIsCursorUser && PressedMouseButtons.Num() == 0 )
	{
		PlatformApplication->SetCapture( nullptr );
	}

	return bHandled;
}

bool FSlateApplication::OnMouseWheel( const float Delta )
{
	return OnMouseWheel(Delta, GetCursorPos());
}

bool FSlateApplication::OnMouseWheel( const float Delta, const FVector2D CursorPos )
{
	FPointerEvent MouseWheelEvent(
		GetUserIndexForMouse(),
		CursorPointerIndex,
		CursorPos,
		CursorPos,
		PressedMouseButtons,
		EKeys::Invalid,
		Delta,
		PlatformApplication->GetModifierKeys()
		);

	return ProcessMouseWheelOrGestureEvent( MouseWheelEvent, nullptr );
}

bool FSlateApplication::ProcessMouseWheelOrGestureEvent( const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessMouseWheelGesture);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::MouseWheel, InWheelEvent);
#endif

	bool bShouldProcessEvent = false;

	if ( InGestureEvent )
	{
		switch ( InGestureEvent->GetGestureType() )
		{
		case EGestureEvent::LongPress:
			bShouldProcessEvent = true;
			break;
		default:
			bShouldProcessEvent = (InGestureEvent->GetGestureDelta() != FVector2f::ZeroVector) ||
				(InGestureEvent->GetGesturePhase() == EGesturePhase::Started) ||
				(InGestureEvent->GetGesturePhase() == EGesturePhase::Ended);
			break;
		}
	}
	else
	{
		bShouldProcessEvent = InWheelEvent.GetWheelDelta() != 0;
	}
	
	if ( !bShouldProcessEvent )
	{
		return false;
	}

	SetLastUserInteractionTime(this->GetCurrentTime());

	// Input preprocessors get the first chance at the input
	if (InputPreProcessors.HandleMouseWheelOrGestureEvent(*this, InWheelEvent, InGestureEvent))
	{
		return true;
	}
	
	// NOTE: We intentionally don't reset LastUserInteractionTimeForThrottling here so that the UI can be responsive while scrolling

	FWidgetPath EventPath = LocateWindowUnderMouse(InWheelEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, InWheelEvent.GetUserIndex());

	return RouteMouseWheelOrGestureEvent(EventPath, InWheelEvent, InGestureEvent).IsEventHandled();
}

FReply FSlateApplication::RouteMouseWheelOrGestureEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent)
{
	TScopeCounter<int32> BeginInput(ProcessingInput);

	FWidgetPath MouseCaptorPath;

	TSharedRef<FSlateUser> User = GetOrCreateUser(InWheelEvent);
	if (User->HasCapture(InWheelEvent.GetPointerIndex()))
	{
		MouseCaptorPath = User->GetCaptorPath(InWheelEvent.GetPointerIndex(), FWeakWidgetPath::EInterruptedPathHandling::ReturnInvalid, &InWheelEvent);
	}
		
	const FWidgetPath& EventPath = MouseCaptorPath.IsValid() ? MouseCaptorPath : WidgetsUnderPointer;

	// Switch worlds widgets in the current path
	FScopedSwitchWorldHack SwitchWorld(EventPath);

	FReply Reply = FEventRouter::Route<FReply>(this, FEventRouter::FBubblePolicy(EventPath), InWheelEvent, [&InGestureEvent] (const FArrangedWidget& CurWidget, const FPointerEvent& Event)
	{
		FReply TempReply = FReply::Unhandled();
		// Gesture event gets first shot, if slate doesn't respond to it, we'll try the wheel event.
		if ( InGestureEvent != nullptr )
		{
			TempReply = CurWidget.Widget->OnTouchGesture(CurWidget.Geometry, *InGestureEvent);
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::TouchGesture, InGestureEvent, TempReply, CurWidget.Widget);
#endif
		}

		// Send the mouse wheel event if we haven't already handled the gesture version of this event.
		if ( !TempReply.IsEventHandled() && Event.GetWheelDelta() != 0 )
		{
			TempReply = CurWidget.Widget->OnMouseWheel(CurWidget.Geometry, Event);
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MouseWheel, &Event, TempReply, CurWidget.Widget);
#endif
		}

		return TempReply;
	}, ESlateDebuggingInputEvent::MouseWheel);

	return Reply;
}

bool FSlateApplication::OnMouseMove()
{
	if (!PlatformApplication->Cursor.IsValid())
    {
    	return false;
    }

    // If the left button is pressed we fake 
	if (IsFakingTouchEvents() && (GetPressedMouseButtons().Num() == 0 || GetPressedMouseButtons().Contains(EKeys::LeftMouseButton)))
	{
		// convert to touch event if we are faking it
		if (bIsFakingTouched)
		{
			return OnTouchMoved(PlatformApplication->Cursor->GetPosition(), 1.0f, 0, FSlateApplicationBase::SlateAppPrimaryPlatformUser, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());
		}

		// Throw out the mouse move event if we're faking touch events but the mouse button isn't down.
		return false;
	}

	bool Result = true;
	const FVector2f CurrentCursorPosition = UE::Slate::CastToVector2f(PlatformApplication->Cursor->GetPosition());
	if (LastPlatformCursorPosition != CurrentCursorPosition)
	{
		// Force the cursor user index to use the platform cursor since we've been notified that the platform 
		// cursor position has changed. This is done intentionally after getting the positions in order to avoid
		// false positives.

		// NOTE: When we swap out the real OS cursor for the faux slate cursor ie. UsePlatformCursorForCursorUser(false)
		// we reset this event count to 0.  This occurs typically when a gamepad is being used and you don't want to manipulate the real
		// OS cursor, instead move around a fake cursor so that you can still do development stuff outside the game window.  Anyway,
		// when this occurs, in the future the OS will send you a long delayed mouse movement.  I'm not exactly sure what's triggering it,
		// it's not a movement from the application, I think it's something more subtle, like swapping true cursor visibility for using
		// a None cursor, it could also be some combination.
		// 
		// In any event, we track the number of events, and if we get more than 3, then we start trying to swap back to the OS cursor.
		// the 3 should give us any buffer needed for either a last frame mouse movement, or a weird condition like noted above.
		PlatformMouseMovementEvents++;

		if (PlatformMouseMovementEvents > 3)
		{
			UsePlatformCursorForCursorUser(true);
		}

		LastMouseMoveTime = GetCurrentTime();

		FPointerEvent MouseEvent(
			GetUserIndexForMouse(),
			CursorPointerIndex,
			CurrentCursorPosition,
			LastPlatformCursorPosition,
			PressedMouseButtons,
			EKeys::Invalid,
			0,
			PlatformApplication->GetModifierKeys()
			);

		Result = ProcessMouseMoveEvent( MouseEvent );
		LastPlatformCursorPosition = CurrentCursorPosition;
	}

	return Result;
}

bool FSlateApplication::OnRawMouseMove( const int32 X, const int32 Y )
{
    // We fake a move only if the left mous button is down
	if (IsFakingTouchEvents() && (GetPressedMouseButtons().Num() == 0 || GetPressedMouseButtons().Contains(EKeys::LeftMouseButton)))
	{
		// convert to touch event if we are faking it
		if (bIsFakingTouched)
		{
			return OnTouchMoved(GetCursorPos(), 1.0f, 0, FSlateApplicationBase::SlateAppPrimaryPlatformUser, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());
		}

		// Throw out the mouse move event if we're faking touch events but the mouse button isn't down.
		return false;
	}

	if ( X != 0 || Y != 0 )
	{
		FPointerEvent MouseEvent(
			GetUserIndexForMouse(),
			CursorPointerIndex,
			GetCursorPos(),
			GetLastCursorPos(),
			FVector2f( X, Y ), 
			PressedMouseButtons,
			PlatformApplication->GetModifierKeys()
		);

		ProcessMouseMoveEvent(MouseEvent);
	}
	
	return true;
}

bool FSlateApplication::ProcessMouseMoveEvent( const FPointerEvent& MouseEvent, bool bIsSynthetic )
{
	SCOPE_CYCLE_COUNTER(STAT_ProcessMouseMove);

	if (IsFakingTouchEvents() && !MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		// If we're faking touch events and the left mouse button is not down, do not process the mouse move event
		return false;
	}
   
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::MouseMove, MouseEvent);
#endif

	if ( !bIsSynthetic )
	{
		if (InputPreProcessors.HandleMouseMoveEvent(*this, MouseEvent))
		{
			return true;
		}

		// Guard against synthesized mouse moves and only track user interaction if the cursor pos changed
		SetLastUserInteractionTime(this->GetCurrentTime());
	}

	// When the event came from the OS, we are guaranteed to be over a slate window.
	// Otherwise, we are synthesizing a MouseMove ourselves, and must verify that the
	// cursor is indeed over a Slate window.  Synthesized device (gamepad) input while
	// the application is inactive also needs to populate the widget path.
	const bool bOverSlateWindow = !bIsSynthetic || IsActive() || PlatformApplication->IsCursorDirectlyOverSlateWindow() || GetHandleDeviceInputWhenApplicationNotActive();
   
	FWidgetPath WidgetsUnderCursor = bOverSlateWindow
		? LocateWindowUnderMouse(MouseEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, MouseEvent.GetUserIndex())
		: FWidgetPath();

	bool bResult;

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ProcessMouseMove_RoutePointerMoveEvent);
		bResult = RoutePointerMoveEvent(WidgetsUnderCursor, MouseEvent, bIsSynthetic);
	}

	return bResult;
}

bool FSlateApplication::OnCursorSet()
{
	GetCursorUser()->RequestCursorQuery();
	return true;
}

void FSlateApplication::NavigateToWidget(const uint32 UserIndex, const TSharedPtr<SWidget>& NavigationDestination, ENavigationSource NavigationSource, EUINavigation NavigationType)
{
	if (NavigationDestination.IsValid())
	{
		FWidgetPath NavigationSourceWP;
		if (NavigationSource == ENavigationSource::WidgetUnderCursor)
		{
			NavigationSourceWP = LocateWindowUnderMouse(GetOrCreateUser(UserIndex)->GetCursorPosition(), GetInteractiveTopLevelWindows(), false, UserIndex);
		}
		else
		{
			NavigationSourceWP = *GetOrCreateUser(UserIndex)->GetFocusPath();
		}

		if (NavigationSourceWP.IsValid())
		{
			FScopedNavigationTransition ScopedNavigationTransition(NavigationType, GetUser(UserIndex));

			bool bAlwaysHandleNavigationAttempt = false;
			ExecuteNavigation(NavigationSourceWP, NavigationDestination, UserIndex, bAlwaysHandleNavigationAttempt);
		}
	}
}

EUINavigation FSlateApplication::NavigateFromWidget(const uint32 InUserIndex, TSharedPtr<SWidget> InNavigationSource, const TArray<EUINavigation>& InNavigationTypes)
{
	FWidgetPath NavigationSourcePath;
	if (InNavigationSource.IsValid())
	{
		FindPathToWidget(InNavigationSource.ToSharedRef(), NavigationSourcePath, EVisibility::All);
	}
	else
	{
		NavigationSourcePath = Users[InUserIndex]->GetFocusPath().Get();
	}

	if (!NavigationSourcePath.IsValid())
	{
		UE_LOGF(LogSlate, Log, "Failed to navigate from widget %ls. No widget currently in Focus", *(InNavigationSource->ToString()));
		return EUINavigation::Invalid;
	}

	// Traverse up the widget tree and attempt to navigate to next available widget by searching neighbor space
	for (int32 WidgetIndex = NavigationSourcePath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		const FArrangedWidget& BoundaryWidget = NavigationSourcePath.Widgets[WidgetIndex];

		for (EUINavigation Direction : InNavigationTypes)
		{
			// Try to advance to next widget
			const FNavigationEvent NavigationEventNext(PlatformApplication->GetModifierKeys(), InUserIndex, Direction, ENavigationGenesis::User);
			if (AttemptNavigation(NavigationSourcePath, NavigationEventNext, FNavigationReply::Escape(), BoundaryWidget))
			{
				return Direction;
			}
		}
	}

	UE_LOGF(LogSlate, Log, "Failed to navigate from widget %ls. No neighboring widgets could be reached.", *(InNavigationSource->ToString()));
	return EUINavigation::Invalid;
}

FSlateApplication::FNavigationResult FSlateApplication::CalculateDestinationWidget(const FWidgetPath& NavigationSource, const FNavigationReply& NavigationReply, EUINavigation NavigationType, uint32 UserIndex, const FArrangedWidget& BoundaryWidget) const
{
	if (!NavigationSource.IsValid())
	{
		return FNavigationResult();
	}

	FNavigationResult Result;

	if ( NavigationReply.GetBoundaryRule() == EUINavigationRule::Explicit )
	{
		const SWidget* FocusRecipient = NavigationReply.GetFocusRecipient().Get();
		if ( FocusRecipient && FocusRecipient->IsEnabled() && FocusRecipient->SupportsKeyboardFocus() )
		{
			Result.DestinationWidget = NavigationReply.GetFocusRecipient();
			Result.bAlwaysHandleNavigationAttempt = true;

#if WITH_SLATE_DEBUGGING
			Result.NavigationMethod = ESlateDebuggingNavigationMethod::Explicit;
#endif
		}
#if WITH_SLATE_DEBUGGING
		else
		{
			const TCHAR* Reason = TEXT("Unknown");
			if (!FocusRecipient)
			{
				Reason = TEXT("Widget is a nullptr");
			}
			else if (!FocusRecipient->IsEnabled())
			{
				Reason = TEXT("Widget disabled");
			}
			else
			{
				ensure(!FocusRecipient->SupportsKeyboardFocus());
				Reason = TEXT("Widget does not support keyboard focus");
			}

			UE_LOGF(LogSlate, VeryVerbose, "Could not Explicitly navigate to widget '%ls' because '%ls'", *FReflectionMetaData::GetWidgetDebugInfo(FocusRecipient), Reason);
		}
#endif
	}
	else if ( NavigationReply.GetBoundaryRule() == EUINavigationRule::Custom )
	{
		const FNavigationDelegate& FocusDelegate = NavigationReply.GetFocusDelegate();
		if ( FocusDelegate.IsBound() )
		{
			// Switch worlds for widgets in the current path 
			FScopedSwitchWorldHack SwitchWorld(NavigationSource);

			Result.DestinationWidget = FocusDelegate.Execute(NavigationType);
			Result.bAlwaysHandleNavigationAttempt = true;

#if WITH_SLATE_DEBUGGING
			Result.NavigationMethod = ESlateDebuggingNavigationMethod::CustomDelegateBound;
#endif
		}
		else
		{
#if WITH_SLATE_DEBUGGING
			Result.NavigationMethod = ESlateDebuggingNavigationMethod::CustomDelegateUnbound;
#endif
		}
	}
	else
	{
		// Find the next widget
		if (NavigationType == EUINavigation::Next || NavigationType == EUINavigation::Previous)
		{
			// Fond the next widget
			FWeakWidgetPath WeakNavigationSource(NavigationSource);
			FWidgetPath NewFocusedWidgetPath = WeakNavigationSource.ToNextFocusedPath(NavigationType, NavigationReply, BoundaryWidget);

			checkf(NewFocusedWidgetPath.IsValid(), TEXT("This was assuming the `NewFocusedWidgetPath` is valid already"));

			// Resolve the Widget Path
			FArrangedWidget& NewFocusedArrangedWidget = NewFocusedWidgetPath.Widgets.Last();
			Result.DestinationWidget = NewFocusedArrangedWidget.Widget;

#if WITH_SLATE_DEBUGGING
			Result.NavigationMethod = ESlateDebuggingNavigationMethod::NextOrPrevious;
#endif
		}
		else
		{
			// Resolve the Widget Path
			const FArrangedWidget& FocusedArrangedWidget = NavigationSource.Widgets.Last();

			// Switch worlds for widgets in the current path 
			FScopedSwitchWorldHack SwitchWorld(NavigationSource);

			Result.DestinationWidget = NavigationSource.GetDeepestWindow()->GetHittestGrid().FindNextFocusableWidget(FocusedArrangedWidget, NavigationType, NavigationReply, BoundaryWidget, UserIndex);

#if WITH_SLATE_DEBUGGING
			Result.NavigationMethod = ESlateDebuggingNavigationMethod::HitTestGrid;
#endif
		}
	}

	return Result;
}

bool FSlateApplication::AttemptNavigation(const FWidgetPath& NavigationSource, const FNavigationEvent& NavigationEvent, const FNavigationReply& NavigationReply, const FArrangedWidget& BoundaryWidget)
{
	if ( !NavigationSource.IsValid() )
	{
		return false;
	}

	const EUINavigation NavigationType = NavigationEvent.GetNavigationType();
	const uint32 UserIndex = NavigationEvent.GetUserIndex();
	const FNavigationResult DestinationResult = CalculateDestinationWidget(NavigationSource, NavigationReply, NavigationType, UserIndex, BoundaryWidget);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastAttemptNavigation(NavigationEvent, NavigationReply, NavigationSource, DestinationResult.DestinationWidget, DestinationResult.NavigationMethod);
#endif

	FScopedNavigationTransition ScopedNavigationTransition(NavigationType, GetUser(UserIndex));

	return ExecuteNavigation(NavigationSource, DestinationResult.DestinationWidget, NavigationEvent.GetUserIndex(), DestinationResult.bAlwaysHandleNavigationAttempt);
}

bool FSlateApplication::ExecuteNavigation(const FWidgetPath& NavigationSource, TSharedPtr<SWidget> DestinationWidget, const uint32 UserIndex, bool bAlwaysHandleNavigationAttempt)
{
#if WITH_SLATE_DEBUGGING
	// TODO Execute Navigation
#endif

	bool bHandled = false;

	// Give the custom viewport navigation event handler a chance to handle the navigation if the NavigationSource is contained within it.
	TSharedPtr<ISlateViewport> Viewport = NavigationSource.GetWindow()->GetViewport();
	if (Viewport.IsValid())
	{
		TSharedPtr<SWidget> ViewportWidget = Viewport->GetWidget().Pin();
		if (ViewportWidget.IsValid())
		{
			if (NavigationSource.ContainsWidget(ViewportWidget.Get()))
			{
				bHandled = Viewport->HandleNavigation(UserIndex, DestinationWidget);
			}
		}
	}

	// Set controller focus if the navigation hasn't been handled have a valid widget
	if (!bHandled)
	{
		if (DestinationWidget.IsValid())
		{
			SetUserFocus(UserIndex, DestinationWidget, EFocusCause::Navigation);
			bHandled = true;
		}
		else if (bAlwaysHandleNavigationAttempt)
		{
			bHandled = true;
		}
	}

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastExecuteNavigation();
#endif

	return bHandled;
}


void FSlateApplication::OnMenuDestroyed(const TSharedRef<IMenu>& Menu)
{
	if (MenuBeingDestroyedEvent.IsBound())
	{
		MenuBeingDestroyedEvent.Broadcast(Menu);
	}
}

bool FSlateApplication::OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue)
{
	FKey Key(KeyName);
	TOptional<int32> UserIndex = GetUserIndexForInputDevice(InputDeviceId);
	if (UserIndex.IsSet() && ensureMsgf(Key.IsValid(), TEXT("OnControllerAnalog(KeyName=%s,InputDeviceId=%d,AnalogValue=%f) key is invalid"), *KeyName.ToString(), InputDeviceId.GetId(), AnalogValue))
	{
		FAnalogInputEvent AnalogInputEvent(Key, PlatformApplication->GetModifierKeys(), InputDeviceId, false, 0, 0, AnalogValue, UserIndex);
		
		return ProcessAnalogInputEvent(AnalogInputEvent);
	}
	return false;
}

bool FSlateApplication::OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	FKey Key(KeyName);
	TOptional<int32> UserIndex = GetUserIndexForInputDevice(InputDeviceId);
	if (UserIndex.IsSet() && ensureMsgf(Key.IsValid(), TEXT("OnControllerButtonPressed(KeyName=%s,InputDeviceId=%d,IsRepeat=%u) key is invalid"), *KeyName.ToString(), InputDeviceId.GetId(), IsRepeat))
	{
		FKeyEvent KeyEvent(Key, PlatformApplication->GetModifierKeys(), InputDeviceId, IsRepeat, 0, 0, UserIndex);
		
		return ProcessKeyDownEvent(KeyEvent);
	}
	return false;
}

bool FSlateApplication::OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	FKey Key(KeyName);
	TOptional<int32> UserIndex = GetUserIndexForInputDevice(InputDeviceId);
	if (UserIndex.IsSet() && ensureMsgf(Key.IsValid(), TEXT("OnControllerButtonReleased(KeyName=%s,InputDeviceId=%d,IsRepeat=%u) key is invalid"), *KeyName.ToString(), InputDeviceId.GetId(), IsRepeat))
	{
		FKeyEvent KeyEvent(Key, PlatformApplication->GetModifierKeys(), InputDeviceId, IsRepeat, 0, 0, UserIndex);
		
		return ProcessKeyUpEvent(KeyEvent);
	}
	return false;
}

bool FSlateApplication::HandleTouchGesture( FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D &Delta, const float Value, bool bIsDirectionInvertedFromDevice )
{
	const FVector2f CurrentCursorPosition = GetCursorPos();

	FPointerEvent GestureEvent(
		DeviceId,
		CurrentCursorPosition,
		CurrentCursorPosition,
		PressedMouseButtons,
		PlatformApplication->GetModifierKeys(),
		GestureType,
		GesturePhase,
		Delta,
		bIsDirectionInvertedFromDevice
	);
	
	FPointerEvent MouseWheelEvent(
		DeviceId,
		CursorPointerIndex,
		CurrentCursorPosition,
		CurrentCursorPosition,
		PressedMouseButtons,
		EKeys::Invalid,
		Value,
		PlatformApplication->GetModifierKeys()
	);
	
	return ProcessMouseWheelOrGestureEvent( MouseWheelEvent, &GestureEvent );
}

bool FSlateApplication::OnTouchGesture( EGestureEvent GestureType, const FVector2D &Delta, const float MouseWheelDelta, bool bIsDirectionInvertedFromDevice )
{
	// Software gesture path (e.g. Mac trackpad via GestureRecognizer) has no device ID.
	// Use device 0 to preserve the original UserIndex=0 routing behavior.
	return HandleTouchGesture(FInputDeviceId::CreateFromInternalId(0), GestureType, EGesturePhase::Update, Delta, MouseWheelDelta, bIsDirectionInvertedFromDevice);
}

bool FSlateApplication::OnNativeGestureBegin(EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId)
{
	return HandleTouchGesture(DeviceId, GestureType, EGesturePhase::Started, Delta, Value, bIsDirectionInvertedFromDevice);
}

bool FSlateApplication::OnNativeGestureUpdate(EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId)
{
	return HandleTouchGesture(DeviceId, GestureType, EGesturePhase::Update, Delta, Value, bIsDirectionInvertedFromDevice);
}

bool FSlateApplication::OnNativeGestureEnd(EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId)
{
	return HandleTouchGesture(DeviceId, GestureType, EGesturePhase::Ended, Delta, Value, bIsDirectionInvertedFromDevice);
}

bool ValidateTouchIndex(int32 TouchIndex)
{
	return true;
}

bool FSlateApplication::OnTouchStarted( const TSharedPtr< FGenericWindow >& PlatformWindow, const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId )
{
	// Don't process touches that overlap or surpass with the cursor pointer index.
	if (TouchIndex >= (int32)ETouchIndex::CursorPointerIndex)
	{
#if WITH_SLATE_DEBUGGING
		// Only log when the touch starts, we don't want to spam the logs.
		UE_LOGF(LogSlate, Warning, "Maximum Touch Index Exceeded, %d, the maximum index allowed is %d", TouchIndex, (((int32)ETouchIndex::CursorPointerIndex) - 1));
#endif
		return false;
	}

	FPointerEvent PointerEvent(
		DeviceId,
		TouchIndex,
		Location,
		Location,
		Force,
		true);
	ProcessTouchStartedEvent( PlatformWindow, PointerEvent );

	return true;
}

void FSlateApplication::ProcessTouchStartedEvent( const TSharedPtr< FGenericWindow >& PlatformWindow, const FPointerEvent& InTouchEvent )
{
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::TouchStart, InTouchEvent);
#endif

	GetOrCreateUser(InTouchEvent)->NotifyTouchStarted(InTouchEvent);
	ProcessMouseButtonDownEvent(PlatformWindow, InTouchEvent);
}

bool FSlateApplication::OnTouchMoved( const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID )
{
	TSharedRef<FSlateUser> User = GetOrCreateUser(DeviceID);
	if (User->IsTouchPointerActive(TouchIndex))
	{
		FPointerEvent PointerEvent(
			DeviceID,
			TouchIndex,
			Location,
			User->GetPreviousPointerPosition(TouchIndex),
			Force,
			true);
		ProcessTouchMovedEvent(PointerEvent);

		return true;
	}

	return false;
}

void FSlateApplication::ProcessTouchMovedEvent( const FPointerEvent& PointerEvent )
{
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::TouchMoved, PointerEvent);
#endif

	ProcessMouseMoveEvent(PointerEvent);
}

bool FSlateApplication::OnTouchEnded( const FVector2D& Location, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID )
{
	TSharedRef<FSlateUser> User = GetOrCreateUser(DeviceID);
	if (User->IsTouchPointerActive(TouchIndex))
	{
		FPointerEvent PointerEvent(
			DeviceID,
			TouchIndex,
			Location,
			Location,
			0.0f,
			true);
		ProcessTouchEndedEvent(PointerEvent);

		return true;
	}

	return false;
}

void FSlateApplication::ProcessTouchEndedEvent(const FPointerEvent& PointerEvent)
{
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::TouchEnd, PointerEvent);
#endif

	ProcessMouseButtonUpEvent(PointerEvent);
}

bool FSlateApplication::OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID)
{
	TSharedRef<FSlateUser> User = GetOrCreateUser(DeviceID);
	if (User->IsTouchPointerActive(TouchIndex))
	{
		FPointerEvent PointerEvent(
			DeviceID,
			TouchIndex,
			Location,
			Location,
			Force,
			true,
			true,
			false);
		ProcessTouchMovedEvent(PointerEvent);

		return true;
	}

	return false;
}

bool FSlateApplication::OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID)
{
	TSharedRef<FSlateUser> User = GetOrCreateUser(DeviceID);
	if (User->IsTouchPointerActive(TouchIndex))
	{
		FPointerEvent PointerEvent(
			DeviceID,
			TouchIndex,
			Location,
			User->GetPreviousPointerPosition(TouchIndex),
			Force,
			true,
			false,
			true);
		ProcessTouchMovedEvent(PointerEvent);

		return true;
	}

	return false;
}

void FSlateApplication::ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable)
{
	check(FGestureDetector::IsGestureSupported(Gesture));

	SimulateGestures[(uint8)Gesture] = bEnable;
}

bool FSlateApplication::OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId)
{
	FMotionEvent MotionEvent( 
		InputDeviceId,
		Tilt,
		RotationRate,
		Gravity,
		Acceleration
		);
	ProcessMotionDetectedEvent(MotionEvent);

	return true;
}

void FSlateApplication::ProcessMotionDetectedEvent( const FMotionEvent& MotionEvent )
{
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::MotionDetected, MotionEvent);
#endif

	if (GSlateInputMotionFiresUserInteractionEvents)
	{
		SetLastUserInteractionTime(this->GetCurrentTime());
	}
	
	if (!InputPreProcessors.HandleMotionDetectedEvent(*this, MotionEvent))
	{
		TSharedRef<FSlateUser> User = GetOrCreateUser(MotionEvent);
		if (User->HasValidFocusPath())
		{
			/* Get the controller focus target for this user */
			TSharedRef<FWidgetPath> EventPathRef = User->GetFocusPath();
			const FWidgetPath& EventPath = EventPathRef.Get();

			FScopedSwitchWorldHack SwitchWorld(EventPath);

			FReply Reply = FEventRouter::Route<FReply>(this, FEventRouter::FBubblePolicy(EventPath), MotionEvent, [](const FArrangedWidget& SomeWidget, const FMotionEvent& InMotionEvent)
				{
					const FReply TempReply = SomeWidget.Widget->OnMotionDetected(SomeWidget.Geometry, InMotionEvent);
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent::MotionDetected, &InMotionEvent, TempReply, SomeWidget.Widget);
#endif
					return TempReply;
				}, ESlateDebuggingInputEvent::MotionDetected);
		}
	}
}

bool FSlateApplication::OnSizeChanged( const TSharedRef< FGenericWindow >& PlatformWindow, const int32 Width, const int32 Height, bool bWasMinimized )
{
	LLM_SCOPE(ELLMTag::UI);

	TSharedPtr< SWindow > Window = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, PlatformWindow );

	if ( Window.IsValid() )
	{
		Window->SetCachedSize( FVector2f( Width, Height ) );

		Renderer->RequestResize( Window, Width, Height );

		if (FPlatformProperties::HasFixedResolution())
		{
			Renderer->SetSystemResolution(Width, Height);
		}
		
		if ( !bWasMinimized && Window->IsRegularWindow() && !Window->HasOSWindowBorder() && Window->IsVisible() && Window->IsDrawingEnabled() )
		{
			PrivateDrawWindows( Window );
		}

		if( !bWasMinimized && Window->IsVisible() && Window->IsRegularWindow() && Window->IsAutosized() )
		{
			// Reduces flickering due to one frame lag when windows are resized automatically
			Renderer->FlushCommands();
		}

		// Inform the notification manager we have activated a window - it may want to force notifications 
		// back to the front of the z-order
		FSlateNotificationManager::Get().ForceNotificationsInFront( Window.ToSharedRef() );
	}

	return true;
}

void FSlateApplication::OnOSPaint( const TSharedRef< FGenericWindow >& PlatformWindow )
{
	// This is only called in a modal move loop and in cooked build, the back buffer already
	// has UI composited so don't do anything to prevent drawing UI over existing UI
	if (GIsEditor)
	{
		TSharedPtr< SWindow > Window = FSlateWindowHelper::FindWindowByPlatformWindow(SlateWindows, PlatformWindow);
		PrivateDrawWindows(Window);
		Renderer->FlushCommands();
	}
}

FWindowSizeLimits FSlateApplication::GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const
{
	TSharedPtr<SWindow> SlateWindow = FSlateWindowHelper::FindWindowByPlatformWindow(SlateWindows, Window);
	if (SlateWindow.IsValid())
	{
		return SlateWindow->GetSizeLimits();
	}
	else
	{
		return FWindowSizeLimits();
	}

}

void FSlateApplication::OnResizingWindow( const TSharedRef< FGenericWindow >& PlatformWindow )
{
	// Flush the rendering command queue to ensure that there aren't pending viewport draw commands for the old viewport size.
	Renderer->FlushCommands();
}

bool FSlateApplication::BeginReshapingWindow( const TSharedRef< FGenericWindow >& PlatformWindow )
{
	if(!IsExternalUIOpened())
	{
		if (!ThrottleHandle.IsValid())
		{
			ThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();
		}

		return true;
	}

	return false;
}

void FSlateApplication::FinishedReshapingWindow(const TSharedRef< FGenericWindow >& PlatformWindow)
{
	if (ThrottleHandle.IsValid())
	{
		FSlateThrottleManager::Get().LeaveResponsiveMode(ThrottleHandle);
	}
}

void FSlateApplication::SignalSystemDPIChanged(const TSharedRef<FGenericWindow>& PlatformWindow)
{
#if WITH_EDITOR
	TSharedPtr< SWindow > SlateWindow = FSlateWindowHelper::FindWindowByPlatformWindow(SlateWindows, PlatformWindow);

	if (SlateWindow.IsValid() && SlateWindow->IsRegularWindow())
	{
		OnSignalSystemDPIChangedEvent.Broadcast(SlateWindow.ToSharedRef());
	}
#endif
}

void FSlateApplication::HandleDPIScaleChanged(const TSharedRef<FGenericWindow>& PlatformWindow)
{
#if WITH_EDITOR
	TSharedPtr< SWindow > SlateWindow = FSlateWindowHelper::FindWindowByPlatformWindow(SlateWindows, PlatformWindow);

	if (SlateWindow.IsValid() && SlateWindow->IsRegularWindow())
	{
		OnWindowDPIScaleChangedEvent.Broadcast(SlateWindow.ToSharedRef());
	}
#endif
}

void FSlateApplication::OnRegularWindowMovedOrResized(const TSharedRef<FGenericWindow>& PlatformWindow)
{
	TSharedPtr< SWindow > SlateWindow = FSlateWindowHelper::FindWindowByPlatformWindow(SlateWindows, PlatformWindow);

	if (SlateWindow.IsValid() && SlateWindow->IsRegularWindow())
	{
		OnRegularWindowMovedOrResizedEvent.Broadcast(SlateWindow.ToSharedRef());
	}
}

void FSlateApplication::OnMovedWindow( const TSharedRef< FGenericWindow >& PlatformWindow, const int32 X, const int32 Y )
{
	TSharedPtr< SWindow > Window = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, PlatformWindow );

	if ( Window.IsValid() )
	{
		Window->SetCachedScreenPosition( FVector2f( X, Y ) );
	}
}

FWindowActivateEvent::EActivationType TranslationWindowActivationMessage( const EWindowActivation ActivationType )
{
	FWindowActivateEvent::EActivationType Result = FWindowActivateEvent::EA_Activate;

	switch( ActivationType )
	{
	case EWindowActivation::Activate:
		Result = FWindowActivateEvent::EA_Activate;
		break;
	case EWindowActivation::ActivateByMouse:
		Result = FWindowActivateEvent::EA_ActivateByMouse;
		break;
	case EWindowActivation::Deactivate:
		Result = FWindowActivateEvent::EA_Deactivate;
		break;
	default:
		check( false );
	}

	return Result;
}

bool FSlateApplication::OnWindowActivationChanged( const TSharedRef< FGenericWindow >& PlatformWindow, const EWindowActivation ActivationType )
{
	TSharedPtr< SWindow > Window = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, PlatformWindow );

	if ( !Window.IsValid() )
	{
		return false;
	}

	FWindowActivateEvent::EActivationType TranslatedActivationType = TranslationWindowActivationMessage( ActivationType );
	FWindowActivateEvent WindowActivateEvent( TranslatedActivationType, Window.ToSharedRef() );

	return ProcessWindowActivatedEvent( WindowActivateEvent );
}

bool FSlateApplication::ProcessWindowActivatedEvent( const FWindowActivateEvent& ActivateEvent )
{
	//UE_LOGF(LogSlate, Warning, "Window being %ls: %p", ActivateEvent.GetActivationType() == FWindowActivateEvent::EA_Deactivate ? TEXT("Deactivated") : TEXT("Activated"), &(ActivateEvent.GetAffectedWindow().Get()));

	TSharedPtr<SWindow> ActiveModalWindow = GetActiveModalWindow();

	if ( ActivateEvent.GetActivationType() != FWindowActivateEvent::EA_Deactivate )
	{
		ReleaseAllPointerCapture();

		const bool bActivatedByMouse = ActivateEvent.GetActivationType() == FWindowActivateEvent::EA_ActivateByMouse;
		
		// Only window activate by mouse is considered a user interaction
		if (bActivatedByMouse)
		{
			SetLastUserInteractionTime(this->GetCurrentTime());
		}
		
		// NOTE: The window is brought to front even when a modal window is active and this is not the modal window one of its children 
		// The reason for this is so that the Slate window order is in sync with the OS window order when a modal window is open.  This is important so that when the modal window closes the proper window receives input from Slate.
		// If you change this be sure to test windows are activated properly and receive input when they are opened when a modal dialog is open.
		FSlateWindowHelper::BringWindowToFront(SlateWindows, ActivateEvent.GetAffectedWindow());

		// Do not process activation messages unless we have no modal windows or the current window is modal or we are over an interactive tooltip
		if ( !ActiveModalWindow.IsValid() || ActivateEvent.GetAffectedWindow() == ActiveModalWindow || ActivateEvent.GetAffectedWindow()->IsDescendantOf(ActiveModalWindow) 
			|| IsWindowHousingInteractiveTooltip(ActivateEvent.GetAffectedWindow()) )
		{
			// Window being ACTIVATED
			{
				// Switch worlds widgets in the current path
				FScopedSwitchWorldHack SwitchWorld( ActivateEvent.GetAffectedWindow() );
				ActivateEvent.GetAffectedWindow()->OnIsActiveChanged( ActivateEvent );
			}

			if ( ActivateEvent.GetAffectedWindow()->IsRegularWindow() )
			{
				ActiveTopLevelWindow = ActivateEvent.GetAffectedWindow();
			}

			// A Slate window was activated
			bSlateWindowActive = true;

			{
				FScopedSwitchWorldHack SwitchWorld( ActivateEvent.GetAffectedWindow() );
				// let the menu stack know of new window being activated.  We may need to close menus as a result
				MenuStack.OnWindowActivated( ActivateEvent.GetAffectedWindow() );
			}

			// Inform the notification manager we have activated a window - it may want to force notifications 
			// back to the front of the z-order
			FSlateNotificationManager::Get().ForceNotificationsInFront( ActivateEvent.GetAffectedWindow() );

			// As we've just been activated, attempt to restore the resolution that the engine previously cached.
			// This allows us to force ourselves back to the correct resolution after alt-tabbing out of a fullscreen
			// window and then going back in again.
			Renderer->RestoreSystemResolution(ActivateEvent.GetAffectedWindow());
		}
		else
		{
			// An attempt is being made to activate another window when a modal window is running
			ActiveModalWindow->BringToFront();
			ActiveModalWindow->FlashWindow();
		}

		TSharedRef<SWindow> Window = ActivateEvent.GetAffectedWindow();
		TSharedPtr<ISlateViewport> Viewport = Window->GetViewport();
		if (Viewport.IsValid())
		{
			TSharedPtr<SWidget> ViewportWidgetPtr = Viewport->GetWidget().Pin();
			if (ViewportWidgetPtr.IsValid())
			{
				TArray< TSharedRef<SWindow> > JustThisWindow;
				JustThisWindow.Add(Window);

				FWidgetPath PathToViewport;
				if (FSlateWindowHelper::FindPathToWidget(JustThisWindow, ViewportWidgetPtr.ToSharedRef(), PathToViewport, EVisibility::All))
				{
					// Activate the viewport and process the reply 
					FReply ViewportActivatedReply = Viewport->OnViewportActivated(ActivateEvent);
					if (ViewportActivatedReply.IsEventHandled())
					{
						ProcessReply(PathToViewport, ViewportActivatedReply, nullptr, nullptr);
					}
				}
			}
		}
	}
	else
	{
		// Window being DEACTIVATED

		// If our currently-active top level window was deactivated, take note of that
		if ( ActivateEvent.GetAffectedWindow()->IsRegularWindow() &&
			ActivateEvent.GetAffectedWindow() == ActiveTopLevelWindow.Pin() )
		{
			ActiveTopLevelWindow.Reset();
		}

		// A Slate window was deactivated.  Currently there is no active Slate window
		bSlateWindowActive = false;

		// Switch worlds for the activated window
		FScopedSwitchWorldHack SwitchWorld( ActivateEvent.GetAffectedWindow() );
		ActivateEvent.GetAffectedWindow()->OnIsActiveChanged( ActivateEvent );

		TSharedRef<SWindow> Window = ActivateEvent.GetAffectedWindow();
		TSharedPtr<ISlateViewport> Viewport = Window->GetViewport();
		if (Viewport.IsValid())
		{
			Viewport->OnViewportDeactivated(ActivateEvent);
		}

		// A window was deactivated; mouse capture should be cleared
		ResetToDefaultPointerInputSettings();
	}

	return true;
}

bool FSlateApplication::OnApplicationActivationChanged( const bool IsActive )
{
	ProcessApplicationActivationEvent( IsActive );
	return true;
}

void FSlateApplication::ProcessApplicationActivationEvent(bool InAppActivated)
{
	if (GUELibraryOverrideSettings.bIsEmbedded)
	{
		return;
	}

	const bool UserSwitchedAway = bAppIsActive && !InAppActivated;
	const bool StateChanged = bAppIsActive != InAppActivated;

	bAppIsActive = InAppActivated;

	// If the user switched to a different application then we should dismiss our pop-ups.  In the case
	// where a user clicked on a different Slate window, OnWindowActivatedMessage() will be call MenuStack.OnWindowActivated()
	// to destroy any windows in our stack that are no longer appropriate to be displayed.
	if (UserSwitchedAway)
	{
		// Close pop-up menus
		DismissAllMenus();

		// Close tool-tips
		ForEachUser([](FSlateUser& User) {
			User.CloseTooltip();
		});
		
		// No slate window is active when our entire app becomes inactive
		bSlateWindowActive = false;

		// Stop all slate-only drag-drop operations
		ForEachUser([](FSlateUser& User) { 
			if (User.IsDragDropping() && User.GetDragDropContent()->IsExternalOperation())
			{
				User.CancelDragDrop();
			}
		});

		// Clear the pressed buttons when we deactivate the application, the button state can no longer be trusted.
		PressedMouseButtons.Reset();
	}
	
	// Only broadcast when state has changed
	if (StateChanged)
	{
		OnApplicationActivationStateChanged().Broadcast(InAppActivated);
	}
}

void FSlateApplication::SetNavigationConfig(TSharedRef<FNavigationConfig> InNavigationConfig)
{
	NavigationConfig->OnUnregister();
	NavigationConfig = InNavigationConfig;
	NavigationConfig->OnRegister();

#if WITH_SLATE_DEBUGGING
	TryDumpNavigationConfig(NavigationConfig);
#endif // WITH_SLATE_DEBUGGING
}

bool FSlateApplication::OnConvertibleLaptopModeChanged()
{
	EConvertibleLaptopMode NewMode = FPlatformMisc::GetConvertibleLaptopMode();

	// Notify that we want the mobile experience when in tablet mode, otherwise use mouse and keyboard
	if (!(FParse::Param(FCommandLine::Get(), TEXT("simmobile")) || FParse::Param(FCommandLine::Get(), TEXT("faketouches"))))
	{
		// Not sure what the correct long-term strategy is. Use bIsFakingTouch for now to get things going.
		if (NewMode == EConvertibleLaptopMode::Tablet)
		{
			bIsFakingTouch = true;
		}
		else
		{
			bIsFakingTouch = false;
		}
	}

	FCoreDelegates::PlatformChangedLaptopMode.Broadcast(NewMode);

	return true;
}


EWindowZone::Type FSlateApplication::GetWindowZoneForPoint( const TSharedRef< FGenericWindow >& PlatformWindow, const int32 X, const int32 Y )
{
	TSharedPtr< SWindow > Window = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, PlatformWindow );

	if ( Window.IsValid() )
	{
		return Window->GetCurrentWindowZone( FVector2f( X, Y ) );
	}

	return EWindowZone::NotInWindow;
}


void FSlateApplication::PrivateDestroyWindow( const TSharedRef<SWindow>& DestroyedWindow )
{
	WindowBeingDestroyedEvent.Broadcast(*DestroyedWindow);

	// Notify the window that it is going to be destroyed.  The window must be completely intact when this is called 
	// because delegates are allowed to leave Slate here
	DestroyedWindow->NotifyWindowBeingDestroyed();

	// Release rendering resources.  
	// This MUST be done before destroying the native window as the native window is required to be valid before releasing rendering resources with some API's
	Renderer->OnWindowDestroyed( DestroyedWindow );

	// Destroy the native window
	DestroyedWindow->DestroyWindowImmediately();

	// Remove the window and all its children from the Slate window list
	FSlateWindowHelper::RemoveWindowFromList(SlateWindows, DestroyedWindow);

	// Shutdown the application if there are no more windows
	{
		bool bAnyRegularWindows = false;
		for( auto WindowIter( SlateWindows.CreateConstIterator() ); WindowIter; ++WindowIter )
		{
			auto Window = *WindowIter;
			if( Window->IsRegularWindow() )
			{
				bAnyRegularWindows = true;
				break;
			}
		}

		if (!bAnyRegularWindows)
		{
			OnExitRequested.ExecuteIfBound();
		}
	}
}

void FSlateApplication::OnWindowClose( const TSharedRef< FGenericWindow >& PlatformWindow )
{
	TSharedPtr< SWindow > Window = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, PlatformWindow );

	if ( Window.IsValid() )
	{
		bool bCanCloseWindow = true;
		TSharedPtr< SViewport > CurrentGameViewportWidget = GameViewportWidget.Pin();
		if (CurrentGameViewportWidget.IsValid())
		{
			TSharedPtr< ISlateViewport > SlateViewport = CurrentGameViewportWidget->GetViewportInterface().Pin();
			if (SlateViewport.IsValid())
			{
				bCanCloseWindow = !SlateViewport->OnRequestWindowClose().bIsHandled;
			}
		}
		
		if (bCanCloseWindow)
		{
		    Window->RequestDestroyWindow();
		}	 
	}
}

EDropEffect::Type FSlateApplication::OnDragEnterText( const TSharedRef< FGenericWindow >& Window, const FString& Text )
{
	const TSharedPtr< FExternalDragOperation > DragDropOperation = FExternalDragOperation::NewText( Text );
	const TSharedPtr< SWindow > EffectingWindow = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, Window );

	EDropEffect::Type Result = EDropEffect::None;
	if ( DragDropOperation.IsValid() && EffectingWindow.IsValid() )
	{
		Result = OnDragEnter( EffectingWindow.ToSharedRef(), DragDropOperation.ToSharedRef() );
	}

	return Result;
}

EDropEffect::Type FSlateApplication::OnDragEnterFiles( const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files )
{
	const TSharedPtr< FExternalDragOperation > DragDropOperation = FExternalDragOperation::NewFiles( Files );
	const TSharedPtr< SWindow > EffectingWindow = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, Window );

	EDropEffect::Type Result = EDropEffect::None;
	if ( DragDropOperation.IsValid() && EffectingWindow.IsValid() )
	{
		Result = OnDragEnter( EffectingWindow.ToSharedRef(), DragDropOperation.ToSharedRef() );
	}

	return Result;
}

EDropEffect::Type FSlateApplication::OnDragEnterExternal( const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files )
{
	const TSharedPtr< FExternalDragOperation > DragDropOperation = FExternalDragOperation::NewOperation( Text, Files );
	const TSharedPtr< SWindow > EffectingWindow = FSlateWindowHelper::FindWindowByPlatformWindow( SlateWindows, Window );

	EDropEffect::Type Result = EDropEffect::None;
	if ( DragDropOperation.IsValid() && EffectingWindow.IsValid() )
	{
		Result = OnDragEnter( EffectingWindow.ToSharedRef(), DragDropOperation.ToSharedRef() );
	}

	return Result;
}

EDropEffect::Type FSlateApplication::OnDragEnter( const TSharedRef< SWindow >& Window, const TSharedRef<FExternalDragOperation>& DragDropOperation )
{
	// We are encountering a new drag and drop operation.
	// Assume we cannot handle it.
	DragIsHandled = false;

	const FVector2f CurrentCursorPosition = GetCursorPos();
	const FVector2f LastCursorPosition = GetLastCursorPos();

	// Tell slate to enter drag and drop mode.
	// Make a faux mouse event for slate, so we can initiate a drag and drop.
	FDragDropEvent DragDropEvent(
		FPointerEvent(
		GetUserIndexForMouse(),
		CursorPointerIndex,
		CurrentCursorPosition,
		LastCursorPosition,
		PressedMouseButtons,
		EKeys::Invalid,
		0,
		PlatformApplication->GetModifierKeys() ),
		DragDropOperation
	);

	ProcessDragEnterEvent( Window, DragDropEvent );
	return EDropEffect::None;
}

bool FSlateApplication::ProcessDragEnterEvent( TSharedRef<SWindow> WindowEntered, const FDragDropEvent& DragDropEvent )
{
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::FScopeProcessInputEvent Scope(ESlateDebuggingInputEvent::DragDrop, DragDropEvent);
#endif

	SetLastUserInteractionTime(this->GetCurrentTime());
	
	FWidgetPath WidgetsUnderCursor = LocateWindowUnderMouse( DragDropEvent.GetScreenSpacePosition(), GetInteractiveTopLevelWindows(), false, DragDropEvent.GetUserIndex());
	// There are no "interactable" widget under the cursor.
	if (!WidgetsUnderCursor.IsValid())
	{
		return false;
	}

	// Switch worlds for widgets in the current path
	FScopedSwitchWorldHack SwitchWorld( WidgetsUnderCursor );

	FReply TriggerDragDropReply = FReply::Handled().BeginDragDrop( DragDropEvent.GetOperation().ToSharedRef() );
	ProcessReply( WidgetsUnderCursor, TriggerDragDropReply, &WidgetsUnderCursor, &DragDropEvent );

	GetOrCreateUser(DragDropEvent)->UpdatePointerPosition(DragDropEvent);

	return true;
}

EDropEffect::Type FSlateApplication::OnDragOver( const TSharedPtr< FGenericWindow >& Window )
{
	EDropEffect::Type Result = EDropEffect::None;

	if (GetCursorUser()->IsDragDropping())
	{
		bool MouseMoveHandled = true;
		FVector2f CursorMovementDelta( 0, 0 );
		const FVector2f CurrentCursorPosition = GetCursorPos();
		const FVector2f LastCursorPosition = GetLastCursorPos();

		if ( LastCursorPosition != CurrentCursorPosition )
		{
			FPointerEvent MouseEvent(
				GetUserIndexForMouse(),
				CursorPointerIndex,
				CurrentCursorPosition,
				LastCursorPosition,
				PressedMouseButtons,
				EKeys::Invalid,
				0,
				PlatformApplication->GetModifierKeys()
			);

			MouseMoveHandled = ProcessMouseMoveEvent( MouseEvent );
			CursorMovementDelta = MouseEvent.GetCursorDelta();
		}

		// Slate is now in DragAndDrop mode. It is tracking the payload.
		// We just need to convey mouse movement.
		if ( CursorMovementDelta.SizeSquared() > 0 )
		{
			DragIsHandled = MouseMoveHandled;
		}

		if ( DragIsHandled )
		{
			Result = EDropEffect::Copy;
		}
	}

	return Result;
}

void FSlateApplication::OnDragLeave( const TSharedPtr< FGenericWindow >& Window )
{
	GetCursorUser()->ResetDragDropContent();
}

EDropEffect::Type FSlateApplication::OnDragDrop(const TSharedPtr< FGenericWindow >& Window)
{
	EDropEffect::Type Result = EDropEffect::None;

	if (GetCursorUser()->IsDragDropping())
	{
		// Update PressedMouseButtons before constructing the synthesized mouse-up event so
		// the value-copy snapshot reflects the post-release state. Same reasoning as OnMouseUp.
		if (GetUserIndexForMouse() == CursorUserIndex)
		{
			PressedMouseButtons.Remove(EKeys::LeftMouseButton);
		}

		FPointerEvent MouseEvent(
			GetUserIndexForMouse(),
			CursorPointerIndex,
			GetCursorPos(),
			GetLastCursorPos(),
			PressedMouseButtons,
			EKeys::LeftMouseButton,
			0,
			PlatformApplication->GetModifierKeys()
		);

		// User dropped into a Slate window. Slate is already in drag and drop mode.
		// It knows what to do based on a mouse up.
		if (ProcessMouseButtonUpEvent(MouseEvent))
		{
			Result = EDropEffect::Copy;
		}
	}

	return Result;
}

bool FSlateApplication::OnWindowAction( const TSharedRef< FGenericWindow >& PlatformWindow, const EWindowAction::Type InActionType)
{
	// Return false to tell the OS layer that it should ignore the action

	if (IsExternalUIOpened())
	{
		return false;
	}

	bool bResult = true;

	for (int32 Index = 0; Index < OnWindowActionNotifications.Num(); Index++)
	{
		if (OnWindowActionNotifications[Index].IsBound())
		{
			if (OnWindowActionNotifications[Index].Execute(PlatformWindow, InActionType))
			{
				// If the delegate returned true, it means that it wants the OS layer to stop processing the action
				bResult = false;
			}
		}
	}

	if (InActionType == EWindowAction::ClickedNonClientArea)
	{
#if WITH_SLATE_DEBUGGING
		if (!CVarSlateDismissMenuStacksOnFocusLost->GetBool())
		{
			return bResult;
		}
#endif

		DismissAllMenus();
	}

	return bResult;
}

void FSlateApplication::OnVirtualDesktopSizeChanged(const FDisplayMetrics& NewDisplayMetric)
{
	CachedDisplayMetrics = NewDisplayMetric;
	CachedDebugTitleSafeRatio = FDisplayMetrics::GetDebugTitleSafeZoneRatio();
	const FPlatformRect& VirtualDisplayRect = NewDisplayMetric.VirtualDisplayRect;
	VirtualDesktopRect = FSlateRect(
		VirtualDisplayRect.Left,
		VirtualDisplayRect.Top,
		VirtualDisplayRect.Right,
		VirtualDisplayRect.Bottom);

	if (Renderer.IsValid())
	{
		Renderer->OnVirtualDesktopSizeChanged(NewDisplayMetric);
	}
}


/* 
 *****************************************************************************/

TSharedRef<FSlateApplication> FSlateApplication::InitializeAsStandaloneApplication(const TSharedRef<FSlateRenderer>& PlatformRenderer)
{
	return InitializeAsStandaloneApplication(PlatformRenderer, MakeShareable(FPlatformApplicationMisc::CreateApplication()));
}


TSharedRef<FSlateApplication> FSlateApplication::InitializeAsStandaloneApplication(const TSharedRef< class FSlateRenderer >& PlatformRenderer, const TSharedRef<class GenericApplication>& InPlatformApplication)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSlateApplication::InitializeAsStandaloneApplication);

	// Initialise High DPI mode. This must be called before any window (including the splash screen is created).
	// We don't need to use the force flag which was added for PIE.
	const bool bForceEnable = false;
	FSlateApplication::InitHighDPI(bForceEnable);

	// create the platform slate application (what FSlateApplication::Get() returns)
	TSharedRef<FSlateApplication> Slate = FSlateApplication::Create(InPlatformApplication);

	// initialize renderer
	FSlateApplication::Get().InitializeRenderer(PlatformRenderer);

	// set the normal UE IsEngineExitRequested() when outer frame is closed
	FSlateApplication::Get().SetExitRequestedHandler(FSimpleDelegate::CreateStatic(&OnRequestExit));

	return Slate;
}

void FSlateApplication::InitializeCoreStyle()
{
	if (!FStarshipCoreStyle::IsInitialized())
	{
#if ALLOW_THEMES
		USlateThemeManager::Get().LoadThemes();
#endif
		FStarshipCoreStyle::ResetToDefault();
		FAppStyle::SetAppStyleSet(FStarshipCoreStyle::GetCoreStyle());
	}

	FUMGCoreStyle::ResetToDefault();
}

void FSlateApplication::SetWidgetReflector(const TSharedRef<IWidgetReflector>& WidgetReflector)
{
	if ( SourceCodeAccessDelegate.IsBound() )
	{
		WidgetReflector->SetSourceAccessDelegate(SourceCodeAccessDelegate);
	}

	if ( AssetAccessDelegate.IsBound() )
	{
		WidgetReflector->SetAssetAccessDelegate(AssetAccessDelegate);
	}

	if ( DebugObjectAccessDelegate.IsBound() )
	{
		WidgetReflector->SetDebugObjectAccessDelegate(DebugObjectAccessDelegate);
	}

	WidgetReflectorPtr = WidgetReflector;
}

bool FSlateApplication::IsDragDropping() const
{
	return GetCursorUser()->IsDragDropping();
}

bool FSlateApplication::IsDragDroppingAffected(const FPointerEvent& InPointerEvent) const
{
	return GetCursorUser()->IsDragDroppingAffected(InPointerEvent);
}

TSharedPtr<FDragDropOperation> FSlateApplication::GetDragDroppingContent() const
{
	return GetCursorUser()->GetDragDropContent();
}

void FSlateApplication::CancelDragDrop()
{
	GetCursorUser()->CancelDragDrop();
}

void FSlateApplication::NavigateFromWidgetUnderCursor(const uint32 InUserIndex, EUINavigation InNavigationType, TSharedRef<SWindow> InWindow)
{
	if (InNavigationType != EUINavigation::Invalid)
	{
		FWidgetPath PathToLocatedWidget = LocateWidgetInWindow(GetOrCreateUser(InUserIndex)->GetCursorPosition(), InWindow, false, InUserIndex);
		if (PathToLocatedWidget.IsValid())
		{
			TSharedPtr<SWidget> WidgetToNavFrom = PathToLocatedWidget.Widgets.Last().Widget;

			if (WidgetToNavFrom.IsValid())
			{
				ProcessReply(PathToLocatedWidget, FReply::Handled().SetNavigation(InNavigationType, ENavigationGenesis::User, ENavigationSource::WidgetUnderCursor), &PathToLocatedWidget, nullptr, InUserIndex);
			}
		}
	}
}

void FSlateApplication::InputPreProcessorsHelper::Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor)
{
	TGuardValue<bool> IteratingGuard(bIsIteratingPreProcessors, true);

	for (const TSharedPtr<IInputProcessor>& Processor : InputPreProcessorsIteratorList)
	{
		Processor->Tick(DeltaTime, SlateApp, Cursor);
	}
}

bool FSlateApplication::InputPreProcessorsHelper::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::KeyDown
		, [&SlateApp, &InKeyEvent](IInputProcessor& Processor) { return Processor.HandleKeyDownEvent(SlateApp, InKeyEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::KeyUp
		, [&SlateApp, &InKeyEvent](IInputProcessor& Processor) { return Processor.HandleKeyUpEvent(SlateApp, InKeyEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::AnalogInput
		, [&SlateApp, &InAnalogInputEvent](IInputProcessor& Processor) { return Processor.HandleAnalogInputEvent(SlateApp, InAnalogInputEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::MouseMove
		, [&SlateApp, &MouseEvent](IInputProcessor& Processor) { return Processor.HandleMouseMoveEvent(SlateApp, MouseEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::MouseButtonDown
		, [&SlateApp, &MouseEvent](IInputProcessor& Processor) { return Processor.HandleMouseButtonDownEvent(SlateApp, MouseEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::MouseButtonUp
		, [&SlateApp, &MouseEvent](IInputProcessor& Processor) { return Processor.HandleMouseButtonUpEvent(SlateApp, MouseEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::MouseButtonDoubleClick
		, [&SlateApp, &MouseEvent](IInputProcessor& Processor) { return Processor.HandleMouseButtonDoubleClickEvent(SlateApp, MouseEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& WheelEvent, const FPointerEvent* GestureEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::MouseWheel
		, [&SlateApp, &WheelEvent, &GestureEvent](IInputProcessor& Processor) { return Processor.HandleMouseWheelOrGestureEvent(SlateApp, WheelEvent, GestureEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::HandleMotionDetectedEvent(FSlateApplication& SlateApp, const FMotionEvent& MotionEvent)
{
	return PreProcessInput(ESlateDebuggingInputEvent::MotionDetected
		, [&SlateApp, &MotionEvent](IInputProcessor& Processor) { return Processor.HandleMotionDetectedEvent(SlateApp, MotionEvent); });
}

bool FSlateApplication::InputPreProcessorsHelper::Add(const FInputPreprocessorRegistration& Registration)
{
	// We check if the processor attempting registration is already registered
	bool bAlreadyRegistered = false;
	for (const FProcessorTypeStorage& Storage : InputPreProcessors)
	{
		if (Storage.Contains(Registration.InputProcessor))
		{
			bAlreadyRegistered = true;
			break;
		}
	}

	if(!bAlreadyRegistered)
	{
		if (!bIsIteratingPreProcessors)
		{
			AddInternal(Registration);
		}
		else
		{
			ProcessorsPendingAddition.Add(Registration);
		}
	}

	ProcessorsPendingRemoval.Remove(Registration.InputProcessor);

	return !bAlreadyRegistered;
}

void FSlateApplication::InputPreProcessorsHelper::AddInternal(const FInputPreprocessorRegistration& Registration)
{
	if (!InputPreProcessors.IsValidIndex((uint32)Registration.Info.Type))
	{
		InputPreProcessors.EmplaceAt((int32)Registration.Info.Type, FProcessorTypeStorage());
	}

	FProcessorTypeStorage& Storage = InputPreProcessors[(uint32)Registration.Info.Type];

	if (Registration.Info.Priority == INDEX_NONE)
	{
		Storage.Add(Registration.InputProcessor);
	}
	else
	{
 		if (Registration.Info.Priority >= Storage.Num())
 		{
 			Storage.SetNum(Registration.Info.Priority); // No need for +1, insertion at the Num position doesn't cause an error. +1 would add unneeded empty spaces.
 		}
		Storage.Insert(Registration.InputProcessor, Registration.Info.Priority);
	}

	// We rebuild the iterator list
	InputPreProcessorsIteratorList.Reset();
	for (const FProcessorTypeStorage& TypeStorage : InputPreProcessors)
	{
		for (const TSharedPtr<IInputProcessor>& Processor : TypeStorage)
		{
			// We won't add the empty spaces in the map to the list
			if (Processor)
			{
				InputPreProcessorsIteratorList.Add(Processor);
			}
		}
	}
}

void FSlateApplication::InputPreProcessorsHelper::Remove(TSharedPtr<IInputProcessor> InputProcessor)
{
	if (bIsIteratingPreProcessors)
	{
		ProcessorsPendingRemoval.Add(InputProcessor);
	}
	else
	{
		for (FProcessorTypeStorage& Storage : InputPreProcessors)
		{
			Storage.Remove(InputProcessor);
		}

		InputPreProcessorsIteratorList.Remove(InputProcessor);
	}

	ProcessorsPendingAddition.RemoveAllSwap([InputProcessor](const FInputPreprocessorRegistration& Registration)
		{
			return Registration.InputProcessor == InputProcessor;
		});
}

void FSlateApplication::InputPreProcessorsHelper::RemoveAll()
{
	if (bIsIteratingPreProcessors)
	{
		for (const FProcessorTypeStorage& Storage : InputPreProcessors)
		{
			ProcessorsPendingRemoval.Append(Storage);
		}
	}
	else
	{
		for (FProcessorTypeStorage& Storage : InputPreProcessors)
		{
			Storage.Reset();
		}

		InputPreProcessorsIteratorList.Reset();
	}

	ProcessorsPendingAddition.Reset();
}

int32 FSlateApplication::InputPreProcessorsHelper::Find(TSharedPtr<IInputProcessor> InputProcessor, const EInputPreProcessorType& Type) const
{
	const uint32 TypeInt = static_cast<uint32>(Type);
	if (InputPreProcessors.IsValidIndex(TypeInt))
	{
		const FProcessorTypeStorage& Storage = InputPreProcessors[TypeInt];
		return Storage.Find(InputProcessor);
	}

	return INDEX_NONE;
}

bool FSlateApplication::InputPreProcessorsHelper::PreProcessInput(ESlateDebuggingInputEvent InputEvent, TFunctionRef<bool(IInputProcessor&)> InputProcessFunc)
{
	TGuardValue<bool> IteratingGuard(bIsIteratingPreProcessors, true);

	bool bShouldExit = false;
	for (const TSharedPtr<IInputProcessor>& Processor : InputPreProcessorsIteratorList)
	{
		bShouldExit = InputProcessFunc(*Processor);

#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastPreProcessInputEvent(InputEvent, Processor->GetDebugName(), bShouldExit);
#endif
		if (bShouldExit)
		{
			break;
		}
	}

	for (int32 Index = ProcessorsPendingRemoval.Num() - 1; Index >= 0; --Index)
	{
		TSharedPtr<IInputProcessor>& Processor = ProcessorsPendingRemoval[Index];

		for (FProcessorTypeStorage& Storage : InputPreProcessors)
		{
			Storage.Remove(Processor);
			InputPreProcessorsIteratorList.Remove(Processor);
		}
	}
	ProcessorsPendingRemoval.Reset();

	for (const FInputPreprocessorRegistration& Registration : ProcessorsPendingAddition)
	{
		AddInternal(Registration);
	}
	ProcessorsPendingAddition.Reset();

	return bShouldExit;
}


#if WITH_SLATE_DEBUGGING
namespace UE::Slate::Private
{

bool VerifyParentChildrenRelationship_Recursive(SWidget* Parent, TMap<const SWidget*, const SWidget*>& AllWidgets)
{
	bool bResult = true;
	Parent->GetAllChildren()->ForEachWidget([&bResult, Parent, &AllWidgets](SWidget& ChildWidget)
		{
			if (&ChildWidget != &SNullWidget::NullWidget.Get())
			{
				if (AllWidgets.Find(&ChildWidget))
				{
					bResult = false;
					UE_LOGF(LogSlate, Warning, "The widget '%ls' is owned by more than one parent. 1:'%ls' 2:'%ls'."
						, *FReflectionMetaData::GetWidgetDebugInfo(ChildWidget)
						, *FReflectionMetaData::GetWidgetDebugInfo(AllWidgets[&ChildWidget])
						, *FReflectionMetaData::GetWidgetDebugInfo(Parent));
				}
				else
				{
					AllWidgets.Add(&ChildWidget, Parent);
				}

				if (ChildWidget.GetParentWidget().Get() != Parent)
				{
					bResult = false;
					UE_LOGF(LogSlate, Warning, "The widget '%ls' has the wrong parent.", *FReflectionMetaData::GetWidgetDebugInfo(ChildWidget));
				}

				bResult = bResult && VerifyParentChildrenRelationship_Recursive(&ChildWidget, AllWidgets);
			}
		});

	return bResult;
}

bool VerifyWidgetLayerId_Recursive(SWidget& Widget, bool bInsideInvalidationRoot)
{
	bool bResult = true;

	if (!bInsideInvalidationRoot)
	{
		if (const FSlateInvalidationRoot* AsInvalidationRoot = Widget.Advanced_AsInvalidationRoot())
		{
			bInsideInvalidationRoot = AsInvalidationRoot->GetLastPaintType() == ESlateInvalidationPaintType::Fast;
		}
	}

	const uint32 LastPaintFrame = Widget.Debug_GetLastPaintFrame();
	const bool bIsDeferredPaint = Widget.GetPersistentState().bDeferredPainting;
	const int32 InLayerId = Widget.GetPersistentState().LayerId;
	const int32 OutLayerId = Widget.GetPersistentState().OutgoingLayerId;

	int32 PreviousInLayerId = InLayerId;
	int32 PreviousOutLayerId = InLayerId;

	Widget.GetAllChildren()->ForEachWidget([&bResult, bInsideInvalidationRoot, LastPaintFrame, bIsDeferredPaint, InLayerId, OutLayerId](SWidget& ChildWidget)
		{
			if (&ChildWidget != &SNullWidget::NullWidget.Get())
			{

				if (ChildWidget.GetVisibility().IsVisible() && (bInsideInvalidationRoot || ChildWidget.Debug_GetLastPaintFrame() == LastPaintFrame))
				{
					const bool bIsChildDeferredPaint = ChildWidget.GetPersistentState().bDeferredPainting;
					if (bIsChildDeferredPaint == bIsDeferredPaint)
					{
						const int32 ChildInLayerId = ChildWidget.GetPersistentState().LayerId;
						const int32 ChildOutLayerId = ChildWidget.GetPersistentState().OutgoingLayerId;

						if (ChildInLayerId== 0)
						{
							return;
						}

						const bool bLayerInIsValid = InLayerId <= ChildInLayerId && InLayerId <= ChildOutLayerId;
						const bool bLayerOutIsValid = OutLayerId >= ChildInLayerId && OutLayerId >= ChildOutLayerId;
						if (!bLayerInIsValid || !bLayerOutIsValid)
						{
							SWidget* ParentWidget = ChildWidget.GetParentWidget().Get();
							check(ParentWidget);
							// The parent may just have tick and will be painted on the next frame (this is not desired but possible.
							if (!ParentWidget->GetProxyHandle().HasAnyInvalidationReason(ParentWidget, EInvalidateWidgetReason::Paint))
							{
								bResult = false;
								UE_LOGF(LogSlate, Warning, "The widget '%ls' LayerId is invalid. Parent: [%d,%d] Child: [%d,%d]."
									, *FReflectionMetaData::GetWidgetDebugInfo(ChildWidget)
									, InLayerId
									, OutLayerId
									, ChildInLayerId
									, ChildOutLayerId);
							}

							return;
						}
					}

					bResult = bResult && VerifyWidgetLayerId_Recursive(ChildWidget, bInsideInvalidationRoot);
				}
			}
		});

	return bResult;
}

void VerifyParentChildrenRelationship(const TSharedRef<SWindow>& WindowToDraw)
{
	if (WindowToDraw != SNullWidget::NullWidget)
	{
		TMap<const SWidget*, const SWidget*> AllWidgets;
		AllWidgets.Add(&WindowToDraw.Get(), nullptr);
		if (!Private::VerifyParentChildrenRelationship_Recursive(&WindowToDraw.Get(), AllWidgets))
		{
			CVarSlateVerifyParentChildrenRelationship->Set(false, CVarSlateVerifyParentChildrenRelationship->GetFlags());
			ensureAlwaysMsgf(false, TEXT("VerifyParentChildrenRelationship failed. See log for more info."));
		}
	}
}


void VerifyWidgetLayerId(const TSharedRef<SWindow>& WindowToDraw)
{
	if (WindowToDraw != SNullWidget::NullWidget)
	{
		if (!Private::VerifyWidgetLayerId_Recursive(WindowToDraw.Get(), false))
		{
			CVarSlateVerifyWidgetLayerId->Set(false, CVarSlateVerifyWidgetLayerId->GetFlags());
			ensureAlwaysMsgf(false, TEXT("VerifyWidgetLayerId failed. See log for more info."));
		}


	}
}

} //namespace
#endif //WITH_SLATE_DEBUGGING
