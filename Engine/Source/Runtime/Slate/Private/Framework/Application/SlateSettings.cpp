// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SlateSettings.cpp: Project configurable Slate settings
=============================================================================*/

#include "SlateSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateSettings)

USlateSettings::USlateSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bExplicitCanvasChildZOrder(false)
{
}

