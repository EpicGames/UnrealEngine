// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Application/SlateUser.h"

#if PLATFORM_MAC || PLATFORM_IOS
#include "Apple/AppleTagRect.h"
#endif

#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/NavigationConfig.h"
#include "Misc/App.h"
#include "Widgets/SWindow.h"
#include "Widgets/SWeakWidget.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#if PLATFORM_MICROSOFT
// Needed to be able to use RECT
#include "Microsoft/WindowsHWrapper.h"
#endif

DECLARE_CYCLE_STAT(TEXT("QueryCursor"), STAT_SlateQueryCursor, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Update Tooltip Time"), STAT_SlateUpdateTooltip, STATGROUP_Slate);

namespace SlateDefs
{
	// How far tooltips should be offset from the mouse cursor position, in pixels
	static const FVector2f TooltipOffsetFromMouse(12.0f, 8.0f);

	// How far tooltips should be pushed out from a force field border, in pixels
	static const FVector2f TooltipOffsetFromForceField(4.0f, 3.0f);
}

static bool bEnableSyntheticCursorMoves = true;
FAutoConsoleVariableRef CVarEnableSyntheticCursorMoves(
	TEXT("Slate.EnableSyntheticCursorMoves"),
	bEnableSyntheticCursorMoves,
	TEXT("")
);

static bool bEnableCursorQueries = true;
FAutoConsoleVariableRef CVarEnableCursorQueries(
	TEXT("Slate.EnableCursorQueries"),
	bEnableCursorQueries,
	TEXT(""));

static float SoftwareCursorScale = 1.0f;
FAutoConsoleVariableRef CVarSoftwareCursorScale(
	TEXT("Slate.SoftwareCursorScale"),
	SoftwareCursorScale,
	TEXT("Scale factor applied to the software cursor. Requires the cursor widget to be scale-aware."));

static float TooltipSummonDelay = 0.15f;
FAutoConsoleVariableRef CVarTooltipSummonDelay(
	TEXT("Slate.TooltipSummonDelay"),
	TooltipSummonDelay,
	TEXT("Delay in seconds before a tooltip is displayed near the mouse cursor when hovering over widgets that supply tooltip data."));
	
static float TooltipIntroDuration = 0.1f;
FAutoConsoleVariableRef CVarTooltipIntroDuration(
	TEXT("Slate.TooltipIntroDuration"),
	TooltipIntroDuration,
	TEXT("How long it takes for a tooltip to animate into view, in seconds."));

static float CursorSignificantMoveDetectionThreshold = 0.0;
FAutoConsoleVariableRef CVarCursorSignificantMoveDetectionThreshold(
	TEXT("Slate.CursorSignificantMoveDetectionThreshold"),
	CursorSignificantMoveDetectionThreshold,
	TEXT("The distance from previous cursor position above which the move will be considered significant (used to trigger the display of the tooltips)."));

static bool bAllowTooltipsWithHiddenCursor = false;
FAutoConsoleVariableRef CVarAllowTooltipsWithHiddenCursor(
	TEXT("Slate.AllowTooltipsWithHiddenCursor"),
	bAllowTooltipsWithHiddenCursor,
	TEXT(""));

static bool bAllowTooltipsWhileMouseDown = false;
FAutoConsoleVariableRef CVarAllowTooltipsWhileMouseDown(
	TEXT("Slate.AllowTooltipsWhileMouseDown"),
	bAllowTooltipsWhileMouseDown,
	TEXT(""));

#if WITH_SLATE_DEBUGGING
static bool bEnableTooltipForceFieldDraw = false;
FAutoConsoleVariableRef CVarEnableTooltipForceFieldDraw(
	TEXT("Slate.EnableTooltipForceFieldDraw"),
	bEnableTooltipForceFieldDraw,
	TEXT("If turned on, the rectangles of effective tooltip force fields will be visualized."));
#endif

//////////////////////////////////////////////////////////////////////////
// FSlateVirtualUserHandle
//////////////////////////////////////////////////////////////////////////

FSlateVirtualUserHandle::FSlateVirtualUserHandle(int32 InUserIndex, int32 InVirtualUserIndex)
	: UserIndex(InUserIndex)
	, VirtualUserIndex(InVirtualUserIndex)
{
}

FSlateVirtualUserHandle::~FSlateVirtualUserHandle()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterUser(UserIndex);
	}
}

//////////////////////////////////////////////////////////////////////////
// FActiveTooltipInfo
//////////////////////////////////////////////////////////////////////////

void FSlateUser::FActiveTooltipInfo::Reset()
{
	if (SourceWidget.IsValid())
	{
		SourceWidget.Pin()->OnToolTipClosing();
	}

	if (Tooltip.IsValid())
	{
		Tooltip.Pin()->OnClosed();
	}

	Tooltip.Reset();
	SourceWidget.Reset();
	TooltipVisualizer.Reset();
	OffsetDirection = ETooltipOffsetDirection::Undetermined;
}

//////////////////////////////////////////////////////////////////////////
// FSlateUser
//////////////////////////////////////////////////////////////////////////

TSharedRef<FSlateUser> FSlateUser::Create(int32 InUserIndex, TSharedPtr<ICursor> InCursor)
{
	return MakeShareable(new FSlateUser(InUserIndex, InCursor));
}

TSharedRef<FSlateUser> FSlateUser::Create(FPlatformUserId InPlatformUserId, TSharedPtr<ICursor> InCursor)
{
	return MakeShareable(new FSlateUser(InPlatformUserId, InCursor));
}

FSlateUser::FSlateUser(int32 InUserIndex, TSharedPtr<ICursor> InCursor)
	: UserIndex(InUserIndex)
	, Cursor(InCursor)
{
	PlatformUser = FPlatformMisc::GetPlatformUserForUserIndex(InUserIndex);
	UE_LOGF(LogSlate, Log, "New Slate User Created. Platform User Id %d, User Index %d, Is Virtual User: %d", PlatformUser.GetInternalId(), UserIndex, IsVirtualUser());

	PointerPositionsByIndex.Add(FSlateApplication::CursorPointerIndex, FVector2f::ZeroVector);
	PreviousPointerPositionsByIndex.Add(FSlateApplication::CursorPointerIndex, FVector2f::ZeroVector);
}

FSlateUser::FSlateUser(FPlatformUserId InPlatformUser, TSharedPtr<ICursor> InCursor)
	: PlatformUser(InPlatformUser)
	, Cursor(InCursor)
{
	// TODO: Remove this part, its backwards compatible for now
	UserIndex = InPlatformUser.GetInternalId();
	
	UE_LOGF(LogSlate, Log, "New Slate User Created.  Platform User Id %d,  Old User Index: %d  , Is Virtual User: %d", PlatformUser.GetInternalId(), UserIndex, IsVirtualUser());
	
	PointerPositionsByIndex.Add(FSlateApplication::CursorPointerIndex, FVector2f::ZeroVector);
	PreviousPointerPositionsByIndex.Add(FSlateApplication::CursorPointerIndex, FVector2f::ZeroVector);
}

FSlateUser::~FSlateUser()
{
#if WITH_SLATE_DEBUGGING
	ClearForceFieldDebug();
#endif

	UE_LOGF(LogSlate, Log, "Slate User Destroyed.  User Index %d, Is Virtual User: %d", UserIndex, IsVirtualUser());
}

TSharedPtr<SWidget> FSlateUser::GetFocusedWidget() const
{
	return WeakFocusPath.IsValid() ? WeakFocusPath.GetLastWidget().Pin() : nullptr;
}

TOptional<EFocusCause> FSlateUser::HasFocus(TSharedPtr<const SWidget> Widget) const
{
	return Widget && GetFocusedWidget() == Widget ? FocusCause : TOptional<EFocusCause>();
}

bool FSlateUser::ShouldShowFocus(TSharedPtr<const SWidget> Widget) const
{
	return HasFocus(Widget).IsSet() && bShowFocus;
}

bool FSlateUser::HasFocusedDescendants(TSharedRef<const SWidget> Widget) const
{
	return WeakFocusPath.IsValid() && WeakFocusPath.GetLastWidget().Pin() != Widget && WeakFocusPath.ContainsWidget(&Widget.Get());
}

bool FSlateUser::IsWidgetInFocusPath(TSharedPtr<const SWidget> Widget) const
{
	return Widget && WeakFocusPath.IsValid() && WeakFocusPath.ContainsWidget(Widget.Get());
}

bool FSlateUser::SetFocus(const TSharedRef<SWidget>& WidgetToFocus, EFocusCause ReasonFocusIsChanging)
{
	return FSlateApplication::Get().SetUserFocus(UserIndex, WidgetToFocus, ReasonFocusIsChanging);
}

void FSlateUser::ClearFocus(EFocusCause ReasonFocusIsChanging)
{
	FSlateApplication::Get().ClearUserFocus(UserIndex, ReasonFocusIsChanging);
}

bool FSlateUser::HasAnyCapture() const
{
	return PointerCaptorPathsByIndex.Num() > 0;
}

bool FSlateUser::HasCursorCapture() const
{
	return HasCapture(FSlateApplication::CursorPointerIndex);
}

bool FSlateUser::HasCapture(uint32 PointerIndex) const
{
	const FWeakWidgetPath* CaptorPath = PointerCaptorPathsByIndex.Find(PointerIndex);
	return CaptorPath && CaptorPath->IsValid();
}

bool FSlateUser::DoesWidgetHaveAnyCapture(TSharedPtr<const SWidget> Widget) const
{
	for (const auto& IndexPathPair : PointerCaptorPathsByIndex)
	{
		if (IndexPathPair.Value.GetLastWidget().Pin() == Widget)
		{
			return true;
		}
	}
	return false;
}

bool FSlateUser::DoesWidgetHaveCursorCapture(TSharedPtr<const SWidget> Widget) const
{
	return DoesWidgetHaveCapture(Widget, FSlateApplication::CursorPointerIndex);
}

bool FSlateUser::DoesWidgetHaveCapture(TSharedPtr<const SWidget> Widget, uint32 PointerIndex) const
{
	const FWeakWidgetPath* CaptorPath = PointerCaptorPathsByIndex.Find(PointerIndex);
	return CaptorPath && CaptorPath->GetLastWidget().Pin() == Widget;
}

bool FSlateUser::SetCursorCaptor(TSharedRef<const SWidget> Widget, const FWidgetPath& EventPath)
{
	return SetPointerCaptor(FSlateApplication::CursorPointerIndex, Widget, EventPath);
}

bool FSlateUser::SetPointerCaptor(uint32 PointerIndex, TSharedRef<const SWidget> Widget, const FWidgetPath& EventPath)
{
	// Bail on any current capture we may have for this pointer before trying to establish the new captor
	ReleaseCapture(PointerIndex);
	
	if (ensureMsgf(EventPath.IsValid(), TEXT("An unknown widget is attempting to set capture to %s"), *Widget->ToString()))
	{
		FWidgetPath NewCaptorPath = EventPath.GetPathDownTo(Widget);
		if (!NewCaptorPath.IsValid() || NewCaptorPath.GetLastWidget() != Widget)
		{
			// The target widget wasn't in the given event path, so try searching for it from the root of the event path
			NewCaptorPath = EventPath.GetPathDownTo(EventPath.GetWindow());
			NewCaptorPath.ExtendPathTo(FWidgetMatcher(Widget));
		}
		
		if (NewCaptorPath.IsValid() && NewCaptorPath.GetLastWidget() == Widget)
		{
			PointerCaptorPathsByIndex.Add(PointerIndex, NewCaptorPath);

#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastMouseCapture(UserIndex, PointerIndex, Widget);
#endif
			return true;
		}
	}
	return false;
}

void FSlateUser::ReleaseAllCapture()
{
	TArray<uint32> CapturedPointerIndices;
	PointerCaptorPathsByIndex.GenerateKeyArray(CapturedPointerIndices);
	for (uint32 CapturedPointerIdx : CapturedPointerIndices)
	{
		ReleaseCapture(CapturedPointerIdx);
	}
}

void FSlateUser::ReleaseCursorCapture()
{
	ReleaseCapture(FSlateApplication::CursorPointerIndex);
}

void FSlateUser::ReleaseCapture(uint32 PointerIndex)
{
	if (const FWeakWidgetPath* CaptorPath = PointerCaptorPathsByIndex.Find(PointerIndex))
	{
		WidgetsUnderPointerLastEventByIndex.Add(PointerIndex, *CaptorPath);
		
		TSharedPtr<SWidget> CaptorWidget = CaptorPath->GetLastWidget().Pin();
		CaptorPath = nullptr;

		// Make sure we don't recursively come back here, in case OnMouseCaptureLost causes
		// release capture again.
		PointerCaptorPathsByIndex.Remove(PointerIndex);
		
		if (CaptorWidget.IsValid())
		{
			CaptorWidget->OnMouseCaptureLost(FCaptureLostEvent(UserIndex, PointerIndex));
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastMouseCaptureLost(UserIndex, PointerIndex, CaptorWidget);
#endif
		}

		if (PointerIndex == FSlateApplication::CursorPointerIndex)
		{
			// If cursor capture changes, we should refresh the cursor state.
			RequestCursorQuery();
		}
	}
}

TArray<FWidgetPath> FSlateUser::GetCaptorPaths()
{
	TArray<FWidgetPath> CaptorPaths;

	TArray<uint32> CaptureIndices;
	PointerCaptorPathsByIndex.GenerateKeyArray(CaptureIndices);
	for (uint32 PointerIndex : CaptureIndices)
	{
		FWidgetPath CaptorPath = GetCaptorPath(PointerIndex);
		if (CaptorPath.IsValid())
		{
			CaptorPaths.Add(CaptorPath);
		}
	}

	return CaptorPaths;
}

FWidgetPath FSlateUser::GetCursorCaptorPath(FWeakWidgetPath::EInterruptedPathHandling::Type InterruptedPathHandling, const FPointerEvent* PointerEvent)
{
	return GetCaptorPath(FSlateApplication::CursorPointerIndex, InterruptedPathHandling, PointerEvent);
}

FWidgetPath FSlateUser::GetCaptorPath(uint32 PointerIndex, FWeakWidgetPath::EInterruptedPathHandling::Type InterruptedPathHandling, const FPointerEvent* PointerEvent)
{
	FWidgetPath CaptorPath;
	if (const FWeakWidgetPath* WeakCaptorPath = PointerCaptorPathsByIndex.Find(PointerIndex))
	{
		if (WeakCaptorPath->ToWidgetPath(CaptorPath, InterruptedPathHandling, PointerEvent) == FWeakWidgetPath::EPathResolutionResult::Truncated)
		{
			// The path was truncated, meaning it's not actually valid anymore, so we want to clear our entry for it out immediately
			WeakCaptorPath = nullptr;
			ReleaseCapture(PointerIndex);
		}
	}
	return CaptorPath;
}

FWeakWidgetPath FSlateUser::GetWeakCursorCapturePath() const
{
	return GetWeakCapturePath(FSlateApplication::CursorPointerIndex);
}

FWeakWidgetPath FSlateUser::GetWeakCapturePath(uint32 PointerIndex) const
{
	const FWeakWidgetPath* WeakCaptorPath = PointerCaptorPathsByIndex.Find(PointerIndex);
	return WeakCaptorPath ? *WeakCaptorPath : FWeakWidgetPath();
}

TArray<TSharedRef<SWidget>> FSlateUser::GetCaptorWidgets() const
{
	TArray<TSharedRef<SWidget>> AllCaptors;
	AllCaptors.Reserve(PointerCaptorPathsByIndex.Num());
	for (const auto& IndexPathPair : PointerCaptorPathsByIndex)
	{
		if (TSharedPtr<SWidget> CaptorWidget = IndexPathPair.Value.GetLastWidget().Pin())
		{
			AllCaptors.Add(CaptorWidget.ToSharedRef());
		}
	}
	return AllCaptors;
}

TSharedPtr<SWidget> FSlateUser::GetCursorCaptor() const
{
	return GetPointerCaptor(FSlateApplication::CursorPointerIndex);
}

TSharedPtr<SWidget> FSlateUser::GetPointerCaptor(uint32 PointerIndex) const
{
	const FWeakWidgetPath* WeakCaptorPath = PointerCaptorPathsByIndex.Find(PointerIndex);
	return WeakCaptorPath ? WeakCaptorPath->GetLastWidget().Pin() : nullptr;
}

void FSlateUser::SetCursorVisibility(bool bDrawCursor)
{
	bCanDrawCursor = bDrawCursor;
	
	if (bCanDrawCursor)
	{
		if (bNeedQueryForVisibleCursor)
		{
			RequestCursorQuery();
			bNeedQueryForVisibleCursor = false;
		}
	}
	else
	{
		bNeedQueryForVisibleCursor = true;
		ProcessCursorReply(FCursorReply::Cursor(EMouseCursor::None));
	}
}

bool FSlateUser::IsCursorVisible() const
{
	// Consider the cursor valid so long as we can draw it and it is not the "None" type
	return bCanDrawCursor && Cursor.IsValid() && Cursor->GetType() != EMouseCursor::None;
}

void FSlateUser::SetCursorPosition(int32 PosX, int32 PosY)
{
	SetPointerPosition(FSlateApplication::CursorPointerIndex, PosX, PosY);
}

void FSlateUser::SetCursorPosition(const UE::Slate::FDeprecateVector2DParameter& NewCursorPos)
{
	SetCursorPosition((int32)NewCursorPos.X, (int32)NewCursorPos.Y);
}

void FSlateUser::SetPointerPosition(uint32 PointerIndex, int32 PosX, int32 PosY)
{
	if (Cursor && PointerIndex == FSlateApplication::CursorPointerIndex)
	{
		UE_LOGF(LogSlate, Verbose, "SlateUser [%d] moving cursor @ (%d, %d)", UserIndex, PosX, PosY );

		Cursor->SetPosition(PosX, PosY);
	}

	UpdatePointerPosition(PointerIndex, FVector2f(PosX, PosY));
}

void FSlateUser::SetPointerPosition(uint32 PointerIndex, const UE::Slate::FDeprecateVector2DParameter& NewPointerPos)
{
	SetPointerPosition(PointerIndex, (int32)NewPointerPos.X, (int32)NewPointerPos.Y);
}

UE::Slate::FDeprecateVector2DResult FSlateUser::GetCursorPosition() const
{
	return GetPointerPosition(FSlateApplication::CursorPointerIndex);
}

UE::Slate::FDeprecateVector2DResult FSlateUser::GetPreviousCursorPosition() const
{
	return GetPreviousPointerPosition(FSlateApplication::CursorPointerIndex);
}

UE::Slate::FDeprecateVector2DResult FSlateUser::GetPointerPosition(uint32 PointerIndex) const
{
	if (Cursor && PointerIndex == FSlateApplication::CursorPointerIndex)
	{
		return UE::Slate::CastToVector2f(Cursor->GetPosition());
	}
	const FVector2f* FoundPosition = PointerPositionsByIndex.Find(PointerIndex);
	return FoundPosition ? *FoundPosition : FVector2f::ZeroVector;
}

UE::Slate::FDeprecateVector2DResult FSlateUser::GetPreviousPointerPosition(uint32 PointerIndex) const
{
	const FVector2f* FoundPosition = PreviousPointerPositionsByIndex.Find(PointerIndex);
	return FoundPosition ? *FoundPosition : GetPointerPosition(PointerIndex);
}

bool FSlateUser::IsWidgetUnderCursor(TSharedPtr<const SWidget> Widget) const
{
	return IsWidgetUnderPointer(Widget, FSlateApplication::CursorPointerIndex);
}

bool FSlateUser::IsWidgetUnderPointer(TSharedPtr<const SWidget> Widget, uint32 PointerIndex) const
{
	const FWeakWidgetPath* WidgetsUnderPointer = WidgetsUnderPointerLastEventByIndex.Find(PointerIndex);
	return Widget && WidgetsUnderPointer && WidgetsUnderPointer->ContainsWidget(Widget.Get());
}

bool FSlateUser::IsWidgetUnderAnyPointer(TSharedPtr<const SWidget> Widget) const
{
	if (Widget)
	{
		for (const auto& IndexPathPair : WidgetsUnderPointerLastEventByIndex)
		{
			if (IndexPathPair.Value.ContainsWidget(Widget.Get()))
			{
				return true;
			}
		}
	}
	return false;
}

bool FSlateUser::IsWidgetDirectlyUnderCursor(TSharedPtr<const SWidget> Widget) const
{
	return IsWidgetDirectlyUnderPointer(Widget, FSlateApplication::CursorPointerIndex);
}

bool FSlateUser::IsWidgetDirectlyUnderPointer(TSharedPtr<const SWidget> Widget, uint32 PointerIndex) const
{
	const FWeakWidgetPath* WidgetsUnderPointer = WidgetsUnderPointerLastEventByIndex.Find(PointerIndex);
	return WidgetsUnderPointer && WidgetsUnderPointer->IsValid() && WidgetsUnderPointer->GetLastWidget().Pin() == Widget;
}

bool FSlateUser::IsWidgetDirectlyUnderAnyPointer(TSharedPtr<const SWidget> Widget) const
{
	for (const auto& IndexPathPair : WidgetsUnderPointerLastEventByIndex)
	{
		if (IndexPathPair.Value.IsValid() && IndexPathPair.Value.GetLastWidget().Pin() == Widget)
		{
			return true;
		}
	}
	return false;
}

FWeakWidgetPath FSlateUser::GetLastWidgetsUnderCursor() const
{
	return GetLastWidgetsUnderPointer(FSlateApplication::CursorPointerIndex);
}

FWeakWidgetPath FSlateUser::GetLastWidgetsUnderPointer(uint32 PointerIndex) const
{
	return WidgetsUnderPointerLastEventByIndex.FindRef(PointerIndex);
}

bool FSlateUser::IsDragDropping() const
{
	return DragDropContent.IsValid();
}

bool FSlateUser::IsDragDroppingAffected(const FPointerEvent& InPointerEvent) const
{
	return DragDropContent.IsValid() && DragDropContent->AffectedByPointerEvent(InPointerEvent);
}

void FSlateUser::CancelDragDrop()
{
	if (DragDropContent.IsValid())
	{
		const FPointerEvent EmptyPointerEvent;
		const FDragDropEvent DragDropEvent(EmptyPointerEvent, DragDropContent);

		for (const auto& IndexPathPair : WidgetsUnderPointerLastEventByIndex)
		{
			FWidgetPath WidgetsToDragLeave = IndexPathPair.Value.ToWidgetPath();
			if (WidgetsToDragLeave.IsValid())
			{
				for (int32 WidgetIndex = WidgetsToDragLeave.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
				{
					WidgetsToDragLeave.Widgets[WidgetIndex].Widget->OnDragLeave(DragDropEvent);
				}
			}
		}

		// Cancel dragdrop operation correctly firing off callbacks
		DragDropContent->OnDrop(false, EmptyPointerEvent);

		// We always reset the cache of widgets under our pointers whenever we enter/exit drag-drop mode
		WidgetsUnderPointerLastEventByIndex.Reset();

		DragDropContent.Reset();
	}
}

void FSlateUser::ShowTooltip(const TSharedRef<IToolTip>& InTooltip, const UE::Slate::FDeprecateVector2DParameter& InLocation, TSharedPtr<SWindow> OwnerWindow)
{
	CloseTooltip();

	ActiveTooltipInfo.Tooltip = InTooltip;

	// Establish the tooltip content in the window (per-owner for correct Wayland parenting)
	TSharedRef<SWindow> TooltipWindow = GetOrCreateTooltipWindow(OwnerWindow);
	TooltipWindowPtr = TooltipWindow;	
	TooltipWindow->SetContent(
		SNew(SWeakWidget)
		.PossiblyNullContent(InTooltip->AsWidget()));

	// Make sure the desired size is valid
	FSlateApplication& SlateApp = FSlateApplication::Get();
	TooltipWindow->SlatePrepass(SlateApp.GetApplicationScale() * TooltipWindow->GetNativeWindow()->GetDPIScaleFactor());

	// Place the window as close to the given location as possible (MoveWindowTo will adjust the window's position to stay onscreen, if needed)
	const FSlateRect Anchor(InLocation.X, InLocation.Y, InLocation.X, InLocation.Y);
	ActiveTooltipInfo.DesiredLocation = SlateApp.CalculateTooltipWindowPosition(Anchor, TooltipWindow->GetDesiredSizeDesktopPixels(), /*bAutoAdjustForDPIScale =*/false, (InTooltip->IsInteractive()) ? EPopupCursorOverlapMode::AllowOverlap : EPopupCursorOverlapMode::PreventOverlap);
	TooltipWindow->MoveWindowTo(ActiveTooltipInfo.DesiredLocation);

	TooltipWindow->SetOpacity(0.0f);
	TooltipWindow->ShowWindow();
	ActiveTooltipInfo.SummonTime = FPlatformTime::Seconds();
}

void FSlateUser::CloseTooltip()
{
	ActiveTooltipInfo.Reset();

	// Hide the tooltip window (don't destroy it - we'll reuse it for that owner)
	TSharedPtr<SWindow> TooltipWindow = TooltipWindowPtr.Pin();
	if (TooltipWindow && TooltipWindow->IsVisible())
	{
		TooltipWindow->HideWindow();
	}
}

FVector2f FSlateUser::GetTooltipPosition() const
{
	if (TooltipWindowPtr.IsValid())
	{
		return TooltipWindowPtr.Pin()->GetPositionInScreen();
	}
	return FVector2f::Zero();
}

void FSlateUser::SetUserNavigationConfig(TSharedPtr<FNavigationConfig> InNavigationConfig)
{
	if (UserNavigationConfig)
	{
		UserNavigationConfig->OnUnregister();
	}

	UserNavigationConfig = InNavigationConfig;
	
	if (InNavigationConfig)
	{
		InNavigationConfig->OnRegister();
	}

#if WITH_SLATE_DEBUGGING
	FSlateApplication::Get().TryDumpNavigationConfig(UserNavigationConfig);
#endif // WITH_SLATE_DEBUGGING
}

bool FSlateUser::HasValidFocusPath() const 
{
	return WeakFocusPath.IsValid();
}

const FWeakWidgetPath& FSlateUser::GetWeakFocusPath() const
{
	return WeakFocusPath;
}

bool FSlateUser::IsTouchPointerActive(int32 TouchPointerIndex) const
{
	return TouchPointerIndex < (int32)ETouchIndex::CursorPointerIndex && PointerPositionsByIndex.Contains(TouchPointerIndex);
}

void FSlateUser::DrawWindowlessDragDropContent(const TSharedRef<SWindow>& WindowToDraw, FSlateWindowElementList& WindowElementList, int32& MaxLayerId)
{
	if (DragDropContent && DragDropContent->IsWindowlessOperation())
	{
		TSharedPtr<SWindow> DragDropWindow = DragDropWindowPtr.Pin();
		if (DragDropWindow && DragDropWindow == WindowToDraw)
		{
			TSharedPtr<SWidget> DecoratorWidget = DragDropContent->GetDefaultDecorator();
			if (DecoratorWidget && DecoratorWidget->GetVisibility().IsVisible())
			{
				FSlateApplication& SlateApp = FSlateApplication::Get();
				const float WindowRootScale = SlateApp.GetApplicationScale() * DragDropWindow->GetNativeWindow()->GetDPIScaleFactor();

				DecoratorWidget->SetVisibility(EVisibility::HitTestInvisible);
				DecoratorWidget->SlatePrepass(WindowRootScale);

				FVector2f DragDropContentInWindowSpace = WindowToDraw->GetWindowGeometryInScreen().AbsoluteToLocal(DragDropContent->GetDecoratorPosition()) * WindowRootScale;
				const FGeometry DragDropContentGeometry = FGeometry::MakeRoot(DecoratorWidget->GetDesiredSize(), FSlateLayoutTransform(DragDropContentInWindowSpace));

				DecoratorWidget->Paint(
					FPaintArgs(&WindowToDraw.Get(), WindowToDraw->GetHittestGrid(), WindowToDraw->GetPositionInScreen(), SlateApp.GetCurrentTime(), SlateApp.GetDeltaTime()), 
					DragDropContentGeometry, WindowToDraw->GetClippingRectangleInWindow(),
					WindowElementList,
					++MaxLayerId,
					FWidgetStyle(),
					WindowToDraw->IsEnabled());
			}
		}
	}
}

void FSlateUser::DrawCursor(const TSharedRef<SWindow>& WindowToDraw, FSlateWindowElementList& WindowElementList, int32& MaxLayerId)
{
	TSharedPtr<SWindow> CursorWindow = CursorWindowPtr.Pin();
	if (bCanDrawCursor && CursorWindow && WindowToDraw == CursorWindow)
	{
		if (TSharedPtr<SWidget> CursorWidget = CursorWidgetPtr.Pin())
		{
			FSlateApplication& SlateApp = FSlateApplication::Get();
			const float WindowRootScale = FSlateApplication::Get().GetApplicationScale() * CursorWindow->GetNativeWindow()->GetDPIScaleFactor();

			CursorWidget->SetVisibility(EVisibility::HitTestInvisible);
			CursorWidget->SlatePrepass(WindowRootScale);

			FVector2f CursorInScreen = GetCursorPosition();
			FVector2f CursorPosInWindowSpace = WindowToDraw->GetWindowGeometryInScreen().AbsoluteToLocal(CursorInScreen) * WindowRootScale;
			CursorPosInWindowSpace += (CursorWidget->GetDesiredSize() * SoftwareCursorScale * -0.5);
			const FGeometry CursorGeometry = FGeometry::MakeRoot(CursorWidget->GetDesiredSize() * SoftwareCursorScale, FSlateLayoutTransform(CursorPosInWindowSpace));

			CursorWidget->Paint(
				FPaintArgs(&WindowToDraw.Get(), WindowToDraw->GetHittestGrid(), WindowToDraw->GetPositionInScreen(), SlateApp.GetCurrentTime(), SlateApp.GetDeltaTime()),
				CursorGeometry, WindowToDraw->GetClippingRectangleInWindow(),
				WindowElementList,
				++MaxLayerId,
				FWidgetStyle(),
				WindowToDraw->IsEnabled());
		}
	}
}

void FSlateUser::QueueSyntheticCursorMove()
{
	// Q: Wait, why 2?
	// A: Synthesized moves are processed last, after all other inputs (other inputs are often what call this), 
	//		but Slate won't have had a chance yet to do any actual widget updating. We need to make sure we synthesize the 
	//		move *after* slate has had a chance to update.
	
	// Q: Ok, then shouldn't we only synthesize the move when we go from 1 -> 0?
	// A: Nope - imagine a user provided input last frame without moving the mouse. 
	//		That queued a synthetic move and was processed (possibly unnecessarily), going from 2 -> 1.
	//	This frame, input has been provided again, resetting this value back to 2.
	//	If we wait until we go from 1 -> 0, we would not synthesize a move until a frame without input, and
	//		we'd indefinitely delay the very update the synthetic move is intended to trigger until the user stopped providing any input.
	NumPendingSyntheticCursorMoves = 2;
}

bool FSlateUser::SynthesizeCursorMoveIfNeeded()
{
	// The slate loading widget thread is not allowed to execute this code as it is unsafe to read the hittest grid in another thread
	if (bEnableSyntheticCursorMoves && IsInGameThread() && ensure(Cursor) && --NumPendingSyntheticCursorMoves >= 0)
	{
		// Synthetic cursor/mouse events accomplish two goals:
		// 1) The UI can change even if the mouse doesn't move.
		//    Synthesizing a mouse move sends out events.
		//    In this case, the current and previous position will be the same.
		//
		// 2) The mouse moves, but the OS decided not to send us an event.
		//    e.g. Mouse moved outside of our window.
		//    In this case, the previous and current positions differ.

		FSlateApplication& SlateApp = FSlateApplication::Get();
		
		FInputDeviceId InputDeviceId = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(GetPlatformUserId());
		
		// The input device might be invalid if a split screen player has logged off but still has their controller plugged in
		if (InputDeviceId.IsValid())
		{			
			const bool bHasHardwareCursor = SlateApp.GetPlatformCursor() == Cursor;
			const TSet<FKey> EmptySet;
			FPointerEvent SyntheticCursorMoveEvent(
				InputDeviceId,
				FSlateApplication::CursorPointerIndex,
				GetCursorPosition(),
				GetPreviousCursorPosition(),
				bHasHardwareCursor ? SlateApp.GetPressedMouseButtons() : EmptySet,
				EKeys::Invalid,
				0,
				bHasHardwareCursor ? SlateApp.GetPlatformApplication()->GetModifierKeys() : FModifierKeysState(),
				UserIndex);

			SlateApp.ProcessMouseMoveEvent(SyntheticCursorMoveEvent, true);
			return true;

		}		
	}
	return false;
}

void FSlateUser::SetFocusPath(const FWidgetPath& NewFocusPath, EFocusCause InFocusCause, bool bInShowFocus)
{
	StrongFocusPath.Reset();
	WeakFocusPath = NewFocusPath;
	FocusCause = InFocusCause;
	bShowFocus = bInShowFocus;
}

void FSlateUser::FinishFrame()
{
	StrongFocusPath.Reset();
}

void FSlateUser::NotifyWindowDestroyed(TSharedRef<SWindow> DestroyedWindow)
{
	if (StrongFocusPath && StrongFocusPath->IsValid() && DestroyedWindow == StrongFocusPath->GetWindow())
	{
		StrongFocusPath.Reset();
	}
}

void FSlateUser::QueryCursor()
{
	bQueryCursorRequested = false;

	// The slate loading widget thread is not allowed to execute this code (it's unsafe to read the hittest grid in another thread)
	if (bCanDrawCursor && Cursor && IsInGameThread() && FApp::CanEverRender())
	{
		SCOPE_CYCLE_COUNTER(STAT_SlateQueryCursor);

		FSlateApplication& SlateApp = FSlateApplication::Get();
		FCursorReply CursorReply = FCursorReply::Unhandled();
		
		// Drag-drop gets first dibs if it exists
		if (DragDropContent)
		{
			CursorReply = DragDropContent->OnCursorQuery();
		}

		if (!CursorReply.IsEventHandled())
		{
			const bool bHasHardwareCursor = SlateApp.GetPlatformCursor() == Cursor;
			const FVector2f CurrentCursorPosition = GetCursorPosition();
			const FVector2f LastCursorPosition = GetPreviousCursorPosition();			
			
			const TSet<FKey> EmptySet;
			const FPointerEvent CursorEvent(
				SlateApp.GetUserIndexForMouse(),
				FSlateApplication::CursorPointerIndex,
				CurrentCursorPosition,
				LastCursorPosition,
				CurrentCursorPosition - LastCursorPosition,
				bHasHardwareCursor ? SlateApp.GetPressedMouseButtons() : EmptySet,
				bHasHardwareCursor ? SlateApp.GetModifierKeys() : FModifierKeysState());

			FWidgetPath WidgetsToQueryForCursor;
			if (HasCursorCapture())
			{
				// Query widgets with mouse capture for the cursor
				FWidgetPath MouseCaptorPath = GetCursorCaptorPath(FWeakWidgetPath::EInterruptedPathHandling::Truncate, &CursorEvent);
				if (MouseCaptorPath.IsValid())
				{
					TSharedRef<SWindow> CaptureWindow = MouseCaptorPath.GetWindow();
					const TSharedPtr<SWindow> ActiveModalWindow = SlateApp.GetActiveModalWindow();

					// Never query the mouse captor path if it is outside an active modal window
					if (!ActiveModalWindow || CaptureWindow == ActiveModalWindow || CaptureWindow->IsDescendantOf(ActiveModalWindow))
					{
						WidgetsToQueryForCursor = MouseCaptorPath;
					}
				}
			}
			else
			{
				WidgetsToQueryForCursor = SlateApp.LocateWindowUnderMouse(CurrentCursorPosition, SlateApp.GetInteractiveTopLevelWindows(), false, UserIndex);
			}

			if (WidgetsToQueryForCursor.IsValid())
			{
				// Switch worlds for widgets in the current path
				FScopedSwitchWorldHack SwitchWorld(WidgetsToQueryForCursor);

				for (int32 WidgetIndex = WidgetsToQueryForCursor.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
				{
					const FArrangedWidget& ArrangedWidget = WidgetsToQueryForCursor.Widgets[WidgetIndex];

					CursorReply = ArrangedWidget.Widget->OnCursorQuery(ArrangedWidget.Geometry, CursorEvent);
					if (CursorReply.IsEventHandled())
					{
#if WITH_SLATE_DEBUGGING
						FSlateDebugging::BroadcastCursorQuery(ArrangedWidget.Widget, CursorReply);
#endif

						if (!CursorReply.GetCursorWidget().IsValid())
						{
							for (; WidgetIndex >= 0; --WidgetIndex)
							{
								TOptional<TSharedRef<SWidget>> CursorWidget = WidgetsToQueryForCursor.Widgets[WidgetIndex].Widget->OnMapCursor(CursorReply);
								if (CursorWidget.IsSet())
								{
									CursorReply.SetCursorWidget(WidgetsToQueryForCursor.GetWindow(), CursorWidget.GetValue());
									break;
								}
							}
						}
						break;
					}
				}

				if (!CursorReply.IsEventHandled() && WidgetsToQueryForCursor.IsValid())
				{
					// Query was NOT handled, and we are still over a slate window.
					CursorReply = FCursorReply::Cursor(EMouseCursor::Default);
					
#if WITH_SLATE_DEBUGGING
					FSlateDebugging::BroadcastCursorQuery(TSharedPtr<SWidget>(), CursorReply);
#endif
				}
			}
			else
			{
				// Set the default cursor when there isn't an active window under the cursor and the mouse isn't captured
				CursorReply = FCursorReply::Cursor(EMouseCursor::Default);

#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastCursorQuery(TSharedPtr<SWidget>(), CursorReply);
#endif
			}
		}
		ProcessCursorReply(CursorReply);
	}
}

void FSlateUser::LockCursor(const TSharedRef<SWidget>& Widget)
{
	if (Cursor)
	{
		// Get a path to this widget so we know the position and size of its geometry
		FWidgetPath WidgetPath;
		const bool bFoundWidget = FSlateApplication::Get().GeneratePathToWidgetUnchecked(Widget, WidgetPath);
		if (ensureMsgf(bFoundWidget, TEXT("Attempting to LockCursor() to widget but could not find widget %s"), *Widget->ToString()))
		{
			LockCursorInternal(WidgetPath);
		}
	}
}

void FSlateUser::UnlockCursor()
{
	if (Cursor)
	{
		Cursor->Lock(nullptr);
		LockingWidgetPath = FWeakWidgetPath();
	}
}

void FSlateUser::UpdateCursor()
{
	if (!Cursor)
	{
		return;
	}

	Cursor->Tick();
	if (LockingWidgetPath.IsValid())
	{
		const FWidgetPath PathToWidget = LockingWidgetPath.ToWidgetPath(FWeakWidgetPath::EInterruptedPathHandling::ReturnInvalid);
		if (PathToWidget.IsValid())
		{
			const FSlateRect ComputedClipRect = PathToWidget.Widgets.Last().Geometry.GetLayoutBoundingRect();
			if (ComputedClipRect != LastComputedLockBounds)
			{
				// The locking widget is still valid, but its bounds have changed - gotta update the lock boundaries on the cursor to match
				LockCursorInternal(PathToWidget);
			}
		}
		else
		{
			// Unlock immediately if the locking widget is no longer around
			UnlockCursor();
		}
	}	

	// When Slate captures the mouse, it is up to us to set the cursor because the OS assumes that we own the mouse.
	if (HasAnyCapture() || bQueryCursorRequested)
	{
		QueryCursor();
	}

	const double MoveEpsilonSquared = CursorSignificantMoveDetectionThreshold * CursorSignificantMoveDetectionThreshold;
	if (FVector2D::DistSquared(GetPreviousCursorPosition(), GetCursorPosition()) > MoveEpsilonSquared)
	{
		LastCursorSignificantMoveTime = FPlatformTime::Seconds();
	}
}

void FSlateUser::ProcessCursorReply(const FCursorReply& CursorReply)
{
	if (Cursor && CursorReply.IsEventHandled())
	{
		if (bCanDrawCursor)
		{
			CursorWidgetPtr = CursorReply.GetCursorWidget();
			if (CursorReply.GetCursorWidget().IsValid())
			{
				CursorReply.GetCursorWidget()->SetVisibility(EVisibility::HitTestInvisible);
				CursorWindowPtr = CursorReply.GetCursorWindow();
				if (!FSlateApplication::Get().IsFakingTouchEvents())
				{
					Cursor->SetType(EMouseCursor::Custom);
				}
			}
			else
			{
				CursorWindowPtr.Reset();
				Cursor->SetType(CursorReply.GetCursorType());
			}
		}
		else
		{
			Cursor->SetType(EMouseCursor::None);
		}
	}
	else
	{
		CursorWindowPtr.Reset();
		CursorWidgetPtr.Reset();
	}
}

void FSlateUser::LockCursorInternal(const FWidgetPath& WidgetPath)
{
	check(Cursor);
	check(WidgetPath.IsValid());

	// Do not attempt to lock the cursor to the window if its not in the foreground.  It would cause annoying side effects
	TSharedPtr<const FGenericWindow> NativeWindow = WidgetPath.GetWindow()->GetNativeWindow();
	if (NativeWindow && (NativeWindow->IsForegroundWindow() || NativeWindow->SupportsDynamicCursorLock()))
	{
		// The last widget in the path should be the widget we are locking the cursor to
		FSlateRect SlateClipRect = WidgetPath.Widgets.Last().Geometry.GetLayoutBoundingRect();
		LastComputedLockBounds = SlateClipRect;
		LockingWidgetPath = WidgetPath;

		// Screen space mapping scales everything. When viewport resolution doesn't match platform resolution, 
		// this causes offset cursor hit-tests in fullscreen. Correct when capturing mouse as viewport widget may be smaller than screen in pixels.
		if (FSlateApplication::Get().GetTransformFullscreenMouseInput() && !GIsEditor && NativeWindow->GetWindowMode() == EWindowMode::Fullscreen)
		{
			const FDisplayMetrics& CachedDisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
			const FVector2f DisplaySize = CachedDisplayMetrics.MonitorInfo.IsEmpty()
				? FVector2f(CachedDisplayMetrics.PrimaryDisplayWidth, CachedDisplayMetrics.PrimaryDisplayHeight)
				: WidgetPath.GetWindow()->GetFullScreenInfo().GetSize2f();
			FVector2f DisplayDistortion = DisplaySize / SlateClipRect.GetSize();

			SlateClipRect.Right = ((SlateClipRect.Right - SlateClipRect.Left) * DisplayDistortion.X) + SlateClipRect.Left;
			SlateClipRect.Bottom = ((SlateClipRect.Bottom - SlateClipRect.Top) * DisplayDistortion.Y) + SlateClipRect.Top;
		}

		// Note: Slate uses floating point bounds, but the cursor needs integers. Round the clip rect inwardly, to guarantee it remains inside the
		// widget.  Truncating the left and/or top would cause the clip rect to fall outside the geometry in cases where the widget's left/top
		// coordinates are fractional.
		RECT ClipRect;
		ClipRect.left = FMath::RoundToPositiveInfinity(SlateClipRect.Left);
		ClipRect.top = FMath::RoundToPositiveInfinity(SlateClipRect.Top);
		ClipRect.right = FMath::RoundToNegativeInfinity(SlateClipRect.Right);
		ClipRect.bottom = FMath::RoundToNegativeInfinity(SlateClipRect.Bottom);

		// Lock the cursor to the widget
		Cursor->Lock(&ClipRect);
	}
}

TSharedRef<SWindow> FSlateUser::GetOrCreateTooltipWindow(TSharedPtr<SWindow> OwnerWindow)
{
	// Platforms without per-toplevel parent constraints share one tooltip window across
	// owners -- avoids spawning a Vulkan viewport per owner.
	if (!FSlateApplication::Get().GetPlatformApplication()->RequiresPerWindowTooltips())
	{
		OwnerWindow.Reset();
	}

	// Look up existing tooltip window for this owner
	if (OwnerWindow)
	{
		TWeakPtr<SWindow>* FoundTooltip = TooltipWindowsByOwner.Find(OwnerWindow);
		if (FoundTooltip && FoundTooltip->IsValid())
		{
			return FoundTooltip->Pin().ToSharedRef();
		}
	}
	else
	{
		// No owner -- check for a previously created ownerless tooltip
		TWeakPtr<SWindow> NullKey;
		TWeakPtr<SWindow>* FoundTooltip = TooltipWindowsByOwner.Find(NullKey);
		if (FoundTooltip && FoundTooltip->IsValid())
		{
			return FoundTooltip->Pin().ToSharedRef();
		}
	}

	// Create a new tooltip window for this owner
	TSharedRef<SWindow> NewTooltipWindow = SWindow::MakeToolTipWindow();
	if (OwnerWindow)
	{
		TooltipWindowsByOwner.Add(OwnerWindow, NewTooltipWindow);
	}
	else
	{
		TooltipWindowsByOwner.Add(nullptr, NewTooltipWindow);
	}

	// Plumb OwnerWindow through to the native layer when per-toplevel parents are required.
	// Otherwise the platform falls back to SDL_GetMouseFocus(), which can disagree with the
	// Slate owner when the cursor briefly sits over a different toplevel at creation time.
	if (OwnerWindow.IsValid()
		&& FSlateApplication::Get().GetPlatformApplication()->RequiresPerWindowTooltips())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(NewTooltipWindow, OwnerWindow.ToSharedRef(), /*bShowImmediately =*/false);
	}
	else
	{
		FSlateApplication::Get().AddWindow(NewTooltipWindow, /*bShowImmediately =*/false);
	}
	return NewTooltipWindow;
}

#if WITH_SLATE_DEBUGGING
void FSlateUser::ClearForceFieldDebug()
{
	if (!DebugForceFieldRects.IsEmpty())
	{
		DebugForceFieldRects.Reset();
		FSlateDebugging::PaintDebugElements.RemoveAll(this);
	}
}

void FSlateUser::HandlePaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry,
	FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId)
{
	for (const FSlateRect& Rect : DebugForceFieldRects)
	{
		FSlateDrawElement::MakeBox(
			InOutDrawElements,
			++InOutLayerId,
			InAllottedGeometry.ToPaintGeometry(
				Rect.GetSize2f(),
				FSlateLayoutTransform(Rect.GetTopLeft2f() - InArgs.GetWindowToDesktopTransform())
			),
			FCoreStyle::Get().GetBrush("Debug.Border"),
			ESlateDrawEffect::None,
			FLinearColor::Blue
		);
	}
}
#endif

void FSlateUser::NotifyTouchStarted(const FPointerEvent& TouchEvent)
{
	UE_CLOGF(PointerPositionsByIndex.Contains(TouchEvent.GetPointerIndex()), LogSlate, Error, "SlateUser [%d] notified of a touch starting for pointer [%d] without finding out it ever ended.", TouchEvent.GetUserIndex(), TouchEvent.GetPointerIndex());
	
	GestureDetector.OnTouchStarted(TouchEvent.GetPointerIndex(), TouchEvent.GetScreenSpacePosition());
	PointerPositionsByIndex.FindOrAdd(TouchEvent.GetPointerIndex()) = TouchEvent.GetScreenSpacePosition();
}

void FSlateUser::NotifyPointerMoveBegin(const FPointerEvent& PointerEvent)
{
	if (PointerEvent.IsTouchEvent())
	{
		GestureDetector.OnTouchMoved(PointerEvent.GetPointerIndex(), PointerEvent.GetScreenSpacePosition());
	}
	PointerPositionsByIndex.FindOrAdd(PointerEvent.GetPointerIndex()) = PointerEvent.GetScreenSpacePosition();
}

void FSlateUser::NotifyPointerMoveComplete(const FPointerEvent& PointerEvent, const FWidgetPath& WidgetsUnderPointer)
{
	// Give the current drag drop operation a chance to do something custom (e.g. update the Drag/Drop preview based on content)
	if (IsDragDroppingAffected(PointerEvent))
	{
		FDragDropEvent DragDropEvent(PointerEvent, DragDropContent);

#if WITH_EDITOR
		//@TODO VREDITOR - Remove and move to interaction component
		if (FSlateApplication::Get().OnDragDropCheckOverride.IsBound() && DragDropEvent.GetOperation())
		{
			DragDropEvent.GetOperation()->SetDecoratorVisibility(false);
			DragDropEvent.GetOperation()->SetCursorOverride(EMouseCursor::None);
			DragDropContent->SetCursorOverride(EMouseCursor::None);
		}
#endif

		FScopedSwitchWorldHack SwitchWorld(WidgetsUnderPointer);
		DragDropContent->OnDragged(DragDropEvent);

		// Update the window we're under for rendering the drag drop operation if it's a windowless drag drop operation.
		if (WidgetsUnderPointer.IsValid())
		{
			DragDropWindowPtr = WidgetsUnderPointer.GetWindow();
		}
		else
		{
			DragDropWindowPtr.Reset();
		}
		
		if (ensure(Cursor))
		{
			FCursorReply CursorReply = DragDropContent->OnCursorQuery();
			if (!CursorReply.IsEventHandled())
			{
				// Set the default cursor when there isn't an active window under the cursor and the mouse isn't captured
				CursorReply = FCursorReply::Cursor(EMouseCursor::Default);
			}

			ProcessCursorReply(CursorReply);
		}
		
	}
	else if (!IsDragDropping())
	{
		DragDropWindowPtr.Reset();
	}

	PreviousPointerPositionsByIndex.Add(PointerEvent.GetPointerIndex(), PointerEvent.GetScreenSpacePosition());
	WidgetsUnderPointerLastEventByIndex.Add(PointerEvent.GetPointerIndex(), FWeakWidgetPath(WidgetsUnderPointer));
}

void FSlateUser::UpdatePointerPosition(const FPointerEvent& PointerEvent)
{
	UpdatePointerPosition(PointerEvent.GetPointerIndex(), PointerEvent.GetScreenSpacePosition());
}

void FSlateUser::UpdatePointerPosition(uint32 PointerIndex, const FVector2f& Position)
{
	PointerPositionsByIndex.FindOrAdd(PointerIndex) = Position;
	PreviousPointerPositionsByIndex.FindOrAdd(PointerIndex) = Position;
}

void FSlateUser::StartDragDetection(const FWidgetPath& PathToWidget, int32 PointerIndex, FKey DragButton, UE::Slate::FDeprecateVector2DParameter StartLocation)
{
	DragStatesByPointerIndex.Add(PointerIndex, FDragDetectionState(PathToWidget, PointerIndex, DragButton, StartLocation));
}

FWidgetPath FSlateUser::DetectDrag(const FPointerEvent& PointerEvent, float DragTriggerDistance)
{
	if (FDragDetectionState* DragState = DragStatesByPointerIndex.Find(PointerEvent.GetPointerIndex()))
	{
		const FVector2f DragDelta = DragState->DragStartLocation - PointerEvent.GetScreenSpacePosition();
		if (DragDelta.SizeSquared() > FMath::Square(DragTriggerDistance))
		{
			FWidgetPath DragDetectionPath = DragState->DetectDragForWidget.ToWidgetPath(FWeakWidgetPath::EInterruptedPathHandling::ReturnInvalid);
			if (DragDetectionPath.IsValid())
			{
				ResetDragDetection();
				return DragDetectionPath;
			}
		}
	}

	return FWidgetPath();
}

bool FSlateUser::IsDetectingDrag(uint32 PointerIndex) const
{
	return DragStatesByPointerIndex.Contains(PointerIndex);
}

void FSlateUser::NotifyPointerReleased(const FPointerEvent& PointerEvent, const FWidgetPath& WidgetsUnderCursor, TSharedPtr<FDragDropOperation> DroppedContent, bool bWasHandled)
{
	const int32 PointerIdx = PointerEvent.GetPointerIndex();

	const FDragDetectionState* DragState = DragStatesByPointerIndex.Find(PointerIdx);
	if (DragState && DragState->TriggerButton == PointerEvent.GetEffectingButton())
	{
		// The user has released the button (or finger) that was supposed to start the drag; stop detecting it.
		DragState = nullptr;
		DragStatesByPointerIndex.Remove(PointerIdx);
	}

	if (!HasCapture(PointerIdx))
	{
		// When we perform a touch end, we need to also send a mouse leave as if it were a cursor.
		if (PointerEvent.IsTouchEvent() && !FSlateApplication::Get().IsFakingTouchEvents())
		{
			for (int32 WidgetIndex = WidgetsUnderCursor.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
			{
				WidgetsUnderCursor.Widgets[WidgetIndex].Widget->OnMouseLeave(PointerEvent);
			}
		}
#if WITH_SLATE_DEBUGGING
		const bool bIsFingerCapturedPostRelease = PointerEvent.IsTouchEvent() && HasCapture(PointerEvent.GetPointerIndex());
		ensureMsgf(!bIsFingerCapturedPostRelease, TEXT("Touch pointer was captured while in the process of being released. Since touch pointers cease to exist when the touch ends, this makes no sense."));
#endif

		// Note: FSlateApplication caches this content off BEFORE routing the pointer up, as OnDrop can result in re-entrance of the pointer up routing
		//		To avoid executing the drop operation twice, our cached DragDropContent is wiped before the first routing.
		//		Thus, we rely on the provided DroppedContent here, which will never be provided as valid more than once
		if (DroppedContent && DroppedContent->AffectedByPointerEvent(PointerEvent))
		{
			// @todo slate : depending on SetEventPath() is not ideal.
			FPointerEvent ModifiedEvent(PointerEvent);
			ModifiedEvent.SetEventPath(WidgetsUnderCursor);
			DroppedContent->OnDrop(bWasHandled, ModifiedEvent);
			WidgetsUnderPointerLastEventByIndex.Remove(PointerIdx);
		}
	}
	
	if (PointerEvent.IsTouchEvent())
	{
		GestureDetector.OnTouchEnded(PointerIdx, PointerEvent.GetScreenSpacePosition());

		// For touch events, we always invalidate capture for the pointer.  There's no reason to ever maintain capture for
		// fingers no longer in contact with the screen.
		ReleaseCapture(PointerIdx);

		// When touch pointers are released, they also effectively cease to exist until a touch begins again
		PointerPositionsByIndex.Remove(PointerIdx);
		PreviousPointerPositionsByIndex.Remove(PointerIdx);
		WidgetsUnderPointerLastEventByIndex.Remove(PointerIdx);
	}
}

void FSlateUser::ResetDragDetection()
{
	DragStatesByPointerIndex.Reset();
}

void FSlateUser::SetDragDropContent(TSharedRef<FDragDropOperation> InDragDropContent)
{
	checkf(!IsDragDropping(), TEXT("Drag and Drop already in progress!"));
	DragDropContent = InDragDropContent;
}

void FSlateUser::ResetDragDropContent()
{
	DragDropContent.Reset();
}

void FSlateUser::UpdateTooltip(const FMenuStack& MenuStack, bool bCanSpawnNewTooltip)
{
#if WITH_SLATE_DEBUGGING
	ClearForceFieldDebug();
#endif

	FSlateApplication& SlateApp = FSlateApplication::Get();
	if (!SlateApp.GetAllowTooltips())
	{
		CloseTooltip();
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_SlateUpdateTooltip);

	const double MotionLessDurationBeforeAllowingNewToolTip = 0.05;
	bCanSpawnNewTooltip = bCanSpawnNewTooltip && (bCanDrawCursor || bAllowTooltipsWithHiddenCursor) && (FPlatformTime::Seconds() - LastCursorSignificantMoveTime > MotionLessDurationBeforeAllowingNewToolTip);

	float DPIScaleFactor = 1.0f; //todo: this value is never changed, we should investigate if it is necessary or not to handle it for the force field.
	FWidgetPath WidgetsToQueryForTooltip;

	const bool bCheckForTooltipChanges =
		IsInGameThread() &&					// We should never allow the slate loading thread to create new windows or interact with the hittest grid
		!SlateApp.IsUsingHighPrecisionMouseMovment() && // If we are using HighPrecision movement then we can't rely on the OS cursor to be accurate
		!IsDragDropping() &&				// We must not currently be in the middle of a drag-drop action
		(!SlateApp.GetPressedMouseButtons().Contains(EKeys::LeftMouseButton) || bAllowTooltipsWhileMouseDown) && // We must not currently be clicking on a widget
		(
			SlateApp.IsActive() || // Assume we need update if app is active
			//@todo DanH: We need to check if OUR cursor is over a slate window, not just the platform cursor. 
			//		See about adding FPlatformApplication::GetSlateWindowUnderPoint(FVector2D) or something.
			SlateApp.GetPlatformApplication()->IsCursorDirectlyOverSlateWindow() // The cursor must be over a Slate window
		);

	if (bCheckForTooltipChanges)
	{
		// We're gonna check each widget under the cursor (including disabled widgets) until we find one with a tooltip to show
		FWidgetPath WidgetsUnderCursor = SlateApp.LocateWindowUnderMouse(GetCursorPosition(), SlateApp.GetInteractiveTopLevelWindows(), /*bIgnoreEnabledStatus =*/true, UserIndex);
		if (WidgetsUnderCursor.IsValid())
		{
			// Don't query tooltip windows themselves for tooltips
			bool bIsOverTooltipWindow = false;
			for (const TPair<TWeakPtr<SWindow>, TWeakPtr<SWindow>>& Pair : TooltipWindowsByOwner)
			{
				if (WidgetsUnderCursor.GetWindow() == Pair.Value.Pin())
				{
					bIsOverTooltipWindow = true;
					break;
				}
			}
			if (!bIsOverTooltipWindow)
			{
				WidgetsToQueryForTooltip = WidgetsUnderCursor;
			}
		}
	}

	TOptional<FSlateRect> ForceFieldRect;
	TSharedPtr<IToolTip> NewTooltip;
	TSharedPtr<SWidget> WidgetProvidingNewTooltip;
	for (int32 WidgetIndex = WidgetsToQueryForTooltip.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		const FArrangedWidget* ArrangedWidget = &WidgetsToQueryForTooltip.Widgets[WidgetIndex];
		const TSharedRef<SWidget>& CurWidget = ArrangedWidget->Widget;

		if (!NewTooltip.IsValid())
		{
			// Make sure the tooltip has something to show before we pick it
			TSharedPtr<IToolTip> WidgetTooltip = CurWidget->GetToolTip();
			if (WidgetTooltip.IsValid() && !WidgetTooltip->IsEmpty())
			{
				WidgetProvidingNewTooltip = CurWidget;
				NewTooltip = WidgetTooltip;
			}
		}

		// Make sure we account for all widgets in the path with a force field, even if we've already found the tooltip we'll be showing
		if (CurWidget->HasToolTipForceField())
		{
			const FSlateRect WidgetRect = ArrangedWidget->Geometry.GetLayoutBoundingRect();
			if (!ForceFieldRect.IsSet())
			{
				ForceFieldRect = WidgetRect;
			}
			else
			{
				// Grow the rect to encompass this geometry to be super safe
				// The parent's rect should always be inclusive of its child, so this should usually be overkill.
				ForceFieldRect = ForceFieldRect->Expand(WidgetRect);
			}
			
			// Expand based on the attribute to specify a custom zone
			TOptional<FSlateRect> ExpansionRect = CurWidget->GetToolTipForceFieldExpansion();
			if (ExpansionRect.IsSet())
			{
				ForceFieldRect = ForceFieldRect->Expand(ExpansionRect.GetValue());
			}
			
			ForceFieldRect = (1.0f / DPIScaleFactor) * ForceFieldRect.GetValue();

#if WITH_SLATE_DEBUGGING
			if (bEnableTooltipForceFieldDraw)
			{
				DebugForceFieldRects.Add(WidgetRect);
			}
#endif
		}
	}

#if WITH_SLATE_DEBUGGING
	if (!DebugForceFieldRects.IsEmpty())
	{
		FSlateDebugging::PaintDebugElements.AddRaw(this, &FSlateUser::HandlePaint);
	}
#endif

	TSharedPtr<SWidget> NewTooltipVisualizer;
	TSharedPtr<IToolTip> ActiveTooltip = ActiveTooltipInfo.Tooltip.Pin();
	bool bTooltipChanged = NewTooltip != ActiveTooltip;
	if (bTooltipChanged)
	{
		// Remove existing tooltip if there is one.
		if (ActiveTooltipInfo.TooltipVisualizer.IsValid())
		{
			ActiveTooltipInfo.TooltipVisualizer.Pin()->OnVisualizeTooltip(nullptr);
		}

		// Notify the new tooltip that it's about to be opened.
		if (NewTooltip && bCanSpawnNewTooltip)
		{
			NewTooltip->OnOpening();
			ActiveTooltipInfo.HasBeenPositioned = false;
		}

		// Some widgets might want to provide an alternative Tooltip Handler.
		if (bCanSpawnNewTooltip || !NewTooltip)
		{
			TSharedPtr<SWidget> NewTooltipWidget = NewTooltip ? NewTooltip->AsWidget() : TSharedPtr<SWidget>();
			for (int32 WidgetIndex = WidgetsToQueryForTooltip.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
			{
				const TSharedRef<SWidget>& CurWidget = WidgetsToQueryForTooltip.Widgets[WidgetIndex].Widget;
				if (CurWidget->OnVisualizeTooltip(NewTooltipWidget))
				{
					// Someone is taking care of visualizing this tooltip
					NewTooltipVisualizer = CurWidget;
					break;
				}
			}
		}
	}
	
	// If a widget under the cursor has a tool-tip forcefield active, then go through any menus
	// in the menu stack that are above that widget's window, and make sure those windows also
	// prevent the tool-tip from encroaching.  This prevents tool-tips from drawing over sub-menus
	// spawned from menu items in a different window, for example.
	if (ForceFieldRect.IsSet() && WidgetsToQueryForTooltip.IsValid())
	{
		if (TSharedPtr<IMenu> MenuInPath = MenuStack.FindMenuInWidgetPath(WidgetsToQueryForTooltip))
		{
			FSlateRect MenuStackRectangle;
			const bool bWasSolutionFound = MenuStack.GetToolTipForceFieldRect(MenuInPath.ToSharedRef(), WidgetsToQueryForTooltip, MenuStackRectangle);
			if (bWasSolutionFound)
			{
				ForceFieldRect = ForceFieldRect->Expand(MenuStackRectangle);
			}
		}
	}

	// Start by targeting the position calculated last frame.
	FVector2f DesiredLocation = ActiveTooltipInfo.DesiredLocation;

	// When a tooltip changes from interactive back to regular, there are a few frames during which IsInteractive()
	// returns false, but the tooltip window is still rendering the interactive content. Computing a new desired location
	// during this state can lead to the tooltip jumping around visibly. This variable indicates when we're in this situation,
	// so that we don't look for a new desired location yet.
	bool bStillAtInteractiveSize = !bTooltipChanged && ActiveTooltip && ActiveTooltipInfo.WasInteractive && TooltipWindowPtr.IsValid() && ActiveTooltipInfo.DesiredSize == TooltipWindowPtr.Pin()->GetDesiredSizeDesktopPixels();

	// Calculate a new desired location for the tooltip if:
	// - it's a non-interactive tooltip, do this every frame so that it moves with the cursor,
	//   unless the tooltip window is still rendering interactive content.
	// - if it's interactive, only do this if the tooltip hasn't been positioned yet.
	if (ActiveTooltip)
	{
		if ((!ActiveTooltip->IsInteractive() && !bStillAtInteractiveSize) || !ActiveTooltipInfo.HasBeenPositioned)
		{
			// New tooltips and non-interactive tooltips appear offset from the cursor position, and they follow the cursor as it moves.	
			DesiredLocation = GetPreviousCursorPosition() + SlateDefs::TooltipOffsetFromMouse;
			ActiveTooltipInfo.WasInteractive = false;
			ActiveTooltipInfo.HasBeenPositioned = true;

			// Allow interactive tooltips to adjust the window location
			if (ActiveTooltip->IsInteractive() && !NewTooltipVisualizer.IsValid())
			{
				FVector2D DesiredLocation2d(DesiredLocation);
				ActiveTooltip->OnSetInteractiveWindowLocation(DesiredLocation2d);
				DesiredLocation = UE::Slate::CastToVector2f(DesiredLocation2d);
			}
		}
	}

	// Move the desired position back from the edges of the screen if needed.
	FVector2f TooltipSize(ForceInitToZero);
	if (TooltipWindowPtr.IsValid())
	{
		TooltipSize = TooltipWindowPtr.Pin()->GetDesiredSizeDesktopPixels();
		FSlateRect Anchor(DesiredLocation.X, DesiredLocation.Y, DesiredLocation.X, DesiredLocation.Y);
		DesiredLocation = SlateApp.CalculatePopupWindowPosition(Anchor, TooltipSize, /*bAutoAdjustForDPIScale =*/false, FVector2f::ZeroVector, EOrientation::Orient_Vertical, EPopupLayoutMode::ToolTip);
	}

	// Repel tooltip from a force field, if necessary
	if (ForceFieldRect.IsSet() && !ActiveTooltipInfo.WasInteractive)
	{
		FVector2f TooltipShift;
		TooltipShift.X = (ForceFieldRect->Right + SlateDefs::TooltipOffsetFromForceField.X) - DesiredLocation.X;
		TooltipShift.Y = (ForceFieldRect->Bottom + SlateDefs::TooltipOffsetFromForceField.Y) - DesiredLocation.Y;

		// Make sure the tooltip needs to be offset
		if (TooltipShift.X != 0.0f || TooltipShift.Y != 0.0f)
		{
			// If in the bottom right corner there's no space for the tooltip, let's try to shift it upwards instead
			const FSlateRect WorkArea = FSlateApplication::Get().GetPreferredWorkArea();
			if (DesiredLocation.X + TooltipShift.X + TooltipSize.X > WorkArea.Right &&
				DesiredLocation.Y + TooltipShift.Y + TooltipSize.Y > WorkArea.Bottom)
			{
				TooltipShift.Y = ForceFieldRect->Top - SlateDefs::TooltipOffsetFromForceField.Y - TooltipSize.Y - DesiredLocation.Y;
			}

			// Find the best edge to move the tooltip towards
			if (ActiveTooltipInfo.OffsetDirection == ETooltipOffsetDirection::Undetermined)
			{
				ETooltipOffsetDirection PotentialOffsetDirection = FMath::Abs(TooltipShift.X) < FMath::Abs(TooltipShift.Y) ? ETooltipOffsetDirection::Right : ETooltipOffsetDirection::Down;
				if (PotentialOffsetDirection == ETooltipOffsetDirection::Right)
				{
					FVector2f TentativeDesiredLocation = DesiredLocation + FVector2f(TooltipShift.X, 0);
					FSlateRect Anchor(TentativeDesiredLocation.X, TentativeDesiredLocation.Y, TentativeDesiredLocation.X, TentativeDesiredLocation.Y);
					FVector2f NewDesiredLocation = SlateApp.CalculatePopupWindowPosition(Anchor, TooltipSize, /*bAutoAdjustForDPIScale =*/false, FVector2f::ZeroVector, EOrientation::Orient_Vertical, EPopupLayoutMode::ToolTip);
					// If after adjustment the tooltip still overlaps with the force field, try the other direction
					if (FSlateRect::DoRectanglesIntersect(ForceFieldRect.GetValue(), FSlateRect(NewDesiredLocation, NewDesiredLocation + TooltipSize)))
					{
						PotentialOffsetDirection = ETooltipOffsetDirection::Down;
					}
				}

				if (PotentialOffsetDirection == ETooltipOffsetDirection::Down)
				{
					FVector2f TentativeDesiredLocation = DesiredLocation + FVector2f(0, TooltipShift.Y);
					FSlateRect Anchor(TentativeDesiredLocation.X, TentativeDesiredLocation.Y, TentativeDesiredLocation.X, TentativeDesiredLocation.Y);
					FVector2f NewDesiredLocation = SlateApp.CalculatePopupWindowPosition(Anchor, TooltipSize, /*bAutoAdjustForDPIScale =*/false, FVector2f::ZeroVector, EOrientation::Orient_Vertical, EPopupLayoutMode::ToolTip);
					// If after adjustment the tooltip still overlaps with the force field, try the other direction
					if (FSlateRect::DoRectanglesIntersect(ForceFieldRect.GetValue(), FSlateRect(NewDesiredLocation, NewDesiredLocation + TooltipSize)))
					{
						PotentialOffsetDirection = ETooltipOffsetDirection::Right;
					}
				}

				ActiveTooltipInfo.OffsetDirection = PotentialOffsetDirection;
			}

			check(ActiveTooltipInfo.OffsetDirection != ETooltipOffsetDirection::Undetermined);
			if (ActiveTooltipInfo.OffsetDirection == ETooltipOffsetDirection::Right)
			{
				// Move right
				DesiredLocation.X += TooltipShift.X;
				ActiveTooltipInfo.OffsetDirection = ETooltipOffsetDirection::Right;
			}
			else
			{
				// Move down
				DesiredLocation.Y += TooltipShift.Y;
				ActiveTooltipInfo.OffsetDirection = ETooltipOffsetDirection::Down;
			}
		}
	}

	// Update the desired location so that interactive tooltips can continue to target it in future frames even after the mouse moves away
	ActiveTooltipInfo.DesiredLocation = DesiredLocation;

	// On platforms where tooltips are parented to a specific toplevel window, detect when the
	// source window changes (e.g. mouse moved from main window to asset browser) and recreate
	// the tooltip so it belongs to the correct window.
	if (SlateApp.GetPlatformApplication()->RequiresPerWindowTooltips())
	{
		TSharedPtr<SWindow> CurrentSourceWindow = WidgetsToQueryForTooltip.IsValid()
			? WidgetsToQueryForTooltip.GetWindow() : TSharedPtr<SWindow>();
		if (CurrentSourceWindow.IsValid() && ActiveTooltipInfo.SourceWindow.IsValid() &&
			CurrentSourceWindow != ActiveTooltipInfo.SourceWindow.Pin())
		{
			ResetTooltipWindow();
			bTooltipChanged = true;
		}
		if (CurrentSourceWindow.IsValid())
		{
			ActiveTooltipInfo.SourceWindow = CurrentSourceWindow;
		}
	}

	// The tool tip changed...
	if (bTooltipChanged)
	{
		// Close any existing tooltips; Unless the current tooltip is interactive and we don't have a valid tooltip to replace it
		if (NewTooltip || (ActiveTooltip && !ActiveTooltip->IsInteractive()))
		{
			CloseTooltip();

			if (NewTooltip && bCanSpawnNewTooltip)
			{
				if (NewTooltipVisualizer)
				{
					ActiveTooltipInfo.TooltipVisualizer = NewTooltipVisualizer;
					ActiveTooltipInfo.Tooltip = NewTooltip;
				}
				else
				{
					ShowTooltip(NewTooltip.ToSharedRef(), DesiredLocation, WidgetsToQueryForTooltip.GetWindow());
					ActiveTooltipInfo.SourceWidget = WidgetProvidingNewTooltip;
				}
				ActiveTooltip = NewTooltip;
			}
		}
	}

	if (TooltipWindowPtr.IsValid())
	{
		// Only enable tooltip transitions if we're running at a decent frame rate
		const bool bAllowAnimations = FSlateApplication::Get().IsRunningAtTargetFrameRate();

		// How long since the tooltip was summoned?
		const double PlatformSeconds = FPlatformTime::Seconds();
		const float TimeSinceSummon = (float)(PlatformSeconds - TooltipSummonDelay - ActiveTooltipInfo.SummonTime);
		const float TooltipOpacity = FMath::Clamp<float>(TimeSinceSummon / TooltipIntroDuration, 0.0f, 1.0f);

		// Update window opacity
		TSharedRef<SWindow> TooltipWindow = TooltipWindowPtr.Pin().ToSharedRef();
		TooltipWindow->SetOpacity(TooltipOpacity);

		// How far tool tips should slide
		const FVector2f SlideDistance(30.0f, 5.0f);

		// Apply steep inbound curve to the movement, so it looks like it quickly decelerating
		const float SlideProgress = bAllowAnimations ? FMath::Pow(1.0f - TooltipOpacity, 3.0f) : 0.0f;

		FVector2f WindowLocation = DesiredLocation + (SlideProgress * SlideDistance);
		if (WindowLocation != TooltipWindow->GetPositionInScreen()
			|| TooltipWindow->GetDesiredSize() != ActiveTooltipInfo.DesiredSize)
		{
			// already handled
			const bool bAutoAdjustForDPIScale = false;

			// Avoid the edges of the desktop
			FSlateRect Anchor(WindowLocation.X, WindowLocation.Y, WindowLocation.X, WindowLocation.Y);
			WindowLocation = SlateApp.CalculateTooltipWindowPosition(Anchor, TooltipWindow->GetDesiredSizeDesktopPixels(), bAutoAdjustForDPIScale, (ActiveTooltip && ActiveTooltip->IsInteractive()) ? EPopupCursorOverlapMode::AllowOverlap : EPopupCursorOverlapMode::PreventOverlap);

			// Cache the size to compare against in future frames
			ActiveTooltipInfo.DesiredSize = TooltipWindow->GetDesiredSize();

			// Update the tool tip window positioning
			// SetCachedScreenPosition is a hack (issue tracked as TTP #347070) which is needed because code in TickWindowAndChildren()/DrawPrepass()
			// assumes GetPositionInScreen() to correspond to the new window location in the same tick. This is true on Windows, but other
			// OSes (Linux in particular) may not update cached screen position until next time events are polled.
			TooltipWindow->SetCachedScreenPosition(WindowLocation);
			TooltipWindow->MoveWindowTo(WindowLocation);
		}

		// Cache whether the tooltip is in interactive mode.
		if (ActiveTooltip.IsValid() && ActiveTooltip->IsInteractive())
		{
			ActiveTooltipInfo.WasInteractive = true;
		}
	}
}

void FSlateUser::ResetTooltipWindow()
{
	// Destroy all per-owner tooltip windows
	for (TPair<TWeakPtr<SWindow>, TWeakPtr<SWindow>>& Pair : TooltipWindowsByOwner)
	{
		TSharedPtr<SWindow> Window = Pair.Value.Pin();
		if (Window)
		{
			Window->RequestDestroyWindow();
		}
	}
	TooltipWindowsByOwner.Empty();
	TooltipWindowPtr.Reset();
}

bool FSlateUser::IsWindowHousingInteractiveTooltip(const TSharedRef<const SWindow>& WindowToTest) const
{
	// Check if this window is ANY of our tooltip windows
	bool bIsTooltipWindow = false;
	for (const TPair<TWeakPtr<SWindow>, TWeakPtr<SWindow>>& Pair : TooltipWindowsByOwner)
	{
		if (WindowToTest == Pair.Value.Pin())
		{
			bIsTooltipWindow = true;
			break;
		}
	}
	if (bIsTooltipWindow)
	{
		const TSharedPtr<IToolTip> ActiveTooltip = ActiveTooltipInfo.Tooltip.Pin();
		return ActiveTooltip && ActiveTooltip->IsInteractive();
	}
	return false;
}
