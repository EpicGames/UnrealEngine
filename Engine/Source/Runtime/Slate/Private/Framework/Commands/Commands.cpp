// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/Commands.h"
#include "Styling/ISlateStyle.h"

#define LOC_DEFINE_REGION

void MakeUICommand_InternalUseOnly( FBindingContext* This, TSharedPtr< FUICommandInfo >& OutCommand, const TCHAR* InSubNamespace, const TCHAR* InCommandName, const TCHAR* InCommandNameUnderscoreTooltip, const ANSICHAR* DotCommandName, const TCHAR* FriendlyName, const TCHAR* InDescription, const EUserInterfaceActionType CommandType, const FInputChord& InDefaultChord, const FInputChord& InAlternateDefaultChord)
{
	static const FString UICommandsStr("UICommands");
	const FString Namespace = InSubNamespace && FCString::Strlen(InSubNamespace) > 0 ? UICommandsStr + TEXT(".") + InSubNamespace : UICommandsStr;

	FUICommandInfo::MakeCommandInfo(
		This->AsShared(),
		OutCommand,
		InCommandName,
		FText::AsLocalizable_Advanced( Namespace, InCommandName, FriendlyName ),
		FText::AsLocalizable_Advanced( Namespace, InCommandNameUnderscoreTooltip, InDescription ),
		FSlateIcon( This->GetStyleSetName(), ISlateStyle::Join( This->GetContextName(), DotCommandName ) ),
		CommandType,
		InDefaultChord,
		InAlternateDefaultChord
	);
}

#undef LOC_DEFINE_REGION
