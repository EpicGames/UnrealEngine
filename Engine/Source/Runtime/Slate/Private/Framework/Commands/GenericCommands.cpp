// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "GenericCommands"

UE_DEFINE_TCOMMANDS(FGenericCommands)

void FGenericCommands::RegisterCommands()
{
	UI_COMMAND(Cut, "Cut", "Cut selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::X))
	UI_COMMAND(Copy, "Copy", "Copy selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::C))
	UI_COMMAND(Paste, "Paste", "Paste clipboard contents", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::V))
#if PLATFORM_MAC
	UI_COMMAND(Duplicate, "Duplicate", "Duplicate selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command, EKeys::D))
#else
	UI_COMMAND(Duplicate, "Duplicate", "Duplicate selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D))
#endif
	UI_COMMAND(Undo, "Undo", "Undo last action", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Z))
	UI_COMMAND(Redo, "Redo", "Redo last action", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Y), FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Z))

	UI_COMMAND(Delete, "Delete", "Delete current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete), FInputChord(EKeys::BackSpace))

	UI_COMMAND(Rename, "Rename", "Rename current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::F2))
	UI_COMMAND(SelectAll, "Select All", "Select everything in the current scope", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A))
}

#undef LOCTEXT_NAMESPACE
