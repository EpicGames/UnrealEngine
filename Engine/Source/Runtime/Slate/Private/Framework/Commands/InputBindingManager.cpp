// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/InputBindingManager.h"
#include "Containers/AnsiString.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "SlateGlobals.h"
#include "Trace/SlateMemoryTags.h"
#include "Misc/RemoteConfigIni.h"

/* FUserDefinedChords helper class
 *****************************************************************************/

/** An identifier for a user defined chord */
struct FUserDefinedChordKey
{
	FUserDefinedChordKey()
	{
	}

	FUserDefinedChordKey(const FName InBindingContext, const FName InCommandName, const EMultipleKeyBindingIndex InChordIndex)
		: BindingContext(InBindingContext)
		, CommandName(InCommandName)
		, ChordIndex(InChordIndex)
	{
	}

	bool operator==(const FUserDefinedChordKey& Other) const
	{
		return BindingContext == Other.BindingContext && CommandName == Other.CommandName && ChordIndex == Other.ChordIndex;
	}

	bool operator!=(const FUserDefinedChordKey& Other) const
	{
		return !(*this == Other);
	}

	FName BindingContext;
	FName CommandName;
	EMultipleKeyBindingIndex ChordIndex;
	bool bIsFromProjectSetting = false;
};

uint32 GetTypeHash( const FUserDefinedChordKey& Key )
{
	return GetTypeHash(Key.BindingContext) ^ GetTypeHash(Key.CommandName);
}

class FUserDefinedChords
{
public:
	void LoadChords();
	void SaveChords() const;
	bool GetUserDefinedChord( const FName BindingContext, const FName CommandName, const EMultipleKeyBindingIndex ChordIndex, FInputChord& OutUserDefinedChord ) const;
	void SetUserDefinedChords( const FUICommandInfo& CommandInfo );
	/** Remove all user defined chords */
	void RemoveAll();
private:

	static FUserDefinedChordKey LoadKey(const TSharedPtr<FJsonObject>& ChordInfoObject);
	static FInputChord LoadChord(const TSharedPtr<FJsonObject>& ChordInfoObject);
	void LoadFromCurrentJsonConfig(const TArray<FString>& ChordJsonArray, bool bFromProjectSettings);

	/* Mapping from a chord key to the user defined chord */
	typedef TMap<FUserDefinedChordKey, FInputChord> FChordsMap;
	TSharedPtr<FChordsMap> Chords;
};

FUserDefinedChordKey FUserDefinedChords::LoadKey(const TSharedPtr<FJsonObject>& ChordInfoObject)
{
	const TSharedPtr<FJsonValue> BindingContextObj = ChordInfoObject->Values.FindRef(TEXT("BindingContext"));
	const TSharedPtr<FJsonValue> CommandNameObj = ChordInfoObject->Values.FindRef(TEXT("CommandName"));
	const TSharedPtr<FJsonValue> ChordIndexObj = ChordInfoObject->Values.FindRef(TEXT("ChordIndex"));

	const FName BindingContext = *BindingContextObj->AsString();
	const FName CommandName = *CommandNameObj->AsString();
	const EMultipleKeyBindingIndex ChordIndex = ChordIndexObj.IsValid() ? static_cast<EMultipleKeyBindingIndex>(static_cast<uint32>(ChordIndexObj->AsNumber())) : EMultipleKeyBindingIndex::Primary;

	return FUserDefinedChordKey(BindingContext, CommandName, ChordIndex);
}

FInputChord FUserDefinedChords::LoadChord(const TSharedPtr<FJsonObject>& ChordInfoObject)
{
	const TSharedPtr<FJsonValue> CtrlObj = ChordInfoObject->Values.FindRef(TEXT("Control"));
	const TSharedPtr<FJsonValue> AltObj = ChordInfoObject->Values.FindRef( TEXT("Alt") );
	const TSharedPtr<FJsonValue> ShiftObj = ChordInfoObject->Values.FindRef( TEXT("Shift") );
	const TSharedPtr<FJsonValue> CmdObj = ChordInfoObject->Values.FindRef(TEXT("Command"));
	const TSharedPtr<FJsonValue> KeyObj = ChordInfoObject->Values.FindRef( TEXT("Key") );

#if PLATFORM_MAC
	// Command is treated like "Control" on Mac and vice-versa, so swap them.
	EModifierKey::Type Modifiers = EModifierKey::FromBools(
										CmdObj.IsValid() ? CmdObj->AsBool() : false,
										AltObj->AsBool(),
										ShiftObj->AsBool(),
										CtrlObj->AsBool()
									);
#else
	EModifierKey::Type Modifiers = EModifierKey::FromBools(
										CtrlObj->AsBool(),
										AltObj->AsBool(),
										ShiftObj->AsBool(),
										CmdObj.IsValid() ? CmdObj->AsBool() : false
									);
#endif	//#if PLATFORM_MAC

	return FInputChord(*KeyObj->AsString(), Modifiers);
}

void FUserDefinedChords::LoadFromCurrentJsonConfig(const TArray<FString>& ChordJsonArray, bool bFromProjectSettings)
{
	// This loads an array of JSON strings representing the FUserDefinedChordKey and FInputChord in a single JSON object
	for (const FString& ChordJson : ChordJsonArray)
	{
		const FString UnescapedContent = FRemoteConfig::ReplaceIniSpecialCharWithChar(ChordJson).ReplaceEscapedCharWithChar();

		TSharedPtr<FJsonObject> ChordInfoObj;
		auto JsonReader = TJsonReaderFactory<>::Create(UnescapedContent);
		if (FJsonSerializer::Deserialize(JsonReader, ChordInfoObj))
		{
			FUserDefinedChordKey ChordKey = LoadKey(ChordInfoObj);
			ChordKey.bIsFromProjectSetting = bFromProjectSettings;
			Chords->Remove(ChordKey);
			FInputChord& UserDefinedChord = Chords->FindOrAdd(ChordKey, LoadChord(ChordInfoObj));
		}
	}
}

void FUserDefinedChords::LoadChords()
{
	if( !Chords.IsValid() )
	{
		Chords = MakeShareable( new FChordsMap );

		// First look through project settings
		TArray<FString> ChordJsonArray;
		bool bFoundProjectChords = (GConfig->GetArray(TEXT("ProjectDefinedChords"), TEXT("ProjectDefinedChords"), ChordJsonArray, GEditorSettingsIni) > 0);
		LoadFromCurrentJsonConfig(ChordJsonArray, true);

		// First, try and load the chords from their new location in the ini file
		// Failing that, try and load them from the older txt file
		bool bFoundChords = (GConfig->GetArray(TEXT("UserDefinedChords"), TEXT("UserDefinedChords"), ChordJsonArray, GEditorKeyBindingsIni) > 0);
		if (!bFoundChords)
		{
			// Backwards compat for when it was named gestures
			bFoundChords = (GConfig->GetArray(TEXT("UserDefinedGestures"), TEXT("UserDefinedGestures"), ChordJsonArray, GEditorKeyBindingsIni) > 0);
		}

		if(bFoundChords)
		{
			LoadFromCurrentJsonConfig(ChordJsonArray, false);
		}
		else
		{
			TSharedPtr<FJsonObject> ChordsObj;

			// This loads a JSON object containing BindingContexts, containing objects of CommandNames, containing the FInputChord information
			FString Content;
			bool bFoundContent = (GConfig->GetArray(TEXT("UserDefinedChords"), TEXT("Content"), ChordJsonArray, GEditorKeyBindingsIni) > 0);
			if (!bFoundContent)
			{
				// Backwards compat for when it was named gestures
				bFoundChords = (GConfig->GetArray(TEXT("UserDefinedGestures"), TEXT("Content"), ChordJsonArray, GEditorKeyBindingsIni) > 0);
			}
			if (bFoundContent)
			{
				const FString UnescapedContent = FRemoteConfig::ReplaceIniSpecialCharWithChar(Content).ReplaceEscapedCharWithChar();

				auto JsonReader = TJsonReaderFactory<>::Create( UnescapedContent );
				FJsonSerializer::Deserialize( JsonReader, ChordsObj );
			}

			if (!ChordsObj.IsValid())
			{
				// Chords have not been loaded from the ini file, try reading them from the txt file now
				TSharedPtr<FArchive> Ar = MakeShareable( IFileManager::Get().CreateFileReader( *( FPaths::ProjectSavedDir() / TEXT("Preferences/EditorKeyBindings.txt") ) ) );
				if( Ar.IsValid() )
				{
					auto TextReader = TJsonReaderFactory<ANSICHAR>::Create( Ar.Get() );
					FJsonSerializer::Deserialize( TextReader, ChordsObj );
				}
			}

			if (ChordsObj.IsValid())
			{
				for(const auto& BindingContextInfo : ChordsObj->Values)
				{
					const FName BindingContext(BindingContextInfo.Key);
					TSharedPtr<FJsonObject> BindingContextObj = BindingContextInfo.Value->AsObject();
					for (const auto& CommandInfo : BindingContextObj->Values)
					{
						const FName CommandName(CommandInfo.Key);
						TSharedPtr<FJsonObject> CommandObj = CommandInfo.Value->AsObject();
						for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
						{
							FUserDefinedChordKey ChordKey(BindingContext, CommandName, static_cast<EMultipleKeyBindingIndex>(i));
							Chords->Remove(ChordKey); // Remove any existing that may have been added by project settings, to overwrite the flag on the key
							Chords->FindOrAdd(ChordKey, LoadChord(CommandObj));
						}
					}
				}
			}
		}
	}
}

void FUserDefinedChords::SaveChords() const
{
	if( Chords.IsValid() )
	{
		FString ChordRawJsonContent;
		TArray<FString> ChordJsonArray;
		for(const TPair<FUserDefinedChordKey, FInputChord>& ChordInfo : *Chords)
		{
			// Don't save keybinds that came out of the project settings. These can however be overridden by user defined keybinds successfully
			if (ChordInfo.Key.bIsFromProjectSetting)
			{
				continue;
			}

			TSharedPtr<FJsonValueObject> ChordInfoValueObj = MakeShareable( new FJsonValueObject( MakeShareable( new FJsonObject ) ) );
			TSharedPtr<FJsonObject> ChordInfoObj = ChordInfoValueObj->AsObject();

			// Set the chord values for the command
			ChordInfoObj->SetField( TEXT("BindingContext"), MakeShareable( new FJsonValueString( ChordInfo.Key.BindingContext.ToString() ) ) );
			ChordInfoObj->SetField( TEXT("CommandName"), MakeShareable( new FJsonValueString( ChordInfo.Key.CommandName.ToString() ) ) );
			ChordInfoObj->SetField(TEXT("ChordIndex"), MakeShareable(new FJsonValueNumber(static_cast<uint8>(ChordInfo.Key.ChordIndex))));
			ChordInfoObj->SetField( TEXT("Control"), MakeShareable( new FJsonValueBoolean( ChordInfo.Value.NeedsControl() ) ) );
			ChordInfoObj->SetField( TEXT("Alt"), MakeShareable( new FJsonValueBoolean( ChordInfo.Value.NeedsAlt() ) ) );
			ChordInfoObj->SetField( TEXT("Shift"), MakeShareable( new FJsonValueBoolean( ChordInfo.Value.NeedsShift() ) ) );
			ChordInfoObj->SetField( TEXT("Command"), MakeShareable( new FJsonValueBoolean( ChordInfo.Value.NeedsCommand() ) ) );
			ChordInfoObj->SetField( TEXT("Key"), MakeShareable( new FJsonValueString( ChordInfo.Value.Key.ToString() ) ) );

			auto JsonWriter = TJsonWriterFactory< TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create( &ChordRawJsonContent );
			FJsonSerializer::Serialize( ChordInfoObj.ToSharedRef(), JsonWriter );

			const FString EscapedContent = FRemoteConfig::ReplaceIniCharWithSpecialChar(ChordRawJsonContent).ReplaceCharWithEscapedChar();
			ChordJsonArray.Add(EscapedContent);
		}

		// Disable file operations to prevent the .ini from being deleted when all the sections are emptied below
		const bool bFileOperationsDisabled = GConfig->AreFileOperationsDisabled();
		if (!bFileOperationsDisabled)
		{
			GConfig->DisableFileOperations();
		}

		// Remove all sections not related to keybinds...
		TArray<FString> SectionNames;
		if (GConfig->GetSectionNames(GEditorKeyBindingsIni, SectionNames))
		{
			for (const FString& SectionKey : SectionNames)
			{
				GConfig->EmptySection(*SectionKey, GEditorKeyBindingsIni);
			}
		}

		if (!bFileOperationsDisabled)
		{
			GConfig->EnableFileOperations();
		}

		GConfig->SetArray(TEXT("UserDefinedChords"), TEXT("UserDefinedChords"), ChordJsonArray, GEditorKeyBindingsIni);

		// Clean up old keys (if they still exist)
		GConfig->RemoveKey(TEXT("UserDefinedGestures"), TEXT("UserDefinedGestures"), GEditorKeyBindingsIni);
		GConfig->RemoveKey(TEXT("UserDefinedChords"), TEXT("Content"), GEditorKeyBindingsIni);
		GConfig->RemoveKey(TEXT("UserDefinedChords"), TEXT("Content"), GEditorKeyBindingsIni);

		// Flush the .ini to ensure all new key bindings are saved
		const bool bRemoveFromCache = false;
		GConfig->Flush(bRemoveFromCache, GEditorKeyBindingsIni);
	}
}

bool FUserDefinedChords::GetUserDefinedChord( const FName BindingContext, const FName CommandName, const EMultipleKeyBindingIndex ChordIndex, FInputChord& OutUserDefinedChord ) const
{
	bool bResult = false;

	if( Chords.IsValid() )
	{
		const FUserDefinedChordKey ChordKey(BindingContext, CommandName, ChordIndex);
		const FInputChord* const UserDefinedChordPtr = Chords->Find(ChordKey);
		if( UserDefinedChordPtr )
		{
			OutUserDefinedChord = *UserDefinedChordPtr;
			bResult = true;
		}
	}

	return bResult;
}

void FUserDefinedChords::SetUserDefinedChords( const FUICommandInfo& CommandInfo)
{
	if( Chords.IsValid() )
	{
		const FName BindingContext = CommandInfo.GetBindingContext();
		const FName CommandName = CommandInfo.GetCommandName();

		for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
			// Recreate the command context, since it may have come from project chord, and now we want to be sure to flag as a user chord
			const FUserDefinedChordKey ChordKey(BindingContext, CommandName, ChordIndex);
			Chords->Remove(ChordKey);
			FInputChord& UserDefinedChord = Chords->FindOrAdd(ChordKey);

			// Save an empty invalid chord if one was not set
			// This is an indication that the user doesn't want this bound and not to use the default chord
			const TSharedPtr<const FInputChord> InputChord = CommandInfo.GetActiveChord(ChordIndex);
			UserDefinedChord = (InputChord.IsValid()) ? *InputChord : FInputChord();
		}
	}
}

void FUserDefinedChords::RemoveAll()
{
	Chords = MakeShareable( new FChordsMap );
}


/* FInputBindingManager structors
 *****************************************************************************/

FInputBindingManager& FInputBindingManager::Get()
{
	static FInputBindingManager Instance;
	return Instance;
}


/* FInputBindingManager interface
 *****************************************************************************/

/**
 * Gets the user defined chord (if any) from the provided command name
 * 
 * @param InBindingContext	The context in which the command is active
 * @param InCommandName		The name of the command to get the chord from
 */
bool FInputBindingManager::GetUserDefinedChord( const FName InBindingContext, const FName InCommandName, const EMultipleKeyBindingIndex InChordIndex, FInputChord& OutUserDefinedChord )
{
	if( !UserDefinedChords.IsValid() )
	{
		UserDefinedChords = MakeShareable( new FUserDefinedChords );
		UserDefinedChords->LoadChords();
	}

	return UserDefinedChords->GetUserDefinedChord( InBindingContext, InCommandName, InChordIndex, OutUserDefinedChord );
}

void FInputBindingManager::CheckForDuplicateDefaultChords( const FBindingContext& InBindingContext, TSharedPtr<FUICommandInfo> InCommandInfo ) const
{
	const bool bCheckDefault = true;
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		if (!InCommandInfo->DefaultChords[i].IsValidChord())
		{
			continue;
		}
		TSharedPtr<FUICommandInfo> ExistingInfo = GetCommandInfoFromInputChord(InBindingContext.GetContextName(), InCommandInfo->DefaultChords[i], bCheckDefault);
		if (ExistingInfo.IsValid())
		{
			if (ExistingInfo->CommandName != InCommandInfo->CommandName)
			{
				// Two different commands with the same name in the same context or parent context
				UE_LOGF(LogSlate, Warning, "The command '%ls.%ls' has the same default chord as '%ls.%ls' [%ls]",
					*InCommandInfo->BindingContext.ToString(),
					*InCommandInfo->CommandName.ToString(),
					*ExistingInfo->BindingContext.ToString(),
					*ExistingInfo->CommandName.ToString(),
					*InCommandInfo->DefaultChords[i].GetInputText().ToString());
			}
		}
	}
}


void FInputBindingManager::NotifyActiveChordChanged( const FUICommandInfo& CommandInfo, const EMultipleKeyBindingIndex InChordIndex )
{
	FContextEntry& ContextEntry = ContextMap.FindChecked( CommandInfo.GetBindingContext() );
	// Slow but doesn't happen frequently
	FChordMap& ActualChordMap = ContextEntry.ChordToCommandInfoMaps[static_cast<uint8>(InChordIndex)];
	for( FChordMap::TIterator It(ActualChordMap); It; ++It )
	{
		// Remove the currently active chord from the map if one exists
		if( It.Value() == CommandInfo.GetCommandName() )
		{
			It.RemoveCurrent();
			// There should only be one active chord for each map
			break;
		}
	}

	if( CommandInfo.GetActiveChord(InChordIndex)->IsValidChord() )
	{
		checkSlow( !ActualChordMap.Contains( *CommandInfo.GetActiveChord(InChordIndex) ) )
			ActualChordMap.Add( *CommandInfo.GetActiveChord(InChordIndex), CommandInfo.GetCommandName() );
	}
	//else if (!bIsAlternateChord) 
	// This is optional, keep it off for now
	//// We removed the primary chord, so swap them and remove the alternate; this is so that it's easier for the tooltip system 
	//{
	//	for (FChordMap::TIterator It(ContextEntry.AlternateChordToCommandInfoMap); It; ++It)
	//	{
	//		if (It.Value() == CommandInfo.GetCommandName())
	//		{
	//			It.RemoveCurrent();
	//			ContextEntry.ChordToCommandInfoMap.Add(It.Key(), It.Value());
	//			break;
	//		}
	//	}
	//}

	// The user defined chords should have already been created
	check( UserDefinedChords.IsValid() );

	UserDefinedChords->SetUserDefinedChords( CommandInfo );

	// Broadcast the chord event when a new one is added
	OnUserDefinedChordChanged.Broadcast(CommandInfo);
}

void FInputBindingManager::SaveInputBindings()
{
	if( UserDefinedChords.IsValid() )
	{
		UserDefinedChords->SaveChords();
	}
}

void FInputBindingManager::RemoveUserDefinedChords()
{
	if( UserDefinedChords.IsValid() )
	{
		UserDefinedChords->RemoveAll();
		UserDefinedChords->SaveChords();
	}
}

void FInputBindingManager::GetCommandInfosFromContext( const FName InBindingContext, TArray< TSharedPtr<FUICommandInfo> >& OutCommandInfos ) const
{
	ContextMap.FindRef( InBindingContext ).CommandInfoMap.GenerateValueArray( OutCommandInfos );
}

void FInputBindingManager::CreateInputCommand( const TSharedRef<FBindingContext>& InBindingContext, TSharedRef<FUICommandInfo> InCommandInfo )
{
	check( InCommandInfo->BindingContext == InBindingContext->GetContextName() );

	// The command name should be valid
	check( InCommandInfo->CommandName != NAME_None );

	LLM_SCOPE_BYTAG(UI_Slate);

	// Should not have already created a chord for this command
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		check(!InCommandInfo->ActiveChords[i]->IsValidChord());
	}
	
	const FName ContextName = InBindingContext->GetContextName();

	FContextEntry& ContextEntry = ContextMap.FindOrAdd( ContextName );
	
	// Our parent context must exist.
	check( InBindingContext->GetContextParent() == NAME_None || ContextMap.Find( InBindingContext->GetContextParent() ) != NULL );

	FCommandInfoMap& CommandInfoMap = ContextEntry.CommandInfoMap;

	if( !ContextEntry.BindingContext.IsValid() )
	{
		ContextEntry.BindingContext = InBindingContext;
	}

	if( InBindingContext->GetContextParent() != NAME_None  )
	{
		check( InBindingContext->GetContextName() != InBindingContext->GetContextParent() );
		// Set a mapping from the parent of the current context to the current context
		ParentToChildMap.AddUnique( InBindingContext->GetContextParent(), InBindingContext->GetContextName() );
	}

	CheckForDuplicateDefaultChords( *InBindingContext, InCommandInfo );

	TSharedPtr<FUICommandInfo> ExistingInfo = CommandInfoMap.FindRef( InCommandInfo->CommandName );
	ensureMsgf( !ExistingInfo.IsValid(), TEXT("A command with name %s already exists in context %s"), *InCommandInfo->CommandName.ToString(), *InBindingContext->GetContextName().ToString() );
	
	// Add the command info to the list of known infos.  It can only exist once.
	CommandInfoMap.Add( InCommandInfo->CommandName, InCommandInfo );

	// See if there are user defined chords for this command

	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex> (i);
		FInputChord UserDefinedChord;
		bool bFoundUserDefinedChord = GetUserDefinedChord(ContextName, InCommandInfo->CommandName, ChordIndex, UserDefinedChord);


		if (!bFoundUserDefinedChord && InCommandInfo->DefaultChords[i].IsValidChord())
		{
			// Find any existing command with the same chord 
			// This is for inconsistency between default and user defined chord.  We need to make sure that if default chords are changed to a chord that a user set to a different command, that the default chord doesn't replace
			// the existing commands chord. Note: Duplicate default chords are found above in CheckForDuplicateDefaultChords
			// This needs to check through the maps for the primary and alternate key bindings.
			FName ExisingCommand = NAME_None;  
			for (uint32 j = 0; j < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords) && ExisingCommand == NAME_None; ++j)
			{
				ExisingCommand = ContextEntry.ChordToCommandInfoMaps[j].FindRef(InCommandInfo->DefaultChords[i]);
			}
			if (ExisingCommand == NAME_None)
			{
				// No existing command has a user defined chord and no user defined chord is available for this command 
				TSharedRef<FInputChord> NewChord = MakeShareable(new FInputChord(InCommandInfo->DefaultChords[i]));
				InCommandInfo->ActiveChords[i] = NewChord;
			}

		}
		else if (bFoundUserDefinedChord)
		{
			// Find any existing command with the same chord 
			// This is for inconsistency between default and user defined chord.  We need to make sure that if default chords are changed to a chord that a user set to a different command, that the default chord doesn't replace
			// the existing commands chord. This needs to check through the maps for the primary and alternate key bindings.
			FName ExisingCommandName = NAME_None;
			for (uint32 j = 0; j < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords) && ExisingCommandName == NAME_None; ++j)
			{
				ExisingCommandName = ContextEntry.ChordToCommandInfoMaps[j].FindRef(UserDefinedChord);
			}

			if (ExisingCommandName != NAME_None)
			{
				// Get the command with using the same chord
				TSharedPtr<FUICommandInfo> PreviousInfo = CommandInfoMap.FindRef(ExisingCommandName);
				if (*PreviousInfo->ActiveChords[i] != PreviousInfo->DefaultChords[i])
				{
					// two user defined chords are the same within a context.  If the keybinding editor was used this wont happen so this must have been directly a modified user setting file
					UE_LOGF(LogSlate, Error, "Duplicate user defined chords found: [%ls,%ls].  Chord for %ls being removed", *InCommandInfo->GetLabel().ToString(), *PreviousInfo->GetLabel().ToString(), (ExistingInfo  ? *ExistingInfo->GetLabel().ToString() : TEXT("")));
				}
				ContextEntry.ChordToCommandInfoMaps[i].Remove(*PreviousInfo->ActiveChords[i]);
				// Remove the existing chord. 
				PreviousInfo->ActiveChords[i] = MakeShareable(new FInputChord());
			}

			TSharedRef<FInputChord> NewChord = MakeShareable(new FInputChord(UserDefinedChord));
			// Set the active chord on the command info
			InCommandInfo->ActiveChords[i] = NewChord;
		}

		// If the active chord is valid, map the chord to the map for fast lookup when processing bindings
		if (InCommandInfo->ActiveChords[i]->IsValidChord())
		{
			checkSlow(!ContextEntry.ChordToCommandInfoMaps[i].Contains(*InCommandInfo->GetActiveChord(ChordIndex)));
			ContextEntry.ChordToCommandInfoMaps[i].Add(*InCommandInfo->GetActiveChord(ChordIndex), InCommandInfo->GetCommandName());
		}
	}
}

void FInputBindingManager::RemoveInputCommand(const TSharedRef<FBindingContext>& InBindingContext, TSharedRef<FUICommandInfo> InUICommandInfo)
{
	check(InUICommandInfo->BindingContext == InBindingContext->GetContextName());

	// The command name should be valid
	check(InUICommandInfo->CommandName != NAME_None);

	LLM_SCOPE_BYTAG(UI_Slate);

	const FName ContextName = InBindingContext->GetContextName();

	FContextEntry& ContextEntry = ContextMap.FindOrAdd(ContextName);

	// Our parent context must exist.
	check(InBindingContext->GetContextParent() == NAME_None || ContextMap.Find(InBindingContext->GetContextParent()) != NULL);

	// Remove the command and its associated chord if it's valid
	ContextEntry.CommandInfoMap.Remove(InUICommandInfo->CommandName);
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		if (InUICommandInfo->ActiveChords[i]->IsValidChord())
		{
			ContextEntry.ChordToCommandInfoMaps[i].Remove(*InUICommandInfo->GetActiveChord(static_cast<EMultipleKeyBindingIndex> (i)));
		}
	}
}

bool FInputBindingManager::CommandPassesFilter(const FName InBindingContext, const FName InCommandName) const
{
	if (const FCommandFilterForContext* CommandFilterForContext = CommandFiltersByContext.Find(InBindingContext))
	{
		if (CommandFilterForContext->CommandDenyList.Contains(InCommandName))
		{
			return false;
		}
		else if (!CommandFilterForContext->CommandAllowList.Contains(InCommandName) && CommandFilterForContext->CommandAllowList.Num() > 0)
		{
			return false;
		}
	}

	return true;
}

bool FInputBindingManager::RegisterCommandList(const FName InBindingContext, const TSharedRef<FUICommandList> CommandList) const
{
	if (ContextMap.Contains(InBindingContext) && OnRegisterCommandList.IsBound())
	{
		OnRegisterCommandList.Broadcast(InBindingContext, CommandList);
		return true;
	}
	return false;
}

TSharedPtr<FUICommandList> FInputBindingManager::RegisterNewCommandList(const FName InBindingContext) const
{
	const TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();

	return RegisterCommandList(InBindingContext, CommandList.ToSharedRef()) ? CommandList : nullptr;
}

bool FInputBindingManager::UnregisterCommandList(const FName InBindingContext, TSharedRef<FUICommandList> CommandList) const
{
	if (ContextMap.Contains(InBindingContext) && OnUnregisterCommandList.IsBound())
	{
		OnUnregisterCommandList.Broadcast(InBindingContext, CommandList);
		return true;
	}
	return false;
}

void FInputBindingManager::AddCommandFilter(const FName InOwnerName, const FName InBindingContext, const FName InCommandName, const ECommandFilterType FilterType)
{
	if (FilterType == ECommandFilterType::DenyList)
	{
		CommandFiltersByContext.FindOrAdd(InBindingContext).CommandDenyList.FindOrAdd(InCommandName).OwnerNames.AddUnique(InOwnerName);
	}
	else if (FilterType == ECommandFilterType::AllowList)
	{
		CommandFiltersByContext.FindOrAdd(InBindingContext).CommandAllowList.FindOrAdd(InCommandName).OwnerNames.AddUnique(InOwnerName);
	}
}

void FInputBindingManager::UnregisterCommandFilterOwner(const FName InOwnerName)
{
	for (auto CommandFiltersByContextIt = CommandFiltersByContext.CreateIterator(); CommandFiltersByContextIt; ++CommandFiltersByContextIt)
	{
		FCommandFilterForContext& CommandFilterForContext = CommandFiltersByContextIt->Value;

		for (auto CommandIt = CommandFilterForContext.CommandDenyList.CreateIterator(); CommandIt; ++CommandIt)
		{
			FCommandFilterOwners& CommandFilterOwners = CommandIt->Value;
			CommandFilterOwners.OwnerNames.Remove(InOwnerName);
			if (CommandFilterOwners.OwnerNames.Num() == 0)
			{
				CommandIt.RemoveCurrent();
			}
		}

		for (auto CommandIt = CommandFilterForContext.CommandAllowList.CreateIterator(); CommandIt; ++CommandIt)
		{
			FCommandFilterOwners& CommandFilterOwners = CommandIt->Value;
			CommandFilterOwners.OwnerNames.Remove(InOwnerName);
			if (CommandFilterOwners.OwnerNames.Num() == 0)
			{
				CommandIt.RemoveCurrent();
			}
		}

		if (CommandFilterForContext.CommandDenyList.Num() == 0 && CommandFilterForContext.CommandAllowList.Num() == 0)
		{
			CommandFiltersByContextIt.RemoveCurrent();
		}
	}
}

const TSharedPtr<FUICommandInfo> FInputBindingManager::FindCommandInContext( const FName InBindingContext, const FInputChord& InChord, bool bCheckDefault ) const
{
	const FContextEntry& ContextEntry = ContextMap.FindRef( InBindingContext );
	
	TSharedPtr<FUICommandInfo> FoundCommand = NULL;

	if( bCheckDefault )
	{
		const FCommandInfoMap& InfoMap = ContextEntry.CommandInfoMap;
		for( FCommandInfoMap::TConstIterator It(InfoMap); It && !FoundCommand.IsValid(); ++It )
		{
			const FUICommandInfo& CommandInfo = *It.Value();
			if( CommandInfo.HasDefaultChord(InChord))
			{
				FoundCommand = It.Value();
			}	
		}
	}
	else
	{
		// faster lookup for active chords, using the mapped values
		FName CommandName = NAME_None;
		for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords) && CommandName == NAME_None; ++i)
		{
			CommandName = ContextEntry.ChordToCommandInfoMaps[i].FindRef(InChord);
		}
		if (CommandName != NAME_None)
		{
			FoundCommand = ContextEntry.CommandInfoMap.FindChecked(CommandName);
		}
	}

	if (FoundCommand.IsValid() && !CommandPassesFilter(InBindingContext, FoundCommand->GetCommandName()))
	{
		return TSharedPtr<FUICommandInfo>();
	}

	return FoundCommand;
}

const TSharedPtr<FUICommandInfo> FInputBindingManager::FindCommandInContext( const FName InBindingContext, const FName CommandName ) const 
{
	const FContextEntry& ContextEntry = ContextMap.FindRef( InBindingContext );
	
	return ContextEntry.CommandInfoMap.FindRef( CommandName );
}


void FInputBindingManager::GetAllChildContexts( const FName InBindingContext, TArray<FName>& AllChildren ) const
{
	AllChildren.Add( InBindingContext );

	TArray<FName> TempChildren;
	ParentToChildMap.MultiFind( InBindingContext, TempChildren );
	for( int32 ChildIndex = 0; ChildIndex < TempChildren.Num(); ++ChildIndex )
	{
		GetAllChildContexts( TempChildren[ChildIndex], AllChildren );
	}
}

const TSharedPtr<FUICommandInfo> FInputBindingManager::GetCommandInfoFromInputChord( const FName InBindingContext, const FInputChord& InChord, bool bCheckDefault ) const
{
	TSharedPtr<FUICommandInfo> FoundCommand = NULL;

	FName CurrentContext = InBindingContext;
	while( CurrentContext != NAME_None && !FoundCommand.IsValid() )
	{
		const FContextEntry& ContextEntry = ContextMap.FindRef( CurrentContext );

		FoundCommand = FindCommandInContext( CurrentContext, InChord, bCheckDefault );

		CurrentContext = ContextEntry.BindingContext->GetContextParent();
	}
	
	if( !FoundCommand.IsValid() )
	{
		TArray<FName> Children;
		GetAllChildContexts( InBindingContext, Children );

		for( int32 ContextIndex = 0; ContextIndex < Children.Num() && !FoundCommand.IsValid(); ++ContextIndex )
		{
			FoundCommand = FindCommandInContext( Children[ContextIndex], InChord, bCheckDefault );
		}
	}
	

	return FoundCommand;
}

/**
 * Returns a list of all known input contexts
 *
 * @param OutInputContexts	The generated list of contexts
 * @return A list of all known input contexts                   
 */
void FInputBindingManager::GetKnownInputContexts( TArray< TSharedPtr<FBindingContext> >& OutInputContexts ) const
{
	for( TMap< FName, FContextEntry >::TConstIterator It(ContextMap); It; ++It )
	{
		OutInputContexts.Add( It.Value().BindingContext );
	}
}

TSharedPtr<FBindingContext> FInputBindingManager::GetContextByName( const FName& InContextName )
{
	FContextEntry* FindResult = ContextMap.Find( InContextName );
	if ( FindResult == NULL )
	{
		return TSharedPtr<FBindingContext>();
	}
	else
	{
		return FindResult->BindingContext;
	}
}

void FInputBindingManager::RemoveContextByName( const FName& InContextName )
{
	ContextMap.Remove(InContextName);
}

void FInputBindingManager::PrintAllInputCommands(bool bBoundOnly)
{
#if WITH_EDITOR

	UE_LOGF(LogSlate, Log, "UIContext List  ---------------");
	TArray<FName> ContextNames;
	{	
		for (TMap< FName, FContextEntry >::TConstIterator It(ContextMap); It; ++It)
		{
			ContextNames.Add(It.Key());
		}

		Algo::Sort(ContextNames, [](const FName& A, const FName& B) { return A.Compare(B) < 0; });

		if (!bBoundOnly)
		{
			for (const FName& ContextName : ContextNames)
			{
				UE_LOGF(LogSlate, Log, "%ls", *ContextName.ToString());
			}
		}
	}

	UE_LOGF(LogSlate, Log, "UICommand List ---------------");
	for (const FName& ContextName : ContextNames)
	{
		TArray<FString> Commands;
		const FContextEntry& ContextEntry = ContextMap.FindChecked(ContextName);
		const FCommandInfoMap& InfoMap = ContextEntry.CommandInfoMap;
		for (FCommandInfoMap::TConstIterator CommandInfoIt(InfoMap); CommandInfoIt; ++CommandInfoIt)
		{
			const FName CommandName = CommandInfoIt.Key();
			if (bBoundOnly)
			{
				TSharedPtr<FUICommandInfo> CommandInfo = CommandInfoIt.Value();
				if (CommandInfo.IsValid())
				{
					const TSharedRef<const FInputChord> FirstValidChord = CommandInfo->GetFirstValidChord();
					if (FirstValidChord->IsValidChord())
					{
						Commands.Add(CommandName.ToString() + TEXT("\t[") + FirstValidChord->GetInputText().ToString() + TEXT("]"));
					}
				}
			}
			else
			{
				Commands.Add(CommandName.ToString());
			}
		}

		Algo::Sort(Commands);

		for (const FString& CommandName : Commands)
		{
			UE_LOGF(LogSlate, Log, "%ls.%ls", *ContextName.ToString(), *CommandName);
		}
	}
#endif // WITH_EDITOR
}

FInputBindingManager::FInputBindingManager()
{
#if WITH_EDITOR

	static FAutoConsoleCommand CVarListKeyboundCommands = FAutoConsoleCommand(
		TEXT("Slate.Commands.ListBound"),
		TEXT(""),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			FInputBindingManager::Get().PrintAllInputCommands(true);
		})
	);

	static FAutoConsoleCommand CVarListAllCommands = FAutoConsoleCommand(
		TEXT("Slate.Commands.ListAll"),
		TEXT(""),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			FInputBindingManager::Get().PrintAllInputCommands(false);
		})
	);

#endif // WITH_EDITOR
}


