// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/InputChord.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputChord)

#define LOCTEXT_NAMESPACE "FInputChord"

/* FInputChord interface
 *****************************************************************************/

FInputChord::FInputChord()
	: Key(EKeys::Invalid)
	, bShift(false)
	, bCtrl(false)
	, bAlt(false)
	, bCmd(false)
{
}

EModifierKey::Type FInputChord::GetModifierKey() const
{
	return EModifierKey::FromBools(bCtrl, bAlt, bShift, bCmd);
}

/**
 * Returns the friendly, localized string name of this key binding
 */
FText FInputChord::GetInputText(const bool bLongDisplayName) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Modifiers"), GetModifierText());
	Args.Add(TEXT("Key"), GetKeyText(bLongDisplayName));

	return FText::Format(LOCTEXT("InputText", "{Modifiers}{Key}"), Args);
}


FText FInputChord::GetKeyText(const bool bLongDisplayName) const
{
	FText OutString;

	if (Key.IsValid() && !Key.IsModifierKey())
	{
		OutString = Key.GetDisplayName(bLongDisplayName);
	}

	return OutString;
}

FText FInputChord::GetModifierText(TOptional<FText> ModifierAppender) const
{
#if PLATFORM_MAC
	const FText CommandText = LOCTEXT("KeyName_Control", "Ctrl");
	const FText ControlText = LOCTEXT("KeyName_Command", "Cmd");
	const FText OptionText = LOCTEXT("KeyName_Option", "Option");
#else
	const FText ControlText = LOCTEXT("KeyName_Control", "Ctrl");
	const FText CommandText = LOCTEXT("KeyName_Command", "Cmd");
#endif
	const FText AltText = LOCTEXT("KeyName_Alt", "Alt");
	const FText ShiftText = LOCTEXT("KeyName_Shift", "Shift");


	const FText AppenderText = Key != EKeys::Invalid ? ModifierAppender.Get(LOCTEXT("ModAppender", "+")) : FText::GetEmpty();

	FFormatNamedArguments Args;
	int32 ModCount = 0;

	if (bCtrl)
	{
		Args.Add(FString::Printf(TEXT("Mod%d"), ++ModCount), ControlText);
	}

	if (bCmd)
	{
		Args.Add(FString::Printf(TEXT("Mod%d"), ++ModCount), CommandText);
	}

	if (bAlt)
	{
#if PLATFORM_MAC
		Args.Add(FString::Printf(TEXT("Mod%d"), ++ModCount), OptionText);
#else
		Args.Add(FString::Printf(TEXT("Mod%d"), ++ModCount), AltText);
#endif
	}

	if (bShift)
	{
		Args.Add(FString::Printf(TEXT("Mod%d"), ++ModCount), ShiftText);
	}

	for (int32 i = 1; i <= 4; ++i)
	{
		if (i > ModCount)
		{
			Args.Add(FString::Printf(TEXT("Mod%d"), i), FText::GetEmpty());
			Args.Add(FString::Printf(TEXT("Appender%d"), i), FText::GetEmpty());
		}
		else
		{
			Args.Add(FString::Printf(TEXT("Appender%d"), i), AppenderText);
		}

	}

	return FText::Format(LOCTEXT("FourModifiers", "{Mod1}{Appender1}{Mod2}{Appender2}{Mod3}{Appender3}{Mod4}{Appender4}"), Args);
}

FInputChord::ERelationshipType FInputChord::GetRelationship( const FInputChord& OtherChord ) const
{
	ERelationshipType Relationship = ERelationshipType::None;

	if (Key == OtherChord.Key)
	{
		if ((bAlt == OtherChord.bAlt) &&
			(bCtrl == OtherChord.bCtrl) &&
			(bShift == OtherChord.bShift) &&
			(bCmd == OtherChord.bCmd))
		{
			Relationship = ERelationshipType::Same;
		}
		else if ((bAlt || !OtherChord.bAlt) &&
				(bCtrl || !OtherChord.bCtrl) &&
				(bShift || !OtherChord.bShift) &&
				(bCmd || !OtherChord.bCmd))
		{
			Relationship = ERelationshipType::Masks;
		}
		else if ((!bAlt || OtherChord.bAlt) &&
				(!bCtrl || OtherChord.bCtrl) &&
				(!bShift || OtherChord.bShift) &&
				(!bCmd || OtherChord.bCmd))
		{
			Relationship = ERelationshipType::Masked;
		}
	}

	return Relationship;
}

#undef LOCTEXT_NAMESPACE

