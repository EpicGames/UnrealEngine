// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/UIAction.h"
#include "Framework/Application/SlateApplication.h"


FUIAction::FUIAction()
	: ExecuteAction()
	, CanExecuteAction()
	, GetActionCheckState()
	, IsActionVisibleDelegate()
	, RepeatMode(EUIActionRepeatMode::RepeatDisabled)
{ 
}

FUIAction::FUIAction(FExecuteAction InitExecuteAction, EUIActionRepeatMode InitRepeatMode)
	: ExecuteAction(InitExecuteAction)
	, CanExecuteAction()
	, GetActionCheckState()
	, IsActionVisibleDelegate()
	, RepeatMode(InitRepeatMode)
{
	CanExecuteAction = FCanExecuteAction::CreateStatic([]()
	{
		return FSlateApplication::IsInitialized()
			&& FSlateApplication::Get().IsNormalExecution();
	});
}

FUIAction::FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, EUIActionRepeatMode InitRepeatMode)
	: ExecuteAction(InitExecuteAction)
	, CanExecuteAction(InitCanExecuteAction)
	, GetActionCheckState()
	, IsActionVisibleDelegate()
	, RepeatMode(InitRepeatMode)
{ 
}

FUIAction::FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FIsActionChecked InitIsCheckedDelegate, EUIActionRepeatMode InitRepeatMode)
	: ExecuteAction(InitExecuteAction)
	, CanExecuteAction(InitCanExecuteAction)
	, GetActionCheckState(FGetActionCheckState::CreateStatic(&FUIAction::IsActionCheckedPassthrough, InitIsCheckedDelegate))
	, IsActionVisibleDelegate()
	, RepeatMode(InitRepeatMode)
{ 
}

FUIAction::FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FGetActionCheckState InitGetActionCheckStateDelegate, EUIActionRepeatMode InitRepeatMode)
	: ExecuteAction(InitExecuteAction)
	, CanExecuteAction(InitCanExecuteAction)
	, GetActionCheckState(InitGetActionCheckStateDelegate)
	, IsActionVisibleDelegate()
	, RepeatMode(InitRepeatMode)
{ 
}

FUIAction::FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FIsActionChecked InitIsCheckedDelegate, FIsActionButtonVisible InitIsActionVisibleDelegate, EUIActionRepeatMode InitRepeatMode)
	: ExecuteAction(InitExecuteAction)
	, CanExecuteAction(InitCanExecuteAction)
	, GetActionCheckState(FGetActionCheckState::CreateStatic(&FUIAction::IsActionCheckedPassthrough, InitIsCheckedDelegate))
	, IsActionVisibleDelegate(InitIsActionVisibleDelegate)
	, RepeatMode(InitRepeatMode)
{ 
}

FUIAction::FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FGetActionCheckState InitGetActionCheckStateDelegate, FIsActionButtonVisible InitIsActionVisibleDelegate, EUIActionRepeatMode InitRepeatMode)
	: ExecuteAction(InitExecuteAction)
	, CanExecuteAction(InitCanExecuteAction)
	, GetActionCheckState(InitGetActionCheckStateDelegate)
	, IsActionVisibleDelegate(InitIsActionVisibleDelegate)
	, RepeatMode(InitRepeatMode)
{ 
}

/////////////////////////////////////////////////////
// FUIActionContext

TSharedPtr<IUIActionContextBase> FUIActionContext::FindContext(const FName InName) const
{
	for (const TSharedPtr<IUIActionContextBase>& Context : Contexts)
	{
		if (Context && Context->GetContextName().IsEqual(InName))
		{
			return Context;
		}
	}

	return nullptr;
}

void FUIActionContext::AddContext(const TSharedPtr<IUIActionContextBase>& InContext)
{
	Contexts.Add(InContext);
}
