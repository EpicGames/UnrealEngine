// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/UICommandDragDropOp.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


TSharedRef<FUICommandDragDropOp> FUICommandDragDropOp::New( FName InItemName, EMultiBlockType InBlockType, bool bInIsDraggingSection, FName InOriginMultiBox, TSharedPtr<SWidget> CustomDectorator, FVector2D DecoratorOffset )
{
	TSharedRef<FUICommandDragDropOp> Operation = MakeShareable(new FUICommandDragDropOp(InItemName, InBlockType, bInIsDraggingSection, InOriginMultiBox, CustomDectorator, DecoratorOffset));
	Operation->Construct();

	return Operation;
}


void FUICommandDragDropOp::OnDragged( const class FDragDropEvent& DragDropEvent )
{
	CursorDecoratorWindow->SetOpacity(0.85f);
	CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition() + Offset);
}


void FUICommandDragDropOp::OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent )
{
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);
	OnDropNotification.ExecuteIfBound();
}


TSharedPtr<SWidget> FUICommandDragDropOp::GetDefaultDecorator() const
{
	TSharedPtr<SWidget> Content;

	if (CustomDecorator.IsValid())
	{
		Content = CustomDecorator;
	}
	else
	{
		Content = SNew(STextBlock)
			.Text(FText::FromName(ItemName));
	}

	// Create hover widget
	return SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Content()
		[	
			Content.ToSharedRef()
		];
}
