// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/UICommandInfo.h"

#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Trace/SlateMemoryTags.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UICommandInfo)


FOnBindingContextChanged FBindingContext::CommandsChanged;


FUICommandInfoDecl FBindingContext::NewCommand( const FName InCommandName, const FText& InCommandLabel, const FText& InCommandDesc )
{
	LLM_SCOPE_BYTAG(UI_Slate);
	return FUICommandInfoDecl( this->AsShared(), InCommandName, InCommandLabel, InCommandDesc );
}

void FBindingContext::AddBundle(const FName Name, const FText& Desc)
{
	if (Bundles.Contains(Name))
	{
		UE_LOGF(LogSlate, Warning, "Ignoring attempt to add bundle %ls because it already exists in context %ls", *Name.ToString(), *ContextName.ToString());
	}
	else if (Name == NAME_None)
	{
		UE_LOGF(LogSlate, Warning, "Ignoring attempt to add bundle to context %ls because no name was given", *ContextName.ToString());
	}
	else
	{
		Bundles.Add(Name, Desc);
	}
}

const FText& FBindingContext::GetBundleLabel(const FName Name)
{
	const FText* Found = Bundles.Find(Name);
	if (Found)
	{
		return *Found;
	}
	else
	{
		return FText::GetEmpty();
	}
}




FUICommandInfoDecl::FUICommandInfoDecl( const TSharedRef<FBindingContext>& InContext, const FName InCommandName, const FText& InLabel, const FText& InDesc, const FName InBundle)
	: Context( InContext )
{
	LLM_SCOPE_BYTAG(UI_Slate);
	Info = MakeShareable( new FUICommandInfo( InContext->GetContextName() ) );
	Info->CommandName = InCommandName;
	Info->Label = InLabel;
	Info->Description = InDesc;
	Info->Bundle = InBundle;
}

FUICommandInfoDecl& FUICommandInfoDecl::DefaultChord( const FInputChord& InDefaultChord, const EMultipleKeyBindingIndex InChordIndex)
{
	Info->DefaultChords[static_cast<uint8>(InChordIndex)] = InDefaultChord;
	return *this;
}
FUICommandInfoDecl& FUICommandInfoDecl::UserInterfaceType( EUserInterfaceActionType InType )
{
	Info->UserInterfaceType = InType;
	return *this;
}

FUICommandInfoDecl& FUICommandInfoDecl::Icon( const FSlateIcon& InIcon )
{
	Info->Icon = InIcon;
	return *this;
}

FUICommandInfoDecl& FUICommandInfoDecl::Description( const FText& InDescription )
{
	Info->Description = InDescription;
	return *this;
}

FUICommandInfoDecl::operator TSharedPtr<FUICommandInfo>() const
{
	FInputBindingManager::Get().CreateInputCommand( Context, Info.ToSharedRef() );
	return Info;
}

FUICommandInfoDecl::operator TSharedRef<FUICommandInfo>() const
{
	FInputBindingManager::Get().CreateInputCommand( Context, Info.ToSharedRef() );
	return Info.ToSharedRef();
}

FUICommandInfo::FUICommandInfo( const FName InBindingContext )
	: BindingContext( InBindingContext )
	, UserInterfaceType( EUserInterfaceActionType::Button )
	, bUseLongDisplayName( true )
{
	LLM_SCOPE_BYTAG(UI_Slate);

	ActiveChords.Empty(2);
	ActiveChords.Add(TSharedRef<FInputChord>(new FInputChord));
	ActiveChords.Add(TSharedRef<FInputChord>(new FInputChord));

	DefaultChords.Init(FInputChord(EKeys::Invalid, EModifierKey::None), 2);
}


const FText FUICommandInfo::GetInputText() const
{	
	// Just get the text from the first valid chord, there isn't enough room for all of them
	return GetFirstValidChord()->GetInputText(bUseLongDisplayName);
}


void FUICommandInfo::MakeCommandInfo( const TSharedRef<class FBindingContext>& InContext, TSharedPtr< FUICommandInfo >& OutCommand, const FName InCommandName, const FText& InCommandLabel, const FText& InCommandDesc, const FSlateIcon& InIcon, const EUserInterfaceActionType InUserInterfaceType, const FInputChord& InDefaultChord, const FInputChord& InAlternateDefaultChord, const FName InBundle)
{
	ensureMsgf( !InCommandLabel.IsEmpty(), TEXT("Command labels cannot be empty") );

	LLM_SCOPE_BYTAG(UI_Slate);

	OutCommand = MakeShareable( new FUICommandInfo( InContext->GetContextName() ) );
	OutCommand->CommandName = InCommandName;
	OutCommand->Label = InCommandLabel;
	OutCommand->Description = InCommandDesc;
	OutCommand->Icon = InIcon;
	OutCommand->UserInterfaceType = InUserInterfaceType;
	OutCommand->DefaultChords[static_cast<uint8>(EMultipleKeyBindingIndex::Primary)] = InDefaultChord;
	OutCommand->DefaultChords[static_cast<uint8>(EMultipleKeyBindingIndex::Secondary)] = InAlternateDefaultChord;
	OutCommand->Bundle = InBundle;
	FInputBindingManager::Get().CreateInputCommand( InContext, OutCommand.ToSharedRef() );
}

void FUICommandInfo::UnregisterCommandInfo(const TSharedRef<class FBindingContext>& InContext, const TSharedRef<FUICommandInfo>& InCommand)
{
	FInputBindingManager::Get().RemoveInputCommand(InContext, InCommand);
}

void FUICommandInfo::SetActiveChord( const FInputChord& NewChord, const EMultipleKeyBindingIndex InChordIndex)
{
	ActiveChords[static_cast<uint8>(InChordIndex)]->Set(NewChord);
	

	// Set the user defined chord for this command so it can be saved to disk later
	FInputBindingManager::Get().NotifyActiveChordChanged( *this, InChordIndex);
}

void FUICommandInfo::RemoveActiveChord(const EMultipleKeyBindingIndex InChordIndex)
{
	// Chord already exists
	// Reset the other chord that has the same binding

	ActiveChords[static_cast<uint8>(InChordIndex)] = MakeShareable(new FInputChord());

	FInputBindingManager::Get().NotifyActiveChordChanged( *this, InChordIndex);
}

TSharedRef<SToolTip> FUICommandInfo::MakeTooltip( const TAttribute<FText>& InText, const TAttribute< EVisibility >& InToolTipVisibility ) const
{
	return 
		SNew(SToolTip)
		.Visibility(InToolTipVisibility.IsBound() ? InToolTipVisibility : EVisibility::Visible)
		.Content()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(STextBlock)
				.Text(InText.IsBound() ? InText : GetDescription())
				.Font(FCoreStyle::Get().GetFontStyle( "ToolTip.Font" ))
				.ColorAndOpacity( FSlateColor::UseForeground() )
			]
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(GetInputText())
				.Font(FCoreStyle::Get().GetFontStyle( "ToolTip.Font" ))
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
			]
		];
}

