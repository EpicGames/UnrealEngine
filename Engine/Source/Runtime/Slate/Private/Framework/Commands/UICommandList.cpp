// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Commands/UICommandList.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputBindingManager.h"


/* FUICommandList interface
 *****************************************************************************/

void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, EUIActionRepeatMode RepeatMode )
{
	MapAction( InUICommandInfo, ExecuteAction, FCanExecuteAction(), FGetActionCheckState(), FIsActionButtonVisible(), RepeatMode );
}


void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, EUIActionRepeatMode RepeatMode )
{
	MapAction( InUICommandInfo, ExecuteAction, CanExecuteAction, FGetActionCheckState(), FIsActionButtonVisible(), RepeatMode );
}


void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FIsActionChecked IsCheckedDelegate, EUIActionRepeatMode RepeatMode )
{
	MapAction( InUICommandInfo, ExecuteAction, CanExecuteAction, IsCheckedDelegate, FIsActionButtonVisible(), RepeatMode );
}


void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FGetActionCheckState GetActionCheckState, EUIActionRepeatMode RepeatMode )
{
	MapAction( InUICommandInfo, ExecuteAction, CanExecuteAction, GetActionCheckState, FIsActionButtonVisible(), RepeatMode );
}


void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FIsActionChecked IsCheckedDelegate, FIsActionButtonVisible IsVisibleDelegate, EUIActionRepeatMode RepeatMode )
{
	MapAction( InUICommandInfo, ExecuteAction, CanExecuteAction, FGetActionCheckState::CreateStatic(&FUIAction::IsActionCheckedPassthrough, IsCheckedDelegate), IsVisibleDelegate, RepeatMode );
}


void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FGetActionCheckState GetActionCheckState, FIsActionButtonVisible IsVisibleDelegate, EUIActionRepeatMode RepeatMode )
{
	FUIAction Action;
	Action.ExecuteAction = ExecuteAction;
	Action.CanExecuteAction = CanExecuteAction;
	Action.GetActionCheckState = GetActionCheckState;
	Action.IsActionVisibleDelegate = IsVisibleDelegate;
	Action.RepeatMode = RepeatMode;

	MapAction( InUICommandInfo, Action );
}


void FUICommandList::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, const FUIAction& InUIAction )
{
	LLM_SCOPE_BYTAG(UI_Slate);

	if (!ensure(InUICommandInfo.IsValid()))
	{
		return;
	}

	ContextsInList.Add( InUICommandInfo->GetBindingContext() );
	UICommandBindingMap.Add( InUICommandInfo, InUIAction );
}

void FUICommandList::MapAction(const TSharedPtr< const FUICommandInfo > InUICommandInfo, const FUIAction& InUIAction, const FUIActionContext& InUIActionContext)
{
	LLM_SCOPE_BYTAG(UI_Slate);
	
	if (!ensure(InUICommandInfo.IsValid()))
	{
		return;
	}

	ContextsInList.Add(InUICommandInfo->GetBindingContext());
	UICommandBindingMap.Add(InUICommandInfo, InUIAction);
	UICommandContextMap.Add(InUICommandInfo, InUIActionContext);
}

void FUICommandList::Append( const TSharedRef<FUICommandList>& InCommandsToAppend )
{
	LLM_SCOPE_BYTAG(UI_Slate);
	
	check( &InCommandsToAppend.Get() != this );

	// Clear out any invalid parents or children
	CleanupPointerArray(ParentUICommandLists);
	CleanupPointerArray(ChildUICommandLists);

	// Add the new parent. Add this to the parent's child list.
	ParentUICommandLists.AddUnique( InCommandsToAppend );
	InCommandsToAppend->ChildUICommandLists.AddUnique( this->AsShared() );
}

void FUICommandList::UnmapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo )
{
	UICommandBindingMap.Remove( InUICommandInfo );
	UICommandContextMap.Remove( InUICommandInfo );
}

bool FUICommandList::IsActionMapped(const TSharedPtr< const FUICommandInfo > InUICommandInfo) const
{
	// Check against already mapped actions
	return UICommandBindingMap.Contains(InUICommandInfo);
}

bool FUICommandList::ExecuteAction(const TSharedRef< const FUICommandInfo > InUICommandInfo) const
{
	const FUIAction* Action = GetActionForCommand(InUICommandInfo);

	if( Action )
	{
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::CommandRun.Broadcast(InUICommandInfo->GetCommandName(), InUICommandInfo->GetLabel());
#endif
		Action->Execute();
		return true;
	}

	return false;
}

bool FUICommandList::CanExecuteAction( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const
{
	const FUIAction* Action = GetActionForCommand(InUICommandInfo);

	if( Action )
	{
		return Action->CanExecute();
	}

	// If there is no action then assume its possible to execute (Some menus with only children that do nothing themselves will have no delegates bound )
	return true;
}


bool FUICommandList::TryExecuteAction( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const
{
	if( !CanExecuteAction( InUICommandInfo ) )
	{
		return false;
	}

	return ExecuteAction( InUICommandInfo );
}


EVisibility FUICommandList::GetVisibility( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const
{
	const FUIAction* Action = GetActionForCommand(InUICommandInfo);

	if (Action)
	{
		return Action->IsVisible();
	}

	// If there is no action then assume it is visible
	return EVisibility::Visible;
}


ECheckBoxState FUICommandList::GetCheckState( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const
{
	const FUIAction* Action = GetActionForCommand(InUICommandInfo);

	if( Action )
	{
		return Action->GetCheckState();
	}

	return ECheckBoxState::Unchecked;
}


bool FUICommandList::ProcessCommandBindings( const FKeyEvent& InKeyEvent ) const 
{
	return ConditionalProcessCommandBindings( InKeyEvent.GetKey(), InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown(), InKeyEvent.IsRepeat() );
}


bool FUICommandList::ProcessCommandBindings( const FPointerEvent& InMouseEvent ) const
{
	return ConditionalProcessCommandBindings( InMouseEvent.GetEffectingButton(), InMouseEvent.IsControlDown(), InMouseEvent.IsAltDown(), InMouseEvent.IsShiftDown(), InMouseEvent.IsCommandDown(), InMouseEvent.IsRepeat() );
}


bool FUICommandList::ProcessCommandBindings( const FKey Key, const FModifierKeysState& ModifierKeysState, const bool bRepeat ) const
{
	return ConditionalProcessCommandBindings(
		Key,
		ModifierKeysState.IsControlDown(),
		ModifierKeysState.IsAltDown(),
		ModifierKeysState.IsShiftDown(),
		ModifierKeysState.IsCommandDown(),
		bRepeat );
}


/* FUICommandList implementation
 *****************************************************************************/

bool FUICommandList::ConditionalProcessCommandBindings( const FKey Key, bool bCtrl, bool bAlt, bool bShift, bool bCmd, bool bRepeat ) const
{
	if ( !FSlateApplication::Get().IsDragDropping() )
	{
		FInputChord CheckChord( Key, EModifierKey::FromBools(bCtrl, bAlt, bShift, bCmd) );

		if( CheckChord.IsValidChord() )
		{
			TSet<FName> AllContextsToCheck;
			GatherContextsForList(AllContextsToCheck);

			for( TSet<FName>::TConstIterator It(AllContextsToCheck); It; ++It )
			{
				FName Context = *It;
	
				// Only active chords process commands
				const bool bCheckDefault = false;

				// Check to see if there is any command in the context activated by the chord
				TSharedPtr<FUICommandInfo> Command = FInputBindingManager::Get().FindCommandInContext( Context, CheckChord, bCheckDefault );

				if( Command.IsValid() && Command->HasActiveChord(CheckChord)  )
				{
					// Find the bound action for this command
					const FUIAction* Action = GetActionForCommand(Command);

					// If there is no Action mapped to this command list, continue to the next context
					if( Action )
					{
						if(Action->CanExecute() && (!bRepeat || Action->CanRepeat()))
						{
							// If the action was found and can be executed, do so now
							Action->Execute();
							return true;
						}
						else
						{
							// An action wasn't bound to the command but a chord was found or an action was found but cant be executed
							return false;
						}
					}
				}
			}
		}
	}

	// No action was processed
	return false;
}


const FUIAction* FUICommandList::GetActionForCommand(TSharedPtr<const FUICommandInfo> Command) const
{
	// Make sure the command is valid
	if ( !ensure(Command.IsValid()) )
	{
		return NULL;
	}

	// Check in my own binding map. This should not be prevented by CanProduceActionForCommand.
	// Any action directly requested from a command list should be returned if it actually exists in the list.
	const FUIAction* Action = UICommandBindingMap.Find( Command );

	if ( !Action )
	{
		// We did not find the action in our own list. Recursively attempt to find the command in children and parents.
		const bool bIncludeChildren = true;
		const bool bIncludeParents = true;
		TSet<TSharedRef<const FUICommandList>> VisitedLists;
		Action = GetActionForCommandRecursively(Command.ToSharedRef(), bIncludeChildren, bIncludeParents, VisitedLists);
	}
	
	return Action;
}


const FUIActionContext* FUICommandList::GetContextForCommand(TSharedPtr<const FUICommandInfo> Command) const
{
	// Make sure the command is valid
	if (!ensure(Command.IsValid()))
	{
		return NULL;
	}

	// Check in my own binding map. This should not be prevented by CanProduceActionForCommand.
	// Any action directly requested from a command list should be returned if it actually exists in the list.
	const FUIActionContext* Context = UICommandContextMap.Find(Command);

	if (!Context)
	{
		// We did not find the action in our own list. Recursively attempt to find the command in children and parents.
		const bool bIncludeChildren = true;
		const bool bIncludeParents = true;
		TSet<TSharedRef<const FUICommandList>> VisitedLists;
		Context = GetContextForCommandRecursively(Command.ToSharedRef(), bIncludeChildren, bIncludeParents, VisitedLists);
	}

	return Context;
}


const FUIAction* FUICommandList::GetActionForCommandRecursively(const TSharedRef<const FUICommandInfo>& Command, bool bIncludeChildren, bool bIncludeParents, TSet<TSharedRef<const FUICommandList>>& InOutVisitedLists) const
{
	// Detect cycles in the graph
	{
		const TSharedRef<const FUICommandList>& ListAsShared = AsShared();
		if ( InOutVisitedLists.Contains(ListAsShared) )
		{
			// This node was already visited. End recursion.
			return NULL;
		}

		InOutVisitedLists.Add( ListAsShared );
	}

	const FUIAction* Action = NULL;

	// Make sure I am capable of processing this command
	bool bCapableOfCommand = true;
	if ( CanProduceActionForCommand.IsBound() )
	{
		bCapableOfCommand = CanProduceActionForCommand.Execute(Command);
	}

	if ( bCapableOfCommand )
	{
		// Check in my own binding map
		Action = UICommandBindingMap.Find( Command );
	
		// If the action was not found, check in my children binding maps
		if ( !Action && bIncludeChildren )
		{
			for ( auto ChildIt = ChildUICommandLists.CreateConstIterator(); ChildIt; ++ChildIt )
			{
				TWeakPtr<FUICommandList> Child = *ChildIt;
				if ( Child.IsValid() )
				{
					const bool bShouldIncludeChildrenOfChild = true;
					const bool bShouldIncludeParentsOfChild = false;
					Action = Child.Pin()->GetActionForCommandRecursively(Command, bShouldIncludeChildrenOfChild, bShouldIncludeParentsOfChild, InOutVisitedLists);
					if ( Action )
					{
						break;
					}
				}
			}
		}
	}

	// If the action was not found, check in my parent binding maps
	if ( !Action && bIncludeParents )
	{
		for ( auto ParentIt = ParentUICommandLists.CreateConstIterator(); ParentIt; ++ParentIt )
		{
			TWeakPtr<FUICommandList> Parent = *ParentIt;
			if ( Parent.IsValid() )
			{
				const bool bShouldIncludeChildrenOfParent = false;
				const bool bShouldIncludeParentsOfParent = true;
				Action = Parent.Pin()->GetActionForCommandRecursively(Command, bShouldIncludeChildrenOfParent, bShouldIncludeParentsOfParent, InOutVisitedLists);
				if ( Action )
				{
					break;
				}
			}
		}
	}

	return Action;
}


const FUIActionContext* FUICommandList::GetContextForCommandRecursively(const TSharedRef<const FUICommandInfo>& Command, bool bIncludeChildren, bool bIncludeParents, TSet<TSharedRef<const FUICommandList>>& InOutVisitedLists) const
{
	// Detect cycles in the graph
	{
		const TSharedRef<const FUICommandList>& ListAsShared = AsShared();
		if ( InOutVisitedLists.Contains(ListAsShared) )
		{
			// This node was already visited. End recursion.
			return NULL;
		}

		InOutVisitedLists.Add( ListAsShared );
	}

	const FUIActionContext* Context = nullptr;

	// Make sure I am capable of processing this command
	bool bCapableOfCommand = true;
	if ( CanProduceActionForCommand.IsBound() )
	{
		bCapableOfCommand = CanProduceActionForCommand.Execute(Command);
	}

	if ( bCapableOfCommand )
	{
		// Check in my own binding map
		Context = UICommandContextMap.Find(Command);
	
		// If the context was not found, check in my children binding maps
		if ( !Context && bIncludeChildren )
		{
			for ( auto ChildIt = ChildUICommandLists.CreateConstIterator(); ChildIt; ++ChildIt )
			{
				TWeakPtr<FUICommandList> Child = *ChildIt;
				if ( Child.IsValid() )
				{
					const bool bShouldIncludeChildrenOfChild = true;
					const bool bShouldIncludeParentsOfChild = false;
					Context = Child.Pin()->GetContextForCommandRecursively(Command, bShouldIncludeChildrenOfChild, bShouldIncludeParentsOfChild, InOutVisitedLists);
					if (Context)
					{
						break;
					}
				}
			}
		}
	}

	// If the context was not found, check in my parent binding maps
	if ( !Context && bIncludeParents )
	{
		for ( auto ParentIt = ParentUICommandLists.CreateConstIterator(); ParentIt; ++ParentIt )
		{
			TWeakPtr<FUICommandList> Parent = *ParentIt;
			if ( Parent.IsValid() )
			{
				const bool bShouldIncludeChildrenOfParent = false;
				const bool bShouldIncludeParentsOfParent = true;
				Context = Parent.Pin()->GetContextForCommandRecursively(Command, bShouldIncludeChildrenOfParent, bShouldIncludeParentsOfParent, InOutVisitedLists);
				if (Context)
				{
					break;
				}
			}
		}
	}

	return Context;
}


void FUICommandList::GatherContextsForList(TSet<FName>& OutAllContexts) const
{
	TSet<TSharedRef<const FUICommandList>> VisitedLists;
	GatherContextsForListRecursively(OutAllContexts, VisitedLists);
}


void FUICommandList::GatherContextsForListRecursively(TSet<FName>& OutAllContexts, TSet<TSharedRef<const FUICommandList>>& InOutVisitedLists) const
{
	// Detect cycles in the graph
	{
		const TSharedRef<const FUICommandList>& ListAsShared = AsShared();
		if ( InOutVisitedLists.Contains(ListAsShared) )
		{
			// This node was already visited. End recursion.
			return;
		}

		InOutVisitedLists.Add( ListAsShared );
	}

	// Include all contexts on this list
	OutAllContexts.Append(ContextsInList);

	// Include all the parent contexts
	for ( auto ParentIt = ParentUICommandLists.CreateConstIterator(); ParentIt; ++ParentIt )
	{
		TWeakPtr<FUICommandList> Parent = *ParentIt;
		if ( Parent.IsValid() )
		{
			Parent.Pin()->GatherContextsForListRecursively(OutAllContexts, InOutVisitedLists);
		}
	}
}
