// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/FDockingDragOperation.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"

/** 
  * Check if the given PotentialParent is a parent of PotentialChild. 
  */
static bool IsParentWidgetOf(const TSharedPtr<SWidget>& PotentialParent, const TSharedPtr<SWidget>& PotentialChild)
{
	const SWidget* Parent = PotentialChild->GetParentWidget().Get();
	while (Parent != nullptr)
	{
		if (PotentialParent.Get() == Parent)
		{
			return true;
		}
		Parent = Parent->GetParentWidget().Get();
	}

	return false;
}

/**
 * Invoked when the drag and drop operation has ended.
 * 
 * @param bDropWasHandled   true when the drop was handled by some widget; false otherwise
 */
void FDockingDragOperation::OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent )
{
	check(CursorDecoratorWindow.IsValid());

	const FVector2D WindowSize = CursorDecoratorWindow->GetSizeInScreen();

	TabBeingDragged->SetDraggedOverDockArea( NULL );

	if (!bDropWasHandled)
	{
		DroppedOntoNothing();
	}
	else
	{
		// The event was handled, so we HAVE to have some window that we dropped onto.
		TSharedRef<SWindow> WindowDroppedInto = MouseEvent.GetWindow();

		TabOwnerAreaOfOrigin->GetTabManager()->GetPrivateApi().SetCanDoDeferredLayoutSave(true);

		TabOwnerAreaOfOrigin->GetTabManager()->GetPrivateApi().OnTabRelocated(TabBeingDragged.ToSharedRef(), WindowDroppedInto);
	}

	// Destroy the CursorDecoratorWindow by calling the base class implementation because we are relocating the content into a more permanent home.
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);

	TabBeingDragged.Reset();
}

/** 
 * Called when the mouse was moved during a drag and drop operation
 *
 * @param DragDropEvent    The event that describes this drag drop operation.
 */
void FDockingDragOperation::OnDragged( const FDragDropEvent& DragDropEvent )
{
	const bool bPreviewingTarget = HoveredDockTarget.TargetNode.IsValid();
	if ( !bPreviewingTarget )
	{
		// The tab is being dragged. Move the the decorator window to match the cursor position.
		FVector2D TargetPosition = DragDropEvent.GetScreenSpacePosition() - GetDecoratorOffsetFromCursor();
		CursorDecoratorWindow->UpdateMorphTargetShape( FSlateRect(TargetPosition.X, TargetPosition.Y, TargetPosition.X + LastContentSize.X, TargetPosition.Y + LastContentSize.Y) );	
		CursorDecoratorWindow->MoveWindowTo( TargetPosition );
	}
}
	
/**
 * DragTestArea widgets invoke this method when a drag enters them
 *
 * @param ThePanel   That tab well that we just dragged something into.
 */
void FDockingDragOperation::OnTabWellEntered( const TSharedRef<class SDockingTabWell>& ThePanel )
{
	if (IsParentWidgetOf(this->TabOwnerAreaOfOrigin, ThePanel->GetDockArea()))
	{
		return;
	}

	// We just pulled the tab into some TabWell (in some DockNode).
	// Hide our decorator window and let the DockNode handle previewing what will happen if we drop the node.
	HoveredTabPanelPtr = ThePanel;
	CursorDecoratorWindow->HideWindow();

	TabBeingDragged->SetDraggedOverDockArea( ThePanel->GetDockArea() );
}

/**
 * DragTestArea widgets invoke this method when a drag leaves them
 *
 * @param ThePanel   That tab well that we just dragged something out of.
 */
void FDockingDragOperation::OnTabWellLeft( const TSharedRef<class SDockingTabWell>& ThePanel, const FGeometry& DockNodeGeometry )
{
	// We just pulled out of some DockNode's TabWell
	HoveredTabPanelPtr.Reset();
	// Show the Preview Window again.
	CursorDecoratorWindow->Resize( DockNodeGeometry.GetLocalSize() );
	CursorDecoratorWindow->ShowWindow();
	CursorDecoratorWindow->ReshapeWindow( DockNodeGeometry.GetLayoutBoundingRect() );

	FCurveSequence Sequence;
	Sequence.AddCurve( 0, 0.05f, ECurveEaseFunction::QuadOut );
	CursorDecoratorWindow->MorphToShape( Sequence, CursorDecoratorWindow->GetOpacity(), CursorDecoratorWindow->GetMorphTargetShape() );

	LastContentSize = DesiredSizeFrom( DockNodeGeometry.GetLocalSize() );

	CursorDecoratorStackNode->OpenTab(TabBeingDragged.ToSharedRef());

	TabBeingDragged->SetDraggedOverDockArea( NULL );
}


FSlateRect FDockingDragOperation::GetPreviewAreaForDirection ( const FSlateRect& DockableArea, SDockingArea::RelativeDirection DockingDirection )
{
	FSlateRect TargetRect = DockableArea;
	const FVector2D Size (TargetRect.Right-TargetRect.Left, TargetRect.Bottom-TargetRect.Top);

	switch( DockingDirection )
	{
		case SDockingNode::LeftOf:
			TargetRect.Right = TargetRect.Left + Size.X * 0.5f;
		break;
		case SDockingNode::Above:
			TargetRect.Bottom = TargetRect.Top + Size.Y * 0.5f;
		break;
		case SDockingNode::RightOf:
			TargetRect.Left = TargetRect.Left + Size.X * 0.5f;
		break;
		case SDockingNode::Below:
			TargetRect.Top = TargetRect.Top + Size.Y * 0.5f;
		break;
		case SDockingNode::Center:
		break;
	}
	return TargetRect;
}


void FDockingDragOperation::SetHoveredTarget( const FDockTarget& InTarget, const FInputEvent& InputEvent )
{
	if ( HoveredDockTarget != InTarget )
	{
		HoveredDockTarget = InTarget;
		TSharedPtr<SDockingNode> HoveredTargetNode = InTarget.TargetNode.Pin();
		
		FCurveSequence Sequence;
		Sequence.AddCurve( 0, 0.1f, ECurveEaseFunction::QuadOut );

		if ( HoveredTargetNode.IsValid() )
		{
			const FGeometry TargetDockNodeGeometry = InputEvent.FindGeometry( HoveredTargetNode.ToSharedRef() );
			FSlateRect TabStackArea = GetPreviewAreaForDirection(
				TargetDockNodeGeometry.GetLayoutBoundingRect(),
				InTarget.DockDirection );

			const float TargetOpacity = CursorDecoratorWindow->GetOpacity();

			CursorDecoratorWindow->MorphToShape( Sequence, TargetOpacity, TabStackArea );

			CursorDecoratorWindow->SetColorAndOpacity( FCoreStyle::Get().GetColor( TEXT("Docking.Cross.PreviewWindowTint") ) );

			TabBeingDragged->SetDraggedOverDockArea( HoveredTargetNode->GetDockArea() );
		}
		else
		{
			CursorDecoratorWindow->MorphToShape( Sequence, CursorDecoratorWindow->GetOpacity(), CursorDecoratorWindow->GetMorphTargetShape() );
			CursorDecoratorWindow->SetColorAndOpacity( FLinearColor::White );

			TabBeingDragged->SetDraggedOverDockArea( NULL );
		}
	}
}


TSharedRef<FDockingDragOperation> FDockingDragOperation::New( const TSharedRef<class SDockTab>& InTabToBeDragged, const FVector2D InTabGrabOffset, TSharedRef<class SDockingArea> InTabOwnerArea, const FVector2D& OwnerAreaSize )
{
	const TSharedRef<FDockingDragOperation> Operation = MakeShareable( new FDockingDragOperation( InTabToBeDragged, InTabGrabOffset, InTabOwnerArea, OwnerAreaSize ) );
	return Operation;
}


TSharedPtr<class SDockTab> FDockingDragOperation::GetTabBeingDragged()
{
	return TabBeingDragged;
}


FVector2D FDockingDragOperation::GetTabGrabOffsetFraction() const
{
	return TabGrabOffsetFraction;
}


bool FDockingDragOperation::CanDockInNode(const TSharedRef<SDockingNode>& DockNode, EViaTabwell IsDockingViaTabwell ) const
{
	const TSharedRef<FTabManager> TargetTabManager = DockNode->GetDockArea()->GetTabManager();
	TSharedRef<FTabManager> TabManagerOfOriginToUse = this->TabOwnerAreaOfOrigin->GetTabManager();

	// if we are dragging a NomadTab force it to use the FGlobalTabManager as the TabManager of origin
	// There are cases where the NomadTab is spawned like a normal tab and his TabManager is not correct
	if (TabBeingDragged->GetTabRole() == ETabRole::NomadTab)
	{
		TabManagerOfOriginToUse = FGlobalTabmanager::Get();
	}

	if (IsParentWidgetOf(TabBeingDragged->GetContent(), DockNode))
	{
		return false;
	}

	if (TabBeingDragged->GetTabRole() == ETabRole::NomadTab)
	{
		if ( IsDockingViaTabwell == FDockingDragOperation::DockingViaTabWell )
		{
			// Nomad tabs can be docked in in any tab well.
			return true;
		}
		else
		{
			return TargetTabManager != FGlobalTabmanager::Get();
		}
	}
	else if (TabBeingDragged->GetTabRole() == ETabRole::MajorTab)
	{
		// Major tabs can only be stacked; they should not 
		// be allowed to split areas. They are also confined to their
		// tab manager of origin.
		// The only exception is an empty area, where docking the tab should be really easy.
		const bool bTabManagerMatches = TargetTabManager == TabManagerOfOriginToUse;
		const bool bCanDockInEmptyArea = DockNode->GetNodeType() == SDockingNode::DockArea && StaticCastSharedRef<SDockingArea>(DockNode)->GetChildNodes().Num() == 0;
		return bTabManagerMatches && (IsDockingViaTabwell == FDockingDragOperation::DockingViaTabWell || bCanDockInEmptyArea);
	}
	else
	{
		// Most commonly, tabs are confined to their tab manager of origin.
		return (TargetTabManager == TabManagerOfOriginToUse);
	}	
}


FDockingDragOperation::~FDockingDragOperation()
{
	if (TabBeingDragged.IsValid())
	{
		DroppedOntoNothing();
	}
}


/** The constructor is protected, so that this class can only be instantiated as a shared pointer. */
FDockingDragOperation::FDockingDragOperation( const TSharedRef<SDockTab>& InTabToBeDragged, const FVector2D InTabGrabOffsetFraction, TSharedRef<class SDockingArea> InTabOwnerArea, const FVector2D& OwnerAreaSize )
	: TabBeingDragged( InTabToBeDragged )
	, TabGrabOffsetFraction( InTabGrabOffsetFraction )
	, TabOwnerAreaOfOrigin( InTabOwnerArea )
	, TabStackOfOrigin(InTabToBeDragged->GetParent() ? InTabToBeDragged->GetParent()->GetParentDockTabStack() : nullptr)
	, HoveredTabPanelPtr( )
	, HoveredDockTarget()
	, LastContentSize( OwnerAreaSize )
{

	// Create the decorator window that we will use during this drag and drop to make the user feel like
	// they are actually dragging a piece of UI.

	// Start the window off hidden.
	const bool bShowImmediately = false;
	CursorDecoratorWindow = FSlateApplication::Get().AddWindow( SWindow::MakeCursorDecorator(), bShowImmediately );
	// Usually cursor decorators figure out their size automatically from content, but we will drive it
	// here because the window will reshape itself to better reflect what will happen when the user drops the Tab.
	CursorDecoratorWindow->SetSizingRule( ESizingRule::FixedSize );
	CursorDecoratorWindow->SetOpacity(0.45f);
	
	CursorDecoratorWindow->SetContent
	(
		SNew(SBorder)
		. BorderImage(FCoreStyle::Get().GetBrush("Docking.Background"))
		[
			SNew(SDockingArea, TabBeingDragged->GetTabManagerPtr().ToSharedRef(), FTabManager::NewPrimaryArea())
			//. OriginalDockArea(OriginalDockArea)
			. InitialContent
			( 
				SAssignNew(CursorDecoratorStackNode, SDockingTabStack, FTabManager::NewStack())
			)
		]
	);

	TabOwnerAreaOfOrigin->GetTabManager()->GetPrivateApi().SetCanDoDeferredLayoutSave(false);

	if ( TabBeingDragged->IsActive() )
	{
		FGlobalTabmanager::Get()->SetActiveTab(NULL);
	}

	if (!TabStackOfOrigin.IsValid())
	{
		// No parent, open the window right away
		CursorDecoratorWindow->Resize(OwnerAreaSize);
		CursorDecoratorWindow->ShowWindow();
		CursorDecoratorWindow->ReshapeWindow(FSlateRect(FVector2D(0.0, 0.0), OwnerAreaSize));

		CursorDecoratorStackNode->OpenTab(TabBeingDragged.ToSharedRef());
		TabBeingDragged->SetDraggedOverDockArea(NULL);
	}
}

/** @return The offset into the tab where the user grabbed in Slate Units. */
const FVector2D FDockingDragOperation::GetDecoratorOffsetFromCursor()
{
	const ETabRole RoleToUse = TabBeingDragged->GetVisualTabRole();
	const FVector2D TabDesiredSize = TabBeingDragged->GetDesiredSize();
	const FVector2D MaxTabSize = FDockingConstants::GetMaxTabSizeFor(RoleToUse);

	return TabGrabOffsetFraction * FVector2D (
		FMath::Min( TabDesiredSize.X, MaxTabSize.X ),
		FMath::Min( TabDesiredSize.Y, MaxTabSize.Y )
	);
}

/** @return the size of the DockNode that looks good in a preview given the initial size of the tab that we grabbed. */
FVector2D FDockingDragOperation::DesiredSizeFrom( const FVector2D& InitialTabSize )
{
	// Just make sure it isn't too big so it doesn't cover up the whole screen.
	const float MaxSideSizeAllowed = 800;
	const float SizeCoefficient = FMath::Clamp( MaxSideSizeAllowed / InitialTabSize.GetMax(), 0.1f, 1.0f );
	return InitialTabSize * SizeCoefficient;
}

void FDockingDragOperation::DroppedOntoNothing()
{
	// If we dropped the tab into an existing DockNode then it would have handled the DropEvent.
	// We are here because that didn't happen, so make a new window with a new DockNode and drop the tab into that.
	TSharedPtr<FTabManager> MyTabManager = TabBeingDragged->GetTabManagerPtr();
	if (!MyTabManager.IsValid())
	{
		return;
	}

	const FVector2D PositionToDrop = CursorDecoratorWindow->GetPositionInScreen();

	const float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(PositionToDrop.X, PositionToDrop.Y);

	TSharedPtr<SWindow> NewWindowParent = MyTabManager->GetPrivateApi().GetParentWindow();

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
		// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
		// and for multi-view rendering to function properly
		.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
		.Title(FGlobalTabmanager::Get()->GetApplicationTitle())
		.AutoCenter(EAutoCenter::None)
		// Divide out scale, it is already factored into position
		.ScreenPosition(PositionToDrop/DPIScale)
		// Make room for the title bar; otherwise windows will get progressive smaller whenver you float them.
		.ClientSize(SWindow::ComputeWindowSizeForContent(CursorDecoratorWindow->GetSizeInScreen()))
		.CreateTitleBar(false);

	TSharedPtr<SDockingTabStack> NewDockNode;

	TSharedPtr<FTabManager> TabManagerToUse;

	if (TabBeingDragged->GetTabRole() == ETabRole::NomadTab)
	{
		TabManagerToUse = FGlobalTabmanager::Get();
		TabBeingDragged->SetTabManager(TabManagerToUse);
	}
	else
	{
		TabManagerToUse = MyTabManager;
	}

	// Create a new dockarea
	TSharedRef<SDockingArea> NewDockArea =
		SNew(SDockingArea, TabManagerToUse.ToSharedRef(), FTabManager::NewPrimaryArea())
		.ParentWindow(NewWindow)
		.InitialContent
		(
		SAssignNew(NewDockNode, SDockingTabStack, FTabManager::NewStack())
		);

	if (TabBeingDragged->GetTabRole() == ETabRole::MajorTab || TabBeingDragged->GetTabRole() == ETabRole::NomadTab)
	{
		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		if (RootWindow.IsValid())
		{
			// We have a root window, so all MajorTabs are nested under it.
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef())->SetContent(NewDockArea);
		}
		else
		{
			// App tabs get put in top-level windows. They show up on the taskbar.
			FSlateApplication::Get().AddWindow(NewWindow)->SetContent(NewDockArea);
		}
	}
	else
	{
		// Other tab types are placed in child windows. Their life is controlled by the top-level windows.
		// They do not show up on the taskbar.

		if (NewWindowParent.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, NewWindowParent.ToSharedRef())->SetContent(NewDockArea);
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow)->SetContent(NewDockArea);
		}
	}

	// Do this after the window parenting so that the window title is set correctly
	NewDockNode->OpenTab(TabBeingDragged.ToSharedRef());

	TabOwnerAreaOfOrigin->GetTabManager()->GetPrivateApi().SetCanDoDeferredLayoutSave(true);

	// Let every widget under this tab manager know that this tab has found a new home.
	TabOwnerAreaOfOrigin->GetTabManager()->GetPrivateApi().OnTabRelocated(TabBeingDragged.ToSharedRef(), NewWindow);
}
