// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/SlateRect.h"
#include "Input/DragAndDrop.h"
#include "Framework/Docking/SDockingNode.h"
#include "Framework/Docking/SDockingArea.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Framework/Docking/SDockingTabWell.h"

/** A Sample implementation of IDragDropOperation */
class FDockingDragOperation : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FDockingDragOperation, FDragDropOperation)

	/**
	 * Represents a target for the user re-arranging some layout.
	 * A user expresses their desire to re-arrange layout by placing a tab relative to some layout node.
	 * e.g. I want my tab left of the viewport tab.
	 */
	struct FDockTarget
	{
		FDockTarget()
		: TargetNode()
		, DockDirection()
		{
		}

		FDockTarget( const TSharedPtr<class SDockingNode>& InTargetNode, SDockingNode::RelativeDirection InDockDirection )
		: TargetNode( InTargetNode )
		, DockDirection( InDockDirection )
		{
		}

		bool operator==( const FDockTarget& Other )
		{
			return
				this->TargetNode == Other.TargetNode &&
				this->DockDirection == Other.DockDirection;
		}

		bool operator!=( const FDockTarget& Other )
		{
			return !((*this)==Other);
		}

		/** We'll put the tab relative to this node */
		TWeakPtr<class SDockingNode> TargetNode;
		/** Relation to node where we will put the tab. */
		SDockingNode::RelativeDirection DockDirection;
	};

	/**
	 * Invoked when the drag and drop operation has ended.
	 * 
	 * @param bDropWasHandled   true when the drop was handled by some widget; false otherwise
	 */
	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;

	/** 
	 * Called when the mouse was moved during a drag and drop operation
	 *
	 * @param DragDropEvent    The event that describes this drag drop operation.
	 */
	virtual void OnDragged( const FDragDropEvent& DragDropEvent ) override;
	
	/**
	 * DragTestArea widgets invoke this method when a drag enters them
	 *
	 * @param ThePanel   That tab well that we just dragged something into.
	 */
	void OnTabWellEntered( const TSharedRef<class SDockingTabWell>& ThePanel );

	/**
	 * DragTestArea widgets invoke this method when a drag leaves them
	 *
	 * @param ThePanel   That tab well that we just dragged something out of.
	 */
	void OnTabWellLeft( const TSharedRef<class SDockingTabWell>& ThePanel, const FGeometry& DockNodeGeometry );

	/**
	 * Given a docking direction and the geometry of the dockable area, figure out the area that will be occupied by a new tab if it is docked there.
	 *
	 * @param DockableArea       The area of a TabStack that you're hovering
	 * @param DockingDirection   Where relative to this TabStack you want to dock: e.g. to the right.
	 *
	 * @return The area that the new tab will occupy.
	 */
	FSlateRect GetPreviewAreaForDirection ( const FSlateRect& DockableArea, SDockingArea::RelativeDirection DockingDirection );

	/** Update which dock target, if any, is currently hovered as a result of the InputEvent */
	void SetHoveredTarget( const FDockTarget& InTarget, const FInputEvent& InputEvent );

	/**
	 * Create this Drag and Drop Content
	 *
	 * @param InTabToBeDragged    The tab being dragged (dragged within or outside of a tabwell).
	 * @param InTabGrabOffset     Where within the tab we grabbed, so we're not dragging by the upper left of the tab.
	 * @param InTabOwnerArea      The DockArea that owns this tab until it is relocated.
	 * @param OwnerAreaSize       Size of the DockArea at the time when we start dragging.
	 *
	 * @return a new FDockingDragOperation
	 */
	static TSharedRef<FDockingDragOperation> New( const TSharedRef<SDockTab>& InTabToBeDragged, const FVector2D InTabGrabOffset, TSharedRef<class SDockingArea> InTabOwnerArea, const FVector2D& OwnerAreaSize );

	/** @return the widget being dragged */
	TSharedPtr<SDockTab> GetTabBeingDragged();

	/** @return location where the user grabbed within the tab as a fraction of the tab's size */
	FVector2D GetTabGrabOffsetFraction() const;

	/**Is this dock tab being placed via a tab well or via a target*/
	enum EViaTabwell
	{
		DockingViaTabWell,
		DockingViaTarget
	};

	/** Checks to see if this tab can dock in this node. Some tabs can only dock via the tab well. */
	bool CanDockInNode( const TSharedRef<SDockingNode>& DockNode, EViaTabwell IsDockingViaTabwell ) const;

	virtual ~FDockingDragOperation();

protected:
	/** The constructor is protected, so that this class can only be instantiated as a shared pointer. */
	FDockingDragOperation( const TSharedRef<class SDockTab>& InTabToBeDragged, const FVector2D InTabGrabOffsetFraction, TSharedRef<class SDockingArea> InTabOwnerArea, const FVector2D& OwnerAreaSize );

	/** @return The offset into the tab where the user grabbed in Slate Units. */
	const FVector2D GetDecoratorOffsetFromCursor();

	/** @return the size of the DockNode that looks good in a preview given the initial size of the tab that we grabbed. */
	static FVector2D DesiredSizeFrom( const FVector2D& InitialTabSize );

	/** The tab was dropped onto nothing or someone interrupted the drag drop operation. */
	void DroppedOntoNothing();

	/** What is actually being dragged in this operation */
	TSharedPtr<class SDockTab> TabBeingDragged;

	/** Where the user grabbed the tab as a fraction of the tab's size */
	FVector2D TabGrabOffsetFraction;

	/** The area from which we initially started dragging */
	TSharedPtr<SDockingArea> TabOwnerAreaOfOrigin;

	/** Decorator widget where we add temp doc tabs to */
	TSharedPtr<SDockingTabStack> CursorDecoratorStackNode;

	/** Tab stack from which we started dragging this tab. */
	TWeakPtr<class SDockingTabStack> TabStackOfOrigin;

	/** The TabWell over which we are currently hovering */
	TWeakPtr<class SDockingTabWell> HoveredTabPanelPtr;

	/** Some target dock node over which we are currently hovering; it could be a TabStack or a DockAre */
	FDockTarget HoveredDockTarget;

	/** What the size of the content was when it was last shown. The user drags splitters to set this size; it is legitimate state. */
	FVector2D LastContentSize;
};
