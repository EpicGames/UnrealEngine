// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/LayoutExtender.h"

void FLayoutExtender::ExtendLayout(FTabId PredicateTabId, ELayoutExtensionPosition Position, FTabManager::FTab TabToAdd)
{
	TabExtensions.Add(PredicateTabId, FExtendedTab(Position, TabToAdd));
}

void FLayoutExtender::ExtendArea(FName ExtensionId, const FAreaExtension& AreaExtension)
{
	AreaExtensions.Add(ExtensionId, FExtendedArea(AreaExtension));
}

SLATE_API void FLayoutExtender::ExtendStack(FName ExtensionId, ELayoutExtensionPosition Position, FTabManager::FTab TabToAdd, bool bShouldBeForegrounded)
{
	StackExtensions.Add(ExtensionId, FExtendedTab(Position, TabToAdd, bShouldBeForegrounded));
}

void FLayoutExtender::ExtendAreaRecursive(const TSharedRef<FTabManager::FArea>& Area) const
{
	FName ExtensionId = Area->GetExtensionId();
	if (ExtensionId == NAME_None)
	{
		return;
	}

	for (auto It = AreaExtensions.CreateConstKeyIterator(ExtensionId); It; ++It)
	{
		It.Value().ExtensionCallback(Area);
	}

	for (TSharedRef<FTabManager::FLayoutNode>& ChildNode : Area->ChildNodes)
	{
		TSharedPtr<FTabManager::FArea> ChildArea = ChildNode->AsArea();
		if (ChildArea)
		{
			ExtendAreaRecursive(ChildArea.ToSharedRef());
		}
	}
}
