// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/LayoutService.h"
#include "HAL/FileManager.h"
#include "Internationalization/Regex.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/StringUtility.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "String/CaseConversion.h"

DEFINE_LOG_CATEGORY_STATIC(LogLayoutService, Log, All);

const TCHAR* DefaultEditorLayoutsSectionName = TEXT("EditorLayouts");

static const TCHAR* GetEditorLayoutsSectionName()
{
	static FString EditorLayoutsSectionName;

	if (EditorLayoutsSectionName.IsEmpty())
	{
		GConfig->GetString(TEXT("Slate"), TEXT("EditorLayoutsSectionName"), EditorLayoutsSectionName, GEditorIni);
		EditorLayoutsSectionName.TrimStartAndEndInline();
		if (EditorLayoutsSectionName.IsEmpty())
		{
			EditorLayoutsSectionName = DefaultEditorLayoutsSectionName;
		}
	}

	return *EditorLayoutsSectionName;
}

static bool GetConfigString(const TCHAR* Key, FString& Value, const FString& Filename, bool bAllowFallback = true)
{
	if (!GConfig->GetString(GetEditorLayoutsSectionName(), Key, Value, Filename))
	{
		if (bAllowFallback && GetEditorLayoutsSectionName() != DefaultEditorLayoutsSectionName)
		{
			return GConfig->GetString(DefaultEditorLayoutsSectionName, Key, Value, Filename);
		}
		return false;
	}

	// Old custom layouts may have an incorrect name, so account for that here.
	if (const bool bIsUserLayout = Filename.StartsWith(FPaths::EngineUserLayoutDir()))
	{
		// Inject corrected label into result
		FRegexPattern Pattern(TEXT("\"\\w+\"")); // words between commas
		FRegexMatcher Matcher(Pattern, Value);

		int32 MatchStart = INDEX_NONE;
		int32 MatchEnd = INDEX_NONE;

		// We want the last match, so run until none found and use the last result
		while (Matcher.FindNext())
		{
			MatchStart = Matcher.GetMatchBeginning();
			MatchEnd = Matcher.GetMatchEnding();
		}

		if (MatchStart != INDEX_NONE && MatchEnd != INDEX_NONE)
		{
			Value = Value.Left(MatchStart + 1) + FString(WriteToString<16>(UE::String::PascalCase(FPaths::GetBaseFilename(Filename)))) + Value.Right(Value.Len() - MatchEnd + 1);
		}
	}

	return true;
}

static bool GetConfigSection(TArray<FString>& Result, const FString& Filename, bool bAllowFallback = true)
{
	if (!GConfig->GetSection(GetEditorLayoutsSectionName(), Result, Filename))
	{
		if (bAllowFallback && GetEditorLayoutsSectionName() != DefaultEditorLayoutsSectionName)
		{
			return GConfig->GetSection(DefaultEditorLayoutsSectionName, Result, Filename);
		}
		return false;
	}
	return true;
}

static const FConfigSection* GetConfigSectionPrivate(const bool Force, const FString& Filename, bool bAllowFallback = true)
{
	const FConfigSection* FoundSection = GConfig->GetSection(GetEditorLayoutsSectionName(), /*Force*/false, Filename);
	if (FoundSection)
	{
		if (bAllowFallback && GetEditorLayoutsSectionName() != DefaultEditorLayoutsSectionName)
		{
			FoundSection = GConfig->GetSection(DefaultEditorLayoutsSectionName, /*Force*/false, Filename);
		}
	}
	return FoundSection;
}

static FString PrepareLayoutStringForIni(const FString& LayoutString)
{
	// Have to store braces as parentheses due to braces causing ini issues
	return LayoutString
		.Replace(TEXT("{"), TEXT("("))
		.Replace(TEXT("}"), TEXT(")"))
		.Replace(TEXT("\r"), TEXT(""))
		.Replace(TEXT("\n"), TEXT(""))
		.Replace(TEXT("\t"), TEXT(""));
}

static FString GetLayoutStringFromIni(const FString& LayoutString, bool& bOutIsJson)
{
	if (FTextStringHelper::IsComplexText(*LayoutString))
	{
		bOutIsJson = false;
		return LayoutString;
	}

	// Revert parenthesis to braces, from ini readable to Json readable
	bOutIsJson = true;
	return LayoutString
		.Replace(TEXT("("), TEXT("{"))
		.Replace(TEXT(")"), TEXT("}"))
		// Consider line terminators from any supported platform, as the ini may have been saved on a different platform.
		.Replace(TEXT("\\\n"), LINE_TERMINATOR)
		.Replace(TEXT("\\\r\n"), LINE_TERMINATOR);
}


const FString& FLayoutSaveRestore::GetAdditionalLayoutConfigIni()
{
	static const FString IniSectionAdditionalConfig = "SlateAdditionalLayoutConfig";
	return IniSectionAdditionalConfig;
}

static TSharedPtr<FJsonObject> ConvertSectionToJson(const TArray<FString>& SectionStrings)
{
	TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();

	for (const FString& SectionPair : SectionStrings)
	{
		FString Key, Value;
		if (SectionPair.Split(TEXT("="), &Key, &Value))
		{
			bool bIsJson = true;
			Value = GetLayoutStringFromIni(Value, bIsJson);

			if (bIsJson)
			{
				TSharedPtr<FJsonObject> ChildObject = MakeShared<FJsonObject>();
				TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Value);
				if (FJsonSerializer::Deserialize(Reader, ChildObject))
				{
					RootObject->SetObjectField(Key, ChildObject);
				}
				else
				{
					bIsJson = false;
				}
			}

			if (!bIsJson)
			{
				RootObject->SetStringField(Key, Value);
			}
		}
	}

	return RootObject;
}

static FString GetLayoutJsonFileName(const FString& InConfigFileName)
{
	const FString JsonFileName = FPaths::GetBaseFilename(InConfigFileName) + TEXT(".json");
#ifdef UE_SAVED_DIR_OVERRIDE
	const FString UserSettingsPath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), TEXT(UE_STRINGIZE(UE_SAVED_DIR_OVERRIDE)), TEXT("Editor"), JsonFileName);
#else
	const FString UserSettingsPath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), FApp::GetEpicProductIdentifier(), TEXT("Editor"), JsonFileName);
#endif
	return UserSettingsPath;
}

static TSharedPtr<FJsonObject> LoadJsonFile(const FString& InFileName)
{
	TSharedPtr<FJsonObject> JsonObject;

	FString JsonContents;
	if (FFileHelper::LoadFileToString(JsonContents, *InFileName))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonContents);
		FJsonSerializer::Deserialize(Reader, JsonObject);
	}

	return JsonObject;
}

static bool SaveJsonFile(const FString& InFileName, TSharedPtr<FJsonObject> JsonObject)
{
	FString NewJsonContents;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&NewJsonContents);
	if (FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer))
	{
		return FFileHelper::SaveStringToFile(NewJsonContents, *InFileName);
	}

	return false;
}

static void SaveLayoutToJson(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InLayoutToSave)
{
	const FString UserSettingsPath = GetLayoutJsonFileName(InConfigFileName);

	TSharedPtr<FJsonObject> AllLayoutsObject = LoadJsonFile(UserSettingsPath);
	if (!AllLayoutsObject.IsValid())
	{
		// doesn't exist
		AllLayoutsObject = MakeShared<FJsonObject>();
	}

	AllLayoutsObject->SetObjectField(InLayoutToSave->GetLayoutName().ToString(), InLayoutToSave->ToJson());

	SaveJsonFile(UserSettingsPath, AllLayoutsObject);
}

static bool LoadLayoutFromJson(const FString& InConfigFileName, const FString& InLayoutName, TSharedPtr<FTabManager::FLayout>& OutLayout)
{
	const FString LayoutJsonFile = GetLayoutJsonFileName(InConfigFileName);

	TSharedPtr<FJsonObject> JsonObject = LoadJsonFile(LayoutJsonFile);
	if (!JsonObject.IsValid())
	{
		return false;
	}

	const TSharedPtr<FJsonObject>* LayoutJson = nullptr;
	if (JsonObject->TryGetObjectField(InLayoutName, LayoutJson))
	{
		OutLayout = FTabManager::FLayout::NewFromJson(*LayoutJson);
		return true;
	}

	return false;
}

void FLayoutSaveRestore::SaveToConfig( const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InLayoutToSave )
{
	if (FGlobalTabmanager::Get()->CanSavePersistentLayouts())
	{
		// Only save to config if it's not the FTabManager::FLayout::NullLayout
		if (InLayoutToSave->GetLayoutName() != FTabManager::FLayout::NullLayout->GetLayoutName())
		{
			const FString LayoutAsString = PrepareLayoutStringForIni(InLayoutToSave->ToString());
			GConfig->SetString(GetEditorLayoutsSectionName(), *InLayoutToSave->GetLayoutName().ToString(), *LayoutAsString, InConfigFileName);

			SaveLayoutToJson(InConfigFileName, InLayoutToSave);
		}
	}
}

TSharedRef<FTabManager::FLayout> FLayoutSaveRestore::LoadFromConfig(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InDefaultLayout,
	const EOutputCanBeNullptr InPrimaryAreaOutputCanBeNullptr)
{
	TArray<FString> DummyArray;
	return FLayoutSaveRestore::LoadFromConfigPrivate(InConfigFileName, InDefaultLayout, InPrimaryAreaOutputCanBeNullptr, false, DummyArray);
}


TSharedRef<FTabManager::FLayout> FLayoutSaveRestore::LoadFromConfig(const FString& InConfigFileName,
	const TSharedRef<FTabManager::FLayout>& InDefaultLayout, const EOutputCanBeNullptr InPrimaryAreaOutputCanBeNullptr, TArray<FString>& OutRemovedOlderLayoutVersions)
{
	return FLayoutSaveRestore::LoadFromConfigPrivate(InConfigFileName, InDefaultLayout, InPrimaryAreaOutputCanBeNullptr, true, OutRemovedOlderLayoutVersions);
}


TSharedRef<FTabManager::FLayout> FLayoutSaveRestore::LoadFromConfigPrivate(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InDefaultLayout,
	const EOutputCanBeNullptr InPrimaryAreaOutputCanBeNullptr, const bool bInRemoveOlderLayoutVersions, TArray<FString>& OutRemovedOlderLayoutVersions)
{
	const FString LayoutNameString = InDefaultLayout->GetLayoutName().ToString();

	TSharedPtr<FTabManager::FLayout> UserLayout;

	// First try to load from JSON, then INI if that does not exist
	if (!LoadLayoutFromJson(InConfigFileName, LayoutNameString, UserLayout))
	{
		FString IniLayoutString;
		// If the Key (InDefaultLayout->GetLayoutName()) already exists in the section GetEditorLayoutsSectionName() of the file InConfigFileName, try to load the layout from that file
		GetConfigString(*LayoutNameString, IniLayoutString, InConfigFileName);
		bool bIsJson = false;
		UserLayout = FTabManager::FLayout::NewFromString( GetLayoutStringFromIni( IniLayoutString, bIsJson ) );
	}

	if (UserLayout.IsValid())
	{
		// Return UserLayout in the following 2 cases:
		// - By default (PrimaryAreaOutputCanBeNullptr = Never or IfNoTabValid)
		// - For the case of PrimaryAreaOutputCanBeNullptr = IfNoOpenTabValid, only if the primary area has at least a valid open tab
		TSharedPtr<FTabManager::FArea> PrimaryArea = UserLayout->GetPrimaryArea().Pin();
		if (PrimaryArea.IsValid() && (InPrimaryAreaOutputCanBeNullptr != EOutputCanBeNullptr::IfNoOpenTabValid || FGlobalTabmanager::Get()->HasValidOpenTabs(PrimaryArea.ToSharedRef())))
		{
			return UserLayout.ToSharedRef();
		}
	}
	// If the file layout could not be loaded and the caller wants to remove old fields
	else if (bInRemoveOlderLayoutVersions)
	{
		// If File and Section exist
		if (const FConfigSection* ConfigSection = GetConfigSectionPrivate(/*Force*/false, InConfigFileName))
		{
			// If Key does not exist (i.e., Section does but not contain that Key)
			if (!ConfigSection->Find(*LayoutNameString))
			{
				// Create LayoutKeyToRemove
				FString LayoutKeyToRemove;
				for (int32 Index = LayoutNameString.Len() - 1; Index > 0; --Index)
				{
					if (LayoutNameString[Index] != TCHAR('.') && (LayoutNameString[Index] < TCHAR('0') || LayoutNameString[Index] > TCHAR('9')))
					{
						LayoutKeyToRemove = LayoutNameString.Left(Index+1);
						break;
					}
				}
				// Look for older versions of this Key
				OutRemovedOlderLayoutVersions.Empty();
				for (const auto& SectionPair : *ConfigSection/*->ArrayOfStructKeys*/)
				{
					FString CurrentKey = SectionPair.Key.ToString();
					if (CurrentKey.Len() > LayoutKeyToRemove.Len() && CurrentKey.Left(LayoutKeyToRemove.Len()) == LayoutKeyToRemove)
					{
						OutRemovedOlderLayoutVersions.Emplace(std::move(CurrentKey));
					}
				}
				// Remove older versions of this Key
				for (const FString& KeyToRemove : OutRemovedOlderLayoutVersions)
				{
					if (GConfig->RemoveKey(GetEditorLayoutsSectionName(), *KeyToRemove, InConfigFileName))
					{
						UE_LOGF(LogLayoutService, Warning, "While key \"%ls\" was not found, and older version exists (key \"%ls\"). This means section \"%ls\" was"
							" created with a previous version of UE and is no longer compatible. The old key has been removed and updated with the new one.",
							*LayoutNameString, *KeyToRemove, GetEditorLayoutsSectionName());
					}
					else
					{
						UE_LOGF(LogLayoutService, Warning, "Could not remove old layout key %ls because it exists in the fallback section %ls instead of the current section %ls",
							*KeyToRemove, DefaultEditorLayoutsSectionName, GetEditorLayoutsSectionName());
					}
				}
			}
		}
	}

	return InDefaultLayout;
}

void FLayoutSaveRestore::SaveSectionToConfig(const FString& InConfigFileName, const FString& InSectionName, const FText& InSectionValue)
{
	SaveSectionToConfig(InConfigFileName, InSectionName, InSectionValue.ToString());
}

void FLayoutSaveRestore::SaveSectionToConfig(const FString& InConfigFileName, const FString& InSectionName, const FString& InSectionValue)
{
	GConfig->SetString(GetEditorLayoutsSectionName(), *InSectionName, *InSectionValue, InConfigFileName);

	const FString JsonFileName = GetLayoutJsonFileName(InConfigFileName);
	TSharedPtr<FJsonObject> JsonObject = LoadJsonFile(JsonFileName);
	if (JsonObject.IsValid())
	{
		JsonObject->SetStringField(InSectionName, *InSectionValue);
		SaveJsonFile(JsonFileName, JsonObject);
	}
}

FText FLayoutSaveRestore::LoadSectionFromConfig(const FString& InConfigFileName, const FString& InSectionName, const bool bIsOptional)
{
	FString ValueString;
	const FString JsonFileName = GetLayoutJsonFileName(InConfigFileName);
	TSharedPtr<FJsonObject> JsonObject = LoadJsonFile(JsonFileName);
	if (JsonObject.IsValid())
	{
		// If optional, we check if the field exists first
		if (!bIsOptional
			|| (bIsOptional && JsonObject->HasTypedField(InSectionName, EJson::String)))
		{
			ValueString = JsonObject->GetStringField(InSectionName);
		}
	}

	if (ValueString.IsEmpty())
	{
		GetConfigString(*InSectionName, ValueString, InConfigFileName);
	}

	FText ValueText;
	FTextStringHelper::ReadFromBuffer(*ValueString, ValueText, GetEditorLayoutsSectionName());

	return ValueText;
}

bool FLayoutSaveRestore::DuplicateConfig(const FString& SourceConfigFileName, const FString& TargetConfigFileName)
{
	const bool bShouldReplace = true;
	const bool bCopyEvenIfReadOnly = true;
	const bool bCopyAttributes = false; // If true, we could copy the read-only flag of DefaultLayout.ini and cause save/load to stop working

	if (IFileManager::Get().Copy(*TargetConfigFileName, *SourceConfigFileName, bShouldReplace, bCopyEvenIfReadOnly, bCopyAttributes) == COPY_Fail)
	{
		return false;
	}


	TSharedPtr<FJsonObject> RootObject;

	// The right layout config to be duplicated is already in a json file
	// Let's load it and use that as source for duplication
	if (TSharedPtr<FJsonObject> SourceJson = LoadJsonFile(GetLayoutJsonFileName(SourceConfigFileName)))
	{
		RootObject = SourceJson;
	}
	else
	{
		// If we end up here, there is no json file for the specified ini layout:
		// convert this layout to a JSON file
		TArray<FString> SectionPairs;
		GetConfigSection(SectionPairs, TargetConfigFileName);
		RootObject = ConvertSectionToJson(SectionPairs);
	}

	const FString TargetJsonFilename = GetLayoutJsonFileName(TargetConfigFileName);
	SaveJsonFile(TargetJsonFilename, RootObject);

	return true;
}

void FLayoutSaveRestore::MigrateConfig( const FString& OldConfigFileName, const FString& NewConfigFileName )
{
	TArray<FString> OldSectionStrings;

	// check whether any layout configuration needs to be migrated
	if (!GetConfigSection(OldSectionStrings, OldConfigFileName) || (OldSectionStrings.Num() == 0))
	{
		return;
	}

	TArray<FString> NewSectionStrings;

	// migrate old configuration if a new layout configuration does not yet exist
	if (!GConfig->GetSection(GetEditorLayoutsSectionName(), NewSectionStrings, NewConfigFileName) || (NewSectionStrings.Num() == 0))
	{
		FString Key, Value;

		for (const FString& SectionString : OldSectionStrings)
		{
			if (SectionString.Split(TEXT("="), &Key, &Value))
			{
				GConfig->SetString(GetEditorLayoutsSectionName(), *Key, *Value, NewConfigFileName);
			}
		}
	}

	// remove old configuration
	if (!GConfig->EmptySection(GetEditorLayoutsSectionName(), OldConfigFileName))
	{
		UE_LOGF(LogLayoutService, Warning, "Could not remove old layout in %ls because it is using the fallback section %ls instead of the current section %ls. New layout file will still be created.",
			*OldConfigFileName, DefaultEditorLayoutsSectionName, GetEditorLayoutsSectionName());
	}
	GConfig->Flush(false, OldConfigFileName);
	GConfig->Flush(false, NewConfigFileName);

	// migrate layout to JSON as well
	const FString NewLayoutJsonFileName = GetLayoutJsonFileName(NewConfigFileName);
	TSharedPtr<FJsonObject> JsonObject = LoadJsonFile(NewLayoutJsonFileName);
	if (!JsonObject.IsValid() || JsonObject->Values.Num() == 0)
	{
		TSharedPtr<FJsonObject> RootObject = ConvertSectionToJson(OldSectionStrings);
		SaveJsonFile(NewLayoutJsonFileName, RootObject);
	}
}


bool FLayoutSaveRestore::IsValidConfig(const FString& InConfigFileName, bool bAllowFallback)
{
	if (GConfig->DoesSectionExist(GetEditorLayoutsSectionName(), *InConfigFileName))
	{
		return true;
	}
	else if (bAllowFallback && GConfig->DoesSectionExist(DefaultEditorLayoutsSectionName, *InConfigFileName))
	{
		return true;
	}

	const FString JsonFileName = GetLayoutJsonFileName(InConfigFileName);
	return IFileManager::Get().FileExists(*JsonFileName);
}
