// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingArea.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Framework/Docking/SDockingTarget.h"
#include "Framework/Docking/FDockingDragOperation.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Docking/STabSidebar.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDockingArea"

namespace UE::Slate::Private
{
	void SetAreaNodeWindowPlacementValues(TSharedPtr<FTabManager::FArea>& AreaToSet, const SWindow& InWindow)
	{
		bool bIsWindowMaximized = InWindow.IsWindowMaximized();
		FSlateRect WindowRect;
		if (bIsWindowMaximized || InWindow.IsWindowMinimized() || !InWindow.IsVisible())
		{
			// Confusing name but does use the restore size when there is one before the actual size of the window.
			WindowRect = InWindow.GetNonMaximizedRectInScreen();
		}
		else
		{
			WindowRect = InWindow.GetRectInScreen();
		}

		// In order to restore SWindows to their correct size, we need to save areas as 
		// client area sizes, since the Constructor for SWindow uses a client size
		if (!InWindow.HasOSWindowBorder())
		{
			const FMargin WindowBorder = InWindow.GetWindowBorderSize();
			WindowRect.Right -= WindowBorder.Left + WindowBorder.Right;
			WindowRect.Bottom -= WindowBorder.Top + WindowBorder.Bottom;
		}

		// Remove DPI Scale when saving layout so that the saved size is DPI independent
		float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowRect.Left, WindowRect.Top);

		if (AreaToSet)
		{
			AreaToSet->SetWindowSize(WindowRect.GetSize() / DPIScale);
		}
		else
		{
			AreaToSet = FTabManager::NewArea(WindowRect.GetSize() / DPIScale);
		}
		AreaToSet->SetWindow(FVector2D(WindowRect.Left, WindowRect.Top) / DPIScale, bIsWindowMaximized);
	}
}


void SDockingArea::Construct( const FArguments& InArgs, const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FArea>& PersistentNode )
{
	MyTabManager = InTabManager;
	InTabManager->GetPrivateApi().OnDockAreaCreated( SharedThis(this) );

	bManageParentWindow = InArgs._ShouldManageParentWindow;
	bIsOverlayVisible = false;

	bIsCenterTargetVisible = false;

	const TAttribute<EVisibility> TargetCrossVisibility = TAttribute<EVisibility>(SharedThis(this), &SDockingArea::TargetCrossVisibility);
	const TAttribute<EVisibility> TargetCrossCenterVisibility = TAttribute<EVisibility>(SharedThis(this), &SDockingArea::TargetCrossCenterVisibility);

	const TSharedRef<SOverlay> SidebarDrawersOverlay = SNew(SOverlay);

	TSharedRef<SWidget> NoOpenTabsWidget =
			SNew(SHorizontalBox)
			.Visibility_Lambda([this]()
			{
				// This text is visible when we have no child tabs
				return GetNumTabs() == 0 ? EVisibility::Visible : EVisibility::Collapsed;
			})
			
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoTabsDockedText", "This asset editor has no docked tabs."))
				.TextStyle( FAppStyle::Get(), "HintText")
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::GetBrush("Icons.Help"))
				.ToolTipText(LOCTEXT("NoTabsDockedTooltip", "To recover your tabs, you can reopen them from the Window menu, or drag and drop them back from floating windows.\n"
												   "You can also reset your editor layout completely from the Window > Load Layout menu, but this affects all editor windows."))
			];

	// In DockSplitter mode we just act as a thin shell around a Splitter widget
	this->ChildSlot
	[
		SNew(SOverlay)
		.Visibility( EVisibility::SelfHitTestInvisible )
		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(LeftSidebar, STabSidebar, SidebarDrawersOverlay)
				.Location(ESidebarLocation::Left)
				.Visibility(EVisibility::Collapsed)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SAssignNew(Splitter, SSplitter)
					.Orientation(PersistentNode->GetOrientation())
				]
				+SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					NoOpenTabsWidget
				]
				
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(RightSidebar, STabSidebar, SidebarDrawersOverlay)
				.Location(ESidebarLocation::Right)
				.Visibility(EVisibility::Collapsed)
			]
		]
		
		+ SOverlay::Slot()
		// Houses the minimize, maximize, restore, and icon
		.Expose(WindowControlsArea)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		
		+SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		[
			SNew(SDockingTarget)
			.Visibility(TargetCrossVisibility)
			.OwnerNode( SharedThis(this) )
			.DockDirection( SDockingNode::LeftOf )
		]
		+SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		[
			SNew(SDockingTarget)
			.Visibility(TargetCrossVisibility)
			.OwnerNode( SharedThis(this) )
			.DockDirection( SDockingNode::RightOf )
		]
		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SDockingTarget)
			.Visibility(TargetCrossVisibility)
			.OwnerNode( SharedThis(this) )
			.DockDirection( SDockingNode::Above )
		]
		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Bottom)
		[
			SNew(SDockingTarget)
			.Visibility(TargetCrossVisibility)
			.OwnerNode( SharedThis(this) )
			.DockDirection( SDockingNode::Below )
		]
		+SOverlay::Slot()
		[
			SNew(SDockingTarget)
			.Visibility(TargetCrossCenterVisibility)
			.OwnerNode( SharedThis(this) )
			.DockDirection( SDockingNode::Center )
		]
		+SOverlay::Slot()
		[
			SidebarDrawersOverlay
		]
	];



	bCleanUpUponTabRelocation = false;

	// If the owner window is set and bManageParentWindow is true, this docknode will close the window when its last tab is removed.
	if (InArgs._ParentWindow.IsValid())
	{
		SetParentWindow(InArgs._ParentWindow.ToSharedRef());
	}
	
		
	// Add initial content if it was provided
	if ( InArgs._InitialContent.IsValid() )
	{
		AddChildNode( InArgs._InitialContent.ToSharedRef() );
	}

	for (const FTabManager::FPanelDrawerTab& InactivePanelDrawerTab : PersistentNode->GetPanelDrawerInactiveTabs())
	{
		TSharedRef<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = MakeShared<UE::Slate::Private::FPanelDrawerData>();
		PanelDrawerData->Size = InactivePanelDrawerTab.Size;
		InactivePanelDrawerTabs.Add(InactivePanelDrawerTab.TabId, MoveTemp(PanelDrawerData));
	}
}

void SDockingArea::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		if ( DragDropOperation->CanDockInNode(SharedThis(this), FDockingDragOperation::DockingViaTarget) )
		{
			ShowCross();
		}		
	}

}

void SDockingArea::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FDockingDragOperation>().IsValid() )
	{
		HideCross();
	}
}

FReply SDockingArea::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FDockingDragOperation>().IsValid() )
	{
		HideCross();
	}

	return FReply::Unhandled();
}

FReply SDockingArea::OnUserAttemptingDock( SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		if (Direction == Center)
		{
			//check(Children.Num() <= 1);

			TSharedRef<SDockingTabStack> NewStack = SNew(SDockingTabStack, FTabManager::NewStack());
			AddChildNode( NewStack );
			NewStack->OpenTab( DragDropOperation->GetTabBeingDragged().ToSharedRef() );
		}
		else
		{
			DockFromOutside( Direction, DragDropEvent );
		}		

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void SDockingArea::OnTabFoundNewHome( const TSharedRef<SDockTab>& RelocatedTab, const TSharedRef<SWindow>& NewOwnerWindow )
{
	HideCross();

	// The last tab has been successfully relocated elsewhere.
	// Destroy this window.
	TSharedPtr<SWindow> ParentWindow = GetParentWindow();
	if (ParentWindow.IsValid() && bManageParentWindow && bCleanUpUponTabRelocation && ParentWindow != NewOwnerWindow)
	{
		ParentWindow->SetRequestDestroyWindowOverride( FRequestDestroyWindowOverride() );
		ParentWindow->RequestDestroyWindow();
	}
}

TSharedPtr<SDockingArea> SDockingArea::GetDockArea()
{
	return SharedThis(this);
}

TSharedPtr<const SDockingArea> SDockingArea::GetDockArea() const
{
	return SharedThis(this);
}

TSharedPtr<SWindow> SDockingArea::GetParentWindow() const
{
	return ParentWindowPtr.IsValid() ? ParentWindowPtr.Pin() : TSharedPtr<SWindow>();
}

void SDockingArea::ShowCross()
{
	bIsOverlayVisible = true;
}

void SDockingArea::HideCross()
{
	bIsOverlayVisible = false;
}

void SDockingArea::CleanUp( ELayoutModification RemovalMethod )
{
	const ECleanupRetVal CleanupResult = CleanUpNodes();
	
	if ( CleanupResult != VisibleTabsUnderNode )
	{
		bIsCenterTargetVisible = true;
		// We may have a window to manage.
		TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin();
		if ( bManageParentWindow && ParentWindow.IsValid() )
		{
			if (RemovalMethod == TabRemoval_Closed)
			{
				MyTabManager.Pin()->GetPrivateApi().OnDockAreaClosing( SharedThis(this) );
				ParentWindow->RequestDestroyWindow();
			}
			else if ( RemovalMethod == TabRemoval_DraggedOut )
			{
				// We can't actually destroy this due to limitations of some platforms.
				// Just hide the window. We will destroy it when the drag and drop is done.
				bCleanUpUponTabRelocation = true;
				ParentWindow->HideWindow();
				MyTabManager.Pin()->GetPrivateApi().OnDockAreaClosing( SharedThis(this) );
			}
		}
	}
	else
	{
		bIsCenterTargetVisible = false;
	}

	// In some cases a dock area will control the window,
	// and we need to move some of the tabs out of the way
	// to make room for window chrome.
	UpdateWindowChromeAndSidebar();
}

void SDockingArea::SetParentWindow( const TSharedRef<SWindow>& NewParentWindow )
{
	if( bManageParentWindow )
	{
		NewParentWindow->SetRequestDestroyWindowOverride( FRequestDestroyWindowOverride::CreateSP( this, &SDockingArea::OnOwningWindowBeingDestroyed ) );
	}

	// Even though we don't manage the parent window's lifetime, we are still responsible for making its window chrome.
	{
		TSharedPtr<IWindowTitleBar> TitleBar;

		FWindowTitleBarArgs Args(NewParentWindow);
		Args.CenterContent = SNullWidget::NullWidget;
		Args.CenterContentAlignment = HAlign_Fill;
		Args.CloseButtonToolTipText = NewParentWindow->GetWindowCloseButtonToolTipText();

		TSharedRef<SWidget> TitleBarWidget = FSlateApplication::Get().MakeWindowTitleBar(Args, TitleBar);
		(*WindowControlsArea)
		[
			TitleBarWidget
		];

		NewParentWindow->SetTitleBar(TitleBar);
	}

	ParentWindowPtr = NewParentWindow;
	NewParentWindow->GetOnWindowActivatedEvent().AddSP(this, &SDockingArea::OnOwningWindowActivated);
}

TSharedPtr<FTabManager::FLayoutNode> SDockingArea::GatherPersistentLayout() const
{
	// Assume that all the nodes were dragged out, and there's no meaningful layout data to be gathered.
	bool bHaveLayoutData = false;

	TSharedPtr<FTabManager::FArea> PersistentNode;

	TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin();
	if ( ParentWindow.IsValid() && bManageParentWindow )
	{
		UE::Slate::Private::SetAreaNodeWindowPlacementValues(PersistentNode, *ParentWindow.Get());
	}
	else
	{
		// An area without a window persists because it must be a primary area.
		// Those must always be restored, even if they are empty.
		PersistentNode = FTabManager::NewPrimaryArea();
		bHaveLayoutData = true;
	}	

	PersistentNode->SetOrientation( this->GetOrientation() );

	for (int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		TSharedPtr<FTabManager::FLayoutNode> PersistentChild = Children[ChildIndex]->GatherPersistentLayout();
		if ( PersistentChild.IsValid() )
		{
			bHaveLayoutData = true;
			PersistentNode->Split( PersistentChild.ToSharedRef() );
		}
	}

	// PanelDrawer layout data
	{
		FTabManager::FPanelDrawerTab PanelDrawerTab;
		if (const TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
		{
			if (const TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = PanelDrawer->GetHostedPanelDrawerData())
			{
				if (PanelDrawerData->HostedTab)
				{
					// Only store the nomad tab in the area that manager their window
					ETabRole HostedTabRole = PanelDrawerData->HostedTab->GetTabRole();
					if ((HostedTabRole == ETabRole::NomadTab && GetParentWindow())
						|| (HostedTabRole != ETabRole::NomadTab && !GetParentWindow()))
					{
						PanelDrawerTab.TabId = PanelDrawerData->HostedTab->GetLayoutIdentifier();
						const FTabManager::FPanelDrawerSize& Size = PanelDrawerData->Size;
						PanelDrawerTab.Size = Size;
					}
				}
			}
		}

		if (PanelDrawerTab.TabId.TabType.IsNone())
		{
			if (HiddenPanelDrawerTabToReopenOnRestore && HiddenPanelDrawerTabToReopenOnRestore->HostedTab)
			{
				PanelDrawerTab.TabId = HiddenPanelDrawerTabToReopenOnRestore->HostedTab->GetLayoutIdentifier();
				const FTabManager::FPanelDrawerSize& Size = HiddenPanelDrawerTabToReopenOnRestore->Size;
				PanelDrawerTab.Size = Size;
			}
		}

		if (!PanelDrawerTab.TabId.TabType.IsNone())
		{
			PersistentNode->SetPanelDrawerActiveTab(MoveTemp(PanelDrawerTab));
		}
	}

	for (const TPair<FTabId, TSharedPtr<UE::Slate::Private::FPanelDrawerData>>& Pair : InactivePanelDrawerTabs)
	{
		FTabManager::FPanelDrawerTab PanelDrawerTab;
		PanelDrawerTab.TabId = Pair.Key;
		const FTabManager::FPanelDrawerSize& Size = Pair.Value->Size;
		PanelDrawerTab.Size = Size;
		PersistentNode->AddPanelDrawerInactiveTab(MoveTemp(PanelDrawerTab));
	}

	if( !bHaveLayoutData )
	{
		PersistentNode.Reset();
	}

	return PersistentNode;
}

TSharedRef<FTabManager> SDockingArea::GetTabManager() const
{
	return MyTabManager.Pin().ToSharedRef();
}

ESidebarLocation SDockingArea::AddTabToSidebar(TSharedRef<SDockTab> TabToAdd)
{
	ESidebarLocation Location = ESidebarLocation::None;

	if(ensure(bCanHaveSidebar))
	{
		// Determine which sidebar to add the tab to. Testing is done in absolute desktop space because we need the tab and the area to be in the same space
		FSlateRect AreaRect = GetPaintSpaceGeometry().GetLayoutBoundingRect();
		FSlateRect LeftRect = FSlateRect(AreaRect.Left, AreaRect.Top, AreaRect.Right / 2, AreaRect.Bottom);
		FSlateRect RightRect = FSlateRect(AreaRect.Right / 2, AreaRect.Top, AreaRect.Right, AreaRect.Bottom);

		FSlateRect TabRect = TabToAdd->GetPaintSpaceGeometry().GetLayoutBoundingRect();

		bool bOverlapsLeft = false;
		FSlateRect LeftOverlap = LeftRect.IntersectionWith(TabRect, bOverlapsLeft);

		bool bOverlapsRight = false;
		FSlateRect RightOverlap = RightRect.IntersectionWith(TabRect, bOverlapsRight);

		if (bOverlapsLeft && bOverlapsRight)
		{
			if (LeftOverlap.GetArea() > RightOverlap.GetArea())
			{
				// Left side
				Location = ESidebarLocation::Left;
			}
			else
			{
				// Right side
				Location = ESidebarLocation::Right;
			}
		}
		else if (bOverlapsLeft)
		{
			// left side
			Location = ESidebarLocation::Left;
		}
		else
		{
			ensure(bOverlapsRight);
			// right side
			Location = ESidebarLocation::Right;
		}

		if (Location == ESidebarLocation::Left)
		{
			LeftSidebar->AddTab(TabToAdd);
		}
		else
		{
			RightSidebar->AddTab(TabToAdd);
		}
	}
	

	return Location;

}

bool SDockingArea::RestoreTabFromSidebar(TSharedRef<SDockTab> TabToRemove)
{
	return LeftSidebar->RestoreTab(TabToRemove) || RightSidebar->RestoreTab(TabToRemove);
}

bool SDockingArea::IsTabInSidebar(TSharedRef<SDockTab> Tab) const
{
	return LeftSidebar->ContainsTab(Tab) || RightSidebar->ContainsTab(Tab);
}

bool SDockingArea::RemoveTabFromSidebar(TSharedRef<SDockTab> Tab)
{
	return LeftSidebar->RemoveTab(Tab) || RightSidebar->RemoveTab(Tab);
}

bool SDockingArea::TryOpenSidebarDrawer(TSharedRef<SDockTab> TabToModify) const
{
	return LeftSidebar->TryOpenSidebarDrawer(TabToModify) || RightSidebar->TryOpenSidebarDrawer(TabToModify);
}

bool SDockingArea::TryDismissSidebarDrawer(TSharedRef<SDockTab> TabToModify) const
{
	return LeftSidebar->TryDismissSidebarDrawer(TabToModify) || RightSidebar->TryDismissSidebarDrawer(TabToModify);
}

bool SDockingArea::TryToggleSidebarDrawer(TSharedRef<SDockTab> TabToModify) const
{
	return LeftSidebar->TryToggleSidebarDrawer(TabToModify) || RightSidebar->TryToggleSidebarDrawer(TabToModify);
}

void SDockingArea::AddSidebarTabsFromRestoredLayout(const FSidebarTabLists& SidebarTabs)
{
	for (const TSharedRef<SDockTab>& Tab : SidebarTabs.LeftSidebarTabs)
	{
		LeftSidebar->AddTab(Tab);
	}

	for (const TSharedRef<SDockTab>& Tab : SidebarTabs.RightSidebarTabs)
	{
		RightSidebar->AddTab(Tab);
	}
}

TArray<TSharedRef<SDockTab>> SDockingArea::GetAllSidebarTabs() const
{
	TArray<TSharedRef<SDockTab>> AllSidebarTabs;
	AllSidebarTabs.Append(LeftSidebar->GetAllTabs());
	AllSidebarTabs.Append(RightSidebar->GetAllTabs());

	return AllSidebarTabs;
}

bool SDockingArea::HostTabIntoPanelDrawer(const TSharedRef<SDockTab>& InTab)
{
	TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin();
	if (!PanelDrawer)
	{
		return false;
	}

	TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = PanelDrawer->GetHostedPanelDrawerData();
	if (PanelDrawerData && PanelDrawerData->HostedTab)
	{
		InactivePanelDrawerTabs.Add(PanelDrawerData->HostedTab->GetLayoutIdentifier(), MoveTemp(PanelDrawerData));
	}


	if (!InactivePanelDrawerTabs.RemoveAndCopyValue(InTab->GetLayoutIdentifier(), PanelDrawerData))
	{
		PanelDrawerData = MakeShared<UE::Slate::Private::FPanelDrawerData>();
	}

	PanelDrawerData->HostedTab = InTab;
	InTab->SetTabManager(GetTabManager());
	InTab->SetParentDockingArea(AsSharedSubobject(this));
	HiddenPanelDrawerTabToReopenOnRestore.Reset();

	constexpr bool bWithAnimation = true;
	PanelDrawer->OpenPanel(bWithAnimation, PanelDrawerData.ToSharedRef());

	return true;
}

void SDockingArea::ClosePanelDrawer()
{
	if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = PanelDrawer->GetHostedPanelDrawerData();
		if (PanelDrawerData && PanelDrawerData->HostedTab)
		{
			InactivePanelDrawerTabs.Add(PanelDrawerData->HostedTab->GetLayoutIdentifier(), MoveTemp(PanelDrawerData));
		}

		constexpr bool bWithAnimation = true;
		PanelDrawer->ClosePanel(bWithAnimation);

		HiddenPanelDrawerTabToReopenOnRestore.Reset();
	}
}

TSharedPtr<SDockTab> SDockingArea::ClosePanelDrawerForTransfer()
{
	TSharedPtr<SDockTab> HostedTab;
	if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = PanelDrawer->GetHostedPanelDrawerData();
		if (PanelDrawerData && PanelDrawerData->HostedTab)
		{
			// Purposely null the stored Hosted Tab
			HostedTab = MoveTemp(PanelDrawerData->HostedTab);
			InactivePanelDrawerTabs.Add(HostedTab->GetLayoutIdentifier(), MoveTemp(PanelDrawerData));
		}

		constexpr bool bWithAnimation = true;
		constexpr bool bForTransfer = true;
		PanelDrawer->ClosePanel(bWithAnimation, bForTransfer);

		HiddenPanelDrawerTabToReopenOnRestore.Reset();
	}

	return HostedTab;
}

bool SDockingArea::IsPanelDrawerOpen() const
{
	if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		return PanelDrawer->IsOpen();
	}
	return false;
}

bool SDockingArea::HasPanelDrawer() const
{
	return PanelDrawerPtr.IsValid();
}

TSharedPtr<SDockTab> SDockingArea::GetPanelDrawerSystemHostedTab(const FTabId& TabId) const
{
	if (HiddenPanelDrawerTabToReopenOnRestore && HiddenPanelDrawerTabToReopenOnRestore->HostedTab)
	{
		if (HiddenPanelDrawerTabToReopenOnRestore->HostedTab->GetLayoutIdentifier() == TabId)
		{
			return HiddenPanelDrawerTabToReopenOnRestore->HostedTab;
		}
	}

	if (const TSharedPtr<UE::Slate::Private::FPanelDrawerData>* PanelDrawerData = InactivePanelDrawerTabs.Find(TabId))
	{
		return (*PanelDrawerData)->HostedTab;
	}

	if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		if (TSharedPtr<SDockTab> HostedTab = PanelDrawer->GetHostedTab())
		{
			if (HostedTab->GetLayoutIdentifier() == TabId)
			{
				return HostedTab;
			}
		}
	}

	return {};
}

TSharedPtr<SDockTab> SDockingArea::GetPanelDrawerHostedTab() const
{
	// Handle the case of this local area hosting a nomad tab
	if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		// Note: Consider only returning the data when open if this cause some issue
		return PanelDrawer->GetHostedTab();
	}

	return {};
}

void SDockingArea::SetPanelDrawerArea(const TSharedPtr<UE::Slate::Private::SPanelDrawerArea>& InPanelDrawerArea)
{
	if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		if (!InPanelDrawerArea)
		{
			HiddenPanelDrawerTabToReopenOnRestore = PanelDrawer->GetHostedPanelDrawerData();
		}

		if (InPanelDrawerArea != PanelDrawer)
		{
			PanelDrawer->ClosePanel(false);
		}
	}

	PanelDrawerPtr = InPanelDrawerArea;

	if (InPanelDrawerArea)
	{
		bool bShouldRestore = true;
		if (TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = InPanelDrawerArea->GetHostedPanelDrawerData())
		{
			if (PanelDrawerData->HostedTab)
			{
				if (PanelDrawerData->HostedTab->GetTabRole() == ETabRole::NomadTab)
				{
					bShouldRestore = false;
					HiddenPanelDrawerTabToReopenOnRestore.Reset();
					PanelDrawerData->HostedTab->SetTabManager(GetTabManager());
					PanelDrawerData->HostedTab->SetParentDockingArea(AsSharedSubobject(this));
					InactivePanelDrawerTabs.Remove(PanelDrawerData->HostedTab->GetLayoutIdentifier());
				}
			}
		}

		if (bShouldRestore)
		{
			RestorePanelDrawerArea();
		}
	}
}

void SDockingArea::DetachPanelDrawerArea()
{
	PanelDrawerPtr.Reset();
}

bool SDockingArea::RestorePanelDrawerArea()
{
	return RestorePanelDrawerArea(TSharedPtr<UE::Slate::Private::SPanelDrawerArea>());
}

bool SDockingArea::RestorePanelDrawerArea(const TSharedPtr<UE::Slate::Private::SPanelDrawerArea>& InPanelDrawerAreaOverride)
{
	TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer;
	if (InPanelDrawerAreaOverride)
	{
		PanelDrawer = InPanelDrawerAreaOverride;
	}
	else
	{
		PanelDrawer = PanelDrawerPtr.Pin();
	}

	if (!PanelDrawer)
	{
		return false;
	}


	if (TSharedPtr<SDockingArea> MajorDockingArea = GetTopLevelDockingArea())
	{
		// This Area live under another area. Let the top level Area select the drawer content first.
		MajorDockingArea->SetPanelDrawerArea(PanelDrawer);
		if (PanelDrawer->IsOpen())
		{
			HiddenPanelDrawerTabToReopenOnRestore.Reset();
			return true;
		}
	}
	else if (!GetParentWindow())
	{
		return false;
	}

	// Consume the queued Restore
	TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = MoveTemp(HiddenPanelDrawerTabToReopenOnRestore);
	if (PanelDrawerData && PanelDrawerData->HostedTab)
	{
		PanelDrawerData->HostedTab->SetTabManager(GetTabManager());
		PanelDrawerData->HostedTab->SetParentDockingArea(AsSharedSubobject(this));
		InactivePanelDrawerTabs.Remove(PanelDrawerData->HostedTab->GetLayoutIdentifier());

		constexpr bool bWithAnimation = false;
		PanelDrawer->OpenPanel(bWithAnimation, PanelDrawerData.ToSharedRef());
		
		return true;
	}

	return false;
}

TMap<FTabId, TSharedRef<SDockTab>> SDockingArea::GetPanelDrawerKeepAliveTabs() const
{
	TMap<FTabId, TSharedRef<SDockTab>> Output;
	Output.Reserve(InactivePanelDrawerTabs.Num() + 1);

	if (TSharedPtr < UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
	{
		TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = PanelDrawer->GetHostedPanelDrawerData();
		if (PanelDrawerData && PanelDrawerData->HostedTab)
		{
			ETabRole HostedTabRole = PanelDrawerData->HostedTab->GetTabRole();
			// Only keep alive the nomad tab if we own the window
			if (HostedTabRole != ETabRole::NomadTab || GetParentWindow())
			{
				Output.Add(PanelDrawerData->HostedTab->GetLayoutIdentifier(), PanelDrawerData->HostedTab.ToSharedRef());
			}
		}
		else if (HiddenPanelDrawerTabToReopenOnRestore && HiddenPanelDrawerTabToReopenOnRestore->HostedTab)
		{
			Output.Add(HiddenPanelDrawerTabToReopenOnRestore->HostedTab->GetLayoutIdentifier(), HiddenPanelDrawerTabToReopenOnRestore->HostedTab.ToSharedRef());
		}

	}

	for (const TPair<FTabId, TSharedPtr<UE::Slate::Private::FPanelDrawerData>>& Pair : InactivePanelDrawerTabs)
	{
		if (Pair.Value && Pair.Value->HostedTab)
		{
			Output.Add(Pair.Key, Pair.Value->HostedTab.ToSharedRef());
		}
	}

	return Output;
}

TSharedPtr<UE::Slate::Private::SPanelDrawerArea> SDockingArea::GetPanelDrawerArea() const
{
	return PanelDrawerPtr.Pin();
}

bool SDockingArea::RemoveHiddenInactivePanelDrawerTab(const TSharedPtr<SDockTab>& TabToRemove)
{
	if (TabToRemove)
	{
		for (TPair<FTabId, TSharedPtr<UE::Slate::Private::FPanelDrawerData>>& Pair : InactivePanelDrawerTabs)
		{
			if (Pair.Value && Pair.Value->HostedTab == TabToRemove)
			{
				Pair.Value->HostedTab.Reset();
				return true;
			}
		}

		if (HiddenPanelDrawerTabToReopenOnRestore && HiddenPanelDrawerTabToReopenOnRestore->HostedTab == TabToRemove)
		{
			HiddenPanelDrawerTabToReopenOnRestore.Reset();
			return true;
		}
	}

	return false;
}

void SDockingArea::CleanPanelDrawer()
{
	// Push our nomad tab to top docking area
	if (TSharedPtr<SDockingArea> GlobalDockArea = GetTopLevelDockingArea())
	{
		if (TSharedPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawer = PanelDrawerPtr.Pin())
		{
			if (TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = PanelDrawer->GetHostedPanelDrawerData())
			{
				if (PanelDrawerData->HostedTab && PanelDrawerData->HostedTab->GetTabRole() == ETabRole::NomadTab)
				{
					GlobalDockArea->HiddenPanelDrawerTabToReopenOnRestore = PanelDrawer->GetHostedPanelDrawerData();
				}
			}
		}

		if (!GlobalDockArea->HiddenPanelDrawerTabToReopenOnRestore)
		{
			if (HiddenPanelDrawerTabToReopenOnRestore)
			{
				if (HiddenPanelDrawerTabToReopenOnRestore->HostedTab && HiddenPanelDrawerTabToReopenOnRestore->HostedTab->GetTabRole() == ETabRole::NomadTab)
				{
					GlobalDockArea->HiddenPanelDrawerTabToReopenOnRestore = MoveTemp(HiddenPanelDrawerTabToReopenOnRestore);
				}
			}
		}


	}

	HiddenPanelDrawerTabToReopenOnRestore.Reset();
	InactivePanelDrawerTabs.Empty();
}

void SDockingArea::SetPanelDrawerHiddenActiveTab(TSharedRef<UE::Slate::Private::FPanelDrawerData>&& InPanelDrawerData)
{
	if (!PanelDrawerPtr.IsValid())
	{
		if (InPanelDrawerData->HostedTab)
		{
			InPanelDrawerData->HostedTab->SetTabManager(GetTabManager());
			InPanelDrawerData->HostedTab->SetParentDockingArea(SharedThis(this));
			HiddenPanelDrawerTabToReopenOnRestore = MoveTemp(InPanelDrawerData);
		}
	}
}

SDockingNode::ECleanupRetVal SDockingArea::CleanUpNodes()
{
	SDockingNode::ECleanupRetVal ReturnValue = SDockingSplitter::CleanUpNodes();
	return ReturnValue;
}

TSharedPtr<SDockingArea> SDockingArea::GetTopLevelDockingArea() const
{
	if (!ParentWindowPtr.IsValid())
	{
		TSharedRef<FGlobalTabmanager> GlobalManager = FGlobalTabmanager::Get();
		if (TSharedPtr<SDockTab> MajorTab = GlobalManager->GetMajorTabForTabManager(GetTabManager()))
		{
			if (TSharedPtr<SWindow> GlobalWindow = MajorTab->GetParentWindow())
			{
				return GlobalManager->GetPrivateApi().GetDockingAreaForPanelDrawer(GlobalWindow);
			}
		}
	}

	return {};
}

EVisibility SDockingArea::TargetCrossVisibility() const
{
	return (bIsOverlayVisible && !bIsCenterTargetVisible)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SDockingArea::TargetCrossCenterVisibility() const
{
	return (bIsOverlayVisible && bIsCenterTargetVisible)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}


void SDockingArea::DockFromOutside(SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = StaticCastSharedPtr<FDockingDragOperation>(DragDropEvent.GetOperation());
		
	//
	// Dock from outside.
	//
	const bool bDirectionMatches = DoesDirectionMatchOrientation( Direction, this->Splitter->GetOrientation() );

	if (!bDirectionMatches && Children.Num() > 1)
	{
		// We have multiple children, but the user wants to add a new node that's perpendicular to their orientation.
		// We need to nest our children into a child splitter so that we can re-orient ourselves.
		{
			// Create a new, re-oriented splitter and copy all the children into it.
			TSharedRef<SDockingSplitter> NewSplitter = SNew(SDockingSplitter, FTabManager::NewSplitter()->SetOrientation(Splitter->GetOrientation()) );
			for( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
			{
				NewSplitter->AddChildNode( Children[ChildIndex], INDEX_NONE );
			}

			// Remove all our children.
			while( Children.Num() > 0 )
			{
				RemoveChildAt(Children.Num()-1);
			}
				
			AddChildNode( NewSplitter );
		}

		// Re-orient ourselves
		const EOrientation NewOrientation = (this->Splitter->GetOrientation() == Orient_Horizontal)
			? Orient_Vertical
			: Orient_Horizontal;

		this->SetOrientation( NewOrientation );
	}

	// Add the new node.
	{
		TSharedRef<SDockingTabStack> NewStack = SNew(SDockingTabStack, FTabManager::NewStack());

		if ( Direction == LeftOf || Direction == Above )
		{
			this->PlaceNode( NewStack, Direction, Children[0] );
		}
		else
		{
			this->PlaceNode( NewStack, Direction, Children.Last() );
		}

		NewStack->OpenTab( DragDropOperation->GetTabBeingDragged().ToSharedRef() );
	}

	HideCross();
}

void SDockingArea::OnOwningWindowBeingDestroyed(const TSharedRef<SWindow>& WindowBeingDestroyed)
{
	TArray< TSharedRef<SDockTab> > AllTabs = GetAllChildTabs();
	
	// Save the visual states of all the tabs.
	for (int32 TabIndex=0; TabIndex < AllTabs.Num(); ++TabIndex)
	{
		AllTabs[TabIndex]->PersistVisualState();
	}

	// First check if it's ok to close all the tabs that we have.
	bool CanDestroy = true;
	for (int32 TabIndex=0; CanDestroy && TabIndex < AllTabs.Num(); ++TabIndex)
	{
		CanDestroy = AllTabs[TabIndex]->CanCloseTab();
	}

	if ( CanDestroy )
	{
		// It's cool to close all tabs, so destroy them all and destroy the window.
		for (int32 TabIndex=0; TabIndex < AllTabs.Num(); ++TabIndex)
		{
			AllTabs[TabIndex]->RemoveTabFromParent();
		}

		// Destroy the window
		FSlateApplication::Get().RequestDestroyWindow(WindowBeingDestroyed);
	}
	else
	{
		// Some of the tabs cannot be closed, so we cannot close the window.
	}

}

void SDockingArea::OnOwningWindowActivated()
{
	// Update the global menu bar when the window activation changes.
	TArray< TSharedRef<SDockTab> > AllTabs = GetAllChildTabs();
	for (int32 TabIndex=0; TabIndex < AllTabs.Num(); ++TabIndex)
	{
		if (AllTabs[TabIndex]->IsForeground())
		{
			FGlobalTabmanager::Get()->UpdateMainMenu(AllTabs[TabIndex], true);
			break;
		}
	}
}

void SDockingArea::OnLiveTabAdded()
{
	bIsCenterTargetVisible = false;
	SDockingNode::OnLiveTabAdded();

	CleanUp(SDockingNode::TabRemoval_None);
}

void SDockingArea::UpdateWindowChromeAndSidebar()
{
	TArray< TSharedRef<SDockingNode> > AllNodes = this->GetChildNodesRecursively();
	if (AllNodes.Num() > 0)
	{
		// Clear out all the reserved space.
		for (auto SomeNode : AllNodes)
		{
			if (SomeNode->GetNodeType() == DockTabStack)
			{
				auto SomeTabStack = StaticCastSharedRef<SDockingTabStack>(SomeNode);
				SomeTabStack->ClearReservedSpace();
			}
		}

		bCanHaveSidebar = true;

		if (TSharedPtr<SWindow> ParentWindow = GetParentWindow())
		{
			bool bAccountForMenuBarPadding = false;
			bool bContainsMajorTabs = false;
			bool bContainsNomadTabs = false;
			if (MyTabManager.Pin()->AllowsWindowMenuBar())
			{
				TArray<TSharedRef<SDockTab>> AllTabs = GetAllChildTabs();
				for (auto& Tab : AllTabs)
				{
					if (Tab->GetParentWindow() == ParentWindow && Tab->GetTabRole() == ETabRole::MajorTab)
					{
						bAccountForMenuBarPadding = true;
						bContainsMajorTabs = true;
						break;
					}
					else if (Tab->GetTabRole() == ETabRole::NomadTab)
					{
						bContainsNomadTabs = true;
					}
				}
			}

			if (LeftSidebar->GetAllTabIds().IsEmpty())
			{
				LeftSidebar->SetVisibility(EVisibility::Collapsed);
			}
			if (RightSidebar->GetAllTabIds().IsEmpty())
			{
				RightSidebar->SetVisibility(EVisibility::Collapsed);
			}

			bCanHaveSidebar = !bContainsMajorTabs;

			if (bCanHaveSidebar)
			{
				if (bAccountForMenuBarPadding)
				{
					// Menu bar padding has already applied an offset so no need to do it again
					LeftSidebar->SetOffset(0);
					RightSidebar->SetOffset(0);
				}
				else
				{
					LeftSidebar->SetOffset(35.f);
					RightSidebar->SetOffset(35.f);
				}
			}

			const bool bNoMajorOrNomadTabs = !(bContainsMajorTabs || bContainsNomadTabs);

			// Reserve some space for the minimize, restore, and close controls
			TSharedRef<SDockingTabStack> WindowControlHousing = this->FindTabStackToHouseWindowControls();
			WindowControlHousing->ReserveSpaceForWindowChrome(SDockingTabStack::EChromeElement::Controls, bAccountForMenuBarPadding, bNoMajorOrNomadTabs);

			// Reserve some space for the app icons
			TSharedRef<SDockingTabStack> IconHousing = this->FindTabStackToHouseWindowIcon();
			IconHousing->ReserveSpaceForWindowChrome(SDockingTabStack::EChromeElement::Icon, bAccountForMenuBarPadding, bNoMajorOrNomadTabs);

			if (ParentWindow->GetTitleBar())
			{
				ParentWindow->GetTitleBar()->SetAllowMenuBar(bAccountForMenuBarPadding);
			}
		}

		AdjustDockedTabsIfNeeded();

		// Call the delegate for when the window is activated, to update the global menu bar without the user having to click on it
		OnOwningWindowActivated();
	}

}

#undef LOCTEXT_NAMESPACE
