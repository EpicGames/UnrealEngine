// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/SDockingNode.h"
#include "Framework/Docking/SDockingSplitter.h"
#include "Framework/Docking/SPanelDrawerArea.h"

class STabSidebar;

namespace UE::Slate::Private
{
	void SetAreaNodeWindowPlacementValues(TSharedPtr<FTabManager::FArea>& AreaToSet, const SWindow& InWindow);
}

/** List of tabs that should be in each sidebar */
struct FSidebarTabLists
{
	TArray<TSharedRef<SDockTab>> LeftSidebarTabs;
	TArray<TSharedRef<SDockTab>> RightSidebarTabs;
};

/**
 * Represents the root node in a hierarchy of DockNodes.
 */
class SDockingArea : public SDockingSplitter
{
public:

	SLATE_BEGIN_ARGS(SDockingArea)
		: _ParentWindow()
		, _ShouldManageParentWindow(true)
		, _InitialContent()
		{
			// Visible by default, but don't absorb clicks
			_Visibility = EVisibility::SelfHitTestInvisible;
		}
		
		/* The window whose content area this dock area is directly embedded within.  By default, ShouldManageParentWindow is
		   set to true, which means the dock area will also destroy the window when the last tab goes away.  Assigning a
		   parent window also allows the docking area to embed title area widgets (minimize, maximize, etc) into its content area */
		SLATE_ARGUMENT( TSharedPtr<SWindow>, ParentWindow )

		/** True if this docking area should close the parent window when the last tab in this docking area goes away */
		SLATE_ARGUMENT( bool, ShouldManageParentWindow )

		 /**
		 * What to put into the DockArea initially. Usually a TabStack, so that some tabs can be added to it.
		 */
		SLATE_ARGUMENT( TSharedPtr<SDockingNode>, InitialContent )

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs, const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FTabManager::FArea>& PersistentNode );

	virtual Type GetNodeType() const override
	{
		return SDockingNode::DockArea;
	}
	
	/** Returns this dock area */
	SLATE_API virtual TSharedPtr<SDockingArea> GetDockArea() override;
	SLATE_API virtual TSharedPtr<const SDockingArea> GetDockArea() const override;

	/** Returns the window that this dock area resides in directly and also manages */
	SLATE_API TSharedPtr<SWindow> GetParentWindow() const;

	SLATE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	SLATE_API virtual FReply OnUserAttemptingDock( SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent ) override;

	SLATE_API void OnTabFoundNewHome( const TSharedRef<SDockTab>& RelocatedTab, const TSharedRef<SWindow>& NewOwnerWindow );

	// Show the dock-from-outside dock targets
	SLATE_API void ShowCross();

	// Hide the dock-from-outside dock targets
	SLATE_API void HideCross();

	/**
	 * Removes redundant stack and splitters. Collapses any widgets that any no longer showing live content.
	 */
	SLATE_API void CleanUp( ELayoutModification RemovalMethod );

	SLATE_API void SetParentWindow( const TSharedRef<SWindow>& NewParentWindow );

	SLATE_API virtual TSharedPtr<FTabManager::FLayoutNode> GatherPersistentLayout() const override;

	SLATE_API TSharedRef<FTabManager> GetTabManager() const;

	/**
	 * Adds a tab to a drawer in the sidebar. 
	 *
	 * @param TabToAdd	The tab to add to the sidebar
	 * @return The location of the sidebar that the tab was added to
	 */
	SLATE_API ESidebarLocation AddTabToSidebar(TSharedRef<SDockTab> TabToAdd);

	/**
	 * Restores a tab from the sidebar to its parent tab stack and removes the tab from the sidebar.  
	 *
	 * @return true if the tab was found in this area and restore
	 */
	SLATE_API bool RestoreTabFromSidebar(TSharedRef<SDockTab> TabToRemove);

	/**
	 * @return true if the specified tab is in the sidebar
	 */
	SLATE_API bool IsTabInSidebar(TSharedRef<SDockTab> Tab) const;

	/**
 	 * Removes a tab from a sidebar
	 * @return true if the specified tab was found and removed
	 */
	SLATE_API bool RemoveTabFromSidebar(TSharedRef<SDockTab> Tab);

	/**
	 * Attempts to open a sidebar drawer that may the tab to open
	 *
	 * @true if the drawer was opened, false if the tab is not in a drawer
	 */
	SLATE_API bool TryOpenSidebarDrawer(TSharedRef<SDockTab> TabToModify) const;

	/**
	 * Attempts to dismiss a sidebar drawer that might be open for the tab
	 *
	 * @true if the drawer was dismissed, false if the tab is pinned or not in a drawer
	 */
	SLATE_API bool TryDismissSidebarDrawer(TSharedRef<SDockTab> TabToModify) const;

	/**
	 * Attempts to toggle a sidebar drawer that may own the given tab
	 *
	 * @true if the drawer toggled, false if not found or some other setting prevented toggling
	 */
	SLATE_API bool TryToggleSidebarDrawer(TSharedRef<SDockTab> TabToModify) const;

	/**
	 * Adds all tabs back to a sidebar that were saved in a sidebar from a previous session
	 */
	SLATE_API void AddSidebarTabsFromRestoredLayout(const FSidebarTabLists& SidebarTabs);

	/**
	 * Gets all tabs in all sidebars in this dock area
	 */
	SLATE_API TArray<TSharedRef<SDockTab>> GetAllSidebarTabs() const;

	bool CanHaveSidebar() const { return bCanHaveSidebar; }

	/** If we have a PanelDrawer the tab host the tab in it */
	bool HostTabIntoPanelDrawer(const TSharedRef<SDockTab>& InTab);

	/** Close the PanelDrawer associated to this area */
	void ClosePanelDrawer();

	/** Close the PanelDrawer associated to this area but set it up for a tab transfer */
	TSharedPtr<SDockTab> ClosePanelDrawerForTransfer();

	/** Return true if the area as an open PanelDrawer */
	bool IsPanelDrawerOpen() const;

	/** Does this area have a DrawerPanel associated with it */
	bool HasPanelDrawer() const;

	/** Check if the tab is hosted by the PanelDrawer system associated to this area (Include previously hosted tabs ready to reopened) */
	TSharedPtr<SDockTab> GetPanelDrawerSystemHostedTab(const FTabId& TabId) const;

	/** Get the currently active tab in the PanelDrawer */
	TSharedPtr<SDockTab> GetPanelDrawerHostedTab() const;

	/** Associate a PanelDrawer to this area */
	void SetPanelDrawerArea(const TSharedPtr<UE::Slate::Private::SPanelDrawerArea>& InPanelDrawerArea);

	/** Detach the PanelDrawer but doesn't setup or use the restore mechanism */
	void DetachPanelDrawerArea();

	/** Used to set the state of the PanelDrawer associated to this area */
	bool RestorePanelDrawerArea();

	/** Get list of tabs this PanelDrawer is keeping alive */
	TMap<FTabId, TSharedRef<SDockTab>> GetPanelDrawerKeepAliveTabs() const;

	/** Get the PanelDrawer currently associated with this area */
	TSharedPtr<UE::Slate::Private::SPanelDrawerArea> GetPanelDrawerArea() const;

	/**
	 * Remove a hidden but keep alive tab by the PanelDrawer.
	 * This may include the PanelDrawer hidden active tab if the area currently doesn't have an associated PanelDrawer
	 */
	bool RemoveHiddenInactivePanelDrawerTab(const TSharedPtr<SDockTab>& TabToRemove);

	/** Remove any data that could cause this Paneldrawer to stay alive. Also transfer some state to the top level area if needed */
	void CleanPanelDrawer();

	/** Use when restoring a layout do not use outside of that */
	void SetPanelDrawerHiddenActiveTab(TSharedRef<UE::Slate::Private::FPanelDrawerData>&& InPanelDrawerData);

protected:
	SLATE_API virtual SDockingNode::ECleanupRetVal CleanUpNodes() override;

private:

	/** Return the top docking area that is owning the window in which this docking area live */
	TSharedPtr<SDockingArea> GetTopLevelDockingArea() const;

	SLATE_API EVisibility TargetCrossVisibility() const;
	SLATE_API EVisibility TargetCrossCenterVisibility() const;
	/** Dock a tab along the outer edge of this DockArea */
	SLATE_API void DockFromOutside(SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent);

	/** We were placed in a window, and it is being destroyed */
	SLATE_API void OnOwningWindowBeingDestroyed(const TSharedRef<SWindow>& WindowBeingDestroyed);

	/** We were placed in a window and it is being activated */
	SLATE_API void OnOwningWindowActivated();
	
	SLATE_API virtual void OnLiveTabAdded() override;

	/**
	 * If this dock area controls a window, then we need
	 * to reserve some room in the upper left and upper right tab wells
	 * so that there is no overlap with the window chrome.
	 *
	 * We also update the sidebar to account for major tabs.  Docking areas for major tabs do not have a sidebar.
	 */
	SLATE_API void UpdateWindowChromeAndSidebar();

	bool RestorePanelDrawerArea(const TSharedPtr<UE::Slate::Private::SPanelDrawerArea>& InPanelDrawerAreaOverride);
private:	
	/** Left and right sidebar widgets */
	TSharedPtr<STabSidebar> LeftSidebar;
	TSharedPtr<STabSidebar> RightSidebar;

	/** The window this dock area is embedded within.  If bIsManagingParentWindow is true, the dock area will also
	    destroy the window when the last tab goes away. */
	TWeakPtr<SWindow> ParentWindowPtr;

	/**
	 * We don't want to waste a lot of space for the minimize, restore, close buttons and other windows controls.
	 * DockAreas that manage a parent window will use this slot to house those controls.
	 */
	SOverlay::FOverlaySlot* WindowControlsArea;

	/** True if this docking area should close the parent window when the last tab in this docking area goes away */
	bool bManageParentWindow;

	/** The tab manager that controls this DockArea */
	TWeakPtr<FTabManager> MyTabManager;

	/** The overlay is visible when the user is dragging a tab over the dock area */
	bool bIsOverlayVisible;

	/** The center target is visible when the overlay is visible and there are no live tabs */
	bool bIsCenterTargetVisible;

	/** True when the last tab has been pulled from this area, meaning that this DockArea will not be necessary once that tab finds a new home. */
	bool bCleanUpUponTabRelocation;

	/** True if this area can ever show sidebars (minor tab areas only) */
	bool bCanHaveSidebar;

	/** Optional Panel drawer for this area */
	TWeakPtr<UE::Slate::Private::SPanelDrawerArea> PanelDrawerPtr;

	/** Layout data for the PanelDrawer, also include the keep alive tabs */
	TSharedPtr<UE::Slate::Private::FPanelDrawerData> HiddenPanelDrawerTabToReopenOnRestore;
	TMap<FTabId, TSharedPtr<UE::Slate::Private::FPanelDrawerData>> InactivePanelDrawerTabs;
};
