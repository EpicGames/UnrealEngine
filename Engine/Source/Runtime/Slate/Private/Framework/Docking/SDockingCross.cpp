// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingCross.h"
#include "Rendering/DrawElements.h"
#include "Framework/Docking/FDockingDragOperation.h"

namespace DockingConstants
{
	static const float ZoneFraction = 0.3f;	
	static const float MaxZoneSize = 150.0f;
	static const float MinZoneSize = 5.0f;
}


void SDockingCross::Construct( const FArguments& InArgs, const TSharedPtr<class SDockingNode>& InOwnerNode )
{
	OwnerNode = InOwnerNode;

}



int32 SDockingCross::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	using namespace DockingConstants;

	const float DockZoneSizeX = FMath::Clamp( AllottedGeometry.GetLocalSize().X * ZoneFraction, MinZoneSize, MaxZoneSize );
	const float DockZoneSizeY = FMath::Clamp( AllottedGeometry.GetLocalSize().Y * ZoneFraction, MinZoneSize, MaxZoneSize );

	// We want to draw this:
	//  +-------------+
	//  |\           /|
	//  |P0---------P1|
	//  | |         | |
	//  | |         | |
	//  | |         | |
	//  | |         | |
	//  |P3---------P2|
	//  |/           \|
	//  +-------------+
	
	const FVector2f P0( DockZoneSizeX, DockZoneSizeY );
	const FVector2f P1( AllottedGeometry.GetLocalSize().X - DockZoneSizeX, DockZoneSizeY );
	const FVector2f P2( AllottedGeometry.GetLocalSize().X - DockZoneSizeX, AllottedGeometry.GetLocalSize().Y - DockZoneSizeY );
	const FVector2f P3( DockZoneSizeX, AllottedGeometry.GetLocalSize().Y - DockZoneSizeY );

	const FVector2f P0_Outer( 0.f,0.f );
	const FVector2f P1_Outer( AllottedGeometry.GetLocalSize().X,0.f );
	const FVector2f P2_Outer( AllottedGeometry.GetLocalSize().X,AllottedGeometry.GetLocalSize().Y );
	const FVector2f P3_Outer( 0.f,AllottedGeometry.GetLocalSize().Y );

	// Inner Box
	{
		TArray<FVector2f> InnerBox;

		InnerBox.Add( P0 );
		InnerBox.Add( P1 );
		InnerBox.Add( P2 );
		InnerBox.Add( P3 );
		InnerBox.Add( P0 );
		
		FSlateDrawElement::MakeLines(
	      OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(InnerBox)
		);
	}

	// Outer Box
	{
		TArray<FVector2f> OuterBox;

		OuterBox.Add( P0_Outer );
		OuterBox.Add( P1_Outer );
		OuterBox.Add( P2_Outer );
		OuterBox.Add( P3_Outer );
		OuterBox.Add( P0_Outer );
		
		FSlateDrawElement::MakeLines(
	      OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(OuterBox)
		);
	}
	
	// Diagonals
	{
		TArray<FVector2f> Points;

		Points.Add( P0_Outer );
		Points.Add( P0 );

		FSlateDrawElement::MakeLines(
	      OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(Points)
		);
	}
	{
		TArray<FVector2f> Points;

		Points.Add(FVector2f(P1_Outer));
		Points.Add(FVector2f(P1));

		FSlateDrawElement::MakeLines(
	      OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(Points)
		);
	}
	{
		TArray<FVector2f> Points;

		Points.Add(FVector2f(P2_Outer));
		Points.Add(FVector2f(P2));

		FSlateDrawElement::MakeLines(
	      OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(Points)
		);
	}{
		TArray<FVector2f> Points;

		Points.Add(FVector2f(P3_Outer));
		Points.Add(FVector2f(P3));

		FSlateDrawElement::MakeLines(
	      OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(Points)
		);
	}
	


	return LayerId;

}

FVector2D SDockingCross::ComputeDesiredSize( float ) const
{
	// DockCross does not really have a desired size.
	return FVector2D( 16, 16 );
}

static FDockingDragOperation::FDockTarget GetDropTarget( const TSharedPtr<SDockingNode>& OwnerNode, const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	using namespace DockingConstants;

	const FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
	// Noramlize per axis; the section boundaries become orthogonal lines. Slopes are 1 and -1!
	const FVector2D NormalizedMousePos = FVector2D(LocalMousePos.X / MyGeometry.Size.X, LocalMousePos.Y/MyGeometry.Size.Y);
	const float MouseSlope = NormalizedMousePos.Y / NormalizedMousePos.X;

	const float DistanceAlongSlopeOne = FVector2D::DotProduct(NormalizedMousePos, FVector2D::UnitVector);

	// Figured which section the mouse lands in.
	//                  
	//   (0,0)         
	//       +--------x-->
	//       |\      /|
	//       | \    / |
	//       |  \  /  |
	//       |   \/   |
	//       |   /\   |
	//       |  /  \  |
	//       | /    \ |
	//       y-------\ (1,1)
	//       |        \.
	//       v         \ slope = 1			

	const FVector2D DeadZoneCenter = FVector2D(0.5f, 0.5f);
	const float DeadZoneRadius = 0.75f;
	const bool bInDeadZone = (NormalizedMousePos - DeadZoneCenter).SizeSquared() < (DeadZoneRadius*DeadZoneRadius);
	
	struct 
	{ 
		bool operator()( const FVector2D& InLocalMousePos, const FGeometry& InGeometry ) const
		{
			// Dock Zones can be as large as they want, but should never get smaller than 5 slate units.
			const float DockZoneSizeX = FMath::Clamp( InGeometry.GetLocalSize().X * ZoneFraction, MinZoneSize, MaxZoneSize );
			const float DockZoneSizeY = FMath::Clamp( InGeometry.GetLocalSize().Y * ZoneFraction, MinZoneSize, MaxZoneSize );

			return
				InLocalMousePos.X < DockZoneSizeX ||
				InLocalMousePos.X > (InGeometry.Size.X - DockZoneSizeX) ||
				InLocalMousePos.Y < DockZoneSizeY ||
				InLocalMousePos.Y > (InGeometry.Size.Y - DockZoneSizeY);
		} 
	} IsInDockZone;

	const bool bIsInDockZone = IsInDockZone(LocalMousePos, MyGeometry);
	if ( !bIsInDockZone )
	{
		return FDockingDragOperation::FDockTarget();
	}
	else if (MouseSlope > 1.0f)
	{
		if (DistanceAlongSlopeOne > 1.0f)
		{
			return FDockingDragOperation::FDockTarget( OwnerNode, SDockingNode::Below );
		}
		else
		{
			return FDockingDragOperation::FDockTarget( OwnerNode, SDockingNode::LeftOf ); 
		}
	}
	else
	{
		if (DistanceAlongSlopeOne > 1.0f)
		{
			return FDockingDragOperation::FDockTarget( OwnerNode, SDockingNode::RightOf ); 
		}
		else
		{
			return FDockingDragOperation::FDockTarget( OwnerNode, SDockingNode::Above ); 
		}
	}
}

void SDockingCross::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		DragDropOperation->SetHoveredTarget(FDockingDragOperation::FDockTarget(), DragDropEvent);
	}
}

FReply SDockingCross::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		const FDockingDragOperation::FDockTarget DropTarget = GetDropTarget(OwnerNode.Pin(), MyGeometry, DragDropEvent);
		DragDropOperation->SetHoveredTarget(DropTarget, DragDropEvent);

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SDockingCross::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FDockingDragOperation>().IsValid() )
	{
		const FDockingDragOperation::FDockTarget DropTarget = GetDropTarget(OwnerNode.Pin(), MyGeometry, DragDropEvent);
		TSharedPtr<SDockingNode> DropTargetNode = DropTarget.TargetNode.Pin();
		if (DropTargetNode.IsValid())
		{
			return DropTargetNode->OnUserAttemptingDock( DropTarget.DockDirection, DragDropEvent );
		}
	}

	return FReply::Unhandled();
}
