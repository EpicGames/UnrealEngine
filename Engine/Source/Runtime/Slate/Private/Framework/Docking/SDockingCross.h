// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Framework/Docking/SDockingNode.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * Targets used by docking code. When re-arranging layout, hovering over targets
 * gives the user a preview of what will happen if they drop on that target.
 * Dropping actually commits the layout-restructuring.
 */
class SDockingCross : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SDockingCross){}
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs, const TSharedPtr<class SDockingNode>& InOwnerNode );

	// SWidget interface
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	// End of SWidget interface
	
private:

	/** The DockNode relative to which we want to dock */
	TWeakPtr<class SDockingNode> OwnerNode;
};

