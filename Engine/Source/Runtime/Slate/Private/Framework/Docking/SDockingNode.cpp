// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingNode.h"
#include "Framework/Docking/SDockingArea.h"


TSharedPtr<SDockingArea> SDockingNode::GetDockArea()
{
	return ParentNodePtr.IsValid() ? ParentNodePtr.Pin()->GetDockArea() : TSharedPtr<SDockingArea>();
}

TSharedPtr<const SDockingArea> SDockingNode::GetDockArea() const
{
	return ParentNodePtr.IsValid() ? ParentNodePtr.Pin()->GetDockArea() : TSharedPtr<SDockingArea>();
}

FReply SDockingNode::OnUserAttemptingDock( SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent )
{
	return FReply::Unhandled();
}

float SDockingNode::GetSizeCoefficient() const
{
	return SizeCoefficient;
}

void SDockingNode::SetSizeCoefficient( float InSizeCoefficient )
{
	SizeCoefficient = InSizeCoefficient;

	OnResized();
}

void SDockingNode::OnLiveTabAdded()
{
	SetVisibility(EVisibility::Visible);

	TSharedPtr<SDockingNode> ParentNode = ParentNodePtr.Pin();
	if (ParentNode.IsValid())
	{
		ParentNode->OnLiveTabAdded();
	}
}

SDockingNode::SDockingNode()
{


}

