// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingSplitter.h"

#include "SDockingArea.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/SDockingTabStack.h"


void SDockingSplitter::Construct( const FArguments& InArgs, const TSharedRef<FTabManager::FSplitter>& PersistentNode )
{
	// In DockSplitter mode we just act as a thin shell around a Splitter widget
	this->ChildSlot
	[
		SAssignNew(Splitter, SSplitter)
		. Orientation( PersistentNode->GetOrientation() )
	];

	this->SetSizeCoefficient(PersistentNode->GetSizeCoefficient());
}

void SDockingSplitter::AddChildNode( const TSharedRef<SDockingNode>& InChild, int32 InLocation )
{
	// Keep children consistent between the DockNode list and the generic widget list in the SSplitter.
	this->Splitter->AddSlot(InLocation)
	.Value( TAttribute<float>(InChild, &SDockingNode::GetSizeCoefficient) )
	.OnSlotResized( SSplitter::FOnSlotResized::CreateSP( InChild, &SDockingNode::SetSizeCoefficient ) )
	.SizeRule( TAttribute<SSplitter::ESizeRule>(InChild, &SDockingNode::GetSizeRule) )
	.MinSize( TAttribute<float>(InChild, &SDockingNode::GetNodeMinSize) )
	[
		InChild
	];	

	if ( InLocation == INDEX_NONE )
	{
		this->Children.Add( InChild );
	}
	else
	{
		this->Children.Insert( InChild, InLocation );
	}

	// Whatever node we added in, we are now its parent.
	InChild->SetParentNode( SharedThis(this) );
}

void SDockingSplitter::OnResized()
{
	if (TSharedPtr<SDockingArea> Area = GetDockArea())
	{
		Area->GetTabManager()->RequestSavePersistentLayout();
	}
}

void SDockingSplitter::ReplaceChild( const TSharedRef<SDockingNode>& InChildToReplace, const TSharedRef<SDockingNode>& Replacement )
{
	// We want to replace this placeholder with whatever is being dragged.
	int32 IndexInParentSplitter = Children.Find( InChildToReplace );
	check(IndexInParentSplitter != INDEX_NONE);
	Children[IndexInParentSplitter] = Replacement;

	Replacement->SetSizeCoefficient(InChildToReplace->GetSizeCoefficient());

	SSplitter::FSlot& Slot = Splitter->SlotAt(IndexInParentSplitter);
	Slot.SetSizeValue(TAttribute<float>(Replacement, &SDockingNode::GetSizeCoefficient));
	Slot.OnSlotResized().BindSP(Replacement, &SDockingNode::SetSizeCoefficient);
	Slot.SetSizingRule(TAttribute<SSplitter::ESizeRule>(Replacement, &SDockingNode::GetSizeRule));
	Slot.SetMinSize(TAttribute<float>(Replacement, &SDockingNode::GetNodeMinSize));
	Slot[Replacement];

	Replacement->SetParentNode( SharedThis(this) );
}

void SDockingSplitter::RemoveChild( const TSharedRef<SDockingNode>& ChildToRemove )
{
	int32 IndexToRemove = Children.Find( ChildToRemove );
	check(IndexToRemove != INDEX_NONE);
	this->RemoveChildAt( IndexToRemove );
}

void SDockingSplitter::RemoveChildAt( int32 IndexOfChildToRemove )
{
	// Keep children consisten between the DockNode list and the generic widget list in the SSplitter.
	Children.RemoveAt(IndexOfChildToRemove);
	Splitter->RemoveAt(IndexOfChildToRemove);
}

bool SDockingSplitter::DoesDirectionMatchOrientation( SDockingNode::RelativeDirection InDirection, EOrientation InOrientation )
{
	return
		( (InDirection == SDockingNode::LeftOf || InDirection == SDockingNode::RightOf) && InOrientation == Orient_Horizontal ) || 
		( (InDirection == SDockingNode::Above || InDirection == SDockingNode::Below) && InOrientation == Orient_Vertical );
}

SDockingNode::ECleanupRetVal SDockingSplitter::MostResponsibility( SDockingNode::ECleanupRetVal A, SDockingNode::ECleanupRetVal B )
{
	return FMath::Min(A, B);
}

void SDockingSplitter::AdjustDockedTabsIfNeeded()
{
	TSharedPtr<SDockingArea> DockingArea = nullptr;

	if (ParentNodePtr != nullptr && ParentNodePtr.IsValid())
	{
		DockingArea = ParentNodePtr.Pin()->GetDockArea().ToSharedRef();
	}
	else if (GetNodeType() == DockArea)
	{
		DockingArea = StaticCastSharedRef<SDockingArea>(AsShared());
	}

	// if is in floating window, set the first docking tab stack to have it's tab well unhidden.
	if (DockingArea != nullptr && DockingArea->GetParentWindow().IsValid())
	{
		for (const TSharedRef<SDockingNode>& ChildNode : Children)
		{
			/*
			 * if the node type is a tab stack AND it is visible, this must be the first tab stack and we should not
			 * hide the tabwell. Unhide it and break out of the method, no need to go further
			 */
			if (ChildNode->GetNodeType() == DockTabStack && ChildNode->GetVisibility() == EVisibility::Visible)
			{
				TSharedRef<SDockingTabStack> TabStack = StaticCastSharedRef<SDockingTabStack>(ChildNode);
				if (TabStack->IsTabWellHidden())
				{
					TabStack->SetTabWellHidden(false);
				}
				break;
			}
			// else if  node type is splitter, the first tab stack might be in there... check its children to see
			else if (ChildNode->GetNodeType() == DockSplitter)
			{
				TSharedRef<SDockingSplitter> ChildSplitter = StaticCastSharedRef<SDockingSplitter>(ChildNode);
				ChildSplitter->AdjustDockedTabsIfNeeded();
			}
		}
	}
}

SDockingNode::ECleanupRetVal SDockingSplitter::CleanUpNodes()
{
	ECleanupRetVal ThisNodePurpose = NoTabsUnderNode;

	for( int32 ChildIndex = 0; ChildIndex < Children.Num();  )
	{
		const TSharedRef<SDockingNode>& ChildNode = Children[ChildIndex];
		const ECleanupRetVal ChildNodePurpose = ChildNode->CleanUpNodes();
		ThisNodePurpose = MostResponsibility( ThisNodePurpose, ChildNodePurpose );

		switch( ChildNode->GetNodeType() )
		{
			case SDockingNode::DockTabStack:
			{
				TSharedRef<SDockingTabStack> ChildAsStack = StaticCastSharedRef<SDockingTabStack>(ChildNode);
				if (ChildNodePurpose == NoTabsUnderNode)
				{
					// This child node presents no tabs and keeps no tab history.
					RemoveChildAt(ChildIndex);
				}
				else
				{
					// This child is useful; keep it and move on to to the next element.
					++ChildIndex;
				}
			}
			break;

			case SDockingNode::DockSplitter:
			{
				TSharedRef<SDockingSplitter> ChildAsSplitter = StaticCastSharedRef<SDockingSplitter>(ChildNode);
				if (ChildNodePurpose == NoTabsUnderNode)
				{
					// Child node no longer useful
					RemoveChildAt(ChildIndex);
				}
				else
				{
					if ( ChildAsSplitter->Children.Num() == 1 || (ChildAsSplitter->GetOrientation() == this->GetOrientation()) )
					{
						const float GrandchildCoefficientScale = ChildAsSplitter->GetSizeCoefficient() / ChildAsSplitter->ComputeChildCoefficientTotal();
						// Child node is redundant
						RemoveChildAt(ChildIndex);
						// Copy the child nodes up one level
						for(int32 GrandchildIndex=0; GrandchildIndex < ChildAsSplitter->Children.Num(); ++GrandchildIndex)
						{
							const TSharedRef<SDockingNode> GrandchildNode = ChildAsSplitter->Children[GrandchildIndex];
							GrandchildNode->SetSizeCoefficient( GrandchildNode->GetSizeCoefficient() * GrandchildCoefficientScale );
							AddChildNode(GrandchildNode, ChildIndex);
							ChildIndex++;
						}
					}
					else
					{
						// Keep the child node
						ChildIndex++;
					}
				}
			}
			break;

			default:
			{
				ensureMsgf( false, TEXT("Unknown node type.") );
			}
			break;
		};
	}

	// At this point we may have ended up with a single splitter child.
	// If so, remove it and adopt all its children.
	if ( this->Children.Num() == 1 && (this->Children[0]->GetNodeType() == SDockingNode::DockSplitter || this->Children[0]->GetNodeType() == SDockingNode::DockArea) )
	{
		TSharedRef<SDockingSplitter> SoleChild = StaticCastSharedRef<SDockingSplitter>( this->Children[0] );
		this->RemoveChildAt(0);
		this->Splitter->SetOrientation( SoleChild->GetOrientation() );

		const float GrandchildCoefficientScale = SoleChild->GetSizeCoefficient() / SoleChild->ComputeChildCoefficientTotal();

		for (int32 GrandchildIndex=0; GrandchildIndex < SoleChild->Children.Num(); ++GrandchildIndex)
		{
			const TSharedRef<SDockingNode>& Grandchild = SoleChild->Children[GrandchildIndex];
			Grandchild->SetSizeCoefficient( Grandchild->GetSizeCoefficient() * GrandchildCoefficientScale );
			this->AddChildNode( Grandchild );
		}
	}

	if (ThisNodePurpose == HistoryTabsUnderNode)
	{
		// Collapse the node because it only has tab history; not live tabs.
		// Note that dock areas should never collapse.
		const bool bIsDockArea = !(this->ParentNodePtr.IsValid());
		if (!bIsDockArea)
		{
			SetVisibility(EVisibility::Collapsed);
		}		
	}
		
	return ThisNodePurpose;
}

float SDockingSplitter::ComputeChildCoefficientTotal() const
{
	float CoefficientTotal = 0;
	for( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		CoefficientTotal += Children[ChildIndex]->GetSizeCoefficient();
	}
	return CoefficientTotal;
}

void SDockingSplitter::PlaceNode( const TSharedRef<SDockingNode>& NodeToPlace, SDockingNode::RelativeDirection Direction, const TSharedRef<SDockingNode>& RelativeToMe )
{
	const bool bDirectionMatches = DoesDirectionMatchOrientation( Direction, this->Splitter->GetOrientation() );
	const bool bHasOneChild = (Children.Num() == 1);
	
	if ( !bDirectionMatches )
	{
		// This splitter's direction doesn't match the user's request to make some room for a new tab stack.
		// But if we only have one child, so we can just re-orient this splitter!
		const EOrientation NewOrientation = (this->Splitter->GetOrientation() == Orient_Horizontal)
			? Orient_Vertical
			: Orient_Horizontal;
		
		if ( bHasOneChild )
		{
			// When we have just a single child, we can just re-orient ourselves.
			// No extra work necessary.
			this->Splitter->SetOrientation( NewOrientation );
		}
		else
		{
			// Our orientation is wrong.
			// We also have more than one child, so we must preserve the orientation of the child nodes.
			// We will do this by making a new splitter, and putting the two tab stacks involved with
			// desired orientation in the new splitter.

			TSharedRef<SDockingSplitter> NewSplitter = SNew(SDockingSplitter, FTabManager::NewSplitter()->SetOrientation(NewOrientation) );
			this->ReplaceChild( RelativeToMe, NewSplitter );
			NewSplitter->AddChildNode(RelativeToMe);
			return NewSplitter->PlaceNode( NodeToPlace, Direction, RelativeToMe );
		}
	}

	
	// Find index relative to which we want to insert.
	const int32 RelativeToMeIndex = Children.Find( RelativeToMe );
	check( RelativeToMeIndex != INDEX_NONE );
		
	// Now actually drop in the new content
	if ( Direction == LeftOf || Direction == Above )
	{
		return this->AddChildNode( NodeToPlace, RelativeToMeIndex );
	}
	else
	{
		return this->AddChildNode( NodeToPlace, RelativeToMeIndex + 1 );
	}
}

void SDockingSplitter::SetOrientation(EOrientation NewOrientation)
{
	Splitter->SetOrientation(NewOrientation);
}

const TArray< TSharedRef<SDockingNode> >& SDockingSplitter::GetChildNodes() const
{
	return Children;
}

TArray< TSharedRef<SDockingNode> > SDockingSplitter::GetChildNodesRecursively() const
{
	TArray< TSharedRef<SDockingNode> > ChildNodes;
	for (int32 i = 0; i < Children.Num(); ++i)
	{
		const TSharedRef<SDockingNode>& Child = Children[i];
		ChildNodes.Add(Child);
		if (Child->GetNodeType() == SDockingNode::DockSplitter || Child->GetNodeType() == SDockingNode::DockArea)
		{
			ChildNodes += StaticCastSharedRef<SDockingSplitter>(Child)->GetChildNodesRecursively();
		}
	}
	return ChildNodes;
}

TArray<TSharedRef<SDockTab>> SDockingSplitter::GetAllChildTabs() const
{
	TArray<TSharedRef<SDockTab>> ChildTabs;
	for (int32 i = 0; i < Children.Num(); ++i)
	{
		const TSharedRef<SDockingNode>& Child = Children[i];
		ChildTabs.Append(Child->GetAllChildTabs());
	}
	return ChildTabs;
}


int32 SDockingSplitter::GetNumTabs() const
{
	int32 NumTabs = 0;
	for (int32 i = 0; i < Children.Num(); ++i)
	{
		const TSharedRef<SDockingNode>& Child = Children[i];
		NumTabs += Child->GetNumTabs();
	}

	return NumTabs;
}

EOrientation SDockingSplitter::GetOrientation() const
{
	return Splitter->GetOrientation();
}


TSharedPtr<FTabManager::FLayoutNode> SDockingSplitter::GatherPersistentLayout() const
{
	// Assume that all the nodes were dragged out, and there's no meaningful layout data to be gathered.
	bool bHaveLayoutData = false;

	TSharedRef<FTabManager::FSplitter> PersistentNode = FTabManager::NewSplitter()
		->SetOrientation(this->GetOrientation())
		->SetSizeCoefficient(this->GetSizeCoefficient());

	for (int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		TSharedPtr<FTabManager::FLayoutNode> PersistentChild = Children[ChildIndex]->GatherPersistentLayout();
		if ( PersistentChild.IsValid() )
		{
			bHaveLayoutData = true;
			PersistentNode->Split( PersistentChild.ToSharedRef() );
		}
	}

	return (bHaveLayoutData)
		? PersistentNode
		: TSharedPtr<FTabManager::FLayoutNode>();
	
}


TSharedRef<SDockingTabStack> SDockingSplitter::FindTabStackToHouseWindowControls() const
{
#if PLATFORM_MAC
	return StaticCastSharedRef<SDockingTabStack>(this->FindTabStack(ETabStackToFind::UpperLeft));
#else
	return StaticCastSharedRef<SDockingTabStack>(this->FindTabStack(ETabStackToFind::UpperRight));
#endif
}

TSharedRef<SDockingTabStack> SDockingSplitter::FindTabStackToHouseWindowIcon() const
{
	return StaticCastSharedRef<SDockingTabStack>(this->FindTabStack(ETabStackToFind::UpperLeft));
}

TSharedRef<SDockingNode> SDockingSplitter::FindTabStack(ETabStackToFind FindMe) const
{
	auto FindFirstVisibleChild = [this]() -> TSharedRef<SDockingNode>
	{
		for (auto ChildNode : Children)
		{
			if (ChildNode->GetVisibility() == EVisibility::Visible)
			{
				return ChildNode;
			}
		}

		// We might find and modify some invisible nodes. Not a problem.
		return Children[0];
	};

	auto FindLastVisibleChild = [this]() -> TSharedRef<SDockingNode>
	{
		for (int32 i = Children.Num() - 1; i >= 0; --i)
		{
			const TSharedRef<SDockingNode>& ChildNode = Children[i];
			if (ChildNode->GetVisibility() == EVisibility::Visible)
			{
				return ChildNode;
			}
		}

		// We might find and modify some invisible nodes. Not a problem.
		return Children.Last();
	};

	// We want the top-most node that is on the left-most or the right-most.
	// In the case of left-most, just grab the first element until we hit a leaf.
	// In the case of right-most grab the first element for vertical splits and the last for horizontal.
	TSharedRef<SDockingNode> Candidate = (FindMe == ETabStackToFind::UpperLeft || this->GetOrientation() == Orient_Vertical)
		? FindFirstVisibleChild()
		: Children.Last();

	const SDockingNode::Type CandidateType = Candidate->GetNodeType();
	if (CandidateType == DockTabStack)
	{
		return Candidate;
	}
	else if ( ensure(CandidateType == DockArea || CandidateType == DockSplitter) )
	{
		return StaticCastSharedRef<SDockingSplitter>(Candidate)->FindTabStack(FindMe);
	}
	else
	{
		return Candidate;
	}

}

SSplitter::ESizeRule SDockingSplitter::GetSizeRule() const
{
	if (Children.Num() > 0)
	{
		for (const TSharedRef<SDockingNode>& ChildNode : Children)
		{
			if (ChildNode->GetSizeRule() == SSplitter::FractionOfParent)
			{
				return SSplitter::FractionOfParent;
			}
		}
		// If all nodes in this docking splitter are sized to content, then it should size to Content.
		return SSplitter::SizeToContent;
	}
	else
	{
		return SSplitter::FractionOfParent;
	}
}
