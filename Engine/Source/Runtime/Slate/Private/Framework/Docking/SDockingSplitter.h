// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/SDockingNode.h"

class SDockingTabStack;

/** Dynamic N-way splitter that provides the resizing functionality in the docking framework. */
class SDockingSplitter : public SDockingNode
{
public:
	SLATE_BEGIN_ARGS(SDockingSplitter){}
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs, const TSharedRef<FTabManager::FSplitter>& PersistentNode );

	virtual Type GetNodeType() const override
	{
		return SDockingNode::DockSplitter;
	}

	/**
	 * Add a new child dock node at the desired location.
	 * Assumes this DockNode is a splitter.
	 *
	 * @param InChild      The DockNode child to add.
	 * @param InLocation   Index at which to add; INDEX_NONE (default) adds to the end of the list
	 */
	SLATE_API void AddChildNode( const TSharedRef<SDockingNode>& InChild, int32 InLocation = INDEX_NONE);

	/**
	 * Replace the child InChildToReplace with Replacement
	 *
	 * @param InChildToReplace     The child of this node to replace
	 * @param Replacement          What to replace with
	 */
	SLATE_API void ReplaceChild( const TSharedRef<SDockingNode>& InChildToReplace, const TSharedRef<SDockingNode>& Replacement );

	/**
	 * Remove a child node from the splitter
	 * 
	 * @param ChildToRemove    The child node to remove from this splitter
	 */
	SLATE_API void RemoveChild( const TSharedRef<SDockingNode>& ChildToRemove );
	
	/**
	 * Remove a child at this index
	 *
	 * @param IndexOfChildToRemove    Remove a child at this index
	 */
	SLATE_API void RemoveChildAt( int32 IndexOfChildToRemove );

	/**
	 * Inserts NodeToPlace next to RelativeToMe.
	 * Next to means above, below, left or right of RelativeToMe.
	 *
	 * @param NodeToPlace        The node to insert.
	 * @param Direction          Where to insert relative to the other node
	 * @param RelativeToMove     Insert relative to this node.
	 */
	SLATE_API void PlaceNode( const TSharedRef<SDockingNode>& NodeToPlace, SDockingNode::RelativeDirection Direction, const TSharedRef<SDockingNode>& RelativeToMe );

	/** Sets the orientation of this splitter */
	SLATE_API void SetOrientation(EOrientation NewOrientation);

	/** Gets an array of all child dock nodes */
	SLATE_API const TArray< TSharedRef<SDockingNode> >& GetChildNodes() const;

	/** Gets an array of all child dock nodes and all of their child dock nodes and so on */
	SLATE_API TArray< TSharedRef<SDockingNode> > GetChildNodesRecursively() const;
	
	/** Recursively searches through all children looking for child tabs */
	SLATE_API virtual TArray< TSharedRef<SDockTab> > GetAllChildTabs() const override;

	/** Gets the number of tabs in all children recursively */
	SLATE_API virtual int32 GetNumTabs() const override;

	/** Should this node auto-size or be a percentage of its parent size */
	SLATE_API virtual SSplitter::ESizeRule GetSizeRule() const override;

	/** Gets the size coefficient of a given child dock node */
	SLATE_API float GetSizeCoefficientForSlot(int32 Index) const;

	/** Gets the orientation of this splitter */
	SLATE_API EOrientation GetOrientation() const;

	SLATE_API virtual TSharedPtr<FTabManager::FLayoutNode> GatherPersistentLayout() const override;

	/**
	 * The tab stack which would be appropriate to use for showing the minimize/maximize/close buttons.
	 * On Windows it is the upper right most. On Mac the upper left most.
	 */
	SLATE_API TSharedRef<SDockingTabStack> FindTabStackToHouseWindowControls() const;

	/** Which tab stack is appropriate for showing the app icon. */
	SLATE_API TSharedRef<SDockingTabStack> FindTabStackToHouseWindowIcon() const;

protected:

	/** Is the docking direction (left, right, above, below) match the orientation (horizontal vs. vertical) */
	static SLATE_API bool DoesDirectionMatchOrientation( SDockingNode::RelativeDirection InDirection, EOrientation InOrientation );

	static SLATE_API SDockingNode::ECleanupRetVal MostResponsibility( SDockingNode::ECleanupRetVal A, SDockingNode::ECleanupRetVal B );

	SLATE_API virtual SDockingNode::ECleanupRetVal CleanUpNodes() override;

	SLATE_API virtual void OnResized() override;

	SLATE_API float ComputeChildCoefficientTotal() const;

	enum class ETabStackToFind
	{
		UpperLeft,
		UpperRight
	};

	/** Helper: Finds the upper left or the upper right tab stack in the hierarchy */
	SLATE_API TSharedRef<SDockingNode> FindTabStack(ETabStackToFind FindMe) const;

	/** The SSplitter widget that SDockingSplitter wraps. */
	TSharedPtr<SSplitter> Splitter;
	
	/** The DockNode children. All these children are kept in sync with the SSplitter's children via the use of the public interface for adding, removing and replacing children. */
	TArray< TSharedRef<class SDockingNode> > Children;

	/**
	 *   As the Dock area is updating after clean up of any added, removed, or newly shown Dock areas or tabs,
	 *   the tabs/tab well may need adjusting. Use this for post clean up docked tab adjustments.
	 */
	SLATE_API void AdjustDockedTabsIfNeeded();
};


