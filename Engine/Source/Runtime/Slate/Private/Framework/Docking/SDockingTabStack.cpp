// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingTabStack.h"
#include "Algo/AllOf.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Docking/SDockingTabWell.h"
#include "Framework/Docking/SDockingCross.h"
#include "Framework/Docking/FDockingDragOperation.h"
#include "Framework/Docking/TabCommands.h"
#include "Framework/Docking/STabDrawer.h"
#include "Brushes/SlateColorBrush.h"

#define LOCTEXT_NAMESPACE "DockTabStack"


static const FVector2D ContextButtonTargetSize(24,24);
static const float TriggerAreaFraction = 0.24f;


/**
 * Like a missing widget, but says it's a document area
 */
class SDocumentAreaWidget
{
public:
	static TSharedRef<class SWidget> MakeDocumentAreaWidget()
	{
		return 
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew( STextBlock )
				.Text(LOCTEXT("DocumentArea", "Document Area") )
				.TextStyle( FCoreStyle::Get(), "EmbossedText" )
			];
	}
};


void SDockingTabStack::Construct( const FArguments& InArgs, const TSharedRef<FTabManager::FStack>& PersistentNode  )
{
	BindTabCommands();

	Tabs = PersistentNode->Tabs;
	this->SetSizeCoefficient(PersistentNode->GetSizeCoefficient());
	NodeMinSize = PersistentNode->GetMinSize();

	// the value of this is determined every time a tab is added
	bShowingTitleBarArea = false;

	bIsDocumentArea = InArgs._IsDocumentArea;

	InlineContentAreaLeft = nullptr;
	InlineContentAreaRight = nullptr;
	TitleBarSlot = nullptr;

	// Animation that toggles the tabs
	{
		ShowHideTabWell = FCurveSequence(0,0.15f);
		if (PersistentNode->bHideTabWell)
		{
			ShowHideTabWell.JumpToStart();
		}
		else
		{
			ShowHideTabWell.JumpToEnd();
		}
		
	}

	// In TabStack mode we glue together a TabWell, InlineContent areas and a ContentOverlay
	// that shows the content of the currently selected Tab.
	//                                         ________ TabWell
	//                                        |
	//  +-------------------------------------v-------------------------------+
	//  |                       +--------------------+                        |
	//  | InlineContentAreaLeft | Tab0 | Tab1 | Tab2 | InlineContentAreaRight | 
	//  +---------------------------------------------------------------------+
	//  |                                                                     |
	//  |                                                                     |  <-- Content area overlay
	//  |                                                                     |
	//  +---------------------------------------------------------------------+
	//

	const FButtonStyle* const UnhideTabWellButtonStyle = &FCoreStyle::Get().GetWidgetStyle< FButtonStyle >( "Docking.UnhideTabwellButton" );

	// create inline title bar content
	TitleBarContent = 
	SNew(SOverlay)
	+ SOverlay::Slot()
	[
		SNew(SHorizontalBox)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Expose(InlineContentAreaLeft)

		+ SHorizontalBox::Slot() 
		.FillWidth(1.0f)
		.VAlign(VAlign_Bottom)
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SVerticalBox)
			.Visibility(EVisibility::SelfHitTestInvisible)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSpacer)
				.Visibility(this, &SDockingTabStack::GetMaximizeSpacerVisibility)
				.Size(FVector2D(0.0f, 10.0f))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				// TabWell
				SAssignNew(TabWell, SDockingTabWell)
				.ParentStackNode(SharedThis(this))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Expose(InlineContentAreaRight)
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
	];

	ChildSlot
	[
		SNew(SVerticalBox)
		.Visibility( EVisibility::SelfHitTestInvisible )

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			// tab well area
			SNew(SBorder)
			.Visibility(this, &SDockingTabStack::GetTabWellVisibility)
			.DesiredSizeScale(this, &SDockingTabStack::GetTabWellScale)
			.BorderImage(this, &SDockingTabStack::GetTabStackBorderImage)
			.VAlign(VAlign_Bottom)
			.OnMouseButtonDown(this, &SDockingTabStack::TabWellRightClicked)
			.Padding(0.0f)
			[
				SNew(SVerticalBox)
				.Visibility(EVisibility::SelfHitTestInvisible)

				+ SVerticalBox::Slot()
				.Expose(TitleBarSlot)
				.AutoHeight()

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SImage)
					.Image(this, &SDockingTabStack::GetTabWellBrush)
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			// tab content area
			SAssignNew(OverlayManagement.ContentAreaOverlay, SOverlay)

			+ SOverlay::Slot()
			[
				// content goes here
				SAssignNew(ContentSlot, SBorder)
				.BorderImage(this, &SDockingTabStack::GetContentAreaBrush)
				.Padding(this, &SDockingTabStack::GetContentPadding)
				.Clipping(EWidgetClipping::ClipToBounds)
				.IsEnabled(this, &SDockingTabStack::IsContentEnabled)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EmptyTabMessage", "Empty Tab!"))
				]
			]

			+ SOverlay::Slot()
			.Padding(0.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				// unhide tab well button (yellow triangle)
				SNew(SButton)
				.ButtonStyle(UnhideTabWellButtonStyle)
				.OnClicked(this, &SDockingTabStack::UnhideTabWell)
				.ContentPadding(0.0f)
				.Visibility(this, &SDockingTabStack::GetUnhideButtonVisibility)
				.DesiredSizeScale(this, &SDockingTabStack::GetUnhideTabWellButtonScale)
				.ButtonColorAndOpacity(this, &SDockingTabStack::GetUnhideTabWellButtonOpacity)
				.ToolTipText(LOCTEXT("UnhideTabWellToolTip", "Show Tabs"))
				[
					// button should be big enough to show its own image
					SNew(SSpacer)
						.Size(UnhideTabWellButtonStyle->Normal.ImageSize)
				]
			]

#if DEBUG_TAB_MANAGEMENT
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("Docking.Border"))
				.BorderBackgroundColor(FLinearColor(1.0f, 0.5f, 0.0f, 0.75f))
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(STextBlock)
					.Text(this, &SDockingTabStack::ShowPersistentTabs)
					.ShadowOffset(FVector2D::UnitVector)
				]
			]
#endif
		]
	];

	if (bIsDocumentArea)
	{
		this->SetNodeContent(SDocumentAreaWidget::MakeDocumentAreaWidget(), FDockingStackOptionalContent());
	}
}

void SDockingTabStack::OnLastTabRemoved()
{
	if (!bIsDocumentArea)
	{
		// Stop holding onto any meaningful window content.
		// The user should not see any content in this DockNode.
		this->SetNodeContent(SNullWidget::NullWidget, FDockingStackOptionalContent());
	}
	else
	{
		this->SetNodeContent(SDocumentAreaWidget::MakeDocumentAreaWidget(), FDockingStackOptionalContent());
	}
}

void SDockingTabStack::OnTabClosed(const TSharedRef<SDockTab>& ClosedTab, SDockingNode::ELayoutModification RemovalMethod)
{
	const FTabId& TabIdBeingClosed = ClosedTab->GetLayoutIdentifier();
	
	// Document-style tabs are positioned per use-case.
	const bool bIsTabPersistable = TabIdBeingClosed.IsTabPersistable();
	if (bIsTabPersistable)
	{
		// Sidebar tabs should still exist in the stacks layout so we can restore it
		if (RemovalMethod != SDockingNode::ELayoutModification::TabRemoval_Sidebar)
		{
			ClosePersistentTab(TabIdBeingClosed);
		}
	}
	else
	{
		RemovePersistentTab(TabIdBeingClosed);
	}	
}


void SDockingTabStack::OnTabRemoved( const FTabId& TabId )
{
	RemovePersistentTab( TabId );
}

void SDockingTabStack::OpenTab(const TSharedRef<SDockTab>& InTab, int32 InsertLocationAmongActiveTabs, bool bKeepInactive)
{
	const int32 InsertIndex = OpenPersistentTab(InTab->GetLayoutIdentifier(), InsertLocationAmongActiveTabs);

	// The tab may be a nomad tab, in which case it should inherit whichever tab manager it is being put into!
	InTab->SetTabManager(GetDockArea()->GetTabManager());

	const FTabId TabId = InTab->GetLayoutIdentifier();

	// the insert index is not the same as the tab index in the array for new tabs so find the tab again to check the tab state.
	const FTabManager::FTab* TabInfo = Tabs.FindByPredicate([TabId](const FTabManager::FTab& TestTab) {return TestTab.TabId == TabId; });
	if (!TabInfo)
	{
		return;
	}

	if (TabInfo->TabState == ETabState::SidebarTab)
	{
		FSidebarTabLists SidebarLists;
		if (TabInfo->SidebarLocation == ESidebarLocation::Left)
		{
			SidebarLists.LeftSidebarTabs.Add(InTab);
		}
		else
		{
			ensure(TabInfo->SidebarLocation == ESidebarLocation::Right);
			SidebarLists.RightSidebarTabs.Add(InTab);
		}

		AddSidebarTab(InTab);
		GetDockArea()->AddSidebarTabsFromRestoredLayout(SidebarLists);
	}
	else
	{
		AddTabWidget(InTab, InsertIndex, bKeepInactive);
		OnLiveTabAdded();
		TabWell->RefreshParentContent();
	}

}

void SDockingTabStack::AddTabWidget(const TSharedRef<SDockTab>& InTab, int32 AtLocation, bool bKeepInactive)
{
	TabWell->AddTab(InTab, AtLocation, bKeepInactive);

	if ( IsTabWellHidden() && TabWell->GetNumTabs() > 1 )
	{
		SetTabWellHidden(false);
	}
	
	// We just added a tab, so if there was a cross up we no longer need it.
	HideCross();
	TSharedPtr<SDockingArea> ParentDockArea = GetDockArea();
	if (ParentDockArea.IsValid())
	{
		ParentDockArea->HideCross();
	}

}

void SDockingTabStack::AddSidebarTab(const TSharedRef<SDockTab>& InTab)
{
	InTab->SetParent(TabWell);
}

float SDockingTabStack::GetTabSidebarSizeCoefficient(const TSharedRef<SDockTab>& InTab)
{
	FTabManager::FTab* Tab = Tabs.FindByPredicate(FTabMatcher(InTab->GetLayoutIdentifier()));
	if (Tab)
	{
		return Tab->SidebarSizeCoefficient;
	}

	return 0;
}

void SDockingTabStack::SetTabSidebarSizeCoefficient(const TSharedRef<SDockTab>& InTab, float InSizeCoefficient)
{
	FTabManager::FTab* Tab = Tabs.FindByPredicate(FTabMatcher(InTab->GetLayoutIdentifier()));
	if (Tab)
	{
		Tab->SidebarSizeCoefficient = InSizeCoefficient;
	}
}

bool SDockingTabStack::IsTabPinnedInSidebar(const TSharedRef<SDockTab>& InTab)
{
	FTabManager::FTab* Tab = Tabs.FindByPredicate(FTabMatcher(InTab->GetLayoutIdentifier()));
	if (Tab)
	{
		return Tab->bPinnedInSidebar;
	}

	return false;
}

void SDockingTabStack::SetTabPinnedInSidebar(const TSharedRef<SDockTab>& InTab, bool bPinnedInSidebar)
{
	FTabManager::FTab* Tab = Tabs.FindByPredicate(FTabMatcher(InTab->GetLayoutIdentifier()));
	if (Tab)
	{
		Tab->bPinnedInSidebar = bPinnedInSidebar;
	}
}

bool SDockingTabStack::IsTabLocked(const TSharedRef<const SDockTab>& InTab) const
{
	const FTabManager::FTab* Tab = Tabs.FindByPredicate(FTabMatcher(InTab->GetLayoutIdentifier()));
	if (Tab)
	{
		return Tab->bIsLockedInPlace.Get(false);
	}

	return false;
}

void SDockingTabStack::SetTabLocked(const TSharedRef<const SDockTab>& InTab, TOptional<bool> bInLocked)
{
	FTabManager::FTab* Tab = Tabs.FindByPredicate(FTabMatcher(InTab->GetLayoutIdentifier()));
	if (Tab)
	{
		Tab->bIsLockedInPlace = bInLocked;
	}
}

const TSlotlessChildren<SDockTab>& SDockingTabStack::GetTabs() const
{
	return TabWell->GetTabs();
}

int32 SDockingTabStack::GetNumTabs() const
{
	return TabWell->GetNumTabs();
}

bool SDockingTabStack::HasTab(const struct FTabMatcher& TabMatcher) const
{
	return Tabs.IndexOfByPredicate(TabMatcher) != INDEX_NONE;
}

FGeometry SDockingTabStack::GetTabStackGeometry() const
{
	return GetTickSpaceGeometry();
}

void SDockingTabStack::RemoveClosedTabsWithName( FName InName )
{
	for (int32 TabIndex=0; TabIndex < Tabs.Num();  )
	{
		const FTabManager::FTab& ThisTab = Tabs[TabIndex];
		if ( ThisTab.TabState == ETabState::ClosedTab && ThisTab.TabId == InName )
		{
			Tabs.RemoveAtSwap(TabIndex);
		}
		else
		{
			++TabIndex;
		}
	}
}

bool SDockingTabStack::IsShowingLiveTabs() const
{
	return this->TabWell->GetNumTabs() > 0;
}

void SDockingTabStack::BringToFront( const TSharedRef<SDockTab>& TabToBringToFront )
{
	TabWell->BringTabToFront(TabToBringToFront);
}

void SDockingTabStack::SetNodeContent(const TSharedRef<SWidget>& InContent, const FDockingStackOptionalContent& OptionalContent)
{
	ContentSlot->SetContent(InContent);
	(*InlineContentAreaLeft)[OptionalContent.ContentLeft];
	(*InlineContentAreaRight)[OptionalContent.ContentRight];

	if(TabWell->GetForegroundTab())
	{
		if (TSharedPtr<SWindow> ParentWindow = TabWell->GetForegroundTab()->GetParentWindow())
		{
			ParentWindow->GetTitleBar()->UpdateBackgroundContent(OptionalContent.TitleBarContentRight);
		}
	}

}

FReply SDockingTabStack::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{	
		if (DragDropOperation->CanDockInNode(SharedThis(this), FDockingDragOperation::DockingViaTarget))
		{
			FGeometry OverlayGeometry = this->FindChildGeometry( MyGeometry, OverlayManagement.ContentAreaOverlay.ToSharedRef() );
	
			if ( OverlayGeometry.IsUnderLocation( DragDropEvent.GetScreenSpacePosition() ) )
			{
				ShowCross();
			}
			else
			{
				HideCross();
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SDockingTabStack::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if (DragDropEvent.GetOperationAs<FDockingDragOperation>().IsValid())
	{
		HideCross();
	}
}

FReply SDockingTabStack::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if (DragDropEvent.GetOperationAs<FDockingDragOperation>().IsValid())
	{
		HideCross();
	}

	return FReply::Unhandled();
}

void SDockingTabStack::OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	// If the new widget is a drawer within the tab, the tab does not need to be activated.
	// This prevents the focus being reset back to the containing tab and instantly closing the drawer.
	if (NewWidgetPath.IsValid())
	{
		if (TSharedPtr<SWidget> LastWidget = NewWidgetPath.GetLastWidget())
		{
			if (LastWidget->GetType() == STabDrawer::StaticWidgetClass().GetWidgetType())
			{
				return;
			}
		}
	}

	const TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();
	if ( ForegroundTab.IsValid() )
	{
		const bool bIsForegroundTabActive = NewWidgetPath.ContainsWidget( this );
	
		if (bIsForegroundTabActive)
		{
			// If a widget inside this tab stack got focused, activate this tab.
			FGlobalTabmanager::Get()->SetActiveTab( ForegroundTab );
			ForegroundTab->ActivateInParent(ETabActivationCause::SetDirectly);
		}
	}
}

FReply SDockingTabStack::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();

	const bool bIsRelevantButtonForTabFocus = MouseEvent.GetPressedButtons().Contains(EKeys::LeftMouseButton)
			|| MouseEvent.GetPressedButtons().Contains(EKeys::RightMouseButton)
			|| MouseEvent.GetPressedButtons().Contains(EKeys::MiddleMouseButton);

	if (bIsRelevantButtonForTabFocus && ForegroundTab.IsValid() && !ForegroundTab->IsActive())
	{
		FGlobalTabmanager::Get()->SetActiveTab( ForegroundTab );
#if PLATFORM_LINUX
		// Don't stop further event handling in case the user wants to move this window.
		// Returning FReply::Handled() here will prevent SWindow from seeing the event.
		// FIXME: In some cases the foreground tab is never the active tab and this handler will consume every mouse down event.
		return FReply::Unhandled();
#else
		return FReply::Handled();
#endif
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SDockingTabStack::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (ActionList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SDockingNode::OnKeyDown(MyGeometry, InKeyEvent);
}

void SDockingTabStack::ShowContextMenu(const TSharedRef<SDockTab>& InContextTab, const FWidgetPath& InWidgetPath)
{
	FSlateApplication::Get().PushMenu(
		AsShared(), 
		InWidgetPath, 
		MakeContextMenu(InContextTab), 
		FSlateApplication::Get().GetCursorPos(), 
		FPopupTransitionEffect::ContextMenu
	);
	
	InContextTab->FlashTab();
}

FReply SDockingTabStack::OnUserAttemptingDock( SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		// We want to replace this placeholder  with whatever is being dragged.				
		CreateNewTabStackBySplitting( Direction )->OpenTab( DragDropOperation->GetTabBeingDragged().ToSharedRef() );

		HideCross();

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

TArray< TSharedRef<SDockTab> > SDockingTabStack::GetAllChildTabs() const
{
	return GetTabs().AsArrayCopy();
}

void SDockingTabStack::CloseForegroundTab()
{
	CloseTab(TabWell->GetForegroundTab());
}

void SDockingTabStack::CloseTab(TSharedPtr<SDockTab> InTab)
{
	if (InTab.IsValid())
	{
		InTab->RequestCloseTab();
	}
}

void SDockingTabStack::CloseTabsInDirectionFromForegroundTab(ETabsToClose TabsToClose, ECloseTabsDirection Direction)
{
	CloseTabsInDirectionFromTab(TabWell->GetForegroundTab(), TabsToClose, Direction);
}

void SDockingTabStack::CloseTabsInDirectionFromTab(TSharedPtr<SDockTab> InTab, ETabsToClose InTabsToClose, ECloseTabsDirection InDirection)
{
	if (!InTab.IsValid())
	{
		return;
	}

	const int32 IndexOfTab = TabWell->GetTabIndex(InTab);
	if (IndexOfTab == INDEX_NONE)
	{
		return;
	}

	const int32 Increment = GetIncrementFromCloseTabsDirection(InDirection);

	int32 DestroyIndex = IndexOfTab + Increment;
	while (DestroyIndex >= 0 && DestroyIndex < TabWell->GetNumTabs())
	{
		const TSharedRef<SDockTab>& Tab = TabWell->GetTabs()[DestroyIndex];
		const ETabRole VisualTabRole = Tab->GetVisualTabRole();
		const bool bCanClose =
			(InTabsToClose == ETabsToClose::CloseAllTabs) ||
			(InTabsToClose == ETabsToClose::CloseDocumentTabs && VisualTabRole == ETabRole::DocumentTab) ||
			(InTabsToClose == ETabsToClose::CloseDocumentAndMajorTabs && (VisualTabRole == ETabRole::DocumentTab || VisualTabRole == ETabRole::MajorTab));

		// Need to increment index if we failed to close a tab to the right or if we're closing tabs to the left regardless of if we succeeded
		if (!bCanClose || !Tab->RequestCloseTab() || InDirection == ECloseTabsDirection::Left)
		{
			DestroyIndex += Increment;
		}
	}
}

void SDockingTabStack::CloseAllButForegroundTab(ETabsToClose TabsToClose)
{
	CloseAllButTab(TabWell->GetForegroundTab(), TabsToClose);
}

void SDockingTabStack::CloseAllButTab(TSharedPtr<SDockTab> InTab, ETabsToClose InTabsToClose)
{
	if (!InTab.IsValid())
	{
		return;
	}
	
	int32 DestroyIndex = 0;
	while ((TabWell->GetNumTabs() > 1) && (DestroyIndex < TabWell->GetNumTabs()))
	{
		const TSharedRef<SDockTab>& Tab = TabWell->GetTabs()[DestroyIndex];

		const ETabRole VisualTabRole = Tab->GetVisualTabRole();
		const bool bCanClose = 
			(InTabsToClose == ETabsToClose::CloseAllTabs) ||
			(InTabsToClose == ETabsToClose::CloseDocumentTabs && VisualTabRole == ETabRole::DocumentTab) ||
			(InTabsToClose == ETabsToClose::CloseDocumentAndMajorTabs && (VisualTabRole == ETabRole::DocumentTab || VisualTabRole == ETabRole::MajorTab));

		if ((Tab == InTab) || !bCanClose || !Tab->RequestCloseTab())
		{
			++DestroyIndex;
		}
	}
}

FReply SDockingTabStack::TabWellRightClicked( const FGeometry& TabWellGeometry, const FPointerEvent& MouseEvent )
{
	if (TabWell.IsValid() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (const TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab())
		{
			const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			ShowContextMenu(ForegroundTab.ToSharedRef(), WidgetPath);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

SDockingNode::ECleanupRetVal SDockingTabStack::CleanUpNodes()
{
	if (TabWell->GetNumTabs() > 0)
	{
		return VisibleTabsUnderNode;
	}
	else if (Tabs.Num() > 0)
	{
		SetVisibility(EVisibility::Collapsed);
		return HistoryTabsUnderNode;
	}
	else
	{
		return NoTabsUnderNode;
	}
}

TSharedRef<SWidget> SDockingTabStack::MakeContextMenu(const TSharedPtr<SDockTab>& InTab)
{
	if (!InTab.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	// Show a menu that allows users to toggle whether
	// a specific tab should hide if it is the sole tab
	// in its tab well.
	constexpr bool bCloseAfterSelection = true;
	constexpr bool bCloseSelfOnly = false;
	FMenuBuilder MenuBuilder( bCloseAfterSelection, nullptr, TSharedPtr<FExtender>(), bCloseSelfOnly, &FCoreStyle::Get() );
	{
		MenuBuilder.BeginSection("DockingTabStackOptions", LOCTEXT("TabOptionsHeading", "Options") );
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CollapseTabWell", "Hide Tabs"),
				LOCTEXT("CollapseTabWellTooltip", "Collapses the tabs headers to save room."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &SDockingTabStack::ToggleTabWellVisibility ),
					FCanExecuteAction::CreateSP( this, &SDockingTabStack::CanHideTabWell )
				) 
			);

			if (IsTabAllowedInSidebar(InTab))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("MoveToSidebar", "Dock to Sidebar"),
					LOCTEXT("MoveToSidebarTooltip", "Moves this tab to a sidebar drawer on the side of the window closest to the tab.\nThe tab can be opened from the drawer and will automatically close again when clicking off it."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SDockingTabStack::MoveTabToSidebar, InTab),
						FCanExecuteAction::CreateSP(this, &SDockingTabStack::CanMoveTabToSidebar, InTab)
					)
				);
			}
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("DockingTabStackCloseTabs");
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CloseTab", "Close"),
				LOCTEXT("CloseTabTooltip", "Close this tab."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SDockingTabStack::CloseTab, InTab),
					FCanExecuteAction::CreateSP(this, &SDockingTabStack::CanCloseTab, InTab)
				)
			);

			const ETabsToClose TabsToClose = ETabsToClose::CloseDocumentAndMajorTabs;

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CloseTabsToTheLeft", "Close Tabs to the Left"),
				LOCTEXT("CloseTabsToTheLeftTooltip", "Closes all tabs to the left of the active tab."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &SDockingTabStack::CloseTabsInDirectionFromTab, InTab, TabsToClose, ECloseTabsDirection::Left),
					FCanExecuteAction::CreateSP( this, &SDockingTabStack::CanCloseTabsInDirectionOfTab, InTab, ECloseTabsDirection::Left)
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CloseTabsToTheRight", "Close Tabs to the Right"),
				LOCTEXT("CloseTabsToTheRightTooltip", "Closes all tabs to the right of the active tab."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &SDockingTabStack::CloseTabsInDirectionFromTab, InTab, TabsToClose, ECloseTabsDirection::Right),
					FCanExecuteAction::CreateSP( this, &SDockingTabStack::CanCloseTabsInDirectionOfTab, InTab, ECloseTabsDirection::Right)
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CloseOtherTabs", "Close Other Tabs"),
				LOCTEXT("CloseOtherTabsTooltip", "Closes all tabs except for the active tab."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &SDockingTabStack::CloseAllButTab, InTab, TabsToClose ),
					FCanExecuteAction::CreateSP(this, &SDockingTabStack::CanCloseAllButTab, InTab)
				)
			);
		}
		MenuBuilder.EndSection();

		// Let the tab extend its own context menu
		InTab->ExtendContextMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

void SDockingTabStack::ShowCross()
{
	const float DockTargetSize = 32.0f;

	if (!OverlayManagement.bShowingCross)
	{
		this->GetDockArea()->ShowCross();

		OverlayManagement.bShowingCross = true;
		OverlayManagement.ContentAreaOverlay->AddSlot()
		. HAlign(HAlign_Fill)
		. VAlign(VAlign_Fill)
		[
			
			SNew( SDockingCross, SharedThis(this) )

			//SNew(SBorder)
			//. BorderImage( FStyleDefaults::GetNoBrush() )
			//. Padding( FMargin(0,0,0,0))
			//. Content()
			//[
			//	SNew(SVerticalBox)
			//	+ SVerticalBox::Slot()
			//.AutoHeight()
			//	[
			//		// TOP ROW
			//		SNew(SHorizontalBox)
			//		+ SHorizontalBox::Slot() .FillWidth(1.0f)
			//		+ SHorizontalBox::Slot()
			//		[
			//			SNew(SDockingTarget)
			//			. OwnerNode( SharedThis(this) )
			//			. DockDirection( SDockingNode::Above )
			//		]
			//		+ SHorizontalBox::Slot() .FillWidth(1.0f)
			//	]
			//	+ SVerticalBox::Slot()
			//.AutoHeight()
			//	[
			//		// MIDDLE ROW
			//		SNew(SHorizontalBox)
			//		+ SHorizontalBox::Slot()
			//		.FillWidth(1.0f)
			//		[
			//			SNew(SDockingTarget)
			//			. OwnerNode( SharedThis(this) )
			//			. DockDirection( SDockingNode::LeftOf )
			//		]
			//		+ SHorizontalBox::Slot().AutoWidth()
			//		[
			//			// The center node is redundant with just moving the tab into place.
			//			// It was also confusing to many.
			//			SNew(SDockingTarget)
			//			. Visibility(EVisibility::Hidden)
			//			. OwnerNode( SharedThis(this) )
			//			. DockDirection( SDockingNode::Center )
			//		]
			//		+ SHorizontalBox::Slot()
			//		.FillWidth(1.0f)
			//		[
			//			SNew(SDockingTarget)
			//			. OwnerNode( SharedThis(this) )
			//			. DockDirection( SDockingNode::RightOf )
			//		]
			//	]
			//	+ SVerticalBox::Slot()
			//.AutoHeight()
			//	.HAlign(HAlign_Center)
			//	[
			//		// BOTTOM ROW
			//		SNew(SHorizontalBox)
			//		+ SHorizontalBox::Slot() .FillWidth(1.0f)
			//		+ SHorizontalBox::Slot().AutoWidth()
			//		[
			//			SNew(SDockingTarget)
			//			. OwnerNode( SharedThis(this) )
			//			. DockDirection( SDockingNode::Below )
			//		]
			//		+ SHorizontalBox::Slot() .FillWidth(1.0f)
			//	]
			//]
		];

	}
}

void SDockingTabStack::HideCross()
{
	if (OverlayManagement.bShowingCross)
	{
		OverlayManagement.ContentAreaOverlay->RemoveSlot();
		OverlayManagement.bShowingCross = false;
	}
}

TSharedPtr<FTabManager::FLayoutNode> SDockingTabStack::GatherPersistentLayout() const
{
	if( Tabs.Num() > 0 )
	{
		// Each live tab might want to save custom visual state.
		{
			const TArray< TSharedRef<SDockTab> > MyTabs = this->GetTabs().AsArrayCopy();
			for (int32 TabIndex=0; TabIndex < MyTabs.Num(); ++TabIndex)
			{
				MyTabs[TabIndex]->PersistVisualState();
			}
		}

		// Persist layout
		TSharedRef<FTabManager::FStack> PersistentStack =
			FTabManager::NewStack()
			->SetSizeCoefficient( this->GetSizeCoefficient() )
			->SetHideTabWell( this->IsTabWellHidden() )
			->SetMinSize( NodeMinSize );

		TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();
		if(ForegroundTab.IsValid())
		{
			PersistentStack->SetForegroundTab(ForegroundTab->GetLayoutIdentifier());
		}

		for (int32 TabIndex=0; TabIndex < Tabs.Num(); ++TabIndex)
		{
			// We do not persist document tabs. Document tabs have a valid InstanceId in addition to a TabType.
			const bool bIsTabPersistable = Tabs[TabIndex].TabId.IsTabPersistable();
			if ( bIsTabPersistable )
			{
				PersistentStack->AddTab(Tabs[TabIndex]);
			}			
		}
		return PersistentStack;
	}
	else
	{
		return TSharedPtr<FTabManager::FLayoutNode>();
	}
}


void SDockingTabStack::ClearReservedSpace()
{
	bShowingTitleBarArea = false;
	TitleBarSlot->SetPadding(0.f);
}

void SDockingTabStack::ReserveSpaceForWindowChrome(EChromeElement Element, bool bIncludePaddingForMenuBar, bool bOnlyMinorTabs)
{
	FMargin ControlsPadding;
	FMargin IconPadding;

#if PLATFORM_MAC
	if (bIncludePaddingForMenuBar)
	{
		static const float TopPaddingForTrafficLightsAndMenuBar = 30.0f;
		// Always add padding on top, because on the Mac there is always either a main menu bar or the "traffic light" buttons (close, minimize, and maximize) above controls.
		// Always add padding to the left, because on the Mac there's no Unreal icon to the left of controls, only the window edge, so we need some space.
		ControlsPadding = FMargin(8.0f, TopPaddingForTrafficLightsAndMenuBar, 0, 0);
	}
	else
	{
		// Without a main menu bar in the title bar, we just need to pad on the left to avoid overlapping with the "traffic light" buttons (close, minimize, and maximize).
		ControlsPadding = FMargin(67.0f, 0, 0, 0);
	}
#else
	static const float TopPaddingForMenuBar = 25.0f;

	static const float LeftPaddingForIcon = FSlateApplication::Get().GetAppIcon()->GetImageSize().X;
	// If we are including top padding for the menu bar we do not need to pad the outer sides since we will be below the left icon and the right controls.
	if (bIncludePaddingForMenuBar)
	{
		ControlsPadding = FMargin(8.f, TopPaddingForMenuBar, 0.f, 0.f);
		IconPadding = FMargin(LeftPaddingForIcon + 12.f, bOnlyMinorTabs ? 5.f : 0.f, 0.f, 0.f);
	}
	else
	{
		ControlsPadding = FMargin(8.f, 2.f, 128.f, 0.f);
		IconPadding = FMargin(25.f, bOnlyMinorTabs ? 5.f : 0.f, 0.f, 0.f);
	}
#endif

	bShowingTitleBarArea = true;
	const FMargin CurrentPadding = TitleBarSlot->GetPadding();
	switch (Element)
	{
		case EChromeElement::Controls:
			TitleBarSlot->SetPadding(CurrentPadding + ControlsPadding);
			break;

		case EChromeElement::Icon:
			TitleBarSlot->SetPadding(CurrentPadding + IconPadding);
			break;

		default:
			ensure(false);
			break;
	}
}

TSharedRef< SDockingTabStack > SDockingTabStack::CreateNewTabStackBySplitting( const SDockingNode::RelativeDirection Direction )
{
	TSharedPtr<SDockingSplitter> ParentNode = ParentNodePtr.Pin();
	check(ParentNode.IsValid());
	
	TSharedRef<SDockingTabStack> NewStack = SNew(SDockingTabStack, FTabManager::NewStack());
	{
		NewStack->SetSizeCoefficient( this->GetSizeCoefficient() );
	}	
	
	ParentNode->PlaceNode( NewStack, Direction, SharedThis(this) );
	return NewStack;
}

void SDockingTabStack::SetParentNode( TSharedRef<class SDockingSplitter> InParent )
{
	SDockingNode::SetParentNode(InParent);

	TitleBarSlot->AttachWidget(TitleBarContent.ToSharedRef());
}

bool SDockingTabStack::IsContentEnabled() const
{
	TSharedRef<FTabManager> TabManager = GetDockArea()->GetTabManager();

	if(!TabManager->IsReadOnly())
	{
		return true;
	}

	// If we are in read only mode, and the foreground tab desires custom behavior (i.e not hidden or disabled) it is enabled
	// and the tab owner is responsible for handling the content in read only mode
	if(TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab())
	{
		return TabManager->GetTabReadOnlyBehavior(ForegroundTab->GetLayoutIdentifier()) == ETabReadOnlyBehavior::Custom;
	}

	return true;
}


/** What should the content area look like for the current tab? */
const FSlateBrush* SDockingTabStack::GetContentAreaBrush() const
{
	TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();
	return (ForegroundTab.IsValid())
		? ForegroundTab->GetContentAreaBrush()
		: FStyleDefaults::GetNoBrush();
}

FMargin SDockingTabStack::GetContentPadding() const
{
	TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();
	return (ForegroundTab.IsValid())
		? ForegroundTab->GetContentPadding()
		: FMargin(0);
}

EVisibility SDockingTabStack::GetTabWellVisibility() const
{	
	const bool bTabWellVisible =
		// If we are playing, we're in transition, so tab is visible.
		ShowHideTabWell.IsPlaying() ||
		// Playing forward expands the tab, so it is always visible then as well.
		!ShowHideTabWell.IsInReverse();

	return (!bTabWellVisible)
		? EVisibility::Collapsed
		: EVisibility::SelfHitTestInvisible;	// Visible, but allow clicks to pass through self (but not children)
}

const FSlateBrush* SDockingTabStack::GetTabWellBrush() const
{
	TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();
	return ( ForegroundTab.IsValid() )
		? ForegroundTab->GetTabWellBrush()
		: FStyleDefaults::GetNoBrush();
}

EVisibility SDockingTabStack::GetUnhideButtonVisibility() const
{
	const bool bShowUnhideButton =
		// If we are playing, we're in transition, so tab is visible.
		ShowHideTabWell.IsPlaying() ||
		// Playing forward expands the tab, so it is always visible then as well.
		ShowHideTabWell.IsInReverse();

	return (bShowUnhideButton)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void SDockingTabStack::ToggleTabWellVisibility()
{
	ShowHideTabWell.Reverse();
}

void SDockingTabStack::MoveForegroundTabToSidebar()
{
	if (TSharedPtr<SDockTab> ForegroundTabPtr = TabWell->GetForegroundTab())
	{
		MoveTabToSidebar(ForegroundTabPtr.ToSharedRef());
	}
}

void SDockingTabStack::MoveTabToSidebar(TSharedPtr<SDockTab> InTab)
{
	if (InTab.IsValid())
	{
		MoveTabToSidebar(InTab.ToSharedRef());
	}
}

void SDockingTabStack::MoveTabToSidebar(TSharedRef<SDockTab> Tab)
{
	const int32 TabIndex = Tabs.IndexOfByPredicate(FTabMatcher(Tab->GetLayoutIdentifier(), ETabState::OpenedTab));
	if(TabIndex != INDEX_NONE)
	{
		ESidebarLocation SidebarLoc = GetDockArea()->AddTabToSidebar(Tab);

		if (SidebarLoc != ESidebarLocation::None)
		{
			Tabs[TabIndex].TabState = ETabState::SidebarTab;
			Tabs[TabIndex].SidebarLocation = SidebarLoc;

			TabWell->RemoveAndDestroyTab(Tab, ELayoutModification::TabRemoval_Sidebar);
		}
	}
}

void SDockingTabStack::RestoreTabFromSidebar(TSharedRef<SDockTab> Tab)
{
	const int32 TabIndex = Tabs.IndexOfByPredicate(FTabMatcher(Tab->GetLayoutIdentifier(), ETabState::SidebarTab));
	if (TabIndex != INDEX_NONE)
	{
		FTabManager::FTab& TabInfo = Tabs[TabIndex];

		TabInfo.SidebarSizeCoefficient = 0;
		// Set the sate to closed so its reopened by OpenTab
		TabInfo.TabState = ETabState::ClosedTab;
		TabInfo.SidebarLocation = ESidebarLocation::None;
		OpenTab(Tab);
	}
}

FReply SDockingTabStack::UnhideTabWell()
{
	SetTabWellHidden(false);

	return FReply::Handled();
}

bool SDockingTabStack::CanHideTabWell() const
{
	const TSharedPtr<SDockingSplitter> ParentNode = ParentNodePtr.Pin();

	if (ParentNode && !ParentNode->GetAllChildTabs().IsEmpty())
	{
		// Is target tab located at the upper and left most among tabs in the parent window(as first child). Unreal icon will overlap golden triangle(unhide button) when the tab is the first child of the window.
		const bool bIsUpperLeftmostTab = (FGlobalTabmanager::Get()->GetActiveTab() == ParentNode->GetAllChildTabs()[0]);
		// Is target tab in the Floating Window. Unreal icon will overlap when the tab is in the floating window(without menu)
		const bool bIsInFloatingWindow = ParentNode->GetDockArea()->GetParentWindow().IsValid();

		return GetNumTabs() == 1 && FGlobalTabmanager::Get()->CanSetAsActiveTab(GetTabs()[0]) && !(bIsUpperLeftmostTab && bIsInFloatingWindow);
	}

	/* in the case where there are no parent splitter or child tabs, it is invalid to hide the tab well.
	 * The likely case for this would be in a sidebar flyout  */
	return false;
}

bool SDockingTabStack::CanCloseForegroundTab() const
{
	return CanCloseTab(TabWell->GetForegroundTab());
}

bool SDockingTabStack::CanCloseTab(TSharedPtr<SDockTab> InTab) const
{
	return InTab.IsValid() && InTab->CanCloseTab();
}

int32 SDockingTabStack::GetIncrementFromCloseTabsDirection(ECloseTabsDirection Direction)
{
	switch (Direction)
	{
		case ECloseTabsDirection::Left: 	return -1;
		case ECloseTabsDirection::Right:	return  1;
		default: 							return  1;
	}
}

bool SDockingTabStack::CanCloseTabsInDirectionOfForegroundTab(ECloseTabsDirection Direction) const
{
	return CanCloseTabsInDirectionOfTab(TabWell->GetForegroundTab(), Direction);
}

bool SDockingTabStack::CanCloseTabsInDirectionOfTab(TSharedPtr<SDockTab> InTab, ECloseTabsDirection InDirection) const
{
	if (!InTab.IsValid())
	{
		return false;
	}

	const int32 TabIndex = TabWell->GetTabIndex(InTab);
	if (TabIndex == INDEX_NONE)
	{
		return false;
	}

	const ETabRole VisualTabRole = InTab->GetVisualTabRole();
	if (VisualTabRole == ETabRole::DocumentTab || VisualTabRole == ETabRole::MajorTab)
	{
		const TArray< TSharedRef<SDockTab> > MyTabs = this->GetTabs().AsArrayCopy();
		const int32 Increment = GetIncrementFromCloseTabsDirection(InDirection);
		for (int32 Index = TabIndex + Increment; Index >= 0 && Index < MyTabs.Num(); Index += Increment)
		{
			TSharedRef<SDockTab> Tab = MyTabs[Index];
			if (Tab->CanCloseTab())
			{
				return true;
			}
		}
	}
	return false;
}

bool SDockingTabStack::CanCloseAllButForegroundTab() const
{
	// If the active tab is a document tab or major tab and there is at least 1 other closeable tab, offer to close the others
	return CanCloseAllButTab(TabWell->GetForegroundTab());
}

bool SDockingTabStack::CanCloseAllButTab(TSharedPtr<SDockTab> InTab) const
{
	if (!InTab.IsValid())
	{
		return false;
	}
	
	const ETabRole VisualTabRole = InTab->GetVisualTabRole();
	if ((VisualTabRole == ETabRole::DocumentTab || VisualTabRole == ETabRole::MajorTab) && (TabWell->GetNumTabs() > 1))
	{
		const TArray< TSharedRef<SDockTab> > MyTabs = this->GetTabs().AsArrayCopy();
		for (int32 TabIndex = 0; TabIndex < MyTabs.Num(); ++TabIndex)
		{
			TSharedRef<SDockTab> Tab = MyTabs[TabIndex];
			if (Tab != InTab && Tab->CanCloseTab())
			{
				return true;
			}
		}
	}

	return false;
}

bool SDockingTabStack::CanMoveForegroundTabToSidebar() const
{
	if(TSharedPtr<SDockTab> ForegroundTabPtr = TabWell->GetForegroundTab())
	{
		return CanMoveTabToSideBar(ForegroundTabPtr.ToSharedRef());
	}

	return false;
}

bool SDockingTabStack::CanMoveTabToSidebar(TSharedPtr<SDockTab> InTab) const
{
	if (InTab.IsValid())
	{
		return CanMoveTabToSideBar(InTab.ToSharedRef());
	}

	return false;
}

bool SDockingTabStack::CanMoveTabToSideBar(TSharedRef<SDockTab> Tab) const
{
	const FTabId TabIdBeingClosed = Tab->GetLayoutIdentifier();
	// Only persistable non-major tabs can be put into a sidebar. There must also be more than one tab or else adding to a sidebar doesnt make si
	return TabIdBeingClosed.IsTabPersistable() && Tab->GetVisualTabRole() != ETabRole::MajorTab && GetDockArea()->GetNumTabs() > 1;
}

bool SDockingTabStack::IsTabAllowedInSidebar(TSharedPtr<SDockTab> Tab) const
{
	// Major tabs are not allowed to be sidebared
	if (Tab.IsValid())
	{
		if (TSharedPtr<FTabManager> TabManager = Tab->GetTabManagerPtr())
		{
			return !Tab->IsTabLocked() && Tab->GetVisualTabRole() != ETabRole::MajorTab && TabManager->IsTabAllowedInSidebar(Tab->GetLayoutIdentifier());
		}
	}
	return false;
}

TSharedPtr<SDockTab> SDockingTabStack::GetSelectedTabFromWell()
{
	if (TabWell.IsValid())
	{
		return TabWell->GetForegroundTab();
	}
	else
	{
		return nullptr;
	}
}

SSplitter::ESizeRule SDockingTabStack::GetSizeRule() const
{
	const TSlotlessChildren<SDockTab> ChildTabs = GetTabs();

	const int32 NumTabs = ChildTabs.Num();
	if (NumTabs <= 0)
	{
		return SSplitter::FractionOfParent;
	}

	const TSharedPtr<SDockingSplitter> ParentSplitter = ParentNodePtr.Pin();
	const TArrayView<const TSharedRef<SDockTab>> ChildTabsArray = MakeArrayView(&ChildTabs[0], ChildTabs.Num());

	const bool bAllTabsAllowAutosize = Algo::AllOf(ChildTabsArray
		,[&ParentSplitter](const TSharedRef<SDockTab>& InTab)
	{
		const EDockTabAutosizePolicy Policy = InTab->GetAutosizePolicy();

		// No parent splitter: treat like generic autosize check
		if (!ParentSplitter.IsValid())
		{
			return Policy != EDockTabAutosizePolicy::None;
		}

		const EOrientation ParentOrientation = ParentSplitter->GetOrientation();

		switch (Policy)
		{
		default:
		case EDockTabAutosizePolicy::None:
			return false;
		case EDockTabAutosizePolicy::LayoutAxis:
			return true;
		case EDockTabAutosizePolicy::HorizontalOnly:
			return ParentOrientation == Orient_Horizontal;
		case EDockTabAutosizePolicy::VerticalOnly:
			return ParentOrientation == Orient_Vertical;
		}
	});

	return bAllTabsAllowAutosize ? SSplitter::SizeToContent : SSplitter::FractionOfParent;
}

void SDockingTabStack::SetTabWellHidden( bool bShouldHideTabWell )
{
	// If the tab well is already hidden or visible, don't replay the animations.
	if ( (bShouldHideTabWell && IsTabWellHidden()) || (!bShouldHideTabWell && !IsTabWellHidden()))
	{
		return;
	}
	
	if (bShouldHideTabWell)
	{
		ShowHideTabWell.PlayReverse( this->AsShared() );
	}
	else
	{
		ShowHideTabWell.Play( this->AsShared() );
	}
}

bool SDockingTabStack::IsTabWellHidden() const
{
	return ShowHideTabWell.IsInReverse();
}

FVector2D SDockingTabStack::GetTabWellScale() const
{
	return FVector2D(1,ShowHideTabWell.GetLerp());
}

FVector2D SDockingTabStack::GetUnhideTabWellButtonScale() const
{
	return FMath::Lerp(FVector2D::UnitVector, 8*FVector2D::UnitVector, ShowHideTabWell.GetLerp());
}

FSlateColor SDockingTabStack::GetUnhideTabWellButtonOpacity() const
{
	return FLinearColor( 1,1,1, 1.0f - ShowHideTabWell.GetLerp() );
}

const FSlateBrush* SDockingTabStack::GetTabStackBorderImage() const
{
	static const FSlateBrush* MajorTabBackgroundBrush = FAppStyle::Get().GetBrush("Brushes.Title");
	static const FSlateBrush* MinorTabBackgroundBrush = FAppStyle::Get().GetBrush("Brushes.Background");
	return bShowingTitleBarArea ? MajorTabBackgroundBrush : MinorTabBackgroundBrush;
}

int32 SDockingTabStack::OpenPersistentTab( const FTabId& TabId, int32 OpenLocationAmongActiveTabs )
{
	int32 ExistingClosedTabIndex = Tabs.IndexOfByPredicate(FTabMatcher(TabId, static_cast<ETabState::Type>(ETabState::ClosedTab|ETabState::SidebarTab)));

	if (ExistingClosedTabIndex == INDEX_NONE)
	{
		// Check for a persistent opened tab that isn't actually opened ( in the live tabs ).
		// This situation can happen in some corner cases, e.g:
		// - If the process is terminated after a periodic layout save.
		// - If the layout save on editor shutdown occurs before an editor mode is deactivated.
		// In any case we want to treat this opened tab as a closed tab instead of creating a new tab.
		const bool bTreatIndexNoneAsWildcard = false;
		const int32 ExistingOpenedTabIndex = Tabs.IndexOfByPredicate(FTabMatcher(TabId, static_cast<ETabState::Type>(ETabState::OpenedTab), bTreatIndexNoneAsWildcard));
		if (ExistingOpenedTabIndex != INDEX_NONE)
		{
			bool bHasLiveTab = false;
			const TArray< TSharedRef<SDockTab> > LiveTabs = this->GetTabs().AsArrayCopy();
			for (int32 TabIndex = 0; TabIndex < LiveTabs.Num(); ++TabIndex)
			{
				if (TabId == LiveTabs[TabIndex]->GetLayoutIdentifier())
				{
					bHasLiveTab = true;
					break;
				}
			}
			if (!bHasLiveTab)
			{
				ExistingClosedTabIndex = ExistingOpenedTabIndex;
			}
		}
	}

	if (OpenLocationAmongActiveTabs == INDEX_NONE)
	{						
		if (ExistingClosedTabIndex != INDEX_NONE)
		{
			FTabManager::FTab& Tab = Tabs[ExistingClosedTabIndex];
			Tab.TabState = Tab.SidebarLocation == ESidebarLocation::None ? ETabState::OpenedTab : ETabState::SidebarTab;
			return ExistingClosedTabIndex;
		}
		else
		{
			// This tab was never opened in the tab stack before; add it.
			Tabs.Add( FTabManager::FTab( TabId, ETabState::OpenedTab ) );
			return Tabs.Num()-1;
		}
	}
	else
	{
		// @TODO: This branch maybe needs to become a separate function: More like MoveOrAddTab

		// We need to open a tab in a specific location.

		// We have the index of the open tab where to insert. But we need the index in the persistent
		// array, which is an ordered list of all tabs ( both open and closed ).
		int32 OpenLocationInGlobalList=INDEX_NONE;
		for (int32 TabIndex = 0, OpenTabIndex=0; TabIndex < Tabs.Num() && OpenLocationInGlobalList == INDEX_NONE; ++TabIndex)
		{
			const bool bThisTabIsOpen = (Tabs[TabIndex].TabState == ETabState::OpenedTab);
			if ( bThisTabIsOpen )
			{
				if (OpenTabIndex == OpenLocationAmongActiveTabs)
				{
					OpenLocationInGlobalList = TabIndex;
				}
				++OpenTabIndex;
			}
		}

		if (OpenLocationInGlobalList == INDEX_NONE)
		{
			OpenLocationInGlobalList = Tabs.Num();
		}

		if ( ExistingClosedTabIndex == INDEX_NONE )
		{
			// Create a new tab.
			Tabs.Insert( FTabManager::FTab( TabId, ETabState::OpenedTab ), OpenLocationInGlobalList );
			return OpenLocationAmongActiveTabs;
		}
		else
		{
			// Move the existing closed tab to the new desired location
			FTabManager::FTab TabToMove = Tabs[ExistingClosedTabIndex];
			Tabs.RemoveAt( ExistingClosedTabIndex );

			// If the element we removed was before the insert location, subtract one since the index was shifted during the removal
			if ( ExistingClosedTabIndex <= OpenLocationInGlobalList )
			{
				OpenLocationInGlobalList--;
			}

			// Mark the tab opened
			TabToMove.TabState = ETabState::OpenedTab;

			Tabs.Insert( TabToMove, OpenLocationInGlobalList );
			return OpenLocationAmongActiveTabs;
		}
	}
}

int32 SDockingTabStack::ClosePersistentTab( const FTabId& TabId )
{
	const int32 TabIndex = Tabs.IndexOfByPredicate(FTabMatcher(TabId, static_cast<ETabState::Type>(ETabState::OpenedTab|ETabState::SidebarTab)));
	if (TabIndex != INDEX_NONE)
	{
		Tabs[TabIndex].TabState = ETabState::ClosedTab;
	}
	return TabIndex;
}

void SDockingTabStack::RemovePersistentTab( const FTabId& TabId )
{
	const int32 TabIndex = Tabs.IndexOfByPredicate(FTabMatcher(TabId));
	if(TabIndex != INDEX_NONE)
	{
		Tabs.RemoveAtSwap(TabIndex);
	}
}

EVisibility SDockingTabStack::GetMaximizeSpacerVisibility() const
{
/*
	if(GetDockArea().IsValid() && GetDockArea()->GetParentWindow().IsValid())
	{
		if (GetDockArea()->GetParentWindow()->IsWindowMaximized())
		{
			return EVisibility::Collapsed;
		}
		else
		{
			return EVisibility::SelfHitTestInvisible;
		}
	}*/
	return EVisibility::Collapsed;
}


#if DEBUG_TAB_MANAGEMENT

FString SDockingTabStack::ShowPersistentTabs() const
{
	FString AllTabs;
	for (int32 TabIndex=0; TabIndex < Tabs.Num(); ++TabIndex)
	{
		AllTabs += (Tabs[TabIndex].TabState == ETabState::OpenedTab ) ? TEXT("[^]") : TEXT("[x]");
		AllTabs += Tabs[TabIndex].TabId.ToString();
		AllTabs += TEXT(" ");
	}
	return AllTabs;
}

#endif

void SDockingTabStack::BindTabCommands()
{
	check(!ActionList.IsValid());

	ActionList = MakeShareable(new FUICommandList);

	const FTabCommands& Commands = FTabCommands::Get();
	ActionList->MapAction(Commands.CloseMajorTab, FExecuteAction::CreateSP(this, &SDockingTabStack::ExecuteCloseMajorTabCommand), FCanExecuteAction::CreateSP(this, &SDockingTabStack::CanExecuteCloseMajorTabCommand));
	ActionList->MapAction(Commands.CloseMinorTab, FExecuteAction::CreateSP(this, &SDockingTabStack::ExecuteCloseMinorTabCommand), FCanExecuteAction::CreateSP(this, &SDockingTabStack::CanExecuteCloseMinorTabCommand));
	ActionList->MapAction(Commands.CloseFocusedTab, FExecuteAction::CreateSP(this, &SDockingTabStack::ExecuteCloseFocusedTabCommand), FCanExecuteAction::CreateSP(this, &SDockingTabStack::CanExecuteCloseFocusedTabCommand));
}

void SDockingTabStack::ExecuteCloseMajorTabCommand()
{
	// Close this stack's foreground tab (if it's a major tab)
	if (CanExecuteCloseMajorTabCommand())
	{
		TabWell->GetForegroundTab()->RequestCloseTab();
	}
}

bool SDockingTabStack::CanExecuteCloseMajorTabCommand()
{
	// Can we close this stack's foreground tab (if it's a major tab)?
	TSharedPtr<SDockTab> ForegroundTab = TabWell->GetForegroundTab();
	return ForegroundTab.IsValid() && !FGlobalTabmanager::Get()->CanSetAsActiveTab(ForegroundTab);
}

void SDockingTabStack::ExecuteCloseMinorTabCommand()
{
	if (CanExecuteCloseMinorTabCommand())
	{
		// Close the global active (minor) tab
		FGlobalTabmanager::Get()->GetActiveTab()->RequestCloseTab();
	}
}

bool SDockingTabStack::CanExecuteCloseMinorTabCommand()
{
	if (TSharedPtr<SDockingArea> DockArea = GetDockArea())
	{
		TSharedPtr<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
		TSharedPtr<SDockTab> ActiveTab = GlobalTabManager->GetActiveTab();
		if (ActiveTab.IsValid())
		{
			if (ActiveTab->GetParentWindow() == DockArea->GetParentWindow())
			{
				// Can close the global active (minor) tab because it's in the same window as this tab stack
				return true;
			}
		}
	}
	return false;
}

void SDockingTabStack::ExecuteCloseFocusedTabCommand()
{
	if (CanExecuteCloseMinorTabCommand())
	{
		ExecuteCloseMinorTabCommand();
	}
	else
	{
		ExecuteCloseMajorTabCommand();
	}
}

bool SDockingTabStack::CanExecuteCloseFocusedTabCommand()
{
	return CanExecuteCloseMinorTabCommand() || CanExecuteCloseMajorTabCommand();
}

void SDockingTabStack::OnResized()
{
	if (TSharedPtr<SDockingArea> DockArea = GetDockArea())
	{
		DockArea->GetTabManager()->RequestSavePersistentLayout();
	}
}

#undef LOCTEXT_NAMESPACE
