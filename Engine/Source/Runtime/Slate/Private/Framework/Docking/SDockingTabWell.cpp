// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingTabWell.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Framework/Docking/FDockingDragOperation.h"
#include "HAL/PlatformApplicationMisc.h"

const FVector2D FDockingConstants::MaxMinorTabSize(160.f, 25.0f);
const FVector2D FDockingConstants::MaxMajorTabSize(210.f, 50.f);
const float FDockingConstants::MaxTabSizeNoNameWidth(53.f);
const float FDockingConstants::MaxTabSizeNoNameCantCloseWidth(32.f);

const FVector2D FDockingConstants::GetMaxTabSizeFor( ETabRole TabRole )
{
	return (TabRole == ETabRole::MajorTab)
		? MaxMajorTabSize
		: MaxMinorTabSize;
}

SDockingTabWell::SDockingTabWell()
: Tabs(this)
{
}

void SDockingTabWell::Construct( const FArguments& InArgs )
{
	ForegroundTabIndex = INDEX_NONE;
	TabBeingDraggedPtr = nullptr;
	ChildBeingDraggedOffset = 0.0f;
	TabGrabOffsetFraction = FVector2D::ZeroVector;
		
	SeparatorBrush = nullptr; // No separater between tabs

	// We need a valid parent here. TabPanels must exist in a SDockingNode
	check( InArgs._ParentStackNode.Get().IsValid() );
	ParentTabStackPtr = InArgs._ParentStackNode.Get();
}

const TSlotlessChildren<SDockTab>& SDockingTabWell::GetTabs() const
{
	return Tabs;
}

int32 SDockingTabWell::GetTabIndex(const TSharedPtr<SDockTab>& InTab) const
{
	if (!InTab.IsValid())
	{
		return INDEX_NONE;
	}

	return Tabs.Find(InTab.ToSharedRef());
}

int32 SDockingTabWell::GetNumTabs() const
{
	return Tabs.Num();
}

void SDockingTabWell::AddTab( const TSharedRef<SDockTab>& InTab, int32 AtIndex, bool bKeepInactive)
{
	InTab->SetParent(SharedThis(this));

	// Add the tab and implicitly activate it.
	if (AtIndex == INDEX_NONE)
	{
		this->Tabs.Add( InTab );
		if (!bKeepInactive)
		{
			BringTabToFront(Tabs.Num() - 1);
		}
	}
	else
	{
		AtIndex = FMath::Clamp( AtIndex, 0, Tabs.Num() );

		if (AtIndex <= ForegroundTabIndex)
		{
			// Update the currently active index (Otherwise we don't broadcast the right info)
			++ForegroundTabIndex;
		}

		this->Tabs.Insert( InTab, AtIndex );

		if (!bKeepInactive)
		{
			BringTabToFront(AtIndex);
		}
	}


	const TSharedPtr<SDockingTabStack> ParentTabStack = ParentTabStackPtr.Pin();
	if (ParentTabStack.IsValid() && ParentTabStack->GetDockArea().IsValid())
	{
		ParentTabStack->GetDockArea()->GetTabManager()->GetPrivateApi().OnTabOpening( InTab );
	}
}


void SDockingTabWell::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	// The specialized TabWell is dedicated to arranging tabs.
	// Tabs have uniform sizing (all tabs the same size).
	// TabWell also ignores widget visibility, as it is not really
	// relevant.


	// The tab that is being dragged by the user, if any.
	TSharedPtr<SDockTab> TabBeingDragged = TabBeingDraggedPtr;
		
	const int32 NumChildren = Tabs.Num();

	// Tabs have a uniform size.
	const FVector2D ChildSize = ComputeChildSize(AllottedGeometry);
	const FVector2D ChildSizeNoName = FVector2D(FDockingConstants::MaxTabSizeNoNameWidth, ChildSize.Y);
	const FVector2D ChildSizeNoNameCantClose = FVector2D(FDockingConstants::MaxTabSizeNoNameCantCloseWidth, ChildSize.Y);

	// Get the correct child size to use for the current tab
	const auto GetChildSizeToUse = [ChildSize, ChildSizeNoName, ChildSizeNoNameCantClose] (const TSharedPtr<SDockTab>& InCurTab)
		{
			if (!InCurTab.IsValid())
			{
				return ChildSize;
			}

			if (InCurTab->IsTabNameHidden())
			{
				if (InCurTab->CanCloseTab())
				{
					return ChildSizeNoName;
				}
				return ChildSizeNoNameCantClose;
			}
			return ChildSize;
		};

	int32 ExpectedTabDropIndex = INDEX_NONE;
	if (TabBeingDragged.IsValid())
	{
		// Get the expected DropIndex of the tab being dragged
		ExpectedTabDropIndex = ComputeChildDropIndex(AllottedGeometry, TabBeingDragged.ToSharedRef());
	}

	// Arrange all the tabs left to right.
	float XOffset = 0;

	int32 TabDropIndexVisualGap = INDEX_NONE;

	for( int32 TabIndex=0; TabIndex < NumChildren; ++TabIndex )
	{
		const TSharedRef<SDockTab> CurTab = Tabs[TabIndex];
		FVector2D ChildSizeToUse = GetChildSizeToUse(CurTab);

		const float DraggedChildCenter = ChildBeingDraggedOffset + ChildSizeToUse.X / 2;

		const float ChildWidthWithOverlap = ChildSizeToUse.X - CurTab->GetOverlapWidth();

		// This tab being dragged is arranged later.  It should not be arranged twice
		if (CurTab == TabBeingDragged)
		{
			continue;
		}

		// Is this spot reserved from the tab that is being dragged?
		if (TabBeingDragged.IsValid() && XOffset <= DraggedChildCenter && DraggedChildCenter < (XOffset + ChildWidthWithOverlap))
		{
			// If the expected tab index is greater than the current tab index it means that we are not allowed to place it in this current TabIndex, so visually don't add the gap, but force it at the same Expected Index
			// Otherwise add it at the same TabIndex
			TabDropIndexVisualGap = ExpectedTabDropIndex > TabIndex ? ExpectedTabDropIndex : TabIndex;
		}

		// If the gap is expected at this TabIndex add the offset here
		if (TabDropIndexVisualGap == TabIndex)
		{
			XOffset += ChildWidthWithOverlap;
		}

		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild(CurTab, FVector2D(XOffset, 0), ChildSizeToUse) );

		XOffset += ChildWidthWithOverlap;
	}
		
	// Arrange the tab currently being dragged by the user, if any
	if ( TabBeingDragged.IsValid() )
	{
		const FVector2D ChildSizeToUse = GetChildSizeToUse(TabBeingDragged);
		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild( TabBeingDragged.ToSharedRef(), FVector2D(ChildBeingDraggedOffset,0), ChildSizeToUse) );
	}
}
	
int32 SDockingTabWell::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// When we are dragging a tab, it must be painted on top of the other tabs, so we cannot
	// just reuse the Panel's default OnPaint.


	// The TabWell has no visualization of its own; it just visualizes its child tabs.
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;

	TSharedPtr<SDockTab> ForegroundTab = GetForegroundTab();
	FArrangedWidget* ForegroundTabGeometry = nullptr;
	
	// Draw all inactive tabs first, from last, to first, so that the inactive tabs
	// that come later, are drawn behind tabs that come before it.
	for (int32 ChildIndex = ArrangedChildren.Num() - 1; ChildIndex >= 0; --ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		if (CurWidget.Widget == ForegroundTab)
		{
			ForegroundTabGeometry = &CurWidget;
		}
		else
		{
			bool bShouldDrawSeparator = false;
			if (SeparatorBrush && SeparatorBrush->DrawAs != ESlateBrushDrawType::NoDrawType && ArrangedChildren.IsValidIndex(ChildIndex + 1))
			{
				const FArrangedWidget& PrevWidget = ArrangedChildren[ChildIndex + 1];
				bShouldDrawSeparator = !CurWidget.Widget->IsHovered() && !PrevWidget.Widget->IsHovered() && PrevWidget.Widget != ForegroundTab;
			}

			FSlateRect ChildClipRect = MyCullingRect.IntersectionWith( CurWidget.Geometry.GetLayoutBoundingRect() );
			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint( Args.WithNewParent(this), CurWidget.Geometry, ChildClipRect, OutDrawElements, MaxLayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );
		
			if(bShouldDrawSeparator)
			{
				const float SeparatorHeight = CurWidget.Geometry.GetLocalSize().Y * .65f;

				// Center the separator
				float Offset = (CurWidget.Geometry.GetLocalSize().Y - SeparatorHeight) / 2.0f;
				FPaintGeometry Geometry = CurWidget.Geometry.ToPaintGeometry(FVector2f(1.0f, SeparatorHeight), FSlateLayoutTransform(FVector2f(CurWidget.Geometry.GetLocalSize().X + 1.0f, Offset)));
		
				// This code rounds the position of the widget so we don't end up on half a pixel and end up with a larger size separator than we want
				FSlateRenderTransform NewTransform = Geometry.GetAccumulatedRenderTransform();
				NewTransform.SetTranslation(NewTransform.GetTranslation().RoundToVector());
				Geometry.SetRenderTransform(NewTransform);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					MaxLayerId,
					Geometry,
					SeparatorBrush,
					ESlateDrawEffect::None,
					SeparatorBrush->GetTint(InWidgetStyle)
				);
			}

			MaxLayerId = FMath::Max( MaxLayerId, CurWidgetsMaxLayerId );
		}
	}

	// Draw active tab in front
	if (ForegroundTab != TSharedPtr<SDockTab>())
	{
		checkSlow(ForegroundTabGeometry);
		FSlateRect ChildClipRect = MyCullingRect.IntersectionWith( ForegroundTabGeometry->Geometry.GetLayoutBoundingRect() );
		const int32 CurWidgetsMaxLayerId = ForegroundTabGeometry->Widget->Paint( Args.WithNewParent(this), ForegroundTabGeometry->Geometry, ChildClipRect, OutDrawElements, MaxLayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );
		MaxLayerId = FMath::Max( MaxLayerId, CurWidgetsMaxLayerId );
	}

	return MaxLayerId;
}

FVector2D SDockingTabWell::ComputeDesiredSize( float ) const
{
	FVector2D DesiredSizeResult(0,0);

	TSharedPtr<SDockTab> TabBeingDragged = TabBeingDraggedPtr;

	for ( int32 TabIndex=0; TabIndex < Tabs.Num(); ++TabIndex )
	{
		// Currently not respecting Visibility because tabs cannot be invisible.
		const TSharedRef<SDockTab>& SomeTab = Tabs[TabIndex];

		// Tab being dragged is computed later
		if(SomeTab != TabBeingDragged)
		{
			const FVector2D SomeTabDesiredSize = SomeTab->GetDesiredSize();
			DesiredSizeResult.X += SomeTabDesiredSize.X;
			DesiredSizeResult.Y = FMath::Max(SomeTabDesiredSize.Y, DesiredSizeResult.Y);
		}
	}

	if ( TabBeingDragged.IsValid() )
	{
		const FVector2D SomeTabDesiredSize = TabBeingDragged->GetDesiredSize();
		DesiredSizeResult.X += SomeTabDesiredSize.X;
		DesiredSizeResult.Y = FMath::Max(SomeTabDesiredSize.Y, DesiredSizeResult.Y);
	}

	return DesiredSizeResult;
}


FChildren* SDockingTabWell::GetChildren()
{
	return &Tabs;
}


FVector2D SDockingTabWell::ComputeChildSize( const FGeometry& AllottedGeometry ) const
{
	const int32 NumChildren = Tabs.Num();

	/** Assume all tabs overlap the same amount. */
	const float OverlapWidth = (NumChildren > 0)
		? Tabs[0]->GetOverlapWidth()
		: 0.0f;

	// All children shall be the same size: evenly divide the alloted area.
	// If we are dragging a tab, don't forget to take it into account when dividing.
	const FVector2D ChildSize = TabBeingDraggedPtr.IsValid()
		? FVector2D( (AllottedGeometry.GetLocalSize().X - OverlapWidth) / ( NumChildren + 1 ) + OverlapWidth, AllottedGeometry.GetLocalSize().Y )
		: FVector2D( (AllottedGeometry.GetLocalSize().X - OverlapWidth) / NumChildren + OverlapWidth, AllottedGeometry.GetLocalSize().Y );

	// Major vs. Minor tabs have different tab sizes.
	// We will make our choice based on the first tab we encounter.
	TSharedPtr<SDockTab> FirstTab = (NumChildren > 0)
		? Tabs[0]
		: TabBeingDraggedPtr;

	// If there are no tabs in this tabwell, assume minor tabs.
	FVector2D MaxTabSize(0,0);
	if ( FirstTab.IsValid() )
	{
		const ETabRole RoleToUse = FirstTab->GetVisualTabRole();
		MaxTabSize = FDockingConstants::GetMaxTabSizeFor(RoleToUse);
	}
	else
	{
		MaxTabSize = FDockingConstants::MaxMinorTabSize;
	}

	// Don't let the tabs get too big, or they'll look ugly.
	return FVector2D (
		FMath::Min( ChildSize.X, MaxTabSize.X ),
		FMath::Min( ChildSize.Y, MaxTabSize.Y )
	);
}


float SDockingTabWell::ComputeDraggedTabOffset( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FVector2D& InTabGrabOffsetFraction ) const
{
	const FVector2D ComputedChildSize = ComputeChildSize(MyGeometry);
	const FVector2D ChildSizeNoName = FVector2D(FDockingConstants::MaxTabSizeNoNameWidth, ComputedChildSize.Y);
	const FVector2D ChildSizeNoNameCantClose = FVector2D(FDockingConstants::MaxTabSizeNoNameCantCloseWidth, ComputedChildSize.Y);

	FVector2D ChildSizeToUse = ComputedChildSize;
	if (TabBeingDraggedPtr.IsValid() && TabBeingDraggedPtr->IsTabNameHidden())
	{
		ChildSizeToUse = TabBeingDraggedPtr->CanCloseTab() ? ChildSizeNoName : ChildSizeNoNameCantClose;
	}

	return MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).X - InTabGrabOffsetFraction .X * ChildSizeToUse.X;
}


int32 SDockingTabWell::ComputeChildDropIndex(const FGeometry& MyGeometry, const TSharedRef<SDockTab>& TabBeingDragged) const
{
	const float ChildWidth = ComputeChildSize(MyGeometry).X;
	const float ChildWidthWithOverlap = ChildWidth - TabBeingDragged->GetOverlapWidth();
	float DraggedChildCenter = ChildBeingDraggedOffset + ChildWidth / 2;

	// Consider difference in TabSize to adjust the dropped index to the correct one
	float XOffset = 0.f;
	for ( int32 TabIndex=0; TabIndex < Tabs.Num(); ++TabIndex )
	{
		if (XOffset >= ChildBeingDraggedOffset)
		{
			break;
		}

		XOffset += ChildWidth;
		const TSharedRef<SDockTab>& SomeTab = Tabs[TabIndex];
		if (SomeTab->IsTabNameHidden())
		{
			if (!SomeTab->CanCloseTab())
			{
				DraggedChildCenter += ChildWidth - FDockingConstants::MaxTabSizeNoNameCantCloseWidth;
			}
			else
			{
				DraggedChildCenter += ChildWidth - FDockingConstants::MaxTabSizeNoNameWidth;
			}
		}
	}

	// If this is the LevelEditor Area other Tabs are not allowed to be placed in the first and/or second position as those position are fixed for the HomeScreen (if enabled) and the LevelEditor
	TArray<TSharedRef<SDockTab>> TabsArray = Tabs.AsArrayCopy();
	TSharedRef<SDockTab>* LevelEditorTab = TabsArray.FindByPredicate([](TSharedRef<SDockTab> InDockTab)
		{
			return InDockTab->GetLayoutIdentifier().TabType == FName("LevelEditor");
		});

	// if we found the LevelEditor tab than this is the LevelEditor PrimaryArea
	const bool bIsLevelEditorPrimaryArea = LevelEditorTab != nullptr;
	const bool bIsHomeScreenEnabled = UE::Editor::HomeScreen::IsHomeScreenEnabled();

	int32 MinClamp = 0;
	if (bIsLevelEditorPrimaryArea)
	{
		MinClamp = bIsHomeScreenEnabled ? /** First 2 are locked from the HomeScreen and the LevelEditor */ 2 : /** First is locked from the LevelEditor */ 1;
	}

	const int32 DropLocationIndex = FMath::Clamp(static_cast<int32>(DraggedChildCenter / ChildWidthWithOverlap), MinClamp, Tabs.Num());
	return DropLocationIndex;
}


FReply SDockingTabWell::StartDraggingTab( TSharedRef<SDockTab> TabToStartDragging, FVector2D InTabGrabOffsetFraction, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FTabManager> TabManager = TabToStartDragging->GetTabManagerPtr();
	if (!TabManager.IsValid())
	{
		return FReply::Handled();
	}

	const bool bCanLeaveTabWell = TabManager->GetPrivateApi().CanTabLeaveTabWell( TabToStartDragging );

	// We are about to start dragging a tab, so make sure its offset is correct
	this->ChildBeingDraggedOffset = ComputeDraggedTabOffset( MouseEvent.FindGeometry(SharedThis(this)), MouseEvent, InTabGrabOffsetFraction );

	// Tha tab well keeps track of which tab we are dragging; we treat is specially during rendering and layout.
	TabBeingDraggedPtr = TabToStartDragging;	
	TabGrabOffsetFraction = InTabGrabOffsetFraction;
	Tabs.Remove(TabToStartDragging);
	

	if (bCanLeaveTabWell)
	{
		// We just removed the foreground tab.
		ForegroundTabIndex = INDEX_NONE;
		ParentTabStackPtr.Pin()->OnTabRemoved(TabToStartDragging->GetLayoutIdentifier());

#if PLATFORM_MAC
		// On Mac we need to activate the app as we may be dragging a window that is set to be invisible if the app is inactive
		FPlatformApplicationMisc::ActivateApplication();
#endif
		TSharedPtr<SDockingArea> DockArea = GetDockArea();

		if (TabToStartDragging->GetTabRole() == ETabRole::MajorTab)
		{
			DockArea->DetachPanelDrawerArea();
		}

		// Start dragging.
		TSharedRef<FDockingDragOperation> DragDropOperation =
			FDockingDragOperation::New(
				TabToStartDragging,
				InTabGrabOffsetFraction,
				DockArea.ToSharedRef(),
				ParentTabStackPtr.Pin()->GetTabStackGeometry().GetLocalSize()
			);

		return FReply::Handled().BeginDragDrop( DragDropOperation );
	}
	else
	{
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
}

void SDockingTabWell::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		if (DragDropOperation->CanDockInNode(ParentTabStackPtr.Pin().ToSharedRef(), FDockingDragOperation::DockingViaTabWell))
		{
			// The user dragged a tab into this TabWell.

			// Update the state of the DragDropOperation to reflect this change.
			DragDropOperation->OnTabWellEntered( SharedThis(this) );

			// Preview the position of the tab in the TabWell
			this->TabBeingDraggedPtr = DragDropOperation->GetTabBeingDragged();

			// Add the tab widget to the well when the tab is dragged in
			Tabs.Add(TabBeingDraggedPtr.ToSharedRef());

			this->TabGrabOffsetFraction = DragDropOperation->GetTabGrabOffsetFraction();
			
			// The user should see the contents of the tab that we're dragging.
			ParentTabStackPtr.Pin()->SetNodeContent(DragDropOperation->GetTabBeingDragged()->GetContent(), FDockingStackOptionalContent());
		}
	}
}

void SDockingTabWell::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		TSharedRef<SDockingTabStack> ParentTabStack = ParentTabStackPtr.Pin().ToSharedRef();
		TSharedPtr<SDockTab> TabBeingDragged = this->TabBeingDraggedPtr;
		// Check for TabBeingDraggedPtr validity as it may no longer be valid when dragging tabs in game
		if ( TabBeingDragged.IsValid() && DragDropOperation->CanDockInNode(ParentTabStack, FDockingDragOperation::DockingViaTabWell) )
		{
			// Update the DragAndDrop operation based on this change.
			const int32 LastForegroundTabIndex = Tabs.Find(TabBeingDragged.ToSharedRef());

			if ( Tabs.Num() > 1 )
			{
				// Also stop showing its content; switch to the next tab that was active.
				if (LastForegroundTabIndex + 1 < Tabs.Num())
				{
					BringTabToFront(LastForegroundTabIndex + 1);
				}
				else
				{
					BringTabToFront(FMath::Max(LastForegroundTabIndex - 1, 0));
				}
			}

			// Remove the tab from the well when it is dragged out
			Tabs.Remove(TabBeingDraggedPtr.ToSharedRef());

			// The user is pulling a tab out of this TabWell.
			TabBeingDragged->SetParent();

			// We are no longer dragging a tab in this tab well, so stop
			// showing it in the TabWell.
			this->TabBeingDraggedPtr.Reset();

			// We may have removed the last tab that this DockNode had.
			if (Tabs.Num() == 0)
			{
				// Let the DockNode know that it is no longer needed.
				ParentTabStack->OnLastTabRemoved();
			}

			GetDockArea()->CleanUp( SDockingNode::TabRemoval_DraggedOut );

			const FGeometry& DockNodeGeometry = ParentTabStack->GetTabStackGeometry();
			DragDropOperation->OnTabWellLeft( SharedThis(this), DockNodeGeometry );
		}
	}
}


FReply SDockingTabWell::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		if (DragDropOperation->CanDockInNode(ParentTabStackPtr.Pin().ToSharedRef(), FDockingDragOperation::DockingViaTabWell))
		{
			// We are dragging the tab through a TabWell.
			// Update the position of the Tab that we are dragging in the panel.
			this->ChildBeingDraggedOffset = ComputeDraggedTabOffset(MyGeometry, DragDropEvent, TabGrabOffsetFraction);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();	
}

FReply SDockingTabWell::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if (DragDropOperation.IsValid())
	{
		if (DragDropOperation->CanDockInNode(ParentTabStackPtr.Pin().ToSharedRef(), FDockingDragOperation::DockingViaTabWell))
		{
			// It's rare, but sometimes a drop operation can happen after we drag a tab out of a docking tab well but before the engine has a
			// chance to notify the next docking tab well that a drag operation has entered it. In this case, just use the tab referenced by the
			// drag/drop operation
			if (!TabBeingDraggedPtr.IsValid())
			{
				TabBeingDraggedPtr = DragDropOperation->GetTabBeingDragged();
			}
			
			if ( ensure( TabBeingDraggedPtr.IsValid() ) )
			{
				// We dropped a Tab into this TabWell.
				const TSharedRef<SDockTab> TabBeingDragged = TabBeingDraggedPtr.ToSharedRef();

				// Figure out where in this TabWell to drop the Tab.				
				const int32 DropLocationIndex = ComputeChildDropIndex(MyGeometry, TabBeingDragged);

				ensure( DragDropOperation->GetTabBeingDragged().ToSharedRef() == TabBeingDragged );

				// Remove the tab when dropped.  If it was being dragged in this it will be added again in a more permanent way by OpenTab
				Tabs.Remove(TabBeingDraggedPtr.ToSharedRef());

				// Actually insert the new tab.
				ParentTabStackPtr.Pin()->OpenTab(TabBeingDragged, DropLocationIndex);

				// We are no longer dragging a tab.
				TabBeingDraggedPtr.Reset();

				// We knew how to handled this drop operation!
				return FReply::Handled();
			}
		}
	}

	// Someone just dropped something in here, but we have no idea what to do with it.
	return FReply::Unhandled();
}

EWindowZone::Type SDockingTabWell::GetWindowZoneOverride() const
{
	// If this is the tab well for the top-level tab stack of a window, then this window zone should be treated like a title bar instead of client area.

	// Get the tab stack that owns this tab well
	if (const TSharedPtr<SDockingTabStack> ParentTabStack = ParentTabStackPtr.Pin())
	{
		// Get the docking area for that tab stack
		const TSharedPtr<SDockingArea> ParentDockingArea = ParentTabStack->GetDockArea();

		// If the docking area is managing a window, then it's at the top level of the window (either the main window or a floating window)
		// Docking areas that are themselves docked within another docking area won't have a parent window
		if (ParentDockingArea && ParentDockingArea->GetParentWindow())
		{
			// Declare the tab well to be a title bar, allowing the user to drag the tab well to move the window
			return EWindowZone::TitleBar;
		}
	}

	// Otherwise, declare this to be simple client area
	return EWindowZone::ClientArea;
}

FReply SDockingTabWell::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (this->HasMouseCapture() && TabBeingDraggedPtr.IsValid()) 
	{
		const TSharedRef<SDockTab> TabBeingDragged = TabBeingDraggedPtr.ToSharedRef();
		this->TabBeingDraggedPtr.Reset();
		const int32 DropLocationIndex = ComputeChildDropIndex(MyGeometry, TabBeingDragged);
	
		// Reorder the tab
		Tabs.Remove(TabBeingDragged);
		Tabs.Insert(TabBeingDragged, DropLocationIndex);
		
		BringTabToFront(TabBeingDragged);
		// We are no longer dragging a tab in this tab well, so stop showing it in the TabWell.
		return FReply::Handled().ReleaseMouseCapture();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SDockingTabWell::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (this->HasMouseCapture())
	{
		// We are dragging the tab through a TabWell.
		// Update the position of the Tab that we are dragging in the panel.
		this->ChildBeingDraggedOffset = ComputeDraggedTabOffset(MyGeometry, MouseEvent, TabGrabOffsetFraction);
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}


void SDockingTabWell::BringTabToFront( int32 TabIndexToActivate )
{
	const bool bActiveIndexChanging = TabIndexToActivate != ForegroundTabIndex;
	
	if ( bActiveIndexChanging )
	{
		const int32 LastForegroundTabIndex = ForegroundTabIndex >= Tabs.Num() 
			? INDEX_NONE
			: ForegroundTabIndex;

		// For positive indexes, don't go out of bounds on the array.
		ForegroundTabIndex = FMath::Min(TabIndexToActivate, Tabs.Num()-1);

		TSharedPtr<SDockingArea> MyDockArea = GetDockArea();
		if ( Tabs.Num() > 0 && MyDockArea.IsValid() )
		{
			const TSharedPtr<SDockTab> PreviousForegroundTab = (LastForegroundTabIndex == INDEX_NONE)
				? TSharedPtr<SDockTab>()
				: Tabs[LastForegroundTabIndex];

			const TSharedPtr<SDockTab> NewForegroundTab = (ForegroundTabIndex == INDEX_NONE)
				? TSharedPtr<SDockTab>()
				: Tabs[ForegroundTabIndex];

			// Avoid useless broadcast
			if (PreviousForegroundTab != NewForegroundTab)
			{
				TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
				TSharedPtr<FTabManager> LocalTabManager = MyDockArea->GetTabManager();

				if (GlobalTabManager != LocalTabManager)
				{
					LocalTabManager->GetPrivateApi().OnTabForegrounded(NewForegroundTab, PreviousForegroundTab);
				}

				GlobalTabManager->GetPrivateApi().OnTabForegrounded(NewForegroundTab, PreviousForegroundTab);
			}
		}
	}

	// Always force a refresh, even if we don't think the active index changed.
	RefreshParentContent();

	// Update the native, global menu bar if a tab is in the foreground.
	if( ForegroundTabIndex != INDEX_NONE )
	{
		const TSharedRef<SDockTab>& ForegroundTab = Tabs[ForegroundTabIndex];
		TSharedPtr<FTabManager> TabManager = ForegroundTab->GetTabManagerPtr();
		if (TabManager.IsValid())
		{
			if (TabManager == FGlobalTabmanager::Get())
			{
				FGlobalTabmanager::Get()->UpdateMainMenu(ForegroundTab, false);
			}
			else
			{
				TabManager->UpdateMainMenu(ForegroundTab, false);
			}
		}
	}
}

/** Activate the tab specified by TabToActivate SDockTab. */
void SDockingTabWell::BringTabToFront( TSharedPtr<SDockTab> TabToActivate )
{
	if (Tabs.Num() > 0)
	{
		for (int32 TabIndex=0; TabIndex < Tabs.Num(); ++TabIndex )
		{
			if (Tabs[TabIndex] == TabToActivate)
			{
				BringTabToFront( TabIndex );
				return;
			}
		}
	}
}

/** Gets the currently active tab (or the currently dragged tab), or a null pointer if no tab is active. */
TSharedPtr<SDockTab> SDockingTabWell::GetForegroundTab() const
{
	if (TabBeingDraggedPtr.IsValid())
	{
		return TabBeingDraggedPtr;
	}
	return (Tabs.Num() > 0 && ForegroundTabIndex > INDEX_NONE) ? Tabs[ForegroundTabIndex] : TSharedPtr<SDockTab>();
}

/** Gets the index of the currently active tab, or INDEX_NONE if no tab is active or a tab is being dragged. */
int32 SDockingTabWell::GetForegroundTabIndex() const
{
	return (Tabs.Num() > 0) ? ForegroundTabIndex : INDEX_NONE;
}

void SDockingTabWell::RemoveAndDestroyTab(const TSharedRef<SDockTab>& TabToRemove, SDockingNode::ELayoutModification RemovalMethod)
{
	int32 TabIndex = Tabs.Find(TabToRemove);

	if (TabIndex != INDEX_NONE)
	{
		const TSharedPtr<SDockingTabStack> ParentTabStack = ParentTabStackPtr.Pin();

		// Remove the old tab from the list of tabs and activate the new tab.
		{
			int32 OldTabIndex = FMath::Max(ForegroundTabIndex, 0);
	
			// Not sure why but we always bring the tab that is about be remove to the foreground before removing it
			BringTabToFront(TabIndex);
			
			// The tab that will be removed is the same that was selected before
			if (TabIndex == OldTabIndex)
			{
				if (OldTabIndex == Tabs.Num() - 1)
				{
					// Select the previous tab
					OldTabIndex = FMath::Max(OldTabIndex - 1, 0);

				}
				else
				{
					// Select the next tab
					++OldTabIndex;
				}
			}

			// Allow the transfer from old to new foreground tab
			BringTabToFront(OldTabIndex);
			
			// Actually remove the tab
			Tabs.RemoveAt(TabIndex);

			// Update the selected tab index if needed
			if (TabIndex <= ForegroundTabIndex)
			{
				--ForegroundTabIndex;
			}

			if (RemovalMethod == SDockingNode::ELayoutModification::TabRemoval_Sidebar && ForegroundTabIndex == INDEX_NONE)
			{
				FGlobalTabmanager::Get()->SetActiveTab(nullptr);
			}
		}
		
		if ( ensure(ParentTabStack.IsValid()) )
		{
			TSharedPtr<SDockingArea> DockAreaPtr = ParentTabStack->GetDockArea();

			ParentTabStack->OnTabClosed( TabToRemove, RemovalMethod );
			
			// We might be closing down an entire dock area, if this is a major tab.
			// Use this opportunity to save its layout
			if (RemovalMethod == SDockingNode::TabRemoval_Closed)
			{
				if (DockAreaPtr.IsValid())
				{
					DockAreaPtr->GetTabManager()->GetPrivateApi().OnTabClosing( TabToRemove );
				}
			}

			if (Tabs.Num() == 0)
			{
				ParentTabStack->OnLastTabRemoved();
			}
			else
			{
				RefreshParentContent();
			}

			if (DockAreaPtr.IsValid())
			{
				DockAreaPtr->CleanUp( RemovalMethod );
			}
		}
	}
}

void SDockingTabWell::RefreshParentContent()
{
	if (Tabs.Num() > 0 && ForegroundTabIndex != INDEX_NONE)
	{
		const TSharedRef<SDockTab>& ForegroundTab = Tabs[ForegroundTabIndex];
		FGlobalTabmanager::Get()->SetActiveTab( ForegroundTab );

		TSharedPtr<SWindow> ParentWindowPtr = ForegroundTab->GetParentWindow();
		if (ParentWindowPtr.IsValid() && ParentWindowPtr != FGlobalTabmanager::Get()->GetRootWindow())
		{
			ParentWindowPtr->SetTitle( ForegroundTab->GetTabLabel() );
		}

		TSharedPtr<SDockingTabStack> ParentTabStack = ParentTabStackPtr.Pin();

		FDockingStackOptionalContent OptionalContent;
		OptionalContent.ContentLeft = ForegroundTab->GetLeftContent();
		OptionalContent.ContentRight = ForegroundTab->GetRightContent();
		OptionalContent.TitleBarContentRight = ForegroundTab->GetTitleBarRightContent();

		ParentTabStack->SetNodeContent(ForegroundTab->GetContent(), OptionalContent);
	}
	else
	{
		ParentTabStackPtr.Pin()->SetNodeContent(SNullWidget::NullWidget, FDockingStackOptionalContent());
	}
}

TSharedPtr<SDockingArea> SDockingTabWell::GetDockArea()
{
	return ParentTabStackPtr.IsValid() ? ParentTabStackPtr.Pin()->GetDockArea() : TSharedPtr<SDockingArea>();
}


TSharedPtr<SDockingTabStack> SDockingTabWell::GetParentDockTabStack()
{
	return ParentTabStackPtr.IsValid() ? ParentTabStackPtr.Pin() : TSharedPtr<SDockingTabStack>();
}
