// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Framework/Docking/SDockingNode.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/SDockingTabStack.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

struct FDockingConstants
{
	static const FVector2D MaxMinorTabSize;
	static const FVector2D MaxMajorTabSize;
	static const float MaxTabSizeNoNameWidth;
	static const float MaxTabSizeNoNameCantCloseWidth;
	static const FVector2D SLATE_API GetMaxTabSizeFor( ETabRole TabRole );
};

/**
 * TabWell is a panel that shows dockable tabs.
 * Tabs can be re-arranged and dragged out of the TabStack.
 */
class SDockingTabWell : public SPanel
{
public:

	SLATE_BEGIN_ARGS( SDockingTabWell )
		: _ParentStackNode( TSharedPtr<SDockingTabStack>( NULL ) )
		{}
		SLATE_ATTRIBUTE( TSharedPtr<SDockingTabStack>, ParentStackNode )
	SLATE_END_ARGS()

	SDockingTabWell();

	void Construct( const FArguments& InArgs );

	/** @return How many tabs there are. */
	int32 GetNumTabs() const;

	/** @return All child tabs in this node */
	const TSlotlessChildren<SDockTab>& GetTabs() const;
	
	/** @return Index of the tab in this dock or INDEX_NONE */
	int32 GetTabIndex(const TSharedPtr<SDockTab>& InTab) const;

	/**
	 * Add a new tab (InTab) to the TabWell at location (AtIndex).
	 *
	 * @param InTab     TheTab to add
	 * @param AtIndex   Add at this index or at the end if INDEX_NONE(default)
	 */
	void AddTab(const TSharedRef<SDockTab>& InTab, int32 AtIndex = INDEX_NONE, bool bKeepInactive = false);
	
	/** Activate the tab specified by TabToActivate index. */
	void BringTabToFront( int32 TabIndexToActivate );
	
	/** Activate the tab specified by TabToActivate SDockTab. */
	void BringTabToFront( TSharedPtr<SDockTab> TabToActivate );

	/** Gets the currently active tab (or the currently dragged tab), or a null pointer if no tab is active. */
	TSharedPtr<SDockTab> GetForegroundTab() const;

	/** Gets the index of the currently active tab, or INDEX_NONE if no tab is active or a tab is being dragged. */
	int32 GetForegroundTabIndex() const;

	/**
	 * Removes the passed in tab from the tab well and trashes it.
	 *
	 * @param TabToRemove     The tab to be removed
	 * @param RemovalMethod   The user action that caused the tab to be removed
	 */
	void RemoveAndDestroyTab(const TSharedRef<SDockTab>& TabToRemove, SDockingNode::ELayoutModification RemovalMethod);

	void RefreshParentContent();
	
	/** Gets the dock area that this resides in */
	TSharedPtr<SDockingArea> GetDockArea();

	/** Gets the parent dockable tab stack this tab well belong to */
	TSharedPtr<SDockingTabStack> GetParentDockTabStack();

	/**
	 * A tab notifies us that the started dragging it, so we should begin re-arranging layout
	 *
	 * @param TabToStartDragging      The tab that notified us that the user started dragging it
	 * @param TabGrabOffsetFraction   The offset into the tab where the user grabbed it; as a fraction of the tab's size.
	 * @param MouseEvent              The mouse event that caused this dragging to start.
	 *
	 * @return The DragDrop operation
	 */
	FReply StartDraggingTab( TSharedRef<SDockTab> TabToStartDragging, FVector2D TabGrabOffsetFraction, const FPointerEvent& MouseEvent );

public:
	FVector2D ComputeChildSize( const FGeometry& AllottedGeometry ) const;

	// SWidget interface
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override;
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual EWindowZone::Type GetWindowZoneOverride() const override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	// End of SWidget interface

private:
	
	/**
	 * Compute the offset of the tab being dragged given:
	 *
	 * @param MyGeomtry                The tab well's geometry
	 * @param MouseEvent               The mouse event that is effecting the drag
	 * @param TabGrabOffsetFraction    How far into the tab the user grabbed it, as a fraction of the tab's size
	 *
	 * @return the offset of the tab from the beginning of the TabWell
	 */
	float ComputeDraggedTabOffset( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FVector2D& TabGrabOffsetFraction ) const;

	/**
	 * The user is dropping a tab on this TabWell, and the TabWell's geometry is currently MyGeometry.
	 * @return the index of the slot into which the tab shoudl go.
	 */
	int32 ComputeChildDropIndex(const FGeometry& MyGeometry, const TSharedRef<SDockTab>& TabBeingDragged) const;

private:

	/** The tabs in this TabWell */
	TSlotlessChildren< SDockTab > Tabs;

	/** A pointer to the DockNode that owns this TabWell */
	TWeakPtr<class SDockingTabStack> ParentTabStackPtr;

	/** The Tab being dragged through the TabWell, if there is one */
	TSharedPtr<SDockTab> TabBeingDraggedPtr;

	/** The brush displayed between tabs */
	const FSlateBrush* SeparatorBrush;

	/** The offset of the Tab being dragged through this panel */
	float ChildBeingDraggedOffset;

	/** Where the user drabbed the tab as a fraction of the tab's size. */
	FVector2D TabGrabOffsetFraction;

	/** The index of the tab that is in the foreground right now. INDEX_NONE if either none are active or a tab is being dragged through. */
	int32 ForegroundTabIndex;
};


