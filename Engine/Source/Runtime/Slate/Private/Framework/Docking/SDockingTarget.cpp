// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SDockingTarget.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Docking/FDockingDragOperation.h"


static const FSlateBrush* BorderBrushFromDockDirection(SDockingNode::RelativeDirection DockDirection)
{
	// Pick the appropriate border based on direction.
	switch( DockDirection )
	{
		case SDockingNode::LeftOf:  return FCoreStyle::Get().GetBrush( "Docking.Cross.BorderLeft" );
		case SDockingNode::Above:   return FCoreStyle::Get().GetBrush( "Docking.Cross.BorderTop" );
		case SDockingNode::RightOf: return FCoreStyle::Get().GetBrush( "Docking.Cross.BorderRight" );
		case SDockingNode::Below:   return FCoreStyle::Get().GetBrush( "Docking.Cross.BorderBottom" );
		default:
		case SDockingNode::Center:  return FCoreStyle::Get().GetBrush( "Docking.Cross.BorderCenter" );
	}
}


///////////////////////////////////////////////////////////////////////////////////
// SDockingTarget
///////////////////////////////////////////////////////////////////////////////////

void SDockingTarget::Construct( const FArguments& InArgs )
{
	OwnerNode = InArgs._OwnerNode;
	DockDirection = InArgs._DockDirection;
	bIsDragHovered = false;

	// Pick the appropriate image based on direction.
	switch( DockDirection )
	{
		case SDockingNode::LeftOf:
			TargetImage = FCoreStyle::Get().GetBrush( "Docking.Cross.DockLeft" );
			TargetImage_Hovered = FCoreStyle::Get().GetBrush( "Docking.Cross.DockLeft_Hovered" );
		break;
		
		case SDockingNode::Above:
			TargetImage = FCoreStyle::Get().GetBrush( "Docking.Cross.DockTop" );
			TargetImage_Hovered = FCoreStyle::Get().GetBrush( "Docking.Cross.DockTop_Hovered" );
		break;
		
		case SDockingNode::RightOf:
			TargetImage = FCoreStyle::Get().GetBrush( "Docking.Cross.DockRight" );
			TargetImage_Hovered = FCoreStyle::Get().GetBrush( "Docking.Cross.DockRight_Hovered" );
		break;
		
		case SDockingNode::Below:
			TargetImage = FCoreStyle::Get().GetBrush( "Docking.Cross.DockBottom" );
			TargetImage_Hovered = FCoreStyle::Get().GetBrush( "Docking.Cross.DockBottom_Hovered" );
		break;

		default:
		case SDockingNode::Center:
			TargetImage = FCoreStyle::Get().GetBrush( "Docking.Cross.DockCenter" );
			TargetImage_Hovered = FCoreStyle::Get().GetBrush( "Docking.Cross.DockCenter_Hovered" );
		break;
	}

	SBorder::Construct( SBorder::FArguments()
		.ColorAndOpacity(FCoreStyle::Get().GetColor("Docking.Cross.Tint"))
		.BorderBackgroundColor(FCoreStyle::Get().GetColor("Docking.Cross.Tint"))
		.BorderImage( BorderBrushFromDockDirection(InArgs._DockDirection) )
		.HAlign(DockDirection == SDockingNode::Center ? HAlign_Center : HAlign_Fill)
		.VAlign(DockDirection == SDockingNode::Center ? VAlign_Center : VAlign_Fill)
		[
			SNew(SImage)
			.Image( this, &SDockingTarget::GetTargetImage )
		]
	);
}


void SDockingTarget::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		bIsDragHovered = true;
		// Provide feedback that this target is hovered
		SetColorAndOpacity(FCoreStyle::Get().GetColor(TEXT("Docking.Cross.HoveredTint")));

		// Let the drag and drop operation know that we are currently hovering over this node
		DragDropOperation->SetHoveredTarget( FDockingDragOperation::FDockTarget(OwnerNode.Pin(), GetDockDirection()), DragDropEvent );
	}
}
	
void SDockingTarget::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDockingDragOperation> DragDropOperation = DragDropEvent.GetOperationAs<FDockingDragOperation>();
	if ( DragDropOperation.IsValid() )
	{
		bIsDragHovered = false;
		// We are no longer hovered, remove the hover feedback
		SetColorAndOpacity(FCoreStyle::Get().GetColor(TEXT("Docking.Cross.Tint")));
		
		// Let the drag and drop operation know that we are no longer hovering over any nodes
		DragDropOperation->SetHoveredTarget( FDockingDragOperation::FDockTarget(), DragDropEvent );
	}
}
	
FReply SDockingTarget::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if (DragDropEvent.GetOperationAs<FDockingDragOperation>().IsValid())
	{
		TSharedPtr<SDockingNode> PinnedOwnerNode = OwnerNode.Pin();

		// We are a direction node, so re-arrange layout as desired
		return PinnedOwnerNode->OnUserAttemptingDock( DockDirection, DragDropEvent );
	}
	else
	{
		return FReply::Unhandled();
	}
}

TSharedPtr<SDockingNode> SDockingTarget::GetOwner() const
{
	return OwnerNode.Pin();
}

SDockingNode::RelativeDirection SDockingTarget::GetDockDirection() const
{
	return DockDirection;
}

const FSlateBrush* SDockingTarget::GetTargetImage() const
{
	return (bIsDragHovered)
		? TargetImage_Hovered
		: TargetImage;
}

