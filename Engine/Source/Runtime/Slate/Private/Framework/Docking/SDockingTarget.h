// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/Docking/SDockingNode.h"

/**
 * Targets used by docking code. When re-arranging layout, hovering over targets
 * gives the user a preview of what will happen if they drop on that target.
 * Dropping actually commits the layout-restructuring.
 */
class SDockingTarget : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SDockingTarget)
		: _OwnerNode()
		, _DockDirection(SDockingNode::LeftOf)
		{}
		/** The DockNode that this target is representing. Docking will occur relative to this DockNode */
		SLATE_ARGUMENT( TSharedPtr<class SDockingNode>, OwnerNode )
		/** In which direction relative to the TabStack to dock */
		SLATE_ARGUMENT( SDockingNode::RelativeDirection, DockDirection )
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	SLATE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	// End of SWidget interface

	/** @return the DockNode which this target represent */
	SLATE_API TSharedPtr<SDockingNode> GetOwner() const;
	
	/** @return the direction which we represent (relative to the OwnerTabStack) */
	SLATE_API SDockingNode::RelativeDirection GetDockDirection() const;

private:
	const FSlateBrush* GetTargetImage() const;

	const FSlateBrush* TargetImage;
	const FSlateBrush* TargetImage_Hovered;

	/** The DockNode relative to which we want to dock */
	TWeakPtr<class SDockingNode> OwnerNode;
	/** The direction in which we want to dock relative to the tab stack */
	SDockingNode::RelativeDirection DockDirection;

	bool bIsDragHovered;
};

