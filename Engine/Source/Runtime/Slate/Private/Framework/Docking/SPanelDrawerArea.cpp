// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/SPanelDrawerArea.h"

#include "Containers/Array.h"
#include "Framework/Docking/STabPanelDrawer.h"
#include "Layout/Geometry.h"
#include "Layout/WidgetPath.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

namespace UE::Slate::Private
{
constexpr float SSplitterHandleWith = 5.f;
constexpr float AnimationLength = 0.15f;

// Workaround for the web browser issues on some platform
TAutoConsoleVariable<bool> CVarNoAnimationOnTabForegroundedEvent(
	TEXT("Slate.NoAnimationOnTabForegroundedEvent"),
	false,
	TEXT("Tell the docking system to broadcast the event in a way that doesn't account for the animations")
);


void SPanelDrawerArea::Construct(const FArguments& InArgs, const TSharedRef<SWidget>& InContent)
{
	AreaContent = InContent;
	OpenCloseAnimation = FCurveSequence(0.f, AnimationLength, ECurveEaseFunction::QuadOut);

	// Add Restoring logic here
	SetupClosedLayout();
}

SPanelDrawerArea::~SPanelDrawerArea()
{
	FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
}

void SPanelDrawerArea::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Compute the desired final size of the panel (Always cached to avoid issue jumping issue)
	if (PanelDrawerData)
	{
		const FTabManager::FPanelDrawerSize& Size = PanelDrawerData->Size;

		float TotalCoefficient = Size.MainContentCoefficient + Size.PanelDrawerCoefficient;
		if (TotalCoefficient == 0)
		{
			DesiredOpenWidth = (AllottedGeometry.GetLocalSize().X - SSplitterHandleWith) / 2;
		}
		else
		{
			DesiredOpenWidth = (AllottedGeometry.GetLocalSize().X - SSplitterHandleWith) * Size.PanelDrawerCoefficient / (Size.MainContentCoefficient + Size.PanelDrawerCoefficient);
		}

		if (OpenCloseAnimation.IsPlaying())
		{
			UpdateAnimatedSlideWidth();
		}
	}
}

void SPanelDrawerArea::OpenPanel(bool bWithAnimation, const TSharedRef<UE::Slate::Private::FPanelDrawerData>& InData)
{
	SetPanelDrawerData(InData);

	if (bWithAnimation)
	{
		constexpr bool bIsOpening = true;
		PlayAnimation(bIsOpening);
	}
	else
	{
		SetupOpenedLayout();
	}

	if (!bIsOpen)
	{
		bIsOpen = true;
		RequestSaveLayout();
		OnExternalStateChange.ExecuteIfBound();

		if (!CVarNoAnimationOnTabForegroundedEvent.GetValueOnGameThread())
		{
			if (PanelDrawerData && PanelDrawerData->HostedTab)
			{
				TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
				TSharedPtr<FTabManager> LocalTabManager = PanelDrawerData->HostedTab->GetTabManagerPtr();

				if (GlobalTabManager != LocalTabManager)
				{
					LocalTabManager->GetPrivateApi().OnTabForegrounded(PanelDrawerData->HostedTab, {});
				}

				GlobalTabManager->GetPrivateApi().OnTabForegrounded(PanelDrawerData->HostedTab, {});
			}
		}
	}
}

void SPanelDrawerArea::ClosePanel(bool bWithAnimation, bool bIsTabBeingTransfered)
{
	if (!bIsOpen)
	{
		return;
	}

	TSharedPtr<SDockTab> OldTabDisplayed;
	if (PanelDrawerData)
	{
		OldTabDisplayed = PanelDrawerData->HostedTab;
	}
	
	if (bIsTabBeingTransfered && PanelDrawerData)
	{
		TSharedRef<UE::Slate::Private::FPanelDrawerData> NewPanelDrawerData = MakeShared<UE::Slate::Private::FPanelDrawerData>(*PanelDrawerData.Get());
		NewPanelDrawerData->HostedTab.Reset();
		SetPanelDrawerData(MoveTemp(NewPanelDrawerData));
	}

	if (bWithAnimation)
	{
		constexpr bool bIsOpening = false;
		PlayAnimation(bIsOpening);
	}
	else
	{
		SetupClosedLayout();
	}

	if (bIsOpen)
	{
		RequestSaveLayout();
		bIsOpen = false;
		OnExternalStateChange.ExecuteIfBound();

		if (CVarNoAnimationOnTabForegroundedEvent.GetValueOnGameThread() || bIsTabBeingTransfered)
		{
			if (OldTabDisplayed)
			{
				TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
				TSharedPtr<FTabManager> LocalTabManager = OldTabDisplayed->GetTabManagerPtr();

				if (GlobalTabManager != LocalTabManager)
				{
					LocalTabManager->GetPrivateApi().OnTabForegrounded({}, OldTabDisplayed);
				}

				GlobalTabManager->GetPrivateApi().OnTabForegrounded({}, OldTabDisplayed);
			}
		}
	}
}

void SPanelDrawerArea::SetupAnimationLayout()
{
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		+ SSplitter::Slot()
		.Value(bIsOpen ? 1.f : PanelDrawerData->Size.MainContentCoefficient)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				// Add Clipping for content that isn't responsive to dynamic sizes
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Clipping(EWidgetClipping::ClipToBounds)
				[
					AreaContent.ToSharedRef()
				]
			]
			// Fake the splitter handle during the slide, only shown when that part of the splitter should be visible but can't be yet.
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Right)
				.WidthOverride(this, &SPanelDrawerArea::GetAnimatedWidthOverrideForSpacer)
				.Visibility(this, &SPanelDrawerArea::GetAnimatedSpacerVisibility)
			]
		]
		+ SSplitter::Slot()
		.Value(bIsOpen ? 0.0f : PanelDrawerData->Size.PanelDrawerCoefficient)
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		[
			SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.WidthOverride(this, &SPanelDrawerArea::GetAnimatedWidthOverrideForPanelDrawer)
			.Visibility(this, &SPanelDrawerArea::GetAnimatedDrawerPanelVisibility)
			.Padding(FMargin(0.0, 0.0, 0.0, 2.0))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SConstraintCanvas)
				+ SConstraintCanvas::Slot()
				.Anchors(FAnchors(0.f, 0.f, 0.f, 1.f))
				.Offset(FMargin(0.f, 0.f, DesiredOpenWidth, 0.f))
				.Alignment(FVector2D(0.f, 0.f))
				[
					PanelDrawerContent.ToSharedRef()
				]
			]
		]
	];
}

void SPanelDrawerArea::SetupClosedLayout()
{
	ChildSlot
	[
		AreaContent.ToSharedRef()
	];

	if (!CVarNoAnimationOnTabForegroundedEvent.GetValueOnGameThread())
	{
		if (PanelDrawerData && PanelDrawerData->HostedTab)
		{
			TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
			TSharedPtr<FTabManager> LocalTabManager = PanelDrawerData->HostedTab->GetTabManagerPtr();

			if (GlobalTabManager != LocalTabManager)
			{
				LocalTabManager->GetPrivateApi().OnTabForegrounded({}, PanelDrawerData->HostedTab);
			}

			GlobalTabManager->GetPrivateApi().OnTabForegrounded({}, PanelDrawerData->HostedTab);
		}
	}

	SetPanelDrawerData({});
}

void SPanelDrawerArea::SetupOpenedLayout()
{
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		.UpdateAttributes(ESlateAttributeInitialization::AfterConstruct)
		+ SSplitter::Slot()
		.OnSlotResized(this, &SPanelDrawerArea::OnMainContentSlotResized)
		.Value(this, &SPanelDrawerArea::GetMainContentCoefficient)
		[
			// Add Clipping for content that isn't responsive to dynamic sizes
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				AreaContent.ToSharedRef()
			]
		]
		+ SSplitter::Slot()
		.OnSlotResized(this, &SPanelDrawerArea::OnPanelDrawerSlotResized)
		.Value(this, &SPanelDrawerArea::GetPanelDrawerCoefficient)
		[
			// Add Clipping for content that isn't responsive to dynamic sizes
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0.0, 0.0, 0.0, 2.0))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				PanelDrawerContent.ToSharedRef()
			]
		]
	];

	if (CVarNoAnimationOnTabForegroundedEvent.GetValueOnGameThread())
	{
		if (PanelDrawerData && PanelDrawerData->HostedTab)
		{
			TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
			TSharedPtr<FTabManager> LocalTabManager = PanelDrawerData->HostedTab->GetTabManagerPtr();

			if (GlobalTabManager != LocalTabManager)
			{
				LocalTabManager->GetPrivateApi().OnTabForegrounded(PanelDrawerData->HostedTab, {});
			}

			GlobalTabManager->GetPrivateApi().OnTabForegrounded(PanelDrawerData->HostedTab, {});
		}
	}
}

void SPanelDrawerArea::OnMainContentSlotResized(float InMainContentCoefficient)
{
	PanelDrawerData->Size.MainContentCoefficient = InMainContentCoefficient;
	RequestSaveLayout();
}

void SPanelDrawerArea::OnPanelDrawerSlotResized(float InPanelDrawerCoefficient)
{
	PanelDrawerData->Size.PanelDrawerCoefficient = InPanelDrawerCoefficient;
	RequestSaveLayout();
}

float SPanelDrawerArea::GetMainContentCoefficient() const
{
	return PanelDrawerData ? PanelDrawerData->Size.MainContentCoefficient : 1.f;
}

float SPanelDrawerArea::GetPanelDrawerCoefficient() const
{
	return PanelDrawerData ? PanelDrawerData->Size.PanelDrawerCoefficient : 0.f;
}

void SPanelDrawerArea::PlayAnimation(bool bIsOpening)
{
	if (OpenCloseAnimation.IsPlaying())
	{
		OpenCloseAnimation.PlayRelative(AsShared(), bIsOpening);
	}
	else
	{
		constexpr bool bIsLooped = false;
		constexpr float StartTime = 0.f;
		constexpr bool bRequireActiveTimer = false;
		if (bIsOpening)
		{
			OpenCloseAnimation.Play(AsShared(), bIsLooped, StartTime, bRequireActiveTimer);
		}
		else
		{
			OpenCloseAnimation.PlayReverse(AsShared(), bIsLooped, StartTime, bRequireActiveTimer);
		}

		UpdateAnimatedSlideWidth();
		SetupAnimationLayout();
		AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
		OpenCloseTimer = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SPanelDrawerArea::UpdateAnimation));
	}
}

EActiveTimerReturnType SPanelDrawerArea::UpdateAnimation(const double InCurrentTime, const float InDeltaTime)
{
	if (!OpenCloseAnimation.IsPlaying())
	{
		if (bIsOpen)
		{
			SetupOpenedLayout();
		}
		else
		{
			SetupClosedLayout();
		}

		FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
		OpenCloseTimer.Reset();

		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void SPanelDrawerArea::UpdateAnimatedSlideWidth()
{
	AnimatedSlideWidth = FMath::Lerp(0.f, DesiredOpenWidth + SSplitterHandleWith, OpenCloseAnimation.GetLerp());
}

FOptionalSize SPanelDrawerArea::GetAnimatedWidthOverrideForSpacer() const
{
	return FMath::Min(5.f, AnimatedSlideWidth);
}

FOptionalSize SPanelDrawerArea::GetAnimatedWidthOverrideForPanelDrawer() const
{
	return FMath::Max(0.f, AnimatedSlideWidth - 5.f);
}

EVisibility SPanelDrawerArea::GetAnimatedSpacerVisibility() const
{
	return AnimatedSlideWidth >= 5.f ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SPanelDrawerArea::GetAnimatedDrawerPanelVisibility() const
{
	return AnimatedSlideWidth < 5.f ? EVisibility::Collapsed : EVisibility::Visible;
}

bool SPanelDrawerArea::IsOpen() const
{
	return bIsOpen;
}

bool SPanelDrawerArea::IsHostingTab(TSharedPtr<SDockTab> InDockTab) const
{
	if (PanelDrawerData)
	{
		return InDockTab == PanelDrawerData->HostedTab;
	}

	return false;
}

FSimpleDelegate& SPanelDrawerArea::GetOnExternalStateChanged()
{
	return OnExternalStateChange;
}

TSharedPtr<SDockTab> SPanelDrawerArea::GetHostedTab() const
{
	if (PanelDrawerData)
	{
		return PanelDrawerData->HostedTab;
	}

	return {};
}

TSharedPtr<UE::Slate::Private::FPanelDrawerData> SPanelDrawerArea::GetHostedPanelDrawerData() const
{
	if (bIsOpen)
	{
		return PanelDrawerData;
	}

	return {};
}

void SPanelDrawerArea::RequestSaveLayout()
{
	if (PanelDrawerData && PanelDrawerData->HostedTab)
	{
		if (TSharedPtr<FTabManager> TabManager = PanelDrawerData->HostedTab->GetTabManagerPtr())
		{
			TabManager->RequestSavePersistentLayout();
		}
	}
}

TSharedRef<SWidget> SPanelDrawerArea::MakePanelDrawerContent() const
{
	if (PanelDrawerData && PanelDrawerData->HostedTab)
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			.MinHeight(27.f)
			[
				SNew(UE::Slate::Private::STabPanelDrawer, PanelDrawerData->HostedTab.ToSharedRef())
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				PanelDrawerData->HostedTab->GetContent()
			];
	}

	return SNullWidget::NullWidget;
}

void SPanelDrawerArea::SetPanelDrawerData(TSharedPtr<UE::Slate::Private::FPanelDrawerData>&& InNewData)
{
	PanelDrawerData = MoveTemp(InNewData);
	PanelDrawerContent = MakePanelDrawerContent();
}

void SPanelDrawerArea::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	if (PanelDrawerContent && PanelDrawerData && PanelDrawerData->HostedTab)
	{
		if (IsOpen())
		{
			const bool bIsDrawerPanelContentFocused = NewWidgetPath.ContainsWidget(PanelDrawerContent.Get());

			if (bIsDrawerPanelContentFocused)
			{
				// If a widget inside this tab stack got focused, activate this tab.
				FGlobalTabmanager::Get()->SetActiveTab(PanelDrawerData->HostedTab);
				PanelDrawerData->HostedTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
		}
	}
}

}
