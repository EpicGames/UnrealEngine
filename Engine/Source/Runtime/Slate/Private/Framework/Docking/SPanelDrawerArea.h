// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "Application/ThrottleManager.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/SCompoundWidget.h"

class FWeakWidgetPath;
class FWidgetPath;
class SDockTab;
class SDockingArea;

struct FFocusEvent;

namespace UE::Slate::Private
{

struct FPanelDrawerData
{
	FTabManager::FPanelDrawerSize Size;
	TSharedPtr<SDockTab> HostedTab;
};

/**
 * A widget that hold the content where a panel drawer can be invoked.
 */
class SPanelDrawerArea : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPanelDrawerArea) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SWidget>& InContent);

	virtual ~SPanelDrawerArea();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void OpenPanel(bool bWithAnimation, const TSharedRef<UE::Slate::Private::FPanelDrawerData>& InData);
	void ClosePanel(bool bWithAnimation, bool bIsTabBeingTransfered = false);

	bool IsOpen() const;
	bool IsHostingTab(TSharedPtr<SDockTab> InDockTab) const;
	TSharedPtr<SDockTab> GetHostedTab() const;
	FSimpleDelegate& GetOnExternalStateChanged();
	TSharedPtr<UE::Slate::Private::FPanelDrawerData> GetHostedPanelDrawerData() const;

private:
	void RequestSaveLayout();

	void SetupAnimationLayout();

	void SetupOpenedLayout();
	void SetupClosedLayout();

	void OnMainContentSlotResized(float InContentSlotSize);
	void OnPanelDrawerSlotResized(float InOpenPanelDrawerSize);

	float GetMainContentCoefficient() const;
	float GetPanelDrawerCoefficient() const;

	void PlayAnimation(bool bIsOpening);
	EActiveTimerReturnType UpdateAnimation(const double InCurrentTime, const float InDeltaTime);
	void UpdateAnimatedSlideWidth();

	FOptionalSize GetAnimatedWidthOverrideForSpacer() const;
	FOptionalSize GetAnimatedWidthOverrideForPanelDrawer() const;
	EVisibility GetAnimatedSpacerVisibility() const;
	EVisibility GetAnimatedDrawerPanelVisibility() const;

	void SetPanelDrawerData(TSharedPtr<UE::Slate::Private::FPanelDrawerData>&& InNewData);
	TSharedRef<SWidget> MakePanelDrawerContent() const;
	void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent);


	bool bIsOpen = false;

	TSharedPtr<SWidget> AreaContent;
	TSharedPtr<SWidget> PanelDrawerContent;
	TSharedPtr<UE::Slate::Private::FPanelDrawerData> PanelDrawerData;

	// Animation Data
	bool bRequestedAnimation = false;
	FCurveSequence OpenCloseAnimation;
	FThrottleRequest AnimationThrottle;
	TSharedPtr<FActiveTimerHandle> OpenCloseTimer;

	float DesiredOpenWidth = 0.f;
	// The animated include also sliding in of the splitter handle
	float AnimatedSlideWidth = 0.f;

	FSimpleDelegate OnExternalStateChange;
};
}
