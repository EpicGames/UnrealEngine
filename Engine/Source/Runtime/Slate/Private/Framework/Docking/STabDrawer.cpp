// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/STabDrawer.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Layout/ArrangedChildren.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"

void STabDrawer::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

STabDrawer::~STabDrawer()
{
	FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
	FSlateThrottleManager::Get().LeaveResponsiveMode(ResizeThrottleHandle);
}

void STabDrawer::SetCurrentSize(float InSize)
{
	CurrentSize = FMath::Clamp(InSize, MinDrawerSize, TargetDrawerSize);
}

void STabDrawer::Construct(const FArguments& InArgs, TSharedRef<SDockTab> InTab, TWeakPtr<SWidget> InTabButton, ETabDrawerOpenDirection InOpenDirection)
{
	OpenDirection = InOpenDirection;

	ForTab = InTab;
	TabButton = InTabButton;
	OpenCloseAnimation = FCurveSequence(0.0f, 0.15f, ECurveEaseFunction::QuadOut);

	CurrentSize = 0;

	ShadowOffset = InArgs._ShadowOffset;
	ExpanderSize = 5.0f;

	SplitterStyle = &FAppStyle::Get().GetWidgetStyle<FSplitterStyle>("Splitter");

	MinDrawerSize = InArgs._MinDrawerSize;

	MaxDrawerSize = InArgs._MaxDrawerSize;

	TargetDrawerSize = FMath::Clamp(InArgs._TargetDrawerSize, MinDrawerSize, MaxDrawerSize);

	OnTargetDrawerSizeChanged = InArgs._OnTargetDrawerSizeChanged;
	OnDrawerFocusLost = InArgs._OnDrawerFocusLost;
	OnDrawerClosed = InArgs._OnDrawerClosed;

	BackgroundBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.DrawerBackground");
	ShadowBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.DrawerShadow");
	BorderBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.Border");
	if (OpenDirection == ETabDrawerOpenDirection::Left)
	{
		BorderSquareEdgeBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.Border_SquareLeft");
	}
	else if (OpenDirection == ETabDrawerOpenDirection::Right)
	{
		BorderSquareEdgeBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.Border_SquareRight");
	}
	else
	{
		BorderSquareEdgeBrush = BorderBrush;
	}

	FSlateApplication::Get().OnFocusChanging().AddSP(this, &STabDrawer::OnGlobalFocusChanging);
	FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &STabDrawer::OnActiveTabChanged));

	bIsResizeHandleHovered = false;
	bIsResizing = false;

	ChildSlot
	[
		SNew(SBox)
		.Clipping(EWidgetClipping::ClipToBounds)
		.Content()
		[
			InArgs._Content.Widget
		]
	];
}

void STabDrawer::Open(bool bAnimateOpen)
{
	if (!bAnimateOpen)
	{
		SetCurrentSize(TargetDrawerSize);
		OpenCloseAnimation.JumpToEnd();
		return;
	}

	OpenCloseAnimation.Play(AsShared(), false, OpenCloseAnimation.IsPlaying() ? OpenCloseAnimation.GetSequenceTime() : 0.0f, false);

	if (!OpenCloseTimer.IsValid())
	{
		AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
		OpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &STabDrawer::UpdateAnimation));
	}
}

void STabDrawer::Close()
{
	if (OpenCloseAnimation.IsForward())
	{
		OpenCloseAnimation.Reverse();
	}

	if (!OpenCloseTimer.IsValid())
	{
		AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
		OpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &STabDrawer::UpdateAnimation));
	}

	// Make sure that this tab isn't active if it's still active when closing.
	// This might happen if the tab drawer lost focus to a non-tab (e.g. Content Drawer), so no other tab has become active.
	// Don't unconditionally clear the active tab, since this drawer might be closing due to another tab taking focus.
	if (FGlobalTabmanager::Get()->GetActiveTab() == ForTab)
	{
		FGlobalTabmanager::Get()->SetActiveTab(nullptr);
	}
}

bool STabDrawer::IsOpen() const
{
	return !OpenCloseAnimation.IsAtStart();
}

bool STabDrawer::IsClosing() const
{
	return OpenCloseAnimation.IsPlaying() && OpenCloseAnimation.IsInReverse();
}

const TSharedRef<SDockTab> STabDrawer::GetTab() const
{
	return ForTab.ToSharedRef();
}

bool STabDrawer::SupportsKeyboardFocus() const 
{
	return true;
}

FVector2D STabDrawer::ComputeDesiredSize(float) const
{
	if (OpenDirection == ETabDrawerOpenDirection::Bottom)
	{
		return FVector2D(1.0f, TargetDrawerSize + ShadowOffset.Y);
	}
	else
	{
		return FVector2D(TargetDrawerSize + ShadowOffset.X, 1.0f);
	}
}

void STabDrawer::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const 
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if (ArrangedChildren.Accepts(ChildVisibility))
	{
		if (OpenDirection == ETabDrawerOpenDirection::Left)
		{
			ArrangedChildren.AddWidget(
				AllottedGeometry.MakeChild(
					ChildSlot.GetWidget(),
					FVector2D(0.0f, ShadowOffset.Y),
					FVector2D(TargetDrawerSize, AllottedGeometry.GetLocalSize().Y - (ShadowOffset.Y * 2))
				)
			);
		}
		else if (OpenDirection == ETabDrawerOpenDirection::Right)
		{
			ArrangedChildren.AddWidget(
				AllottedGeometry.MakeChild(
					ChildSlot.GetWidget(),
					FVector2D(ShadowOffset),
					FVector2D(TargetDrawerSize, AllottedGeometry.GetLocalSize().Y - (ShadowOffset.Y * 2))
				)
			);
		}
		else
		{
			ArrangedChildren.AddWidget(
				AllottedGeometry.MakeChild(
					ChildSlot.GetWidget(),
					ShadowOffset,
					FVector2D(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetDrawerSize)
				)
			);
		}
		
	}
}

FReply STabDrawer::OnMouseButtonDown(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) 
{
	FReply Reply = FReply::Unhandled();
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(AllottedGeometry);
		const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

		if (ResizeHandleGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
		{
			bIsResizing = true;
			InitialResizeGeometry = ResizeHandleGeometry;
			InitialSizeAtResize = CurrentSize;
			ResizeThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();

			Reply = FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}

	return Reply;

}

FReply STabDrawer::OnMouseButtonUp(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) 
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsResizing == true)
	{
		bIsResizing = false;
		FSlateThrottleManager::Get().LeaveResponsiveMode(ResizeThrottleHandle);

		OnTargetDrawerSizeChanged.ExecuteIfBound(SharedThis(this), TargetDrawerSize);
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply STabDrawer::OnMouseMove(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent)
{
	const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

	bIsResizeHandleHovered = ResizeHandleGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());

	if (bIsResizing && this->HasMouseCapture() && !MouseEvent.GetCursorDelta().IsZero())
	{
		const FVector2D MousePos = MouseEvent.GetScreenSpacePosition();
		float DeltaSize = 0.0f;

		if (OpenDirection == ETabDrawerOpenDirection::Left)
		{
			DeltaSize = InitialResizeGeometry.AbsoluteToLocal(MousePos).X;
		}
		else if (OpenDirection == ETabDrawerOpenDirection::Right)
		{
			DeltaSize = -InitialResizeGeometry.AbsoluteToLocal(MousePos).X;
		}
		else
		{
			DeltaSize = -InitialResizeGeometry.AbsoluteToLocal(MousePos).Y;
		}



		TargetDrawerSize = FMath::Clamp(InitialSizeAtResize + DeltaSize, MinDrawerSize, MaxDrawerSize);
		SetCurrentSize(InitialSizeAtResize + DeltaSize);


		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void STabDrawer::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	bIsResizeHandleHovered = false;
}

FCursorReply STabDrawer::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return bIsResizing || bIsResizeHandleHovered ? FCursorReply::Cursor(OpenDirection == ETabDrawerOpenDirection::Bottom ? EMouseCursor::ResizeUpDown : EMouseCursor::ResizeLeftRight) : FCursorReply::Unhandled();
}

int32 STabDrawer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	static FSlateColor ShadowColor = FAppStyle::Get().GetSlateColor("Colors.Foldout");

	const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(AllottedGeometry);
	const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

	const FVector2D LocalSize = AllottedGeometry.GetLocalSize();
	FVector2D ContentsLocalOrigin;
	FVector2D ContentsLocalSize;
	if (OpenDirection == ETabDrawerOpenDirection::Left)
	{
		ContentsLocalOrigin = FVector2D(0.0f, ShadowOffset.Y);
		ContentsLocalSize = FVector2D(TargetDrawerSize, LocalSize.Y - (ShadowOffset.Y * 2));
	}
	else if (OpenDirection == ETabDrawerOpenDirection::Right)
	{
		ContentsLocalOrigin = ShadowOffset;
		ContentsLocalSize = FVector2D(TargetDrawerSize, LocalSize.Y - (ShadowOffset.Y * 2));
	}
	else
	{
		ContentsLocalOrigin = ShadowOffset;
		ContentsLocalSize = FVector2D(LocalSize.X - (ShadowOffset.X * 2), TargetDrawerSize);
	}

	const FPaintGeometry OffsetPaintGeom = RenderTransformedChildGeometry.ToPaintGeometry(ContentsLocalSize, FSlateLayoutTransform(ContentsLocalOrigin));

	// Draw the resize handle
	if (bIsResizing || bIsResizeHandleHovered)
	{
		const FSlateBrush* SplitterBrush = &SplitterStyle->HandleHighlightBrush;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			ResizeHandleGeometry.ToPaintGeometry(),
			SplitterBrush,
			ESlateDrawEffect::None,
			SplitterBrush->GetTint(InWidgetStyle));
	}

	// Main Shadow
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		RenderTransformedChildGeometry.ToPaintGeometry(),
		ShadowBrush,
		ESlateDrawEffect::None,
		ShadowBrush->GetTint(InWidgetStyle));


	// Background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		OffsetPaintGeom,
		BackgroundBrush,
		ESlateDrawEffect::None,
		BackgroundBrush->GetTint(InWidgetStyle));

	int32 OutLayerId = SCompoundWidget::OnPaint(Args, RenderTransformedChildGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	TSharedPtr<SWidget> TabButtonSP = TabButton.Pin();

	// Top border
	if (OpenDirection == ETabDrawerOpenDirection::Bottom || TabButtonSP == nullptr)
	{
		// When opened from the bottom, draw the full border.
		// Cutting out the "notch" for the corresponding tab is only supported in left/right orientations.
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId,
			OffsetPaintGeom,
			BorderBrush,
			ESlateDrawEffect::None,
			BorderBrush->GetTint(InWidgetStyle));
	}
	else
	{
		// Example of how border box is drawn with the tab notch cut out on the right side
		// (OpenDirection == ETabDrawerOpenDirection::Right)
		//
		//                    + - - - - - - +
		//                    : /---------\ :
		// ClipAboveTabButton : |         | : 
		//                    : |         | :
		//            TabTopY + - - - - - - +
		//                    : |           :  |
		// ClipAtTabButton    : |           :  |  (right edge outside clip is clipped off)
		//                    : |           :  |
		//         TabBottomY + - - - - - - +
		//                    : |         | :
		// ClipBelowTabButton : |         | :
		//                    : \---------/ :
		//                    + - - - - - - +
		//                                  <-->
		//                              NotchOffset
		//
		// Originally, I tried making the middle clip region thinner (to clip out the notch)
		// while keeping the geometry identical, but this looks worse when the tab notch needs to
		// be at the top or bottom, since the top/bottom edge of the border wouldn't extend all the
		// way to the edge.
		const FGeometry TabButtonGeometry = TabButtonSP->GetPaintSpaceGeometry();

		// Compute the top/bottom of the tab in our local space.
		const float BorderWidth = BorderBrush->OutlineSettings.Width;
		const float TabTopY = RenderTransformedChildGeometry.AbsoluteToLocal(TabButtonGeometry.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector)).Y + 0.5f * BorderWidth;
		const float TabBottomY = RenderTransformedChildGeometry.AbsoluteToLocal(TabButtonGeometry.GetAbsolutePositionAtCoordinates(FVector2D::UnitVector)).Y - 0.5f * BorderWidth;

		// Create the geometry for the notched portion, where one edge extends past the clipping rect.
		const FVector2D NotchOffsetSize(TabButtonGeometry.GetLocalSize().X, 0.0f);
		const FVector2D NotchOffsetTranslate = OpenDirection == ETabDrawerOpenDirection::Left ? -NotchOffsetSize : FVector2D::ZeroVector;
		const FPaintGeometry NotchOffsetPaintGeom = RenderTransformedChildGeometry.ToPaintGeometry(ContentsLocalSize + NotchOffsetSize, FSlateLayoutTransform(ContentsLocalOrigin + NotchOffsetTranslate));

		// Split the border box into three clipping zones.
		const FPaintGeometry ClipAboveTabButton = RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(LocalSize.X, TabTopY), FSlateLayoutTransform(FVector2f(0, 0)));
		const FPaintGeometry ClipAtTabButton = RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(LocalSize.X, TabBottomY - TabTopY), FSlateLayoutTransform(FVector2f(0, TabTopY)));
		const FPaintGeometry ClipBelowTabButton = RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(LocalSize.X, LocalSize.Y - TabBottomY), FSlateLayoutTransform(FVector2f(0, TabBottomY)));

		// If the tab button touches a corner on the edge of the border, switch the brush to
		// draw that corner squared-off. When a tab is near the very top or bottom of its sidebar,
		// this makes the outline look slightly nicer and more connected.
		const int32 UpperCornerIndex = OpenDirection == ETabDrawerOpenDirection::Left ? 0 : 1;
		const int32 LowerCornerIndex = OpenDirection == ETabDrawerOpenDirection::Left ? 3 : 2;
		const bool bTabTouchesUpperCorner = TabTopY < ShadowOffset.Y + BorderBrush->OutlineSettings.CornerRadii[UpperCornerIndex];
		const bool bTabTouchesLowerCorner = TabBottomY > LocalSize.Y - ShadowOffset.Y - BorderBrush->OutlineSettings.CornerRadii[LowerCornerIndex];
		const FSlateBrush* AboveTabBrush = bTabTouchesUpperCorner ? BorderSquareEdgeBrush : BorderBrush;
		const FSlateBrush* BelowTabBrush = bTabTouchesLowerCorner ? BorderSquareEdgeBrush : BorderBrush;

		// Draw portion above the tab
		OutDrawElements.PushClip(FSlateClippingZone(ClipAboveTabButton));
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId,
			OffsetPaintGeom,
			AboveTabBrush,
			ESlateDrawEffect::None,
			AboveTabBrush->GetTint(InWidgetStyle));
		OutDrawElements.PopClip();

		// Draw "notched" portion next to the tab
		OutDrawElements.PushClip(FSlateClippingZone(ClipAtTabButton));
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId,
			NotchOffsetPaintGeom,
			BorderSquareEdgeBrush,
			ESlateDrawEffect::None,
			BorderSquareEdgeBrush->GetTint(InWidgetStyle));
		OutDrawElements.PopClip();

		// Draw portion below the tab
		OutDrawElements.PushClip(FSlateClippingZone(ClipBelowTabButton));
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId,
			OffsetPaintGeom,
			BelowTabBrush,
			ESlateDrawEffect::None,
			BelowTabBrush->GetTint(InWidgetStyle));
		OutDrawElements.PopClip();
	}

	return OutLayerId+1;

}

FGeometry STabDrawer::GetRenderTransformedGeometry(const FGeometry& AllottedGeometry) const
{
	if(OpenDirection == ETabDrawerOpenDirection::Left)
	{
		return AllottedGeometry.MakeChild(FSlateRenderTransform(FVector2D(CurrentSize - TargetDrawerSize, 0.0f)));
	}
	else if (OpenDirection == ETabDrawerOpenDirection::Right)
	{
		return AllottedGeometry.MakeChild(FSlateRenderTransform(FVector2D(TargetDrawerSize - CurrentSize, 0.0f)));
	}
	else
	{
		return AllottedGeometry.MakeChild(FSlateRenderTransform(FVector2D(0.0f, TargetDrawerSize - CurrentSize)));
	}
}

FGeometry STabDrawer::GetResizeHandleGeometry(const FGeometry& AllottedGeometry) const
{
	FGeometry RenderTransformedGeometry = GetRenderTransformedGeometry(AllottedGeometry);

	if (OpenDirection == ETabDrawerOpenDirection::Left)
	{
		return RenderTransformedGeometry.MakeChild(
			FVector2D(ExpanderSize, AllottedGeometry.GetLocalSize().Y - ShadowOffset.Y * 2),
			FSlateLayoutTransform(FVector2D(RenderTransformedGeometry.GetLocalSize().X-ShadowOffset.X, ShadowOffset.Y))
		);
	}
	else if (OpenDirection == ETabDrawerOpenDirection::Right)
	{
		return RenderTransformedGeometry.MakeChild(
			FVector2D(ExpanderSize, AllottedGeometry.GetLocalSize().Y - ShadowOffset.Y * 2),
			FSlateLayoutTransform(ShadowOffset - FVector2D(ExpanderSize, 0.0f))
		);
	}
	else
	{
		return RenderTransformedGeometry.MakeChild(
			FVector2D(AllottedGeometry.GetLocalSize().X - ShadowOffset.X * 2, ExpanderSize),
			FSlateLayoutTransform(ShadowOffset - FVector2D(0.0f, ExpanderSize))
		);
	}
}

EActiveTimerReturnType STabDrawer::UpdateAnimation(double CurrentTime, float DeltaTime)
{
	SetCurrentSize(FMath::Lerp(0.0f, TargetDrawerSize, OpenCloseAnimation.GetLerp()));

	if (!OpenCloseAnimation.IsPlaying())
	{
		if (OpenCloseAnimation.IsAtStart())
		{
			OnDrawerClosed.ExecuteIfBound(SharedThis(this));
		}

		FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
		OpenCloseTimer.Reset();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

static bool IsLegalWidgetFocused(const FWidgetPath& FocusPath, const TArrayView<TSharedRef<SWidget>> LegalFocusWidgets)
{
	for (const TSharedRef<SWidget>& Widget : LegalFocusWidgets)
	{
		if (FocusPath.ContainsWidget(&Widget.Get()))
		{
			return true;
		}
	}

	return false;
}

void STabDrawer::OnGlobalFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget)
{
	// Sometimes when dismissing focus can change which will trigger this again
	static bool bIsRentrant = false;

	if (!bIsRentrant)
	{
		TGuardValue<bool> RentrancyGuard(bIsRentrant, true);

		TSharedRef<STabDrawer> ThisWidget = SharedThis(this);
		TArray<TSharedRef<SWidget>, TInlineAllocator<4>> LegalFocusWidgets;
		LegalFocusWidgets.Add(ThisWidget);
		LegalFocusWidgets.Add(ChildSlot.GetWidget());
		if (TSharedPtr<SWidget> TabButtonSP = TabButton.Pin())
		{
			LegalFocusWidgets.Add(TabButtonSP.ToSharedRef());
		}

		bool bShouldLoseFocus = false;
		// Do not close due to slow tasks as those opening send window activation events
		if (!GIsSlowTask && !FSlateApplication::Get().GetActiveModalWindow().IsValid())
		{
			if (IsLegalWidgetFocused(NewFocusedWidgetPath, MakeArrayView(LegalFocusWidgets)))
			{
				// New focus is on this tab, so make it active
				if (!IsClosing())
				{
					FGlobalTabmanager::Get()->SetActiveTab(ForTab);
				}
			}
			else if (NewFocusedWidgetPath.IsValid())
			{
				// New focus is on something else, try to check if it's a menu or child window
				TSharedRef<SWindow> NewWindow = NewFocusedWidgetPath.GetWindow();
				TSharedPtr<SWindow> MyWindow = FSlateApplication::Get().FindWidgetWindow(ThisWidget);

				// See if this is a child window (like a color picker being opened from details), and if so, don't dismiss
				// Rely on OnActiveTabChanged below to lose focus if the child window actually contains tabs
				if (!NewWindow->IsDescendantOf(MyWindow))
				{
					if (TSharedPtr<SWidget> MenuHost = FSlateApplication::Get().GetMenuHostWidget())
					{
						FWidgetPath MenuHostPath;

						// See if the menu being opened is owned by the drawer contents and if so the menu should not be dismissed
						FSlateApplication::Get().GeneratePathToWidgetUnchecked(MenuHost.ToSharedRef(), MenuHostPath);
						if (!MenuHostPath.ContainsWidget(&ChildSlot.GetWidget().Get()))
						{
							bShouldLoseFocus = true;
						}
					}
					else
					{
						bShouldLoseFocus = true;
					}
				}
			}
			else
			{
				bShouldLoseFocus = true;
			}
		}

		if (bShouldLoseFocus)
		{
			OnDrawerFocusLost.ExecuteIfBound(ThisWidget);
		}
	}
}

void STabDrawer::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
	// This tab lost the active status to some other tab; treat this like focus was lost
	if (PreviouslyActive == ForTab && NewlyActivated.IsValid())
	{
		OnDrawerFocusLost.ExecuteIfBound(SharedThis(this));
	}
}
