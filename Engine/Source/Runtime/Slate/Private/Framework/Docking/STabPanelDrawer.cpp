// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/STabPanelDrawer.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/FDockingDragOperation.h"
#include "Framework/Docking/SDockingArea.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Layout/WidgetPath.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "TabPanelDrawer"

namespace UE::Slate::Private
{
void STabPanelDrawer::Construct(const FArguments& InArgs, const TSharedRef<SDockTab> InTab)
{
	DisplayedTab = InTab;

	// Post 5.7 clean a bit the logic here to share more of it with the SDockTab
	// This makes a gradient that displays whether or not a viewport is active
	static FLinearColor ActiveBorderColor = FAppStyle::Get().GetSlateColor("Docking.Tab.ActiveTabIndicatorColor").GetSpecifiedColor();
	static FLinearColor ActiveBorderColorTransparent = FLinearColor(ActiveBorderColor.R, ActiveBorderColor.G, ActiveBorderColor.B, 0.0f);
	static TArray<FLinearColor> GradientStops{ ActiveBorderColorTransparent, ActiveBorderColor, ActiveBorderColorTransparent };
	DarkerTabBrush = *DisplayedTab->GetImageBrush();
	DarkerTabBrush.TintColor = DisplayedTab->GetTabWellBrush()->TintColor.GetSpecifiedColor();

	const FButtonStyle* const CloseButtonStyle = &DisplayedTab->GetCurrentStyle().CloseButtonStyle;

	const FButtonStyle* const OpenInNewWindowButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Docking.OpenTabInWindow.Button");

	FMargin GrayLineMargin(2.f, 2.f, 2.f, 0.f);

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(DisplayedTab->GetImageBrush())
		]

		+ SOverlay::Slot()
		.Padding(GrayLineMargin)
		[
			SNew(SImage)
			.Image(&DarkerTabBrush)
		]

		// Overlay for active tab indication.
		+SOverlay::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)

		[
			SNew(SComplexGradient)
			.Visibility(DisplayedTab.Get(), &SDockTab::GetActiveTabIndicatorVisibility)
			.DesiredSizeOverride(FVector2D(1.0f, 1.0f))
			.GradientColors(GradientStops)
			.Orientation(EOrientation::Orient_Vertical)
		]

		// Overlay for flashing a tab for attention
		+SOverlay::Slot()
		[
			SNew(SBorder)
			// Don't allow flasher tab overlay to absorb mouse clicks
			.Visibility(EVisibility::HitTestInvisible)
			.Padding(DisplayedTab.Get(), &SDockTab::GetTabPadding)
			.BorderImage(DisplayedTab.Get(), &SDockTab::GetFlashOverlayImageBrush)
			.BorderBackgroundColor(DisplayedTab.Get(), &SDockTab::GetFlashColor)
		]

		+ SOverlay::Slot()
		.Padding(GrayLineMargin)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(TabLabelOverlay, SOverlay)
			+ SOverlay::Slot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &STabPanelDrawer::GetTabLabelOverlayPadding)))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.MaxDesiredWidth(this, &STabPanelDrawer::GetTabLabelOverlayMaxSize)
				[
					// Label sub HBox
					SNew(SHorizontalBox)
					.ToolTipText(DisplayedTab.Get(), &SDockTab::GetTabLabel)
					// Tab Label
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						DisplayedTab->LabelWidget.ToSharedRef()
					]

					// Tab Label Suffix
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						DisplayedTab->LabelSuffix.ToSharedRef()
					]
				]
			]

			+ SOverlay::Slot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(DisplayedTab.Get(), &SDockTab::GetTabPadding)))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				.Visibility(EVisibility::Visible)
				.ToolTip(InArgs._ToolTip)
				.ToolTipText(DisplayedTab.Get(), &SDockTab::GetTabLabel)

				// Tab Icon
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(DisplayedTab->IsTabNameHidden() ? FMargin(0) : FMargin(0, 0, 5, 0))
				[
					SNew(SBorder)
					// Don't allow active tab overlay to absorb mouse clicks
					.Padding(DisplayedTab.Get(), &SDockTab::GetTabIconBorderPadding)
					.Visibility(EVisibility::HitTestInvisible)
					// Overlay for color-coded tab effect
					.BorderImage(DisplayedTab.Get(), &SDockTab::GetColorOverlayImageBrush)
					.BorderBackgroundColor(DisplayedTab.Get(), &SDockTab::GetTabColor)
					[
						DisplayedTab->IconWidget.ToSharedRef()
					]
				]

				+ SHorizontalBox::Slot()
				[
					SNullWidget::NullWidget
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SAssignNew(OpenTabInNewWindowButton, SButton)
						.ButtonStyle(OpenInNewWindowButtonStyle)
						.OnClicked(this, &STabPanelDrawer::OnButtonOpenTabInNewWindowClicked)
						.ContentPadding(FMargin(0.0, 1.5, 0.0, 0.0))
						.ToolTipText(LOCTEXT("OpenTabInNewWindowButtonTooltip", "Pop out into a floating window."))
						[
							SNew(SSpacer)
							.Size(OpenInNewWindowButtonStyle->Normal.ImageSize)
						]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SAssignNew(ButtonSpacer, SSpacer)
					.Size(OpenInNewWindowButtonStyle->Normal.ImageSize * 3 / 4)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SAssignNew(DismissButton, SButton)
					.ButtonStyle(CloseButtonStyle) 
					.OnClicked(this, &STabPanelDrawer::OnButtonDismissTabClicked)
					.ContentPadding(FMargin(0.0, 1.5, 0.0, 0.0))
					.ToolTipText(LOCTEXT("DismissPanelDrawerButtonTooltipText", "Dismiss Tab"))
					[
						SNew(SSpacer)
						.Size(CloseButtonStyle->Normal.ImageSize)
					]
				]
			]
		]
	];
}

void STabPanelDrawer::DismissTab() const
{
	if (TSharedPtr<SDockingArea> DockArea = DisplayedTab->ParentDockingAreaPtr.Pin())
	{
		DockArea->ClosePanelDrawer();
	}
	else
	{
		DisplayedTab->RequestCloseTab();
	}
}

FReply STabPanelDrawer::OnButtonDismissTabClicked() const
{
	DismissTab();
	return FReply::Handled();
}

FReply STabPanelDrawer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
		{
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// This should be on mouse up but for consistency with the other tabs we will put it here for now
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MakeContextMenu(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
			return FReply::Handled();
		}

	}

	return FReply::Unhandled();
}

FReply STabPanelDrawer::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply STabPanelDrawer::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (HasMouseCapture())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return FReply::Handled().ReleaseMouseCapture();
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
		{
			if (MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
			{
				DismissTab();
			}

			return FReply::Handled().ReleaseMouseCapture();
		}
	}
	return FReply::Unhandled();
}

FReply STabPanelDrawer::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Need to remember where within a tab we grabbed
	const FVector2D TabGrabOffset = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const FVector2D TabPanelDrawerSize = MyGeometry.GetLocalSize();

	FVector2D TabGrabOffsetFraction = FVector2D(
		TabPanelDrawerSize.X != 0 ? FMath::Clamp(TabGrabOffset.X / TabPanelDrawerSize.X, 0.0, 1.0) : 0.0,
		TabPanelDrawerSize.Y != 0 ? FMath::Clamp(TabGrabOffset.Y / TabPanelDrawerSize.Y, 0.0, 1.0) : 0.0
		);

	FVector2D OriginalSize(TabPanelDrawerSize.X, TabPanelDrawerSize.Y + DisplayedTab->GetContent()->GetTickSpaceGeometry().GetLocalSize().Y);

	if (TSharedPtr<SDockingArea> PinnedParent = DisplayedTab->ParentDockingAreaPtr.Pin())
	{
		//See if we can drag tabs contain in this manager
		TSharedPtr<FTabManager> TabManager = DisplayedTab->GetTabManagerPtr();


		if (TabManager.IsValid() && TabManager->GetCanDoDragOperation())
		{
			PinnedParent->ClosePanelDrawerForTransfer();
			TSharedRef<FDockingDragOperation> DragDropOperation =
				FDockingDragOperation::New(
					DisplayedTab.ToSharedRef(),
					TabGrabOffsetFraction,
					PinnedParent.ToSharedRef(),
					OriginalSize
				);

			return FReply::Handled().BeginDragDrop(DragDropOperation);
		}
		else
		{
			return FReply::Handled();
		}
	}
	
	return FReply::Unhandled();
}

FReply STabPanelDrawer::OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (!HasMouseCapture())
	{
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply STabPanelDrawer::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

TSharedRef<SWidget> STabPanelDrawer::MakeContextMenu() const
{
	constexpr bool bCloseAfterSelection = true;
	constexpr bool bCloseSelfOnly = false;

	// Legacy menu system. We should revisit it to use the UToolsMenu system
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, TSharedPtr<FExtender>(), bCloseSelfOnly, &FCoreStyle::Get());
	{
		MenuBuilder.BeginSection("TabPanelDrawerCloseTab");
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DimissTab", "Dismiss Tab"),
				LOCTEXT("DimissTabTooltip", "Close the Panel Drawer, but keep the tab alive if reopened later."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STabPanelDrawer::DismissTab)
					)
				);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CloseTab", "Close Tab"),
				LOCTEXT("CloseTabTooltil", "Close this tab."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(DisplayedTab.Get(), &SDockTab::RemoveTabFromParent),
					FCanExecuteAction::CreateSP(DisplayedTab.Get(), &SDockTab::CanCloseTab)
					)
				);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("TabPanelDrawer", LOCTEXT("LayoutMenuSection", "Layout"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("OpenTabInNewWindow", "Open Tab in a new window"),
				LOCTEXT("OpenTabInNewWindowTooltip", "Pop out into a floating window."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STabPanelDrawer::OpenTabInNewWindow)
				)
			);

			/* Remove for the 5.7 we need to do a bit more work here for it to be usefull/
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DockInLayout", "Dock in layout"),
				LOCTEXT("DockInLayoutTooltip", "Open Tab outside of the Panel Drawer in its current docked layout."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STabPanelDrawer::InvokeTabOutsidePanelDrawer)
				)
			);*/
		}
		MenuBuilder.EndSection();

		DisplayedTab->ExtendContextMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

FReply STabPanelDrawer::OnButtonOpenTabInNewWindowClicked() const
{
	OpenTabInNewWindow();
	return FReply::Handled();
}

void STabPanelDrawer::OpenTabInNewWindow() const
{
	// This should be refactored into a utility function post 5.7
	const TSharedPtr<FTabManager> MyTabManager = DisplayedTab->GetTabManagerPtr();
	if (!MyTabManager.IsValid())
	{
		return;
	}

	DisplayedTab->RemoveTabFromParent_Internal();

	TSharedPtr<SWindow> NewWindowParent = MyTabManager->GetPrivateApi().GetParentWindow();

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
		// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
		// and for multi-view rendering to function properly
		.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
		.Title(FGlobalTabmanager::Get()->GetApplicationTitle())
		.AutoCenter(EAutoCenter::None)
		// Divide out scale, it is already factored into position
		.ScreenPosition(GetCachedGeometry().LocalToAbsolute(FVector2D(0, 0)))
		// Make room for the title bar; otherwise windows will get progressive smaller whenver you float them.
		.ClientSize(SWindow::ComputeWindowSizeForContent(DisplayedTab->GetContent()->GetTickSpaceGeometry().GetLocalSize()))
		.CreateTitleBar(false);

	TSharedPtr<SDockingTabStack> NewDockNode;
	TSharedPtr<FTabManager> TabManagerToUse;
	if (DisplayedTab->GetTabRole() == ETabRole::NomadTab)
	{
		TabManagerToUse = FGlobalTabmanager::Get();
		DisplayedTab->SetTabManager(FGlobalTabmanager::Get());
	}
	else
	{
		TabManagerToUse = MyTabManager;
	}

	// Create a new dockarea
	TSharedRef<SDockingArea> NewDockArea =
		SNew(SDockingArea, TabManagerToUse.ToSharedRef(), FTabManager::NewPrimaryArea())
		.ParentWindow(NewWindow)
		.InitialContent
		(
			SAssignNew(NewDockNode, SDockingTabStack, FTabManager::NewStack())
		);

	if (DisplayedTab->GetTabRole() == ETabRole::MajorTab || DisplayedTab->GetTabRole() == ETabRole::NomadTab)
	{
		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		if (RootWindow.IsValid())
		{
			// We have a root window, so all MajorTabs are nested under it.
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef())->SetContent(NewDockArea);
		}
		else
		{
			// App tabs get put in top-level windows. They show up on the taskbar.
			FSlateApplication::Get().AddWindow(NewWindow)->SetContent(NewDockArea);
		}
	}
	else
	{
		// Other tab types are placed in child windows. Their life is controlled by the top-level windows.
		// They do not show up on the taskbar.
		if (NewWindowParent.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, NewWindowParent.ToSharedRef())->SetContent(NewDockArea);
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow)->SetContent(NewDockArea);
		}
	}

	// Do this after the window parenting so that the window title is set correctly
	NewDockNode->OpenTab(DisplayedTab.ToSharedRef());

	MyTabManager->GetPrivateApi().SetCanDoDeferredLayoutSave(true);

	// Let every widget under this tab manager know that this tab has found a new home.
	MyTabManager->GetPrivateApi().OnTabRelocated(DisplayedTab.ToSharedRef(), NewWindow);
}

void STabPanelDrawer::InvokeTabOutsidePanelDrawer() const
{
	// Try invoke tab will force the tab to open in the docking layout
	if (TSharedPtr<FTabManager> TabManager = DisplayedTab->GetTabManagerPtr())
	{
		TabManager->TryInvokeTab(DisplayedTab->GetLayoutIdentifier());
	}
}

FMargin STabPanelDrawer::GetTabLabelOverlayPadding() const
{
	if (DisplayedTab)
	{
		FMargin TabLabelMargin = DisplayedTab->GetTabPadding();
		const float LargestSide = FMath::Max(TabLabelMargin.Left, TabLabelMargin.Right);
		TabLabelMargin.Left = LargestSide;
		TabLabelMargin.Right = LargestSide;
		return TabLabelMargin;
	}

	return {};
}

FOptionalSize STabPanelDrawer::GetTabLabelOverlayMaxSize() const
{
	TSharedPtr<SWidget> TabLabelOverlayAsShared = TabLabelOverlay.Pin();
	if (TabLabelOverlayAsShared && DisplayedTab)
	{
		const FGeometry& PaintGeo = TabLabelOverlayAsShared->GetPaintSpaceGeometry();
		float TotalWidth = PaintGeo.GetLocalSize().X;
		if (TotalWidth > 0.f)
		{
			FMargin TabLabelMargin = DisplayedTab->GetTabPadding();
			float LargestSide = FMath::Max(TabLabelMargin.Left, TabLabelMargin.Right);

			if (TSharedPtr<SWidget> OpenTabInNewWindowButtonAsShared = OpenTabInNewWindowButton.Pin())
			{
				LargestSide += OpenTabInNewWindowButtonAsShared->GetPaintSpaceGeometry().GetLocalSize().X;
			}

			if (TSharedPtr<SWidget> ButtonSpacerAsShared = ButtonSpacer.Pin())
			{
				// Double the size for to use the same spacing between the most left button and the label
				LargestSide += ButtonSpacerAsShared->GetPaintSpaceGeometry().GetLocalSize().X * 2;
			}

			if (TSharedPtr<SWidget> DismissButtonAsShared = DismissButton.Pin())
			{
				LargestSide += DismissButtonAsShared->GetPaintSpaceGeometry().GetLocalSize().X;
			}

			return FOptionalSize(FMath::Max(0.f, TotalWidth - LargestSide * 2));
		}
	}

	return {};
}

}

#undef LOCTEXT_NAMESPACE

