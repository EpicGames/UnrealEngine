// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"

namespace UE::Slate::Private
{

/**
 * Does the display of the tab header in when hosted in a panel drawer
 */
class STabPanelDrawer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STabPanelDrawer) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab> InTab);

private:

	void DismissTab() const;
	TSharedRef<SWidget> MakeContextMenu() const;
	void OpenTabInNewWindow() const;
	void InvokeTabOutsidePanelDrawer() const;
	FMargin GetTabLabelOverlayPadding() const;
	FOptionalSize GetTabLabelOverlayMaxSize() const;

	FReply OnButtonDismissTabClicked() const;
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent);
	FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent);
	FReply OnButtonOpenTabInNewWindowClicked() const;

	TSharedPtr<SDockTab> DisplayedTab;
	TWeakPtr<SWidget> TabLabelOverlay;
	TWeakPtr<SWidget> OpenTabInNewWindowButton;
	TWeakPtr<SWidget> ButtonSpacer;
	TWeakPtr<SWidget> DismissButton;
	FSlateBrush DarkerTabBrush;
};

}