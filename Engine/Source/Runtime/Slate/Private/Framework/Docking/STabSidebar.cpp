// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/STabSidebar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Docking/SDockingTabWell.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/STabDrawer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Misc/App.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/Text/SlateTextBlockLayout.h"

#define LOCTEXT_NAMESPACE "TabSidebar"

DECLARE_DELEGATE_OneParam(FOnTabDrawerButtonPressed, TSharedRef<SDockTab>);
DECLARE_DELEGATE_TwoParams(FOnTabDrawerPinButtonToggled, TSharedRef<SDockTab>, bool);

static bool IsTabPinned(TSharedRef<SDockTab> Tab)
{
	return Tab->GetParentDockTabStack()->IsTabPinnedInSidebar(Tab);
}

static void SetTabPinned(TSharedRef<SDockTab> Tab, bool bIsPinned)
{
	return Tab->GetParentDockTabStack()->SetTabPinnedInSidebar(Tab, bIsPinned);
}

/**
 * Vertical text block for use in the tab drawer button.
 * Text is aligned to the top of the widget if it fits without clipping;
 * otherwise it is ellipsized and fills the widget height.
 */
class STabDrawerTextBlock : public SLeafWidget
{
public:
	enum class ERotation
	{
		Clockwise,
		CounterClockwise,
	};

	SLATE_BEGIN_ARGS(STabDrawerTextBlock)
		: _Text()
		, _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		, _Rotation(ERotation::Clockwise)
		, _OverflowPolicy()
		{}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ATTRIBUTE(ERotation, Rotation)
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Text = InArgs._Text;
		TextStyle = *InArgs._TextStyle;
		Rotation = InArgs._Rotation;
		TextLayoutCache = MakeUnique<FSlateTextBlockLayout>(
			this, FTextBlockStyle::GetDefault(), TOptional<ETextShapingMethod>(), TOptional<ETextFlowDirection>(),
			FCreateSlateTextLayout(), FPlainTextLayoutMarshaller::Create(), nullptr);
		TextLayoutCache->SetTextOverflowPolicy(InArgs._OverflowPolicy.IsSet() ? InArgs._OverflowPolicy : TextStyle.OverflowPolicy);
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		// We're going to figure out the bounds of the corresponding horizontal text, and then rotate it into a vertical orientation.
		const FVector2D LocalSize = AllottedGeometry.GetLocalSize();
		const FVector2D DesiredHorizontalTextSize = TextLayoutCache->GetDesiredSize();
		const FVector2D ActualHorizontalTextSize(FMath::Min(DesiredHorizontalTextSize.X, LocalSize.Y), FMath::Min(DesiredHorizontalTextSize.Y, LocalSize.X));

		// Now determine the center of the vertical text by rotating the dimensions of the horizontal text.
		// The center should align it to the top of the widget.
		const FVector2D VerticalTextSize(ActualHorizontalTextSize.Y, ActualHorizontalTextSize.X);
		const FVector2D VerticalTextCenter = VerticalTextSize / 2.0f;

		// Now determine where the horizontal text should be positioned so that it is centered on the vertical text:
		//      +-+
		//      |v|
		//      |e|
		// [ horizontal ]
		//      |r|
		//      |t|
		//      +-+
		const FVector2D HorizontalTextPosition = VerticalTextCenter - ActualHorizontalTextSize / 2.0f;

		// Define the text's geometry using the horizontal bounds, then rotate it 90/-90 degrees into place to become vertical.
		const FSlateRenderTransform RotationTransform(FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(Rotation.Get() == ERotation::Clockwise ? 90 : -90))));
		const FGeometry TextGeometry = AllottedGeometry.MakeChild(ActualHorizontalTextSize, FSlateLayoutTransform(HorizontalTextPosition), RotationTransform, FVector2D(0.5f, 0.5f));

		return TextLayoutCache->OnPaint(Args, TextGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
	}

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		// The text's desired size reflects the horizontal/untransformed text.
		// Switch the dimensions for vertical text.
		const FVector2D DesiredHorizontalTextSize = TextLayoutCache->ComputeDesiredSize(
			FSlateTextBlockLayout::FWidgetDesiredSizeArgs(
				Text.Get(),
				FText(),
				0.0f,
				false,
				ETextWrappingPolicy::DefaultWrapping,
				ETextTransformPolicy::None,
				FMargin(),
				1.0f,
				true,
				ETextJustify::Left),
			LayoutScaleMultiplier, TextStyle);
		return FVector2D(DesiredHorizontalTextSize.Y, DesiredHorizontalTextSize.X);
	}

	void SetText(TAttribute<FText> InText)
	{
		Text = InText;
	}

	void SetRotation(TAttribute<ERotation> InRotation)
	{
		Rotation = InRotation;
	}

private:
	TAttribute<FText> Text;
	FTextBlockStyle TextStyle;
	TAttribute<ERotation> Rotation;
	TUniquePtr<FSlateTextBlockLayout> TextLayoutCache;
};

class STabDrawerButton : public SCompoundWidget
{

	SLATE_BEGIN_ARGS(STabDrawerButton)
	{}
		SLATE_EVENT(FOnTabDrawerButtonPressed, OnDrawerButtonPressed)
		SLATE_EVENT(FOnTabDrawerPinButtonToggled, OnDrawerPinButtonToggled)
		SLATE_EVENT(FOnGetContent, OnGetContextMenuContent)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedRef<SDockTab> ForTab, ESidebarLocation InLocation)
	{
		const FVector2D Size = FDockingConstants::GetMaxTabSizeFor(ETabRole::PanelTab);

		DockTabStyle = &FAppStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab");

		// Sometimes tabs can be renamed so ensure that we pick up the rename
		ForTab->SetOnTabRenamed(SDockTab::FOnTabRenamed::CreateSP(this, &STabDrawerButton::OnTabRenamed));

		OnDrawerButtonPressed = InArgs._OnDrawerButtonPressed;
		OnDrawerPinButtonToggled = InArgs._OnDrawerPinButtonToggled;
		OnGetContextMenuContent = InArgs._OnGetContextMenuContent;
		Tab = ForTab;
		Location = InLocation;

		static FLinearColor ActiveBorderColor = FAppStyle::Get().GetSlateColor("Docking.Tab.ActiveTabIndicatorColor").GetSpecifiedColor();
		static FLinearColor ActiveBorderColorTransparent = FLinearColor(ActiveBorderColor.R, ActiveBorderColor.G, ActiveBorderColor.B, 0.0f);
		static TArray<FLinearColor> GradientStops{ ActiveBorderColorTransparent, ActiveBorderColor, ActiveBorderColorTransparent };

		ChildSlot
		.Padding(0, 0, 0, 0)
		[
			SNew(SBox)
			.WidthOverride(Size.Y) // Swap desired dimensions for a vertical tab
			.HeightOverride(Size.X)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SAssignNew(MainButton, SButton)
					.ToolTip(ForTab->GetToolTip() ? ForTab->GetToolTip() : TAttribute<TSharedPtr<IToolTip>>())
					.ToolTipText(ForTab->GetToolTip() ? TAttribute<FText>() : ForTab->GetTabLabel())
					.ContentPadding(FMargin(0.0f, DockTabStyle->TabPadding.Top, 0.0f, DockTabStyle->TabPadding.Bottom))
					// activate tab on mouse down (not mouse down-up) for consistency with non-sidebar tabs
					.OnPressed_Lambda([this](){OnDrawerButtonPressed.ExecuteIfBound(Tab.ToSharedRef()); })
					.ForegroundColor(FSlateColor::UseForeground())
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Padding(0.0f, 5.0f, 0.0f, 5.0f)
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(ForTab->GetTabIcon())
							.DesiredSizeOverride(FVector2D(16,16))
							//.RenderTransform(Rotate90)
							//.RenderTransformPivot(FVector2D(.5f, .5f))
						]
						+ SVerticalBox::Slot()
						.Padding(0.0f, 5.0f, 0.0f, 5.0f)
						.FillHeight(1.0f)
						.HAlign(HAlign_Center)
						[
							SAssignNew(Label, STabDrawerTextBlock)
								.TextStyle(&DockTabStyle->TabTextStyle)
								.Text(ForTab->GetTabLabel())
								.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
								.Clipping(EWidgetClipping::ClipToBounds)
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(0.0f, 5.0f, 0.0f, 4.0f)
						[
							SAssignNew(PinButton, SCheckBox)
							.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
							.Visibility(this, &STabDrawerButton::GetPinButtonVisibility)
							.ToolTipText(this, &STabDrawerButton::GetPinButtonToolTipText)
							.IsChecked(this, &STabDrawerButton::IsPinButtonChecked)
							.OnCheckStateChanged(this, &STabDrawerButton::OnPinButtonCheckStateChanged)
							.Padding(2.0f)
							.HAlign(HAlign_Center)
							[
								SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Image(this, &STabDrawerButton::GetPinButtonImage)
							]
						]
					]
				]
				+ SOverlay::Slot()
				[
					SAssignNew(OpenBorder, SBorder)
					.Visibility(EVisibility::HitTestInvisible)
				]
				+ SOverlay::Slot()
				.HAlign(Location == ESidebarLocation::Left ? HAlign_Left : HAlign_Right)
				[
					SAssignNew(ActiveIndicator, SComplexGradient)
					.DesiredSizeOverride(FVector2D(1.0f, 1.0f))
					.GradientColors(GradientStops)
					.Orientation(EOrientation::Orient_Horizontal)
					.Visibility(this, &STabDrawerButton::GetActiveTabIndicatorVisibility)
				]
			]
		];

		UpdateAppearance(nullptr);
	}

	void UpdateAppearance(const TSharedPtr<SDockTab> OpenedDrawer)
	{
		bool bShouldAppearOpened = OpenedDrawer.IsValid();

		STabDrawerTextBlock::ERotation Rotation;
		switch (Location)
		{
		case ESidebarLocation::Left:
			Rotation = bShouldAppearOpened ? STabDrawerTextBlock::ERotation::CounterClockwise : STabDrawerTextBlock::ERotation::Clockwise;
			break;
		case ESidebarLocation::Right:
		default:
			Rotation = bShouldAppearOpened ? STabDrawerTextBlock::ERotation::Clockwise : STabDrawerTextBlock::ERotation::CounterClockwise;
			break;
		}

		check(Label);
		Label->SetRotation(Rotation);

		if (OpenedDrawer == Tab)
		{
			// this button is the one with the tab that is actually opened so show the tab border
			OpenBorder->SetVisibility(EVisibility::HitTestInvisible);
			MainButton->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Docking.SidebarButton.Opened"));

			switch (Location)
			{
			case ESidebarLocation::Left:
				OpenBorder->SetBorderImage(FAppStyle::Get().GetBrush("Docking.Sidebar.Border_SquareRight"));
				break;
			case ESidebarLocation::Right:
			default:
				OpenBorder->SetBorderImage(FAppStyle::Get().GetBrush("Docking.Sidebar.Border_SquareLeft"));
				break;
			}
		}
		else
		{
			OpenBorder->SetVisibility(EVisibility::Collapsed);
			MainButton->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Docking.SidebarButton.Closed"));
		}
	}

	void OnTabRenamed(TSharedRef<SDockTab> ForTab)
	{
		if (ensure(ForTab == Tab))
		{
			Label->SetText(ForTab->GetTabLabel());

			if (TSharedPtr<IToolTip> ToolTip = ForTab->GetToolTip())
			{
				MainButton->SetToolTip(ToolTip);
			}
			else
			{
				MainButton->SetToolTipText(ForTab->GetTabLabel());
			}
		}
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnGetContextMenuContent.IsBound())
		{
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, OnGetContextMenuContent.Execute(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual FSlateColor GetForegroundColor() const
	{
		if (ActiveIndicator->GetVisibility() != EVisibility::Collapsed)
		{
			return DockTabStyle->ActiveForegroundColor;
		}
		else if (IsHovered())
		{
			return DockTabStyle->HoveredForegroundColor;
		}

		return FSlateColor::UseStyle();
	}

private:
	EVisibility GetActiveTabIndicatorVisibility() const
	{
		return Tab->IsActive() ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
	}

	EVisibility GetPinButtonVisibility() const
	{
		return (IsTabPinned(Tab.ToSharedRef()) || IsHovered() || Tab->IsActive()) ? EVisibility::Visible : EVisibility::Hidden;
	}

	FText GetPinButtonToolTipText() const
	{
		if (IsTabPinned(Tab.ToSharedRef()))
		{
			return LOCTEXT("UnpinTabToolTip", "Unpin Tab");
		}
		else
		{
			return LOCTEXT("PinTabToolTip", "Pin Tab");
		}
	}

	ECheckBoxState IsPinButtonChecked() const
	{
		if (IsTabPinned(Tab.ToSharedRef()))
		{
			return ECheckBoxState::Checked;
		}
		else
		{
			return ECheckBoxState::Unchecked;
		}
	}

	const FSlateBrush* GetPinButtonImage() const
	{
		if (IsTabPinned(Tab.ToSharedRef()))
		{
			return FAppStyle::Get().GetBrush("Icons.Pinned");
		}
		else
		{
			return FAppStyle::Get().GetBrush("Icons.Unpinned");
		}
	}

	void OnPinButtonCheckStateChanged(ECheckBoxState State)
	{
		OnDrawerPinButtonToggled.ExecuteIfBound(Tab.ToSharedRef(), State == ECheckBoxState::Checked);
	}

	TSharedPtr<SDockTab> Tab;
	TSharedPtr<STabDrawerTextBlock> Label;
	TSharedPtr<SWidget> ActiveIndicator;
	TSharedPtr<SBorder> OpenBorder;
	TSharedPtr<SButton> MainButton;
	TSharedPtr<SCheckBox> PinButton;
	FOnGetContent OnGetContextMenuContent;
	FOnTabDrawerButtonPressed OnDrawerButtonPressed;
	FOnTabDrawerPinButtonToggled OnDrawerPinButtonToggled;
	const FDockTabStyle* DockTabStyle;
	ESidebarLocation Location;
};

STabSidebar::~STabSidebar()
{
	// ensure all drawers are removed when closing a sidebar
	RemoveAllDrawers();
}

void STabSidebar::Construct(const FArguments& InArgs, TSharedRef<SOverlay> InDrawersOverlay)
{
	Location = InArgs._Location;
	DrawersOverlay = InDrawersOverlay;

	ChildSlot
	.Padding(FMargin(
		Location == ESidebarLocation::Right ? 2.0f : 0.0f,
		0.0f,
		Location == ESidebarLocation::Left ? 2.0f : 0.0f,
		0.0f))
	[
		SNew(SBorder)
		.Padding(0.0f)
		.BorderImage(FAppStyle::Get().GetBrush("Docking.Sidebar.Background"))
		[
			SAssignNew(TabBox, SVerticalBox)
		]
	];

}

void STabSidebar::SetOffset(float Offset)
{
	ChildSlot.Padding(0.0f, Offset+4, 0.0f, 0.0f);
}

void STabSidebar::AddTab(TSharedRef<SDockTab> Tab)
{
	if(!ContainsTab(Tab))
	{
		SetVisibility(EVisibility::SelfHitTestInvisible);

		TSharedRef<STabDrawerButton> TabButton =
			SNew(STabDrawerButton, Tab, Location)
			.OnDrawerButtonPressed(this, &STabSidebar::OnTabDrawerButtonPressed)
			.OnDrawerPinButtonToggled(this, &STabSidebar::OnTabDrawerPinButtonToggled)
			.OnGetContextMenuContent(this, &STabSidebar::OnGetTabDrawerContextMenuWidget, Tab);

		// Figure out the size this tab should be when opened later. We do it now when the tab still has valid geometry.  Once it is moved to the sidebar it will not.
		float TargetDrawerSizePct = Tab->GetParentDockTabStack()->GetTabSidebarSizeCoefficient(Tab);
		if (TargetDrawerSizePct == 0)
		{
			TSharedPtr<SWindow> MyWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			if (MyWindow.IsValid())
			{
				TargetDrawerSizePct = Tab->GetParentDockTabStack()->GetPaintSpaceGeometry().GetLocalSize().X / MyWindow->GetPaintSpaceGeometry().GetLocalSize().X;
				Tab->GetParentDockTabStack()->SetTabSidebarSizeCoefficient(Tab, TargetDrawerSizePct);
			}
		}

		// We don't currently allow more than one pinned tab per sidebar, so enforce that
		// Note: it's possible to relax this if users actually want multiple pinned tabs
		if (FindFirstPinnedTab())
		{
			SetTabPinned(Tab, false);
		}

		TabBox->AddSlot()
			// Make the tabs evenly fill the sidebar until they reach the max size
			.FillHeight(1.0f)
			.MaxHeight(FDockingConstants::GetMaxTabSizeFor(ETabRole::PanelTab).X)
			.HAlign(HAlign_Left)
			[
				TabButton
			];

		Tabs.Emplace(Tab, TabButton);

		// If this tab is a pinned tab, then open the drawer automatically after it's added
		if (IsTabPinned(Tab))
		{
			OpenDrawerNextFrame(Tab, /*bAnimateOpen=*/ false);
		}
	}
}

bool STabSidebar::RemoveTab(TSharedRef<SDockTab> TabToRemove)
{
	int32 FoundIndex = Tabs.IndexOfByPredicate(
		[TabToRemove](auto TabPair)
		{
			return TabPair.Key == TabToRemove;
		});


	if(FoundIndex != INDEX_NONE)
	{
		TPair<TSharedRef<SDockTab>, TSharedRef<STabDrawerButton>> TabPair = Tabs[FoundIndex];

		Tabs.RemoveAt(FoundIndex);
		TabBox->RemoveSlot(TabPair.Value);

		RemoveDrawer(TabToRemove);
		SummonPinnedTabIfNothingOpened();

		// Clear the pinned flag when the tab is removed from the sidebar.
		// (Users probably expect that pinning a tab, restoring it/closing it,
		// then moving it to the sidebar again will leave it unpinned the second time.)
		SetTabPinned(TabToRemove, false);

		if (Tabs.Num() == 0)
		{
			SetVisibility(EVisibility::Collapsed);
		}
	}

	return FoundIndex != INDEX_NONE;
}

bool STabSidebar::RestoreTab(TSharedRef<SDockTab> TabToRestore)
{
	if(RemoveTab(TabToRestore))
	{
		TabToRestore->GetParentDockTabStack()->RestoreTabFromSidebar(TabToRestore);
		return true;
	}

	return false;
}

bool STabSidebar::ContainsTab(TSharedPtr<SDockTab> Tab) const
{
	return Tabs.ContainsByPredicate(
		[Tab](auto TabPair)
		{
			return TabPair.Key == Tab;
		});
}

TArray<FTabId> STabSidebar::GetAllTabIds() const
{
	TArray<FTabId> TabIds;
	for (auto TabPair : Tabs)
	{
		TabIds.Add(TabPair.Key->GetLayoutIdentifier());
	}

	return TabIds;
}

TArray<TSharedRef<SDockTab>> STabSidebar::GetAllTabs() const
{
	TArray<TSharedRef<SDockTab>> DockTabs;
	for (auto TabPair : Tabs)
	{
		DockTabs.Add(TabPair.Key);
	}

	return DockTabs;
}

bool STabSidebar::TryOpenSidebarDrawer(TSharedRef<SDockTab> ForTab)
{
	int32 FoundIndex = Tabs.IndexOfByPredicate(
		[ForTab](auto TabPair)
		{
			return TabPair.Key == ForTab;
		});

	if (FoundIndex != INDEX_NONE)
	{
		OpenDrawerNextFrame(ForTab, /*bAnimateOpen=*/ true);
		return true;
	}

	return false;
}

bool STabSidebar::TryDismissSidebarDrawer(TSharedRef<SDockTab> ForTab)
{
	int32 FoundIndex = Tabs.IndexOfByPredicate(
		[ForTab](auto TabPair)
		{
			return TabPair.Key == ForTab;
		});

	if (FoundIndex != INDEX_NONE)
	{
		if (!IsTabPinned(ForTab))
		{
			CloseDrawerInternal(ForTab);
			return true;
		}
	}

	return false;
}

bool STabSidebar::TryToggleSidebarDrawer(TSharedRef<SDockTab> ForTab)
{
	int32 FoundIndex = Tabs.IndexOfByPredicate(
		[ForTab](auto TabPair)
		{
			return TabPair.Key == ForTab;
		});

	if (FoundIndex != INDEX_NONE)
	{
		if (ForTab->IsActive())
		{
			// When clicking on the button of an active (but unpinned) tab, close that tab drawer
			if (!IsTabPinned(ForTab))
			{
				CloseDrawerInternal(ForTab);
				return true;
			}
		}
		else
		{
			// Otherwise clicking on an inactive tab should open the drawer
			OpenDrawerInternal(ForTab, /*bAnimateOpen=*/ true);
			return true;
		}
	}

	return false;
}

void STabSidebar::OnTabDrawerButtonPressed(TSharedRef<SDockTab> ForTab)
{
	TryToggleSidebarDrawer(ForTab);
}

void STabSidebar::OnTabDrawerPinButtonToggled(TSharedRef<SDockTab> ForTab, bool bIsPinned)
{
	// Set pin state for given tab; clear the pin state for all other tabs
	for (const auto& TabAndButton : Tabs)
	{
		const TSharedRef<SDockTab>& Tab = TabAndButton.Key;
		SetTabPinned(Tab, Tab == ForTab ? bIsPinned : false);
	}

	// Open any newly-pinned tab
	if (bIsPinned)
	{
		OpenDrawerInternal(ForTab, /*bAnimateOpen=*/ true);
	}
}

void STabSidebar::OnTabDrawerFocusLost(TSharedRef<STabDrawer> Drawer)
{
	// Don't automatically close a pinned tab that is in the foreground
	if (IsTabPinned(Drawer->GetTab()) && GetForegroundTab() == Drawer->GetTab())
	{
		return;
	}

	CloseDrawerInternal(Drawer->GetTab());
}

void STabSidebar::OnTabDrawerClosed(TSharedRef<STabDrawer> Drawer)
{
	RemoveDrawer(Drawer->GetTab());
}

void STabSidebar::OnTargetDrawerSizeChanged(TSharedRef<STabDrawer> Drawer, float NewSize)
{
	TSharedRef<SDockTab> Tab = Drawer->GetTab();

	const float TargetDrawerSizePct = NewSize / DrawersOverlay->GetPaintSpaceGeometry().GetLocalSize().X;
	Tab->GetParentDockTabStack()->SetTabSidebarSizeCoefficient(Tab, TargetDrawerSizePct);
}

TSharedRef<SWidget> STabSidebar::OnGetTabDrawerContextMenuWidget(TSharedRef<SDockTab> ForTab)
{
	const bool bCloseAfterSelection = true;
	const bool bCloseSelfOnly = false;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, TSharedPtr<FExtender>(), bCloseSelfOnly, &FAppStyle::Get());
	{
		MenuBuilder.BeginSection("RestoreOptions", LOCTEXT("RestoreOptions", "Options"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AutoHideTab", "Undock from Sidebar"),
				LOCTEXT("HideTabWellTooltip", "Moves this tab out of the sidebar and back to a full tab where it previously was before it was added to the sidebar."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STabSidebar::OnRestoreTab, ForTab)
				)
			);

		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CloseOptions");
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CloseTab", "Close Tab"),
				LOCTEXT("CloseTabTooltip", "Close this tab, removing it from the sidebar and its parent tab well."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STabSidebar::OnCloseTab, ForTab)
				)
			);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void STabSidebar::OnRestoreTab(TSharedRef<SDockTab> TabToRestore)
{
	RestoreTab(TabToRestore);
}

void STabSidebar::OnCloseTab(TSharedRef<SDockTab> TabToClose)
{
	if(TabToClose->RequestCloseTab())
	{
		RemoveTab(TabToClose);
		TabToClose->GetParentDockTabStack()->OnTabClosed(TabToClose, SDockingNode::TabRemoval_Closed);
	}
}

void STabSidebar::RemoveDrawer(TSharedRef<SDockTab> ForTab)
{
	if(TSharedPtr<STabDrawer> OpenedDrawer = FindOpenedDrawer(ForTab))
	{
		TSharedRef<STabDrawer> OpenedDrawerRef = OpenedDrawer.ToSharedRef();

		bool bRemoveSuccessful = DrawersOverlay->RemoveSlot(OpenedDrawerRef);
		ensure(bRemoveSuccessful);

		OpenedDrawers.Remove(OpenedDrawerRef);
	}

	ForTab->OnTabDrawerClosed();

	UpdateDrawerAppearance();
}

void STabSidebar::RemoveAllDrawers()
{
	PendingTabToOpen.Reset();
	bAnimatePendingTabOpen = false;

	// Closing drawers can remove them from the opened drawers list so copy the list first
	TArray<TSharedRef<STabDrawer>> OpenedDrawersCopy = OpenedDrawers;

	for (TSharedRef<STabDrawer>& Drawer : OpenedDrawersCopy)
	{
		RemoveDrawer(Drawer->GetTab());
	}
}

EActiveTimerReturnType STabSidebar::OnOpenPendingDrawerTimer(double CurrentTime, float DeltaTime)
{
	if (TSharedPtr<SDockTab> Tab = PendingTabToOpen.Pin())
	{
		// Wait until the drawers overlay has been arranged once to open the drawer
		// It might not have geometry yet if we're adding back tabs on startup
		if (DrawersOverlay->GetTickSpaceGeometry().GetLocalSize().IsZero())
		{
			return EActiveTimerReturnType::Continue;
		}

		OpenDrawerInternal(Tab.ToSharedRef(), bAnimatePendingTabOpen);
	}

	OpenPendingDrawerTimerHandle.Reset();
	PendingTabToOpen.Reset();
	bAnimatePendingTabOpen = false;

	return EActiveTimerReturnType::Stop;
}

void STabSidebar::OpenDrawerNextFrame(TSharedRef<SDockTab> ForTab, bool bAnimateOpen)
{
	PendingTabToOpen = ForTab;
	bAnimatePendingTabOpen = bAnimateOpen;
	if (!OpenPendingDrawerTimerHandle.IsValid())
	{
		OpenPendingDrawerTimerHandle = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &STabSidebar::OnOpenPendingDrawerTimer));
	}
}

void STabSidebar::OpenDrawerInternal(TSharedRef<SDockTab> ForTab, bool bAnimateOpen)
{
	if (FindOpenedDrawer(ForTab))
	{
		// Drawer already opened so don't do anything
		return;
	}

	PendingTabToOpen.Reset();
	bAnimatePendingTabOpen = false;

	const FGeometry DrawersOverlayGeometry = DrawersOverlay->GetTickSpaceGeometry();
	const FGeometry MyGeometry = GetTickSpaceGeometry();

	// Calculate padding for the drawer itself
	const float MinDrawerSize = MyGeometry.GetLocalSize().X - 4.0f; // overlap with sidebar border slightly

	const FVector2D ShadowOffset(8, 8);

	FMargin SlotPadding(
		Location == ESidebarLocation::Left ? MinDrawerSize : 0.0f,
		-ShadowOffset.Y,
		Location == ESidebarLocation::Right ? MinDrawerSize : 0.0f,
		-ShadowOffset.Y
	);

	const float AvailableWidth = DrawersOverlayGeometry.GetLocalSize().X - SlotPadding.GetTotalSpaceAlong<EOrientation::Orient_Horizontal>();

	const float MaxPct = .5f;
	const float MaxDrawerSize = AvailableWidth * 0.50f;

	float TargetDrawerSizePct = ForTab->GetParentDockTabStack()->GetTabSidebarSizeCoefficient(ForTab);
	TargetDrawerSizePct = FMath::Clamp(TargetDrawerSizePct, .0f, .5f);


	const float TargetDrawerSize = AvailableWidth * TargetDrawerSizePct;

	const TPair<TSharedRef<SDockTab>, TSharedRef<STabDrawerButton>>* TabEntry = Tabs.FindByPredicate(
		[ForTab](auto TabPair) {
			return TabPair.Key == ForTab;
		});
	const TWeakPtr<SWidget> TabButton = TabEntry ? TWeakPtr<SWidget>(TabEntry->Value) : nullptr;

	TSharedRef<STabDrawer> NewDrawer =
		SNew(STabDrawer, ForTab, TabButton, Location == ESidebarLocation::Left ? ETabDrawerOpenDirection::Left : ETabDrawerOpenDirection::Right)
		.MinDrawerSize(MinDrawerSize)
		.TargetDrawerSize(TargetDrawerSize)
		.MaxDrawerSize(MaxDrawerSize)
		.OnDrawerFocusLost(this, &STabSidebar::OnTabDrawerFocusLost)
		.OnDrawerClosed(this, &STabSidebar::OnTabDrawerClosed)
		.OnTargetDrawerSizeChanged(this, &STabSidebar::OnTargetDrawerSizeChanged)
		[
			ForTab->GetContent()
		];

	DrawersOverlay->AddSlot()
		.Padding(SlotPadding)
		.HAlign(Location == ESidebarLocation::Left ? HAlign_Left : HAlign_Right)
		[
			NewDrawer
		];

	NewDrawer->Open(bAnimateOpen);

	OpenedDrawers.Add(NewDrawer);

	// This changes the focus and will trigger focus-related events, such as closing other tabs,
	// so it's important that we only call it after we added the new drawer to OpenedDrawers.
	FSlateApplication::Get().SetKeyboardFocus(NewDrawer);

	ForTab->OnTabDrawerOpened();

	UpdateDrawerAppearance();
}

void STabSidebar::CloseDrawerInternal(TSharedRef<SDockTab> ForTab)
{
	if (TSharedPtr<STabDrawer> OpenedDrawer = FindOpenedDrawer(ForTab))
	{
		OpenedDrawer->Close();
		TSharedRef<STabDrawer> OpenedDrawerRef = OpenedDrawer.ToSharedRef();

		bool bRemoveSuccessful = DrawersOverlay->RemoveSlot(OpenedDrawerRef);
		ensure(bRemoveSuccessful);

		OpenedDrawers.Remove(OpenedDrawerRef);
	}

	SummonPinnedTabIfNothingOpened();
	UpdateDrawerAppearance();
}

void STabSidebar::SummonPinnedTabIfNothingOpened()
{
	// If there's already a tab in the foreground, don't bring the pinned tab forward
	if (GetForegroundTab())
	{
		return;
	}

	// But if there's no current foreground tab, then bring forward a pinned tab (there should be at most one)
	// This should happen when:
	// - the current foreground tab is not pinned and loses focus
	// - the current foreground tab's drawer is manually closed by pressing on the tab button
	// - closing or restoring the current foreground tab
	if (TSharedPtr<SDockTab> PinnedTab = FindFirstPinnedTab())
	{
		OpenDrawerInternal(PinnedTab.ToSharedRef(), /*bAnimateOpen=*/ true);
	}
}

void STabSidebar::UpdateDrawerAppearance()
{
	TSharedPtr<SDockTab> OpenedTab;
	if (OpenedDrawers.Num() > 0)
	{
		OpenedTab = OpenedDrawers.Last()->GetTab();
	}

	for (auto& TabPair : Tabs)
	{
		TabPair.Value->UpdateAppearance(OpenedTab);
	}
}

TSharedPtr<SDockTab> STabSidebar::FindFirstPinnedTab() const
{
	const TPair<TSharedRef<SDockTab>, TSharedRef<STabDrawerButton>>* PinnedTab =
		Tabs.FindByPredicate(
			[](const TPair<TSharedRef<SDockTab>, TSharedRef<STabDrawerButton>>& TabAndButton)
			{
				return IsTabPinned(TabAndButton.Key);
			});
	return PinnedTab ? TSharedPtr<SDockTab>(PinnedTab->Key) : nullptr;
}

TSharedPtr<SDockTab> STabSidebar::GetForegroundTab() const
{
	const int32 Index =
		OpenedDrawers.FindLastByPredicate(
			[](const TSharedRef<STabDrawer>& Drawer)
			{
				return Drawer->IsOpen() && !Drawer->IsClosing();
			});
	return Index == INDEX_NONE ? nullptr : TSharedPtr<SDockTab>(OpenedDrawers[Index]->GetTab());
}

TSharedPtr<STabDrawer> STabSidebar::FindOpenedDrawer(TSharedRef<SDockTab> ForTab) const
{
	const TSharedRef<STabDrawer>* OpenedDrawer =
		OpenedDrawers.FindByPredicate(
			[&ForTab](TSharedRef<STabDrawer>& Drawer)
			{
				return ForTab == Drawer->GetTab();
			});
	return OpenedDrawer ? TSharedPtr<STabDrawer>(*OpenedDrawer) : nullptr;
}

#undef LOCTEXT_NAMESPACE
