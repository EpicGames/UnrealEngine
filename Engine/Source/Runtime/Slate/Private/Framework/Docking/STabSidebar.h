// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SDockTab;
class SOverlay;
class STabDrawerButton;
class SVerticalBox;
class STabDrawer;
struct FTabId;
enum class ESidebarLocation : uint8;

/**
 * A Sidebar is a widget that contains STabDrawers which can be opened and closed from the drawer to allow temporary access to the tab
 * A drawer is automatically dismissed when it or any of its children loses focus
 * 
 * Tabs can be pinned in the sidebar: this prevents them from being automatically dismissed when they lose focus,
 * and it also causes them to be summoned back when all other drawers are dismissed
 */
class STabSidebar : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STabSidebar)
	{}
		SLATE_ARGUMENT(ESidebarLocation, Location)
	SLATE_END_ARGS()

public:
	~STabSidebar();

	void Construct(const FArguments& InArgs, TSharedRef<SOverlay> InDrawersOverlay);

	/**
	 * Sets an offset for the sidebar from the top of the window
	 */
	void SetOffset(float Offset);

	/**
	 * Adds a tab to the sidebar
	 */
	void AddTab(TSharedRef<SDockTab> Tab);

	/**
	 * Removes a tab from the sidebar. Does not restore it to the parent stack
	 *
	 * Note it is not sufficient to call this to clean up the tab completely. Call RequestCloseTab on the dock tab to do that.
	 * @return true if the tab was found and removal was successful
	 */
	bool RemoveTab(TSharedRef<SDockTab> TabToRemove);

	/**
	 * Restores a tab to the parent tab stack and removes it from this sidebar
	 *
	 * @return true if the tab was found and removal was successful
	 */
	bool RestoreTab(TSharedRef<SDockTab> TabToRestore);

	/**
	 * @return true if this sidebar contains the provided tab
	 */
	bool ContainsTab(TSharedPtr<SDockTab> Tab) const;

	/**
	 * Get all layout identifiers for tabs in this sidebar
	 */
	TArray<FTabId> GetAllTabIds() const;

	/**
	 * Get all tabs in this sidebar
	 */
	TArray<TSharedRef<SDockTab>> GetAllTabs() const;

	/**
	 * Attempt to open a drawer in the sidebar for a specified tab
	 *
	 * @return true if the passed in tab is contained in this sidebar and was opened
	 */
	bool TryOpenSidebarDrawer(TSharedRef<SDockTab> ForTab);

	/**
	 * Attempt to dismiss a drawer in the sidebar for a specified tab
	 *
	 * @return true if the passed in tab is contained in this sidebar and was dismissed
	 */
	bool TryDismissSidebarDrawer(TSharedRef<SDockTab> ForTab);

	/**
	 * Attempt to toggle a drawer in the sidebar for a specified tab
	 *
	 * @return true if the passed in tab is contained in this sidebar and was toggled
	 */
	bool TryToggleSidebarDrawer(TSharedRef<SDockTab> ForTab);
private:
	void OnTabDrawerButtonPressed(TSharedRef<SDockTab> ForTab);
	void OnTabDrawerPinButtonToggled(TSharedRef<SDockTab> ForTab, bool bIsPinned);
	void OnTabDrawerFocusLost(TSharedRef<STabDrawer> Drawer);
	void OnTabDrawerClosed(TSharedRef<STabDrawer> Drawer);
	void OnTargetDrawerSizeChanged(TSharedRef<STabDrawer> Drawer, float NewSize);

	TSharedRef<SWidget> OnGetTabDrawerContextMenuWidget(TSharedRef<SDockTab> ForTab);
	void OnRestoreTab(TSharedRef<SDockTab> TabToRestore);
	void OnCloseTab(TSharedRef<SDockTab> TabToClose);
	
	/**
	 * Removes a single drawer for a specified tab from this sidebar
	 * Removal is done instantly not waiting for any close animation
	 */
	void RemoveDrawer(TSharedRef<SDockTab> ForTab);

	/**
	 * Removes all drawers instantly (including drawers for pinned tabs)
	 */
	void RemoveAllDrawers();

	EActiveTimerReturnType OnOpenPendingDrawerTimer(double CurrentTime, float DeltaTime);
	void OpenDrawerNextFrame(TSharedRef<SDockTab> ForTab, bool bAnimateOpen);
	void OpenDrawerInternal(TSharedRef<SDockTab> ForTab, bool bAnimateOpen);
	void CloseDrawerInternal(TSharedRef<SDockTab> ForTab);

	/**
	 * Reopens the pinned tab only if there are no other open drawers;
	 * this should be used to bring pinned tabs back after other tabs lose focus/are closed
	 */
	void SummonPinnedTabIfNothingOpened();

	/**
	 * Updates the appearance of open drawers
	 */
	void UpdateDrawerAppearance();

	/**
	 * Returns the first tab in this sidebar that is marked pinned
	 */
	TSharedPtr<SDockTab> FindFirstPinnedTab() const;

	/**
	 * Returns the tab for the last-opened drawer that is still open,
	 * excluding any drawers that are in the process of closing
	 */
	TSharedPtr<SDockTab> GetForegroundTab() const;

	/**
	 * Returns the drawer for the given tab if it's open
	 */
	TSharedPtr<STabDrawer> FindOpenedDrawer(TSharedRef<SDockTab> ForTab) const;

private:
	TSharedPtr<SVerticalBox> TabBox;
	TArray<TPair<TSharedRef<SDockTab>, TSharedRef<STabDrawerButton>>> Tabs;
	TSharedPtr<FActiveTimerHandle> OpenPendingDrawerTimerHandle;
	ESidebarLocation Location;
	TSharedPtr<SOverlay> DrawersOverlay;
	/** Generally speaking one drawer is only ever open at once but we animate any previous drawer closing so there could be more than one while an animation is playing */
	TArray<TSharedRef<STabDrawer>> OpenedDrawers;
	/** Any pending drawer tab to open */
	TWeakPtr<SDockTab> PendingTabToOpen;
	/** Whether to animate the pending tab open */
	bool bAnimatePendingTabOpen = false;
}; 
