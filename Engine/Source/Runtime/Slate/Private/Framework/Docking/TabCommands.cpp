// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/TabCommands.h"

#define LOCTEXT_NAMESPACE "TabCommands"

void FTabCommands::RegisterCommands()
{
	UI_COMMAND(CloseMajorTab, "Close Major Tab", "Closes the focused major tab", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::W))
	UI_COMMAND(CloseMinorTab, "Close Minor Tab", "Closes the current window's active minor tab", EUserInterfaceActionType::Button, FInputChord())
	UI_COMMAND(CloseFocusedTab, "Close Focused Tab", "Closes the focused tab, prioritizing minor tabs over major tabs", EUserInterfaceActionType::Button, FInputChord())
}

#undef LOCTEXT_NAMESPACE
