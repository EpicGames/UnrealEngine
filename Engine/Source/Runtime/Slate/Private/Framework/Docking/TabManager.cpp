// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Docking/TabManager.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Docking/SDockingNode.h"
#include "Framework/Docking/SDockingSplitter.h"
#include "Framework/Docking/SDockingArea.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Framework/Docking/SDockingTabWell.h"
#include "Framework/Docking/SPanelDrawerArea.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Misc/NamePermissionList.h"
#include "Trace/SlateMemoryTags.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"

const FVector2D FTabManager::FallbackWindowSize( 1000, 600 );
TMap<FTabId, FVector2D> FTabManager::DefaultTabWindowSizeMap;

DEFINE_LOG_CATEGORY_STATIC(LogTabManager, Display, All);

TAutoConsoleVariable<bool> CVarPanelDrawerToggle(
	TEXT("EnablePanelDrawer"),
	true,
	TEXT("Enables or disables the panel drawer. When the drawer is off invocations will just create a normal tab.")
);

#define LOCTEXT_NAMESPACE "TabManager"

static const FString UE_TABMANAGER_OPENED_TAB_STRING = "OpenedTab";
static const FString UE_TABMANAGER_CLOSED_TAB_STRING = "ClosedTab";
static const FString UE_TABMANAGER_SIDEBAR_TAB_STRING = "SidebarTab";
static const FString UE_TABMANAGER_INVALID_TAB_STRING = "InvalidTab";

static FString StringFromTabState(ETabState::Type TabState)
{
	switch (TabState)
	{
	case ETabState::OpenedTab:
		return UE_TABMANAGER_OPENED_TAB_STRING;
	case ETabState::ClosedTab:
		return UE_TABMANAGER_CLOSED_TAB_STRING;
	case ETabState::SidebarTab:
		return UE_TABMANAGER_SIDEBAR_TAB_STRING;
	default:
		return UE_TABMANAGER_INVALID_TAB_STRING;
	}
}

static FString StringFromSidebarLocation(ESidebarLocation Location)
{
	switch (Location)
	{
	case ESidebarLocation::Left:
		return TEXT("Left");
	case ESidebarLocation::Right:
		return TEXT("Right");
	default:
		return TEXT("None");
	}
}

static ESidebarLocation SidebarLocationFromString(const FString& AsString)
{
	if (AsString == TEXT("Left"))
	{
		return ESidebarLocation::Left;
	}
	else if (AsString == TEXT("Right"))
	{
		return ESidebarLocation::Right;
	}
	else
	{
		return ESidebarLocation::None;
	}
}

static ETabState::Type TabStateFromString(const FString& AsString)
{
	if (AsString == UE_TABMANAGER_OPENED_TAB_STRING)
	{
		return ETabState::OpenedTab;
	}
	else if (AsString == UE_TABMANAGER_CLOSED_TAB_STRING)
	{
		return ETabState::ClosedTab;
	}
	else if (AsString == UE_TABMANAGER_INVALID_TAB_STRING)
	{
		return ETabState::InvalidTab;
	}
	else if (AsString == UE_TABMANAGER_SIDEBAR_TAB_STRING)
	{
		return ETabState::SidebarTab;
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid tab state."));
		return ETabState::OpenedTab;
	}
}

namespace UE::Slate::Private
{
	TSharedRef<SWindow> GetTabManagerTopWindow(const TSharedRef<SWindow>& InWindow)
	{
		TSharedRef<SWindow> TopLevelWindow = InWindow;

		EWindowType WindowType = TopLevelWindow->GetType();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		while (!(WindowType == EWindowType::Normal || WindowType == EWindowType::GameWindow))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// The window might be a menu or a tooltip if this the case check the parent window instead
			if (TSharedPtr<SWindow> ParentWindow = TopLevelWindow->GetParentWindow())
			{
				TopLevelWindow = ParentWindow.ToSharedRef();
			}
			else
			{
				break;
			}
			WindowType = TopLevelWindow->GetType();
		}

		return TopLevelWindow;
	}
}

//////////////////////////////////////////////////////////////////////////
// 
//////////////////////////////////////////////////////////////////////////

FTabManager::FLiveTabSearch::FLiveTabSearch(FName InSearchForTabId)
	: SearchForTabId(InSearchForTabId)
{
}

TSharedPtr<SDockTab> FTabManager::FLiveTabSearch::Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const
{
	if ( SearchForTabId != NAME_None )
	{
		return Manager.FindExistingLiveTab(FTabId(SearchForTabId));
	}
	else
	{
		return Manager.FindExistingLiveTab(FTabId(PlaceholderId));
	}
}

TSharedPtr<SDockTab> FTabManager::FRequireClosedTab::Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const
{
	return TSharedPtr<SDockTab>();
}

FTabManager::FLastMajorOrNomadTab::FLastMajorOrNomadTab(FName InFallbackTabId)
	: FallbackTabId(InFallbackTabId)
{
}

TSharedPtr<SDockTab> FTabManager::FLastMajorOrNomadTab::Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const
{
	TSharedPtr<SDockTab> FoundTab;
	if ( UnmanagedTab->GetTabRole() == ETabRole::MajorTab )
	{
		FoundTab = Manager.FindLastTabInWindow(Manager.LastMajorDockWindow.Pin());
		if ( !FoundTab.IsValid() && FallbackTabId != NAME_None )
		{
			FoundTab = Manager.FindExistingLiveTab(FTabId(FallbackTabId));
		}
	}

	return FoundTab;
}

const TSharedRef<FTabManager::FLayout> FTabManager::FLayout::NullLayout = FTabManager::NewLayout("NullLayout")->AddArea(FTabManager::NewPrimaryArea());


TSharedRef<FTabManager::FLayoutNode> FTabManager::FLayout::NewFromString_Helper( TSharedPtr<FJsonObject> JsonObject )
{
	struct local
	{
		static FTabManager::FArea::EWindowPlacementInternal PlacementFromString( const FString& AsString )
		{
			if (AsString == GET_ENUMERATOR_NAME_STRING_VIEW_CHECKED(FTabManager::FArea::EWindowPlacementInternal, Placement_NoWindow))
			{
				return FTabManager::FArea::EWindowPlacementInternal::Placement_NoWindow;
			}
			else if (AsString == GET_ENUMERATOR_NAME_STRING_VIEW_CHECKED(FTabManager::FArea::EWindowPlacementInternal, Placement_Automatic))
			{
				return FTabManager::FArea::EWindowPlacementInternal::Placement_Automatic;
			}
			else if (AsString == GET_ENUMERATOR_NAME_STRING_VIEW_CHECKED(FTabManager::FArea::EWindowPlacementInternal, Placement_Specified))
			{
				return FTabManager::FArea::EWindowPlacementInternal::Placement_Specified;
			}
			else if (AsString == GET_ENUMERATOR_NAME_STRING_VIEW_CHECKED(FTabManager::FArea::EWindowPlacementInternal, Placement_ParentSpecified))
			{
				return FTabManager::FArea::EWindowPlacementInternal::Placement_ParentSpecified;
			}
			else
			{
				ensureMsgf(false, TEXT("Invalid placement mode."));
				return FTabManager::FArea::EWindowPlacementInternal::Placement_Automatic;
			}
		}

		static EOrientation OrientationFromString( const FString& AsString )
		{
			static const FString Orient_Horizontal_Str = "Orient_Horizontal";
			static const FString Orient_Vertical_Str = "Orient_Vertical";

			if (AsString == Orient_Horizontal_Str)
			{
				return Orient_Horizontal;
			}
			else if (AsString == Orient_Vertical_Str)
			{
				return Orient_Vertical;
			}
			else
			{
				ensureMsgf(false, TEXT("Invalid orientation."));
				return Orient_Horizontal;
			}
			
		}

	};

	const FString NodeType = JsonObject->GetStringField(TEXT("Type"));
	if (NodeType == TEXT("Area"))
	{
		TSharedPtr<FTabManager::FArea> NewArea;

		FTabManager::FArea::EWindowPlacementInternal WindowPlacement = local::PlacementFromString( JsonObject->GetStringField(TEXT("WindowPlacement")) );
		switch( WindowPlacement )
		{
			default:
			case FTabManager::FArea::EWindowPlacementInternal::Placement_NoWindow:
			{
				NewArea = FTabManager::NewPrimaryArea();
			}
			break;

			case FTabManager::FArea::EWindowPlacementInternal::Placement_Automatic:
			{
				FVector2D WindowSize;

				WindowSize.X = (float)JsonObject->GetNumberField( TEXT("WindowSize_X") );
				WindowSize.Y = (float)JsonObject->GetNumberField( TEXT("WindowSize_Y") );

				NewArea = FTabManager::NewArea( WindowSize );
			}
			break;

			case FTabManager::FArea::EWindowPlacementInternal::Placement_ParentSpecified:
			case FTabManager::FArea::EWindowPlacementInternal::Placement_Specified:
			{
				FVector2D WindowPosition = FVector2D::ZeroVector;
				FVector2D WindowSize;

				WindowPosition.X = (float)JsonObject->GetNumberField( TEXT("WindowPosition_X") );
				WindowPosition.Y = (float)JsonObject->GetNumberField( TEXT("WindowPosition_Y") );

				WindowSize.X = (float)JsonObject->GetNumberField( TEXT("WindowSize_X") );
				WindowSize.Y = (float)JsonObject->GetNumberField( TEXT("WindowSize_Y") );

				bool bIsMaximized = JsonObject->GetBoolField(TEXT("bIsMaximized"));

				NewArea = FTabManager::NewArea( WindowSize );
				NewArea->SetWindow( WindowPosition, bIsMaximized );

				if (WindowPlacement == FTabManager::FArea::EWindowPlacementInternal::Placement_ParentSpecified)
				{
					NewArea->WindowPlacement = FTabManager::FArea::EWindowPlacementInternal::Placement_ParentSpecified;
				}
			}
			break;
		}
		
		NewArea->SetSizeCoefficient((float)JsonObject->GetNumberField( TEXT("SizeCoefficient") ) );
		NewArea->SetOrientation( local::OrientationFromString( JsonObject->GetStringField(TEXT("Orientation")) ) );

		// Panel Drawer data
		{
			FString ActivePanelDrawerTabType;
			if (JsonObject->TryGetStringField(TEXT("ActivePanelDrawerTab"), ActivePanelDrawerTabType))
			{
				FPanelDrawerTab ActivePanelDrawerTab;
				ActivePanelDrawerTab.TabId = FTabId(*ActivePanelDrawerTabType);
				ActivePanelDrawerTab.Size.MainContentCoefficient = JsonObject->GetNumberField(TEXT("MainContentCoefficient"));
				ActivePanelDrawerTab.Size.PanelDrawerCoefficient = JsonObject->GetNumberField(TEXT("PanelDrawerCoefficient"));
				NewArea->SetPanelDrawerActiveTab(MoveTemp(ActivePanelDrawerTab));
			}

			const TArray<TSharedPtr<FJsonValue>>* InactivePanelDrawerTabsPtr = nullptr;
			if (JsonObject->TryGetArrayField(TEXT("InactivePanelDrawerTabs"), InactivePanelDrawerTabsPtr))
			{
				for (const TSharedPtr<FJsonValue>& PanelDrawerTabJsonValue : *InactivePanelDrawerTabsPtr)
				{
					TSharedPtr<FJsonObject> PanelDrawerTabJson = PanelDrawerTabJsonValue->AsObject();
					FPanelDrawerTab PanelDrawerTab;
					PanelDrawerTab.TabId = FTabId(*PanelDrawerTabJson->GetStringField(TEXT("TabId")));
					PanelDrawerTab.Size.MainContentCoefficient = PanelDrawerTabJson->GetNumberField(TEXT("MainContentCoefficient"));
					PanelDrawerTab.Size.PanelDrawerCoefficient = PanelDrawerTabJson->GetNumberField(TEXT("PanelDrawerCoefficient"));
					NewArea->AddPanelDrawerInactiveTab(MoveTemp(PanelDrawerTab));
				}
			}
		}

		TArray< TSharedPtr<FJsonValue> > ChildNodeValues = JsonObject->GetArrayField(TEXT("nodes"));
		for( int32 ChildIndex=0; ChildIndex < ChildNodeValues.Num(); ++ChildIndex )
		{
			NewArea->Split( NewFromString_Helper( ChildNodeValues[ChildIndex]->AsObject() ) );
		}

		return NewArea.ToSharedRef();
	}
	else if ( NodeType == TEXT("Splitter") )
	{
		TSharedRef<FTabManager::FSplitter> NewSplitter =  FTabManager::NewSplitter();
		NewSplitter->SetSizeCoefficient((float)JsonObject->GetNumberField(TEXT("SizeCoefficient")) );
		NewSplitter->SetOrientation( local::OrientationFromString( JsonObject->GetStringField(TEXT("Orientation")) ) );
		TArray< TSharedPtr<FJsonValue> > ChildNodeValues = JsonObject->GetArrayField(TEXT("nodes"));
		for( int32 ChildIndex=0; ChildIndex < ChildNodeValues.Num(); ++ChildIndex )
		{
			NewSplitter->Split( NewFromString_Helper( ChildNodeValues[ChildIndex]->AsObject() ) );
		}
		return NewSplitter;
	}
	else if ( NodeType == TEXT("Stack") )
	{
		TSharedRef<FTabManager::FStack> NewStack = FTabManager::NewStack();
		NewStack->SetSizeCoefficient((float)JsonObject->GetNumberField(TEXT("SizeCoefficient")) );
		NewStack->SetHideTabWell( JsonObject->GetBoolField(TEXT("HideTabWell")) );
		{
			double MinSize = 0.0;
			if (JsonObject->TryGetNumberField(TEXT("MinSize"), MinSize))
			{
				NewStack->SetMinSize((float)MinSize);
			}
		}

		if(JsonObject->HasField(TEXT("ForegroundTab")))
		{
			FName TabId = FName( *JsonObject->GetStringField(TEXT("ForegroundTab")) );
			TabId = FGlobalTabmanager::Get()->GetTabTypeForPotentiallyLegacyTab(TabId);
			NewStack->SetForegroundTab( FTabId(TabId) );
		}

		TArray< TSharedPtr<FJsonValue> > TabsAsJson = JsonObject->GetArrayField( TEXT("Tabs") );
		for (int32 TabIndex=0; TabIndex < TabsAsJson.Num(); ++TabIndex)
		{
			TSharedPtr<FJsonObject> TabAsJson = TabsAsJson[TabIndex]->AsObject();
			FName TabId = FName( *TabAsJson->GetStringField(TEXT("TabId")) );
			TabId = FGlobalTabmanager::Get()->GetTabTypeForPotentiallyLegacyTab(TabId);

			FString SidebarLocation;
			float SidebarSizeCoefficient = .15f;
			bool bPinnedInSidebar = false;
			if (TabAsJson->TryGetStringField(TEXT("SidebarLocation"), SidebarLocation))
			{
				TabAsJson->TryGetNumberField(TEXT("SidebarCoeff"), SidebarSizeCoefficient);
				TabAsJson->TryGetBoolField(TEXT("SidebarPinned"), bPinnedInSidebar);
			}

			// For now always have the tab unlocked until the feature is made public, not assigning a value to it will by default keep them unlocked but also allow us internally to have the LevelEditor and HomeScreen tab decide their own state
			NewStack->AddTab(TabId, TabStateFromString( TabAsJson->GetStringField(TEXT("TabState"))), SidebarLocationFromString(SidebarLocation), SidebarSizeCoefficient, bPinnedInSidebar);
		}
		return NewStack;
	}
	else
	{
		ensureMsgf(false, TEXT("Unrecognized node type."));
		return FTabManager::NewArea(FTabManager::FallbackWindowSize);
	}	
}

TSharedPtr<FTabManager::FLayout> FTabManager::FLayout::NewFromString( const FString& LayoutAsText )
{
	TSharedPtr<FJsonObject> JsonObject;

	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( LayoutAsText );
	if (FJsonSerializer::Deserialize( Reader, JsonObject ))
	{
		return NewFromJson( JsonObject );
	}
	
	return TSharedPtr<FTabManager::FLayout>();
}

TSharedPtr<FTabManager::FLayout> FTabManager::FLayout::NewFromJson( const TSharedPtr<FJsonObject>& LayoutAsJson )
{
	if (!LayoutAsJson.IsValid())
	{
		return TSharedPtr<FTabManager::FLayout>();
	}

	const FString LayoutName = LayoutAsJson->GetStringField(TEXT("Name"));
	TSharedRef<FTabManager::FLayout> NewLayout = FTabManager::NewLayout( *LayoutName );
	int32 PrimaryAreaIndex = FMath::TruncToInt((float)LayoutAsJson->GetNumberField(TEXT("PrimaryAreaIndex")) );

	TArray< TSharedPtr<FJsonValue> > Areas = LayoutAsJson->GetArrayField(TEXT("Areas"));
	for(int32 AreaIndex=0; AreaIndex < Areas.Num(); ++AreaIndex)
	{
		TSharedRef<FTabManager::FArea> NewArea = StaticCastSharedRef<FTabManager::FArea>( NewFromString_Helper( Areas[AreaIndex]->AsObject() ) );
		NewLayout->AddArea( NewArea );
		if (AreaIndex == PrimaryAreaIndex)
		{
			NewLayout->PrimaryArea = NewArea;
		}
	}
		
	return NewLayout;
}

FName FTabManager::FLayout::GetLayoutName() const
{
	return LayoutName;
}

TSharedRef<FJsonObject> FTabManager::FLayout::ToJson() const
{
	TSharedRef<FJsonObject> LayoutJson = MakeShareable( new FJsonObject() );
	LayoutJson->SetStringField( TEXT("Type"), TEXT("Layout") );
	LayoutJson->SetStringField( TEXT("Name"), LayoutName.ToString() );

	LayoutJson->SetNumberField( TEXT("PrimaryAreaIndex"), INDEX_NONE );

	TArray< TSharedPtr<FJsonValue> > AreasAsJson;
	for ( int32 AreaIndex=0; AreaIndex < Areas.Num(); ++AreaIndex )
	{
		if (PrimaryArea.Pin() == Areas[AreaIndex])
		{
			LayoutJson->SetNumberField( TEXT("PrimaryAreaIndex"), AreaIndex );
		}
		AreasAsJson.Add( MakeShareable( new FJsonValueObject( PersistToString_Helper( Areas[AreaIndex] ) ) ) );
	}
	LayoutJson->SetArrayField( TEXT("Areas"), AreasAsJson );

	return LayoutJson;
}

FString FTabManager::FLayout::ToString() const
{
	TSharedRef<FJsonObject> LayoutJson = this->ToJson();

	FString LayoutAsString;
	TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create( &LayoutAsString );
	if (!FJsonSerializer::Serialize(LayoutJson, Writer))
	{
		UE_LOGF(LogSlate, Error, "Failed save layout as Json string: %ls", *GetLayoutName().ToString());
	}

	return LayoutAsString;
}


TSharedRef<FJsonObject> FTabManager::FLayout::PersistToString_Helper(const TSharedRef<FLayoutNode>& NodeToPersist)
{
	TSharedRef<FJsonObject> JsonObj = MakeShareable(new FJsonObject());

	TSharedPtr<FTabManager::FStack> NodeAsStack = NodeToPersist->AsStack();
	TSharedPtr<FTabManager::FSplitter> NodeAsSplitter = NodeToPersist->AsSplitter();
	TSharedPtr<FTabManager::FArea> NodeAsArea = NodeToPersist->AsArea();


	JsonObj->SetNumberField( TEXT("SizeCoefficient"), NodeToPersist->SizeCoefficient );

	if ( NodeAsArea.IsValid() )
	{
		JsonObj->SetStringField( TEXT("Type"), TEXT("Area") );
		JsonObj->SetStringField( TEXT("Orientation"), (NodeAsArea->GetOrientation() == Orient_Horizontal) ? TEXT("Orient_Horizontal") : TEXT("Orient_Vertical")  );

		if ( NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_Automatic )
		{
			JsonObj->SetStringField( TEXT("WindowPlacement"), GET_ENUMERATOR_NAME_STRING_CHECKED(FArea::EWindowPlacementInternal, Placement_Automatic) );
			JsonObj->SetNumberField( TEXT("WindowSize_X"), NodeAsArea->UnscaledWindowSize.X );
			JsonObj->SetNumberField( TEXT("WindowSize_Y"), NodeAsArea->UnscaledWindowSize.Y );
		}
		else if (NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_NoWindow)
		{
			JsonObj->SetStringField( TEXT("WindowPlacement"), GET_ENUMERATOR_NAME_STRING_CHECKED(FArea::EWindowPlacementInternal, Placement_NoWindow) );
		}
		else if ( NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_Specified 
			|| NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_ParentSpecified )
		{
			if (NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_Specified)
			{
				JsonObj->SetStringField(TEXT("WindowPlacement"), GET_ENUMERATOR_NAME_STRING_CHECKED(FArea::EWindowPlacementInternal, Placement_Specified) );
			}
			else
			{
				JsonObj->SetStringField(TEXT("WindowPlacement"), GET_ENUMERATOR_NAME_STRING_CHECKED(FArea::EWindowPlacementInternal, Placement_ParentSpecified) );
			}

			JsonObj->SetNumberField( TEXT("WindowPosition_X"), NodeAsArea->UnscaledWindowPosition.X );
			JsonObj->SetNumberField( TEXT("WindowPosition_Y"), NodeAsArea->UnscaledWindowPosition.Y );
			JsonObj->SetNumberField( TEXT("WindowSize_X"), NodeAsArea->UnscaledWindowSize.X );
			JsonObj->SetNumberField( TEXT("WindowSize_Y"), NodeAsArea->UnscaledWindowSize.Y );
			JsonObj->SetBoolField( TEXT("bIsMaximized"), NodeAsArea->bIsMaximized );
		}

		if (!NodeAsArea->ActivePanelDrawerTab.TabId.TabType.IsNone())
		{
			JsonObj->SetStringField(TEXT("ActivePanelDrawerTab"), NodeAsArea->ActivePanelDrawerTab.TabId.ToString());
			JsonObj->SetNumberField(TEXT("MainContentCoefficient"), NodeAsArea->ActivePanelDrawerTab.Size.MainContentCoefficient);
			JsonObj->SetNumberField(TEXT("PanelDrawerCoefficient"), NodeAsArea->ActivePanelDrawerTab.Size.PanelDrawerCoefficient);
		}

		TArray<TSharedPtr<FJsonValue>> InactivePanelDrawerTabs;
		for (const FPanelDrawerTab& PanelDrawerTab : NodeAsArea->InactivePanelDrawerTabs)
		{
			TSharedRef<FJsonObject> PanelDrawerTabJson = MakeShared<FJsonObject>();
			PanelDrawerTabJson->SetStringField(TEXT("TabId"), PanelDrawerTab.TabId.ToString());
			PanelDrawerTabJson->SetNumberField(TEXT("MainContentCoefficient"), PanelDrawerTab.Size.MainContentCoefficient);
			PanelDrawerTabJson->SetNumberField(TEXT("PanelDrawerCoefficient"), PanelDrawerTab.Size.PanelDrawerCoefficient);
			InactivePanelDrawerTabs.Add(MakeShared<FJsonValueObject>(MoveTemp(PanelDrawerTabJson)));
		}

		if (!InactivePanelDrawerTabs.IsEmpty())
		{
			JsonObj->SetArrayField(TEXT("InactivePanelDrawerTabs"), InactivePanelDrawerTabs);
		}

		TArray< TSharedPtr<FJsonValue> > Nodes;
		for ( const TSharedRef<FLayoutNode>& ChildNode : NodeAsArea->GetChildNodes() )
		{
			Nodes.Add( MakeShareable( new FJsonValueObject( PersistToString_Helper( ChildNode ) ) ) );
		}
		JsonObj->SetArrayField( TEXT("Nodes"), Nodes );
	}
	else if ( NodeAsSplitter.IsValid() )
	{
		JsonObj->SetStringField( TEXT("Type"), TEXT("Splitter") );
		JsonObj->SetStringField( TEXT("Orientation"), (NodeAsSplitter->GetOrientation() == Orient_Horizontal) ? TEXT("Orient_Horizontal") : TEXT("Orient_Vertical")  );

		TArray< TSharedPtr<FJsonValue> > Nodes;
		for ( const TSharedRef<FLayoutNode>& ChildNode : NodeAsSplitter->GetChildNodes() )
		{
			Nodes.Add( MakeShareable( new FJsonValueObject( PersistToString_Helper( ChildNode ) ) ) ) ;
		}
		JsonObj->SetArrayField( TEXT("Nodes"), Nodes );
	}
	else if ( NodeAsStack.IsValid() )
	{
		JsonObj->SetStringField( TEXT("Type"), TEXT("Stack") );
		JsonObj->SetBoolField( TEXT("HideTabWell"), NodeAsStack->bHideTabWell );
		if (NodeAsStack->MinSize != 0.f)
		{
			JsonObj->SetNumberField( TEXT("MinSize"), NodeAsStack->MinSize );
		}

		if (NodeAsStack->ForegroundTabId.ShouldSaveLayout())
		{
			JsonObj->SetStringField(TEXT("ForegroundTab"), NodeAsStack->ForegroundTabId.ToString());
		}
		

		TArray< TSharedPtr<FJsonValue> > TabsAsJson;
		for(const FTab& Tab : NodeAsStack->Tabs)
		{
			if (Tab.TabId.ShouldSaveLayout())
			{	
				TSharedRef<FJsonObject> TabAsJson = MakeShareable( new FJsonObject() );
				TabAsJson->SetStringField( TEXT("TabId"), Tab.TabId.ToString() );
				TabAsJson->SetStringField(TEXT("TabState"), StringFromTabState(Tab.TabState));

				if (Tab.TabState == ETabState::SidebarTab && Tab.SidebarLocation != ESidebarLocation::None)
				{
					TabAsJson->SetStringField(TEXT("SidebarLocation"), StringFromSidebarLocation(Tab.SidebarLocation));
					TabAsJson->SetNumberField(TEXT("SidebarCoeff"), Tab.SidebarSizeCoefficient);
					TabAsJson->SetBoolField(TEXT("SidebarPinned"), Tab.bPinnedInSidebar);
				}

				TabsAsJson.Add( MakeShareable( new FJsonValueObject(TabAsJson) ) );
			}
		}
		JsonObj->SetArrayField( TEXT("Tabs"), TabsAsJson );
	}
	else
	{
		ensureMsgf( false, TEXT("Unable to persist layout node of unknown type.") );
	}

	return JsonObj;
}

void FTabManager::FLayout::ProcessExtensions(const FLayoutExtender& Extender)
{
	// Extend areas first
	for (TSharedRef<FTabManager::FArea>& Area : Areas)
	{
		Extender.ExtendAreaRecursive(Area);
	}

	struct FTabInformation
	{
		FTabInformation(FTabManager::FLayout& Layout)
		{
			for (TSharedRef<FTabManager::FArea>& Area : Layout.Areas)
			{
				Gather(*Area);
			}
		}

		void Gather(FTabManager::FSplitter& Splitter)
		{
			for (TSharedRef<FTabManager::FLayoutNode>& Child : Splitter.ChildNodes)
			{
				TSharedPtr<FTabManager::FStack> Stack = Child->AsStack();
				if (Stack.IsValid())
				{
					StackToParentSplitterMap.Add(Stack.Get(), &Splitter);

					AllStacks.Add(Stack.Get());

					for (FTabManager::FTab& Tab : Stack->Tabs)
					{
						AllDefinedTabs.Add(Tab.TabId);
					}

					continue;
				}

				TSharedPtr<FTabManager::FSplitter> ChildSplitter = Child->AsSplitter();
				if (ChildSplitter.IsValid())
				{
					Gather(*ChildSplitter);
					continue;
				}

				TSharedPtr<FTabManager::FArea> Area = Child->AsArea();
				if (Area.IsValid())
				{
					Gather(*Area);
					continue;
				}
			}
		}

		bool Contains(FTabId TabId) const
		{
			return AllDefinedTabs.Contains(TabId);
		}

		TMap<FTabManager::FStack*, FTabManager::FSplitter*> StackToParentSplitterMap;
		TArray<FTabManager::FStack*> AllStacks;
		TSet<FTabId> AllDefinedTabs;
	};
	FTabInformation AllTabs(*this);

	TArray<FTab, TInlineAllocator<1>> ExtendedTabs;

	for (FTabManager::FStack* Stack : AllTabs.AllStacks)
	{
		// First add to the front of the stack
		Extender.FindStackExtensions(Stack->GetExtensionId(), ELayoutExtensionPosition::Before, ExtendedTabs);
		int32 InsertedTabIndex = 0;
		for (FTab& NewTab : ExtendedTabs)
		{
			if (!AllTabs.Contains(NewTab.TabId))
			{
				Stack->Tabs.Insert(NewTab, InsertedTabIndex++);
			}
		}

		// This is the per-tab extension section
		FSplitter* ParentSplitter = AllTabs.StackToParentSplitterMap.FindRef(Stack);
		for (int32 TabIndex = 0; TabIndex < Stack->Tabs.Num();)
		{
			FTabId TabId = Stack->Tabs[TabIndex].TabId;

			Extender.FindTabExtensions(TabId, ELayoutExtensionPosition::Before, ExtendedTabs);
			for (FTab& NewTab : ExtendedTabs)
			{
				if (!AllTabs.Contains(NewTab.TabId))
				{
					Stack->Tabs.Insert(NewTab, TabIndex++);
				}
			}

			++TabIndex;

			Extender.FindTabExtensions(TabId, ELayoutExtensionPosition::After, ExtendedTabs);
			for (FTab& NewTab : ExtendedTabs)
			{
				if (!AllTabs.Contains(NewTab.TabId))
				{
					Stack->Tabs.Insert(NewTab, TabIndex++);
				}
			}

	
			if (ParentSplitter)
			{
				Extender.FindTabExtensions(TabId, ELayoutExtensionPosition::Below, ExtendedTabs);
				if (ExtendedTabs.Num())
				{
					for (FTab& NewTab : ExtendedTabs)
					{
						if (!AllTabs.Contains(NewTab.TabId))
						{
							ParentSplitter->InsertAfter(Stack->AsShared(),
								FTabManager::NewStack()
								->SetHideTabWell(true)
								->AddTab(NewTab)
							);
						}
					}
				}

				Extender.FindTabExtensions(TabId, ELayoutExtensionPosition::Above, ExtendedTabs);
				if (ExtendedTabs.Num())
				{
					for (FTab& NewTab : ExtendedTabs)
					{
						if (!AllTabs.Contains(NewTab.TabId))
						{
							ParentSplitter->InsertBefore(Stack->AsShared(),
								FTabManager::NewStack()
								->SetHideTabWell(true)
								->AddTab(NewTab)
							);
						}
					}
				}
			}
		}

		// Finally add to the end of the stack
		Extender.FindStackExtensions(Stack->GetExtensionId(), ELayoutExtensionPosition::After, ExtendedTabs);
		InsertedTabIndex = Stack->Tabs.Num();
		for (FTab& NewTab : ExtendedTabs)
		{
			if (!AllTabs.Contains(NewTab.TabId))
			{
				Stack->Tabs.Insert(NewTab, InsertedTabIndex++);
			}
		}
		
		// Check if any of the extended tabs were marked as needing to be foregrounded
		const FTabId ForegroundTab = Extender.FindFirstForegroundTab(Stack->GetExtensionId());
		if (ForegroundTab.TabType != NAME_None)
		{
			Stack->SetForegroundTab(ForegroundTab);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FTabManager::PrivateApi
//////////////////////////////////////////////////////////////////////////

TSharedPtr<SWindow> FTabManager::FPrivateApi::GetParentWindow() const
{
	TSharedPtr<SDockTab> OwnerTab = TabManager.OwnerTabPtr.Pin();
	if ( OwnerTab.IsValid() )
	{
		// The tab was dragged out of some context that is owned by a MajorTab.
		// Whichever window possesses the MajorTab should be the parent of the newly created window.
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( OwnerTab.ToSharedRef() );
		return ParentWindow;
	}
	else
	{
		// This tab is not nested within a major tab, so it is a major tab itself.
		// Ask the global tab manager for its root window.
		return FGlobalTabmanager::Get()->GetRootWindow();
	}
}

void FTabManager::FPrivateApi::OnDockAreaCreated( const TSharedRef<SDockingArea>& NewlyCreatedDockArea )
{
	CleanupPointerArray(TabManager.DockAreas);
	TabManager.DockAreas.Add( NewlyCreatedDockArea );
}

void FTabManager::FPrivateApi::OnTabRelocated( const TSharedRef<SDockTab>& RelocatedTab, const TSharedPtr<SWindow>& NewOwnerWindow )
{
	TabManager.OnTabRelocated(RelocatedTab, NewOwnerWindow);
}

void FTabManager::FPrivateApi::OnTabOpening( const TSharedRef<SDockTab>& TabBeingOpened )
{
	TabManager.OnTabOpening(TabBeingOpened);
}

void FTabManager::FPrivateApi::OnTabClosing( const TSharedRef<SDockTab>& TabBeingClosed )
{
	TabManager.OnTabClosing(TabBeingClosed);
}

void FTabManager::FPrivateApi::OnDockAreaClosing( const TSharedRef<SDockingArea>& DockAreaThatIsClosing )
{
	TSharedPtr<FTabManager::FArea> PersistentDockAreaLayout = StaticCastSharedPtr<FTabManager::FArea>(DockAreaThatIsClosing->GatherPersistentLayout());

	if ( PersistentDockAreaLayout.IsValid() )
	{
		TabManager.CollapsedDockAreas.Add( PersistentDockAreaLayout.ToSharedRef() );
	}

	TabManager.HandleClosingAreaPanelDrawerData(DockAreaThatIsClosing);

}

void FTabManager::FPrivateApi::OnTabManagerClosing()
{
	TabManager.OnTabManagerClosing();
}

bool FTabManager::FPrivateApi::CanTabLeaveTabWell(const TSharedRef<const SDockTab>& TabToTest) const
{
	return TabManager.bCanDoDragOperation && !(TabToTest->GetLayoutIdentifier() == TabManager.MainNonCloseableTabID);
}

const TArray< TWeakPtr<SDockingArea> >& FTabManager::FPrivateApi::GetLiveDockAreas() const
{
	return TabManager.DockAreas;
}

void FTabManager::FPrivateApi::OnTabForegrounded(const TSharedPtr<SDockTab>& NewForegroundTab, const TSharedPtr<SDockTab>& BackgroundedTab)
{
	TabManager.OnTabForegrounded(NewForegroundTab, BackgroundedTab);
}

static void SetWindowVisibility(const TArray< TWeakPtr<SDockingArea> >& DockAreas, bool bWindowShouldBeVisible)
{
	for (int32 DockAreaIndex = 0; DockAreaIndex < DockAreas.Num(); ++DockAreaIndex)
	{
		TSharedPtr<SWindow> DockAreaWindow = DockAreas[DockAreaIndex].Pin()->GetParentWindow();
		if (DockAreaWindow.IsValid())
		{
			if (bWindowShouldBeVisible)
			{
				DockAreaWindow->ShowWindow();
			}
			else
			{
				DockAreaWindow->HideWindow();
			}
		}
	}
}

void FTabManager::FPrivateApi::ShowWindows()
{
	CleanupPointerArray(TabManager.DockAreas);
	SetWindowVisibility(TabManager.DockAreas, true);
}

void FTabManager::FPrivateApi::HideWindows()
{
	CleanupPointerArray(TabManager.DockAreas);
	SetWindowVisibility(TabManager.DockAreas, false);
}

void FTabManager::FPrivateApi::SetCanDoDeferredLayoutSave(bool bInCanDoDeferredLayoutSave)
{
	if (!bInCanDoDeferredLayoutSave)
	{
		TabManager.ClearPendingLayoutSave();
	}
	TabManager.bCanDoDeferredLayoutSave = bInCanDoDeferredLayoutSave;
}

TSharedPtr<SDockingArea> FTabManager::FPrivateApi::GetDockingAreaForPanelDrawer(const TSharedPtr<SWindow>& InWindow) const
{
	return TabManager.GetDockingAreaForPanelDrawer(InWindow);
}

FTabManager::FPrivateApi& FTabManager::GetPrivateApi()
{
	return *PrivateApi;
}


void FTabManager::SetAllowWindowMenuBar(bool bInAllowWindowMenuBar)
{
	bAllowPerWindowMenu = bInAllowWindowMenuBar;
}

void FTabManager::SetMenuMultiBox(const TSharedPtr<FMultiBox> NewMenuMutliBox, const TSharedPtr<SWidget> NewMenuWidget)
{
	// We only use the platform native global menu bar on Mac
	MenuMultiBox = NewMenuMutliBox;
	MenuWidget = NewMenuWidget;

	UpdateMainMenu(OwnerTabPtr.Pin(), false);
}

void FTabManager::UpdateMainMenu(TSharedPtr<SDockTab> ForTab, const bool bForce)
{
	bool bIsMajorTab = true;

	TSharedPtr<SWindow> ParentWindowOfOwningTab;
	if (ForTab && (ForTab->GetTabRole() == ETabRole::MajorTab || ForTab->GetVisualTabRole() == ETabRole::MajorTab))
	{
		ParentWindowOfOwningTab = ForTab->GetParentWindow();
	}
	else if (auto OwnerTabPinned = OwnerTabPtr.Pin())
	{
		ParentWindowOfOwningTab = OwnerTabPinned->GetParentWindow();
	}
	else if (auto MainNonCloseableTabPinned = FindExistingLiveTab(MainNonCloseableTabID))
	{
		ParentWindowOfOwningTab = MainNonCloseableTabPinned->GetParentWindow();
	}

	if (bAllowPerWindowMenu)
	{
		if (ParentWindowOfOwningTab)
		{
			ParentWindowOfOwningTab->GetTitleBar()->UpdateWindowMenu(MenuWidget);
		}
	}
	else
	{
		MenuMultiBox.Reset();
		MenuWidget.Reset();
		if (ParentWindowOfOwningTab)
		{
			ParentWindowOfOwningTab->GetTitleBar()->UpdateWindowMenu(nullptr);
		}
	}
}

void FTabManager::SetMainTab(const FTabId& InMainTabID)
{
	MainNonCloseableTabID = InMainTabID;
}

void FTabManager::SetMainTab(const TSharedRef<const SDockTab>& InTab)
{
	if(!InTab->GetLayoutIdentifier().TabType.IsNone())
	{
		SetMainTab(InTab->GetLayoutIdentifier());
	}
	else
	{
		PendingMainNonClosableTab = InTab;
	}
	
}

void FTabManager::SetReadOnly(bool bInReadOnly)
{
	if(bReadOnly != bInReadOnly)
	{
		bReadOnly = bInReadOnly;
		OnReadOnlyModeChanged.Broadcast(bReadOnly);
	}
}

bool FTabManager::IsReadOnly()
{
	return bReadOnly;
}

bool FTabManager::IsTabCloseable(const TSharedRef<const SDockTab>& InTab) const
{
	return MainNonCloseableTabID != InTab->GetLayoutIdentifier();
}

const TSharedRef<FWorkspaceItem> FTabManager::GetLocalWorkspaceMenuRoot() const
{
	return LocalWorkspaceMenuRoot.ToSharedRef();
}

TSharedRef<FWorkspaceItem> FTabManager::AddLocalWorkspaceMenuCategory( const FText& CategoryTitle )
{
	return LocalWorkspaceMenuRoot->AddGroup( CategoryTitle );
}

void FTabManager::AddLocalWorkspaceMenuItem( const TSharedRef<FWorkspaceItem>& CategoryItem )
{
	LocalWorkspaceMenuRoot->AddItem( CategoryItem );
}

void FTabManager::ClearLocalWorkspaceMenuCategories()
{
	LocalWorkspaceMenuRoot->ClearItems();
}

TSharedPtr<FTabManager::FStack> FTabManager::FLayoutNode::AsStack()
{
	return TSharedPtr<FTabManager::FStack>();
}

TSharedPtr<FTabManager::FSplitter> FTabManager::FLayoutNode::AsSplitter()
{
	return TSharedPtr<FTabManager::FSplitter>();
}

TSharedPtr<FTabManager::FArea> FTabManager::FLayoutNode::AsArea()
{
	return TSharedPtr<FTabManager::FArea>();
}

void FTabManager::SetOnPersistLayout( const FOnPersistLayout& InHandler )
{
	OnPersistLayout_Handler = InHandler;
}

void FTabManager::CloseAllAreas()
{
	for ( int32 LiveAreaIndex=0; LiveAreaIndex < DockAreas.Num(); ++LiveAreaIndex )
	{
		const TSharedPtr<SDockingArea> SomeDockArea = DockAreas[LiveAreaIndex].Pin();
		if (SomeDockArea)
		{
			HandleClosingAreaPanelDrawerData(SomeDockArea);

			const TSharedPtr<SWindow> ParentWindow = SomeDockArea->GetParentWindow();
			if (ParentWindow.IsValid())
			{
				ParentWindow->RequestDestroyWindow();
			}
		}
	}
	DockAreas.Empty();

	CollapsedDockAreas.Empty();
	InvalidDockAreas.Empty();
}


TSharedRef<FTabManager::FLayout> FTabManager::PersistLayout() const
{
	TSharedRef<FLayout> PersistentLayout = FTabManager::NewLayout( this->ActiveLayoutName );
	
	// Persist layout for all LiveAreas
	for ( int32 LiveAreaIndex=0; LiveAreaIndex < DockAreas.Num(); ++LiveAreaIndex )
	{
		TSharedPtr<FArea> PersistedNode;
		TSharedPtr<SDockingArea> ChildDockingArea = DockAreas[LiveAreaIndex].Pin();

		if ( ChildDockingArea.IsValid() )
		{
			TSharedPtr<FTabManager::FLayoutNode> LayoutNode = ChildDockingArea->GatherPersistentLayout();
			if (LayoutNode.IsValid())
			{
				PersistedNode = LayoutNode->AsArea();
			}
		}

		if ( PersistedNode.IsValid() )
		{
			PersistentLayout->AddArea( PersistedNode.ToSharedRef() );
			if (PersistedNode->WindowPlacement == FArea::EWindowPlacementInternal::Placement_NoWindow)
			{
				ensure( !PersistentLayout->PrimaryArea.IsValid() );
				PersistentLayout->PrimaryArea = PersistedNode;

				if (this == &FGlobalTabmanager::Get().Get())
				{
					// Convert the area placement to use the ParentSpecified mode so that the global that manager can restore the main frame window
					if (TSharedPtr<SWindow> ParentWindow = ChildDockingArea->GetParentWindow())
					{
						UE::Slate::Private::SetAreaNodeWindowPlacementValues(PersistedNode, *ParentWindow.Get());
						PersistedNode->WindowPlacement = FArea::EWindowPlacementInternal::Placement_ParentSpecified;
					}
				}
			}
		}
	}

	// Gather existing persistent layouts for CollapsedAreas
	for ( int32 CollapsedAreaIndex=0; CollapsedAreaIndex < CollapsedDockAreas.Num(); ++CollapsedAreaIndex )
	{
		PersistentLayout->AddArea( CollapsedDockAreas[CollapsedAreaIndex] );
	}

	// Gather existing persistent layouts for InvalidAreas
	for (int32 InvalidAreaIndex = 0; InvalidAreaIndex < InvalidDockAreas.Num(); ++InvalidAreaIndex)
	{
		PersistentLayout->AddArea(InvalidDockAreas[InvalidAreaIndex]);
	}

	return PersistentLayout;
}

void FTabManager::SavePersistentLayout()
{
	ClearPendingLayoutSave();

	const TSharedRef<FLayout> LayoutState = this->PersistLayout();
	OnPersistLayout_Handler.ExecuteIfBound(LayoutState);
}

void FTabManager::RequestSavePersistentLayout()
{
	// if we already have a request pending, remove it and schedule a new one
	// this is to avoid hitches when eg. resizing a docked tab
	ClearPendingLayoutSave();

	if (!bCanDoDeferredLayoutSave)
	{
		return;
	}

	auto OnTick = [ThisWeak = AsWeak()](float FrameTime)
	{
		if (TSharedPtr<FTabManager> This = ThisWeak.Pin())
		{
			This->PendingLayoutSaveHandle.Reset();
			This->SavePersistentLayout();
		}
		return false;
	};

	PendingLayoutSaveHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(OnTick), 5.0f);
}

void FTabManager::ClearPendingLayoutSave()
{
	if (PendingLayoutSaveHandle.IsValid())
	{
		FTSTicker::RemoveTicker(PendingLayoutSaveHandle);
		PendingLayoutSaveHandle.Reset();
	}
}

FTabSpawnerEntry& FTabManager::RegisterTabSpawner(const FName TabId, const FOnSpawnTab& OnSpawnTab, const FCanSpawnTab& CanSpawnTab)
{
	ensure(!TabSpawner.Contains(TabId));
	ensure(!FGlobalTabmanager::Get()->IsLegacyTabType(TabId));

	LLM_SCOPE_BYTAG(UI_Slate);

	TSharedRef<FTabSpawnerEntry> NewSpawnerEntry = MakeShareable(new FTabSpawnerEntry(TabId, OnSpawnTab, CanSpawnTab));
	TabSpawner.Add(TabId, NewSpawnerEntry);

	return NewSpawnerEntry.Get();
}

bool FTabManager::UnregisterTabSpawner( const FName TabId )
{
	return TabSpawner.Remove( TabId ) > 0;
}

void FTabManager::UnregisterAllTabSpawners()
{
	TabSpawner.Empty();
}

TSharedPtr<SWidget> FTabManager::RestoreFrom(const TSharedRef<FLayout>& Layout, const TSharedPtr<SWindow>& ParentWindow, const bool bEmbedTitleAreaContent,
	const EOutputCanBeNullptr RestoreAreaOutputCanBeNullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTabManager::RestoreFrom);

	ActiveLayoutName = Layout->LayoutName;

	TSharedPtr<SDockingArea> PrimaryDockArea;
	for (int32 AreaIndex=0; AreaIndex < Layout->Areas.Num(); ++AreaIndex )
	{
		const TSharedRef<FArea>& ThisArea = Layout->Areas[AreaIndex];
		// Set all InvalidTab tabs to OpenedTab so the Editor tries to load them. All non-recognized tabs will be set to InvalidTab later.
		SetTabsTo(ThisArea, ETabState::OpenedTab, ETabState::InvalidTab);

		// Use in the placement has a fallback for old layout where the primary area might not have been saved
		const bool bIsPrimaryArea = ThisArea == Layout->GetPrimaryArea() || ThisArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_NoWindow;

		// If this is the primary area containing the LevelEditor based on the HomeScreen CVar it will either force the HomeScreen and LevelEditor position (and always spawn the HomeScreen) or just the LevelEditor position
		if (bIsPrimaryArea)
		{
			FixLayoutLoadingPrimaryArea(ThisArea);
		}

		// Check the Area after it was cleared of possible HomeScreen tab that remained during testing in the layout config
		const bool bShouldCreate = bIsPrimaryArea || HasValidTabs(ThisArea);

		if ( bShouldCreate )
		{
			TSharedPtr<SDockingArea> RestoredDockArea;
			const bool bHasValidOpenTabs = bIsPrimaryArea || HasValidOpenTabs(ThisArea);

			if (bHasValidOpenTabs)
			{
				RestoredDockArea = RestoreArea(ThisArea, ParentWindow, bEmbedTitleAreaContent, RestoreAreaOutputCanBeNullptr);
				// Invalidate all tabs in ThisArea because they were not recognized
				if (!RestoredDockArea)
				{
					if (bIsPrimaryArea)
					{
						UE_LOGF(LogSlate, Warning, "Primary area was not valid for RestoreAreaOutputCanBeNullptr = %d.", int(RestoreAreaOutputCanBeNullptr));
					}
					SetTabsTo(ThisArea, ETabState::InvalidTab, ETabState::OpenedTab);
					InvalidDockAreas.Add(ThisArea);
				}
			}
			else
			{
				CollapsedDockAreas.Add(ThisArea);
			}

			if (bIsPrimaryArea && RestoredDockArea.IsValid() && ensure(!PrimaryDockArea.IsValid()))
			{
				PrimaryDockArea = RestoredDockArea;
			}
		}
	}

	// Sanity check
	if (RestoreAreaOutputCanBeNullptr == EOutputCanBeNullptr::Never && !PrimaryDockArea.IsValid())
	{
		UE_LOGF(LogSlate, Warning, "FTabManager::RestoreFrom(): RestoreAreaOutputCanBeNullptr was set to EOutputCanBeNullptr::Never but"
			" RestoreFrom() is returning nullptr. I.e., the PrimaryDockArea could not be created. If returning nullptr is possible, set"
			" RestoreAreaOutputCanBeNullptr to an option that could return nullptr (e.g., IfNoTabValid, IfNoOpenTabValid). This code might"
			" ensure(false) or even check(false) in the future.");
	}

	UpdateStats();

	FinishRestore();

	return PrimaryDockArea;
}

TSharedPtr<SWidget> FTabManager::RestorePanelDrawer(const TSharedRef<SWidget>& InContent, const TSharedRef<SWindow>& ParentWindow)
{
	TSharedPtr<SDockingArea> HostArea = GetDockingAreaForPanelDrawer(ParentWindow);

	if (!ensureAlwaysMsgf(HostArea, TEXT("Cannot create a drawer in a window that is not managed by the docking system")))
	{
		return {};
	}

	TSharedRef<UE::Slate::Private::SPanelDrawerArea> NewPanelDrawerArea = SNew(UE::Slate::Private::SPanelDrawerArea, InContent);
	NewPanelDrawerArea->GetOnExternalStateChanged().BindSP(this, &FTabManager::OnPanelDrawerStateChanged);
	HostArea->SetPanelDrawerArea(NewPanelDrawerArea);

	return MoveTemp(NewPanelDrawerArea);
}

bool FTabManager::HasPanelDrawer(const TSharedPtr<SWindow>& ParentWindow) const
{
	if (TSharedPtr<SDockingArea> DockingArea = GetDockingAreaForPanelDrawer(ParentWindow))
	{
		return DockingArea->HasPanelDrawer();
	}

	return false;
}

bool FTabManager::IsPanelDrawerOpen(const TSharedPtr<SWindow>& ParentWindow) const
{
	if (TSharedPtr<SDockingArea> DockingArea = GetDockingAreaForPanelDrawer(ParentWindow))
	{
		return DockingArea->IsPanelDrawerOpen();
	}

	return false;
}

void FTabManager::ClosePanelDrawer(const TSharedPtr<SWindow>& ParentWindow)
{
	if (TSharedPtr<SDockingArea> DockingArea = GetDockingAreaForPanelDrawer(ParentWindow))
	{
		if (DockingArea->IsPanelDrawerOpen())
		{
			DockingArea->ClosePanelDrawer();
		}
	}
}

FDelegateHandle FTabManager::RegisterOnPanelDrawerStateChanges(FOnPanelDrawerStateChanged::FDelegate&& InDelegate)
{
	return OnPanelDrawerStateChangedDelegate.Add(MoveTemp(InDelegate));
}

void FTabManager::UnregisterOnPanelDrawerStateChanges(FDelegateHandle InHandle)
{
	OnPanelDrawerStateChangedDelegate.Remove(InHandle);
}

struct FPopulateTabSpawnerMenu_Args
{
	FPopulateTabSpawnerMenu_Args( const TSharedRef< TArray< TWeakPtr<FTabSpawnerEntry> > >& InAllSpawners, const TSharedRef<FWorkspaceItem>& InMenuNode, int32 InLevel )
		: AllSpawners( InAllSpawners )
		, MenuNode( InMenuNode )
		, Level( InLevel )	
	{
	}

	TSharedRef< TArray< TWeakPtr<FTabSpawnerEntry> > > AllSpawners;
	TSharedRef<FWorkspaceItem> MenuNode;
	int32 Level;
};

/** Scoped guard to ensure that we reset the GuardedValue to false */
struct FScopeGuard
{
	FScopeGuard( bool & InGuardedValue )
	: GuardedValue(InGuardedValue)
	{
		GuardedValue = true;
	}

	~FScopeGuard()
	{
		GuardedValue = false;	
	}

	private:
		bool& GuardedValue;

};

void FTabManager::PopulateTabSpawnerMenu_Helper( FMenuBuilder& PopulateMe, FPopulateTabSpawnerMenu_Args Args )
{
	const TArray< TSharedRef<FWorkspaceItem> >& ChildItems = Args.MenuNode->GetChildItems();

	bool bFirstItemOnLevel = true;

	for ( int32 ChildIndex=0; ChildIndex < ChildItems.Num(); ++ChildIndex )
	{
		const TSharedRef<FWorkspaceItem>& ChildItem = ChildItems[ChildIndex];
		const TSharedPtr<FTabSpawnerEntry> SpawnerNode = ChildItem->AsSpawnerEntry();
		if ( SpawnerNode.IsValid() )
		{
			// LEAF NODE.
			// Make a menu item for summoning a tab.
			if (Args.AllSpawners->Contains(SpawnerNode.ToSharedRef()))
			{
				MakeSpawnerMenuEntry(PopulateMe, SpawnerNode);
			}
		}
		else
		{
			// GROUP NODE
			// If it's not empty, create a section and populate it
			if ( ChildItem->HasChildrenIn(*Args.AllSpawners) )
			{
				const FPopulateTabSpawnerMenu_Args Payload( Args.AllSpawners, ChildItem, Args.Level+1 );

				if ( Args.Level % 2 == 0 )
				{
					FString SectionNameStr = ChildItem->GetDisplayName().BuildSourceString();
					SectionNameStr.ReplaceInline(TEXT(" "), TEXT(""));

					PopulateMe.BeginSection(*SectionNameStr, ChildItem->GetDisplayName());
					{
						PopulateTabSpawnerMenu_Helper(PopulateMe, Payload);
					}
					PopulateMe.EndSection();
				}
				else
				{
					PopulateMe.AddSubMenu(
						ChildItem->GetDisplayName(),
						ChildItem->GetTooltipText(),
						FNewMenuDelegate::CreateRaw( this, &FTabManager::PopulateTabSpawnerMenu_Helper, Payload ),
						false,
						ChildItem->GetIcon()
					);
				}

				bFirstItemOnLevel = false;
			}
		}
	}
}

void FTabManager::MakeSpawnerMenuEntry( FMenuBuilder &PopulateMe, const TSharedPtr<FTabSpawnerEntry> &InSpawnerNode ) 
{
	// We don't want to add a menu entry for this tab if it is hidden, or if we are in read only mode and it is asking to be hidden
	if (InSpawnerNode->MenuType.Get() != ETabSpawnerMenuType::Hidden && !(bReadOnly && InSpawnerNode->ReadOnlyBehavior == ETabReadOnlyBehavior::Hidden) )
	{
		const FText Label = GetTabLabelBasedOnSpawner(InSpawnerNode);

		PopulateMe.AddMenuEntry(
			Label,
			InSpawnerNode->GetTooltipText(),
			InSpawnerNode->GetIcon(),
			GetUIActionForTabSpawnerMenuEntry(InSpawnerNode),
			NAME_None,
			EUserInterfaceActionType::Check
			);
	}
}

void FTabManager::PopulateLocalTabSpawnerMenu(FMenuBuilder& PopulateMe)
{
	PopulateTabSpawnerMenu(PopulateMe, LocalWorkspaceMenuRoot.ToSharedRef());
}

void FTabManager::PopulateTabSpawnerMenu(FMenuBuilder& PopulateMe, TSharedRef<FWorkspaceItem> MenuStructure)
{
	PopulateTabSpawnerMenu(PopulateMe, MenuStructure, true);
}

TArray< TWeakPtr<FTabSpawnerEntry> > FTabManager::CollectSpawners()
{
	TArray< TWeakPtr<FTabSpawnerEntry> > AllSpawners;

	// Editor-specific tabs
	for ( FTabSpawner::TIterator SpawnerIterator(TabSpawner); SpawnerIterator; ++SpawnerIterator )
	{
		const TSharedRef<FTabSpawnerEntry>& SpawnerEntry = SpawnerIterator.Value();
		if ( SpawnerEntry->bAutoGenerateMenuEntry )
		{
			if (IsAllowedTab(SpawnerEntry->TabType))
			{
				AllSpawners.AddUnique(SpawnerEntry);
			}
		}
	}

	// General Tabs
	for ( FTabSpawner::TIterator SpawnerIterator(*NomadTabSpawner); SpawnerIterator; ++SpawnerIterator )
	{
		const TSharedRef<FTabSpawnerEntry>& SpawnerEntry = SpawnerIterator.Value();
		if ( SpawnerEntry->bAutoGenerateMenuEntry )
		{
			if (IsAllowedTab(SpawnerEntry->TabType))
			{
				AllSpawners.AddUnique(SpawnerEntry);
			}
		}
	}

	return AllSpawners;
}

void FTabManager::PopulateTabSpawnerMenu( FMenuBuilder& PopulateMe, TSharedRef<FWorkspaceItem> MenuStructure, bool bIncludeOrphanedMenus )
{
	TSharedRef< TArray< TWeakPtr<FTabSpawnerEntry> > > AllSpawners = MakeShared< TArray< TWeakPtr<FTabSpawnerEntry> > >(CollectSpawners());
	
	if ( bIncludeOrphanedMenus )
	{
		// Put all orphaned spawners at the top of the menu so programmers go and find them a nice home.
		for (const TWeakPtr<FTabSpawnerEntry>& WeakSpawner : *AllSpawners)
		{
			const TSharedPtr<FTabSpawnerEntry> Spawner = WeakSpawner.Pin();
			if (!Spawner)
			{
				continue;
			}

			const bool bHasNoPlaceInMenuStructure = !Spawner->GetParent().IsValid();
			if ( bHasNoPlaceInMenuStructure )
			{
				this->MakeSpawnerMenuEntry(PopulateMe, Spawner);
			}
		}
	}

	PopulateTabSpawnerMenu_Helper( PopulateMe, FPopulateTabSpawnerMenu_Args( AllSpawners, MenuStructure, 0 ) );
}

void FTabManager::PopulateTabSpawnerMenu( FMenuBuilder &PopulateMe, const FName& TabType )
{
	TSharedPtr<FTabSpawnerEntry> Spawner = FindTabSpawnerFor(TabType);
	if (Spawner.IsValid())
	{
		MakeSpawnerMenuEntry(PopulateMe, Spawner);
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "PopulateTabSpawnerMenu failed to find entry for %ls", *(TabType.ToString()));
	}
}

void FTabManager::DrawAttention( const TSharedRef<SDockTab>& TabToHighlight )
{
	// Bring the tab to front.
	const TSharedPtr<SDockingArea> DockingArea = TabToHighlight->GetDockArea();
	if (DockingArea.IsValid())
	{
		const TSharedRef<FTabManager> ManagerOfTabToHighlight = DockingArea->GetTabManager();

		if (ManagerOfTabToHighlight != FGlobalTabmanager::Get())
		{
			FGlobalTabmanager::Get()->DrawAttentionToTabManager(ManagerOfTabToHighlight);
		}

		TSharedPtr<SWindow> OwnerWindow = DockingArea->GetParentWindow();

		if (SWindow* OwnerWindowPtr = OwnerWindow.Get())
		{
			// When should we force a window to the front?
			// 1) The owner window is already active, so we know the user is using this screen.
			// 2) This window is a child window of another already active window (same as 1).
			// 3) Slate is currently processing input, which would imply we got this request at the behest of a user's click or press.
			if (OwnerWindowPtr->IsActive() || OwnerWindowPtr->HasActiveParent() || FSlateApplication::Get().IsProcessingInput())
			{
				OwnerWindowPtr->BringToFront();
			}
		}

		if (!DockingArea->TryOpenSidebarDrawer(TabToHighlight))
		{
			if (TSharedPtr<SDockingTabStack> DockingTabStack = TabToHighlight->GetParentDockTabStack())
			{
				DockingTabStack->BringToFront(TabToHighlight);
			}
		}

		TabToHighlight->FlashTab();

		FGlobalTabmanager::Get()->UpdateMainMenu(TabToHighlight, true);
	}
}

void FTabManager::InsertNewDocumentTab(FName PlaceholderId, FName NewTabId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab)
{
	InsertDocumentTab(PlaceholderId, NewTabId, SearchPreference, UnmanagedTab, true);
}

void FTabManager::InsertNewDocumentTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab)
{
	InsertDocumentTab(PlaceholderId, PlaceholderId, SearchPreference, UnmanagedTab, true);
}

void FTabManager::InsertNewDocumentTab( FName PlaceholderId, ESearchPreference::Type SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab )
{
	switch (SearchPreference)
	{
	case ESearchPreference::PreferLiveTab:
	{
		FLiveTabSearch Search;
		InsertDocumentTab(PlaceholderId, PlaceholderId, Search, UnmanagedTab, true);
		break;
	}

	case ESearchPreference::RequireClosedTab:
	{
		FRequireClosedTab Search;
		InsertDocumentTab(PlaceholderId, PlaceholderId, Search, UnmanagedTab, true);
		break;
	}

	default:
		check(false);
		break;
	}
}

void FTabManager::RestoreDocumentTab( FName PlaceholderId, ESearchPreference::Type SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab )
{
	switch (SearchPreference)
	{
	case ESearchPreference::PreferLiveTab:
	{
		FLiveTabSearch Search;
		InsertDocumentTab(PlaceholderId, PlaceholderId, Search, UnmanagedTab, false);
		break;
	}

	case ESearchPreference::RequireClosedTab:
	{
		FRequireClosedTab Search;
		InsertDocumentTab(PlaceholderId, PlaceholderId, Search, UnmanagedTab, false);
		break;
	}

	default:
		check(false);
		break;
	}
}

TSharedPtr<SDockTab> FTabManager::TryInvokeTab(const FTabId& TabId, bool bInvokeAsInactive)
{
	TSharedPtr<SDockTab> NewTab = InvokeTab_Internal(TabId, bInvokeAsInactive, true);
	if (!NewTab.IsValid())
	{
		return NewTab;
	}

	TSharedPtr<SWindow> ParentWindowPtr = NewTab->GetParentWindow();
	if ((NewTab->GetTabRole() == ETabRole::MajorTab || NewTab->GetTabRole() == ETabRole::NomadTab) && ParentWindowPtr.IsValid() && ParentWindowPtr != FGlobalTabmanager::Get()->GetRootWindow())
	{
		ParentWindowPtr->SetTitle(NewTab->GetTabLabel());
	}
#if PLATFORM_MAC
	FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
#endif
	return NewTab;
}

TSharedPtr<SDockTab> FTabManager::TryToggleSidebarTab(const FTabId& TabId)
{
	if (!InvokeTab_CanInvokeTab(TabId))
	{
		return {};
	}

	TSharedPtr<SDockTab> ExistingTab = InvokeTab_FindOrReuseExistingTab(TabId);

	bool bToggled = false;
	if (ExistingTab.IsValid())
	{
		const TSharedPtr<SDockingArea> DockingArea = ExistingTab->GetDockArea();
		if (DockingArea.IsValid())
		{
			bToggled = DockingArea->TryToggleSidebarDrawer(ExistingTab.ToSharedRef());
		}
	}

	return bToggled ? ExistingTab : TSharedPtr<SDockTab>();
}

TSharedPtr<SDockTab> FTabManager::TryDismissSidebarTab(const FTabId& TabId)
{
	if (!InvokeTab_CanInvokeTab(TabId))
	{
		return {};
	}

	TSharedPtr<SDockTab> ExistingTab = InvokeTab_FindOrReuseExistingTab(TabId);

	bool bDismissed = false;
	if (ExistingTab.IsValid())
	{
		const TSharedPtr<SDockingArea> DockingArea = ExistingTab->GetDockArea();
		if (DockingArea.IsValid())
		{
			bDismissed = DockingArea->TryDismissSidebarDrawer(ExistingTab.ToSharedRef());
		}
	}

	return bDismissed ? ExistingTab : TSharedPtr<SDockTab>();
}

bool FTabManager::InvokeTab_CanInvokeTab(const FTabId& TabId) const
{
	if (!IsAllowedTab(TabId))
	{
		UE_LOGF(LogTabManager, Warning, "Cannot spawn tab for '%ls'", *(TabId.ToString()));
		return false;
	}

	return true;
}

TSharedPtr<SDockTab> FTabManager::InvokeTab_FindOrReuseExistingTab(const FTabId& TabId) const
{
	TSharedPtr<const FTabSpawnerEntry> Spawner = FindTabSpawnerFor(TabId.TabType);

	if (!Spawner.IsValid())
	{
		UE_LOGF(LogTabManager, Warning, "Cannot spawn tab because no spawner is registered for '%ls'", *(TabId.ToString()));
	}
	else
	{
		return Spawner->OnFindTabToReuse.IsBound()
			? Spawner->OnFindTabToReuse.Execute(TabId)
			: Spawner->SpawnedTabPtr.Pin();
	}

	return {};
}

void FTabManager::InvokeTab_DrawAttentionToTab(const TSharedPtr<SDockTab>& InvokedTab)
{
	if (InvokedTab)
	{
		TSharedPtr<SDockTab> MajorTab;
		if (TSharedPtr<FTabManager> ExistingTabManager = InvokedTab->GetTabManagerPtr())
		{
			MajorTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(ExistingTabManager.ToSharedRef());
		}

		// Rules for drawing attention to a tab:
		// 1. Tab is not active
		// 2. Tab's owning major tab is not in the foreground (making the tab we want to draw attention to is not visible)
		// 3. Tab is nomad and is not in the foreground
		// 4. Tab is not in a closed DrawerPanel
		// If the tab is not active or the tabs major tab is not in the foreground, activate it
		if (!InvokedTab->IsActive() || (MajorTab && !MajorTab->IsForeground()) || !InvokedTab->IsForeground())
		{
			bool bShouldDrawAttention = true;

			if (TSharedPtr<SDockingArea> DockArea = InvokedTab->GetDockArea())
			{
				if (DockArea->GetPanelDrawerSystemHostedTab(InvokedTab->GetLayoutIdentifier()) && DockArea->GetPanelDrawerHostedTab() != InvokedTab)
				{
					bShouldDrawAttention = false;
				}
			}

			if (bShouldDrawAttention)
			{
				// Draw attention to this tab if it didn't already have focus
				DrawAttention(InvokedTab.ToSharedRef());
			}
		}
	}
}

TSharedPtr<SDockTab> FTabManager::InvokeTab_Internal(const FTabId& TabId, bool bInvokeAsInactive, bool bForceOpenWindowIfNeeded)
{
	// Tab Spawning Rules:
	// 
	//     * Find live instance --yes--> use it.
	//         |no
	//         v
	//     * [non-Document only]
	//       Find closed instance with matching TabId --yes--> restore it.	  
	//         |no
	//         v
	//     * Find any tab of matching TabType (closed or open) --yes--> spawn next to it.
	//         | no
	//         v
	//     * Is a nomad tab and we are NOT the global tab manager --yes--> try to invoke in the global tab manager
	//         | no
	//         v
	//     * Spawn in a new window.

	if (!InvokeTab_CanInvokeTab(TabId))
	{
		return {};
	}

	TSharedPtr<SDockTab> Tab = InvokeTab_FindOrReuseExistingTab(TabId);
	if (Tab.IsValid())
	{
		bool bCanReuseTab = false;
		if (!Tab->ParentPtr.IsValid())
		{
			if (TSharedPtr<SDockingArea> TabDockingArea = Tab->ParentDockingAreaPtr.Pin())
			{
				if (TabDockingArea->GetPanelDrawerHostedTab() == Tab || TabDockingArea->RemoveHiddenInactivePanelDrawerTab(Tab))
				{
					bCanReuseTab = true;
				}
			}
			else if (TSharedPtr<FTabManager> TabManager = Tab->GetTabManagerPtr())
			{
				bCanReuseTab = TabManager->RemoveFromHiddenPanelDrawerTabs(Tab);
			}
		}

		if (!bCanReuseTab)
		{
			InvokeTab_DrawAttentionToTab(Tab);
			return Tab;
		}
	}

	// Tab is not live. Figure out where to spawn it.
	TSharedPtr<SDockingTabStack> StackToSpawnIn = bForceOpenWindowIfNeeded ? AttemptToOpenTab( TabId, true ) : FindPotentiallyClosedTab( TabId );
	if (StackToSpawnIn.IsValid())
	{
		if (!Tab)
		{
			Tab = SpawnTab(TabId, GetPrivateApi().GetParentWindow());
		}
		else
		{
			// Remove the tab from the panel drawer (post 5.7 make this code less spread more owned by the tab)
			Tab->RemoveTabFromParent_Internal();
		}

		if (Tab.IsValid())
		{
			StackToSpawnIn->OpenTab(Tab.ToSharedRef(), INDEX_NONE, bInvokeAsInactive);
			Tab->PlaySpawnAnim();
			FGlobalTabmanager::Get()->UpdateMainMenu(Tab.ToSharedRef(), false);
		}

		return Tab;
	}
	else if ( FGlobalTabmanager::Get() != SharedThis(this) && NomadTabSpawner->Contains(TabId.TabType) )
	{
		// This tab could have been spawned in the global tab manager since it has a nomad tab spawner
		return FGlobalTabmanager::Get()->InvokeTab_Internal(TabId, bInvokeAsInactive, bForceOpenWindowIfNeeded);
	}
	else if (Tab)
	{
		// Remove the tab from the panel drawer (post 5.7 make this code less spread more owned by the tab)
		Tab->RemoveTabFromParent_Internal();

		// Todo post 5.7 rework this code to remove logic duplication in the docking system (Open a window for the tab)
		TSharedPtr<SWindow> NewWindowParent = GetPrivateApi().GetParentWindow();

		TSharedRef<SWindow> NewWindow = SNew(SWindow)
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
			// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
			// and for multi-view rendering to function properly
			.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
			.Title(FGlobalTabmanager::Get()->GetApplicationTitle())
			.AutoCenter(EAutoCenter::None)
			// Divide out scale, it is already factored into position
			.ScreenPosition(Tab->GetContent()->GetTickSpaceGeometry().LocalToAbsolute(FVector2D(0, 0)))
			// Make room for the title bar; otherwise windows will get progressive smaller whenver you float them.
			.ClientSize(SWindow::ComputeWindowSizeForContent(Tab->GetContent()->GetTickSpaceGeometry().GetLocalSize()))
			.CreateTitleBar(false);

		TSharedPtr<SDockingTabStack> NewDockNode;
		TSharedPtr<FTabManager> TabManagerToUse;
		if (Tab->GetTabRole() == ETabRole::NomadTab)
		{
			TabManagerToUse = FGlobalTabmanager::Get();
			Tab->SetTabManager(FGlobalTabmanager::Get());
		}
		else
		{
			TabManagerToUse = AsShared();
		}

		// Create a new dockarea
		TSharedRef<SDockingArea> NewDockArea =
			SNew(SDockingArea, TabManagerToUse.ToSharedRef(), FTabManager::NewPrimaryArea())
			.ParentWindow(NewWindow)
			.InitialContent
			(
				SAssignNew(NewDockNode, SDockingTabStack, FTabManager::NewStack())
			);

		if (Tab->GetTabRole() == ETabRole::MajorTab || Tab->GetTabRole() == ETabRole::NomadTab)
		{
			TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
			if (RootWindow.IsValid())
			{
				// We have a root window, so all MajorTabs are nested under it.
				FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef())->SetContent(NewDockArea);
			}
			else
			{
				// App tabs get put in top-level windows. They show up on the taskbar.
				FSlateApplication::Get().AddWindow(NewWindow)->SetContent(NewDockArea);
			}
		}
		else
		{
			// Other tab types are placed in child windows. Their life is controlled by the top-level windows.
			// They do not show up on the taskbar.
			if (NewWindowParent.IsValid())
			{
				FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, NewWindowParent.ToSharedRef())->SetContent(NewDockArea);
			}
			else
			{
				FSlateApplication::Get().AddWindow(NewWindow)->SetContent(NewDockArea);
			}
		}

		// Do this after the window parenting so that the window title is set correctly
		NewDockNode->OpenTab(Tab.ToSharedRef());

		// Let every widget under this tab manager know that this tab has found a new home.
		OnTabRelocated(Tab.ToSharedRef(), NewWindow);

		return Tab;
	}
	else
	{
		const TSharedRef<FArea> NewAreaForTab = GetAreaForTabId(TabId);
		NewAreaForTab
			->Split
			(
				FTabManager::NewStack()
				->AddTab(TabId, ETabState::OpenedTab)
			);

		TSharedPtr<SDockingArea> DockingArea = RestoreArea(NewAreaForTab, GetPrivateApi().GetParentWindow());
		if (DockingArea && DockingArea->GetAllChildTabs().Num() > 0)
		{
			const TSharedPtr<SDockTab> NewlyOpenedTab = DockingArea->GetAllChildTabs()[0];
			check(NewlyOpenedTab.IsValid());
			return NewlyOpenedTab.ToSharedRef();
		}
	}

	return nullptr;
}

TSharedPtr<SDockingTabStack> FTabManager::FindPotentiallyClosedTab( const FTabId& ClosedTabId )
{
	return AttemptToOpenTab( ClosedTabId );
}

TSharedPtr<SDockingTabStack> FTabManager::AttemptToOpenTab( const FTabId& ClosedTabId, bool bForceOpenWindowIfNeeded )
{
	TSharedPtr<SDockingTabStack> StackWithClosedTab;

	FTabMatcher TabMatcher( ClosedTabId );

	// Search among the COLLAPSED AREAS
	const int32 CollapsedAreaWithMatchingTabIndex = FindTabInCollapsedAreas( TabMatcher );
	if ( CollapsedAreaWithMatchingTabIndex != INDEX_NONE )
	{
		TSharedRef<FTabManager::FArea> CollapsedAreaWithMatchingTab = CollapsedDockAreas[CollapsedAreaWithMatchingTabIndex];

		// If this is not the global tab manager and the tab is a NomadTab in a floating window, then remove it from the collapsed area.
		if (FGlobalTabmanager::Get() != SharedThis(this) &&
			NomadTabSpawner->Contains(ClosedTabId.TabType) &&
			CollapsedAreaWithMatchingTab->WindowPlacement != FTabManager::FArea::EWindowPlacementInternal::Placement_NoWindow)
		{
			RemoveTabFromCollapsedAreas(TabMatcher);
		}
		else
		{
			TSharedPtr<SDockingArea> RestoredArea = RestoreArea(CollapsedDockAreas[CollapsedAreaWithMatchingTabIndex],
				GetPrivateApi().GetParentWindow(), false, EOutputCanBeNullptr::Never, bForceOpenWindowIfNeeded);
			check(RestoredArea.IsValid());
			// We have just un-collapsed this dock area.
			// Don't rely on the collapsed tab index: RestoreArea() can end up kicking the task graph which could do other tab work and modify the CollapsedDockAreas array.
			CollapsedDockAreas.Remove(CollapsedAreaWithMatchingTab);
			if (RestoredArea.IsValid())
			{
				StackWithClosedTab = FindTabInLiveArea(TabMatcher, StaticCastSharedRef<SDockingArea>(RestoredArea->AsShared()));
			}
		}
	}

	if ( !StackWithClosedTab.IsValid() )
	{
		// Search among the LIVE AREAS
		StackWithClosedTab = FindTabInLiveAreas( TabMatcher );
	}

	return StackWithClosedTab;
}

FUIAction FTabManager::GetUIActionForTabSpawnerMenuEntry(TSharedPtr<FTabSpawnerEntry> InTabMenuEntry)
{
	auto CanExecuteMenuEntry = [](TWeakPtr<FTabSpawnerEntry> SpawnerNode) -> bool
	{
		TSharedPtr<FTabSpawnerEntry> SpawnerNodePinned = SpawnerNode.Pin();
		if (SpawnerNodePinned.IsValid() && SpawnerNodePinned->MenuType.Get() == ETabSpawnerMenuType::Enabled)
		{
			return SpawnerNodePinned->CanSpawnTab.IsBound() ? SpawnerNodePinned->CanSpawnTab.Execute(FSpawnTabArgs(TSharedPtr<SWindow>(), SpawnerNodePinned->TabType)) : true;
		}

		return false;
	};

	return FUIAction(
		FExecuteAction::CreateSP(SharedThis(this), &FTabManager::InvokeTabForMenu, InTabMenuEntry->TabType),
		FCanExecuteAction::CreateStatic(CanExecuteMenuEntry, TWeakPtr<FTabSpawnerEntry>(InTabMenuEntry)),
		FIsActionChecked::CreateSP(InTabMenuEntry.ToSharedRef(), &FTabSpawnerEntry::IsSoleTabInstanceSpawned)
		);
}

void FTabManager::InvokeTabForMenu( FName TabId )
{
	TryInvokeTab(TabId);
}

void FTabManager::InsertDocumentTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab, bool bPlaySpawnAnim)
{
	InsertDocumentTab(PlaceholderId, PlaceholderId, SearchPreference, UnmanagedTab, bPlaySpawnAnim);
}

void FTabManager::InsertDocumentTab(FName PlaceholderId, FName NewTabId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab, bool bPlaySpawnAnim)
{
	bool bWasUnmanagedTabOpened = true;
	const bool bTabNotManaged = ensure( ! FindTabInLiveAreas( FTabMatcher(UnmanagedTab->GetLayoutIdentifier()) ).IsValid() );
	UnmanagedTab->SetLayoutIdentifier( FTabId(NewTabId, LastDocumentUID++) );
	
	if (bTabNotManaged)
	{
		OpenUnmanagedTab(PlaceholderId, SearchPreference, UnmanagedTab);
	}

	DrawAttention(UnmanagedTab);
	if (bPlaySpawnAnim)
	{
		UnmanagedTab->PlaySpawnAnim();
	}
}

void FTabManager::OpenUnmanagedTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab)
{
	TSharedPtr<SDockTab> LiveTab = SearchPreference.Search(*this, PlaceholderId, UnmanagedTab);
		
	if (LiveTab.IsValid())
	{
		LiveTab->GetParent()->GetParentDockTabStack()->OpenTab( UnmanagedTab );
	}
	else
	{
		TSharedPtr<SDockingTabStack> StackToSpawnIn = AttemptToOpenTab( PlaceholderId, true );
		if (StackToSpawnIn.IsValid())
		{
			StackToSpawnIn->OpenTab(UnmanagedTab);
		}
		else
		{
			UE_LOGF(LogTabManager, Warning, "Unable to insert tab '%ls'.", *(PlaceholderId.ToString()));
			LiveTab = InvokeTab_Internal( FTabId( PlaceholderId ) );
			if (LiveTab.IsValid())
			{
				LiveTab->GetParent()->GetParentDockTabStack()->OpenTab( UnmanagedTab );
			}
		}
	}
}

FTabManager::FTabManager( const TSharedPtr<SDockTab>& InOwnerTab, const TSharedRef<FTabSpawner> & InNomadTabSpawner )
: NomadTabSpawner(InNomadTabSpawner)
, OwnerTabPtr( InOwnerTab )
, PrivateApi( MakeShareable(new FPrivateApi(*this)) )
, LastDocumentUID( 0 )
, TabPermissionList( MakeShareable(new FNamePermissionList()) )
{
	LocalWorkspaceMenuRoot = FWorkspaceItem::NewGroup(LOCTEXT("LocalWorkspaceRoot", "Local Workspace Root"));
}

TSharedPtr<SDockingArea> FTabManager::RestoreArea(const TSharedRef<FArea>& AreaToRestore, const TSharedPtr<SWindow>& InParentWindow, const bool bEmbedTitleAreaContent, const EOutputCanBeNullptr OutputCanBeNullptr, bool bForceOpenWindowIfNeeded)
{
	// Sidebar tabs for this area
	FSidebarTabLists SidebarTabs;

	TemporarilySidebaredTabs.Empty();

	if (TSharedPtr<SDockingNode> RestoredNode = RestoreArea_Helper(AreaToRestore, InParentWindow, bEmbedTitleAreaContent, SidebarTabs, OutputCanBeNullptr, bForceOpenWindowIfNeeded))
	{
		TSharedRef<SDockingArea> RestoredArea = StaticCastSharedRef<SDockingArea>(RestoredNode->AsShared());

		RestoredArea->CleanUp(SDockingNode::TabRemoval_None);

		RestoredArea->AddSidebarTabsFromRestoredLayout(SidebarTabs);

		for (const TSharedRef<SDockTab>& Tab : SidebarTabs.LeftSidebarTabs)
		{
			TemporarilySidebaredTabs.Add(Tab);
		}

		for (const TSharedRef<SDockTab>& Tab : SidebarTabs.RightSidebarTabs)
		{
			TemporarilySidebaredTabs.Add(Tab);
		}

		return RestoredArea;
	}
	else
	{
		check(OutputCanBeNullptr != EOutputCanBeNullptr::Never);
		return nullptr;
	}
}

TSharedPtr<SDockingNode> FTabManager::RestoreArea_Helper(const TSharedRef<FLayoutNode>& LayoutNode, const TSharedPtr<SWindow>& ParentWindow, const bool bEmbedTitleAreaContent,
	FSidebarTabLists& OutSidebarTabs, const EOutputCanBeNullptr OutputCanBeNullptr, bool bForceOpenWindowIfNeeded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTabManager::RestoreArea_Helper);

#if WITH_EDITOR
	FSlateApplication::FScopedPreventDebuggingMode Scope(LOCTEXT("RestoringTabsDebugScope", "Disabling debug due to being in tab restore, breakpoints in constructors can infinitely stall during restore."));
#endif

	TSharedPtr<FTabManager::FStack> NodeAsStack = LayoutNode->AsStack();
	TSharedPtr<FTabManager::FSplitter> NodeAsSplitter = LayoutNode->AsSplitter();
	TSharedPtr<FTabManager::FArea> NodeAsArea = LayoutNode->AsArea();
	const bool bCanOutputBeNullptr = (OutputCanBeNullptr != EOutputCanBeNullptr::Never);

	if (NodeAsStack.IsValid())
	{
		TSharedPtr<SDockTab> WidgetToActivate;

		TSharedPtr<SDockingTabStack> NewStackWidget;
		// Should we init NewStackWidget before the for loop? It depends on OutputCanBeNullptr
		bool bIsNewStackWidgetInit = false;
		// 1. If EOutputCanBeNullptr::Never, function cannot return nullptr
		if (OutputCanBeNullptr == EOutputCanBeNullptr::Never)
		{
			bIsNewStackWidgetInit = true;
		}
		// 2. If EOutputCanBeNullptr::IfNoTabValid, we must init the SWidget as soon as any tab is valid for spawning
		else if (OutputCanBeNullptr == EOutputCanBeNullptr::IfNoTabValid)
		{
			// Note: IsValidTabForSpawning does not check whether SpawnTab() will return nullptr
			for (const FTab& SomeTab : NodeAsStack->Tabs)
			{
				if (IsValidTabForSpawning(SomeTab))
				{
					bIsNewStackWidgetInit = true;
					break;
				}
			}
		}
		// 3. If EOutputCanBeNullptr::IfNoOpenTabValid, we must init the SWidget as soon as any open tab is valid for spawning. For efficiency, done in the for loop
		// 4. Else, case not handled --> error
		else if (OutputCanBeNullptr != EOutputCanBeNullptr::IfNoOpenTabValid)
		{
			check(false);
		}
		// Initialize the SWidget already?
		if (bIsNewStackWidgetInit)
		{
			NewStackWidget = SNew(SDockingTabStack, NodeAsStack.ToSharedRef());
			NewStackWidget->SetSizeCoefficient(LayoutNode->GetSizeCoefficient());
		}
		// Open Tabs
		for (FTab& SomeTab : NodeAsStack->Tabs)
		{
			if ((SomeTab.TabState == ETabState::OpenedTab || SomeTab.TabState == ETabState::SidebarTab) && IsValidTabForSpawning(SomeTab))
			{
				const bool bCanUnrecognizedTabBeNullptr = true;
				const TSharedPtr<SDockTab> NewTabWidget = SpawnTab(SomeTab.TabId, ParentWindow, bCanUnrecognizedTabBeNullptr);
				TSharedPtr<FTabSpawnerEntry> Spawner = FindTabSpawnerFor(SomeTab.TabId.TabType);

				if (NewTabWidget.IsValid())
				{
					if (SomeTab.TabId == NodeAsStack->ForegroundTabId)
					{
						ensure(SomeTab.TabState == ETabState::OpenedTab);
						WidgetToActivate = NewTabWidget;
					}

					// First time initialization: Only if at least a valid NewTabWidget
					if (!NewStackWidget)
					{
						NewStackWidget = SNew(SDockingTabStack, NodeAsStack.ToSharedRef());
						NewStackWidget->SetSizeCoefficient(LayoutNode->GetSizeCoefficient());
					}

					// If the config didn't have a locked state saved check if the spawner one is set and use that one instead
					// The config have priority over the Spawner one since it should be used as a default value
					if (!SomeTab.bIsLockedInPlace.IsSet() && Spawner->bIsLocked.IsSet())
					{
						NewStackWidget->SetTabLocked(NewTabWidget.ToSharedRef(), Spawner->bIsLocked.GetValue());
					}

					if (SomeTab.TabState == ETabState::OpenedTab)
					{
						NewStackWidget->AddTabWidget(NewTabWidget.ToSharedRef());
					}
					else
					{
						// Let the stack know we have a tab that belongs in its stack that is currently in a sidebar
						NewStackWidget->AddSidebarTab(NewTabWidget.ToSharedRef());
						if (SomeTab.SidebarLocation == ESidebarLocation::Left)
						{
							OutSidebarTabs.LeftSidebarTabs.Add(NewTabWidget.ToSharedRef());
						}
						else
						{
							ensure(SomeTab.SidebarLocation == ESidebarLocation::Right);
							OutSidebarTabs.RightSidebarTabs.Add(NewTabWidget.ToSharedRef());
						}
					}
				}
			}
		}

		if(WidgetToActivate.IsValid())
		{
			WidgetToActivate->ActivateInParent(ETabActivationCause::SetDirectly);

			if ((WidgetToActivate->GetTabRole() == ETabRole::MajorTab || WidgetToActivate->GetTabRole() == ETabRole::NomadTab)
				&& ParentWindow.IsValid() && ParentWindow != FGlobalTabmanager::Get()->GetRootWindow())
			{
				ParentWindow->SetTitle(WidgetToActivate->GetTabLabel());
			}
		}

		return NewStackWidget;

	}
	else if ( NodeAsArea.IsValid() )
	{
		const bool bDockNeedsNewWindow = NodeAsArea.IsValid()
			&& !(NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_NoWindow 
				|| NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_ParentSpecified);

		TSharedPtr<SDockingArea> NewDockAreaWidget;

		if ( bDockNeedsNewWindow )
		{
			// The layout node we are restoring is a dock area.
			// It needs a new window into which it will land.
			const bool bIsChildWindow = ParentWindow.IsValid();

			const bool bAutoPlacement = (NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_Automatic);
			TSharedRef<SWindow> NewWindow = (bAutoPlacement)
				? SNew(SWindow)
					.AutoCenter( EAutoCenter::PreferredWorkArea )
					.ClientSize( NodeAsArea->UnscaledWindowSize )
					.CreateTitleBar( false )
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
					// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
					// and for multi-view rendering to function properly
					.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
					.IsInitiallyMaximized( NodeAsArea->bIsMaximized )
				: SNew(SWindow)
					.AutoCenter( EAutoCenter::None )
					.ScreenPosition( NodeAsArea->UnscaledWindowPosition )
					.ClientSize( NodeAsArea->UnscaledWindowSize )
					.CreateTitleBar( false )
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
					// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
					// and for multi-view rendering to function properly
					.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
					.IsInitiallyMaximized( NodeAsArea->bIsMaximized );

			// Set a default title; restoring the splitter content may override this if it activates a tab
			NewWindow->SetTitle(FGlobalTabmanager::Get()->GetApplicationTitle());

			// We need to add the new window now before we recursively restore any content.
			// The reason for this is that NewWindow may become the ParentWindow for another window as we restore content.
			// We destroy the new window as we unwind if it ends up being extraneous
			if (bIsChildWindow)
			{
				FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, ParentWindow.ToSharedRef());
			}
			else
			{
				FSlateApplication::Get().AddWindow(NewWindow);
			}

			TArray<TSharedRef<SDockingNode>> DockingNodes;
			if (CanRestoreSplitterContent(DockingNodes, NodeAsArea.ToSharedRef(), NewWindow, OutSidebarTabs, OutputCanBeNullptr))
			{
				NewWindow->SetContent(SAssignNew(NewDockAreaWidget, SDockingArea, SharedThis(this), NodeAsArea.ToSharedRef()).ParentWindow(NewWindow));

				// Restore content
				if (!bCanOutputBeNullptr)
				{
					RestoreSplitterContent(NodeAsArea.ToSharedRef(), NewDockAreaWidget.ToSharedRef(), NewWindow, OutSidebarTabs);
				}
				else
				{
					RestoreSplitterContent(DockingNodes, NewDockAreaWidget.ToSharedRef());
				}

				if (bIsChildWindow)
				{
					// Recursively check to see how many actually spawned tabs there are in this dock area.					
					const int32 TotalNumTabs = NewDockAreaWidget->GetNumTabs();

					// If there are none and we aren't requested to force open the window, then destroy the window.
					if (TotalNumTabs == 0 && !bForceOpenWindowIfNeeded)
					{
						NewWindow->RequestDestroyWindow();
					}
				}
			}
			else
			{
				NewWindow->RequestDestroyWindow();
			}
		}
		else
		{
			if (NodeAsArea->WindowPlacement == FArea::EWindowPlacementInternal::Placement_ParentSpecified)
			{
				if (ParentWindow)
				{
					/**
					 * The SWindow api doesn't help here as some function use the DPI resolution and some don't, but this out of scope for this change.
					 * This code should be refactored to be shared with the window logic also it doesn't support multi DPI scale factor across multiple monitor
					 */
					const float DPIScaleFactor = ParentWindow->GetDPIScaleFactor();
					UE::Slate::FDeprecateVector2DResult WindowSize = ParentWindow->GetWindowSizeFromClientSize(NodeAsArea->UnscaledWindowSize * DPIScaleFactor, DPIScaleFactor);
					FVector2D WindowPosition = NodeAsArea->UnscaledWindowPosition * DPIScaleFactor;

					// Get desktop metrics
					FDisplayMetrics DisplayMetrics;
					FSlateApplicationBase::Get().GetCachedDisplayMetrics(DisplayMetrics);
					const FPlatformRect& VirtualDisplayRect = DisplayMetrics.VirtualDisplayRect;
					
					if (WindowPosition.X < VirtualDisplayRect.Left ||
						WindowPosition.X >= VirtualDisplayRect.Right ||
						WindowPosition.Y < VirtualDisplayRect.Top ||
						WindowPosition.Y >= VirtualDisplayRect.Bottom)
					{
						// The window is out of bound of the virtual display rect just use the preferred work area
						FSlateRect PreferredWorkArea = FSlateApplicationBase::Get().GetPreferredWorkArea();
						WindowSize = PreferredWorkArea.GetSize();
						WindowPosition.X = PreferredWorkArea.Left;
						WindowPosition.Y = PreferredWorkArea.Top;
					}

					ParentWindow->ReshapeWindow(WindowPosition, WindowSize);
					if (NodeAsArea->bIsMaximized)
					{
						ParentWindow->Maximize();
					}
				}
			}

			TArray<TSharedRef<SDockingNode>> DockingNodes;
			if (CanRestoreSplitterContent(DockingNodes, NodeAsArea.ToSharedRef(), ParentWindow, OutSidebarTabs, OutputCanBeNullptr))
			{
				SAssignNew(NewDockAreaWidget, SDockingArea, SharedThis(this), NodeAsArea.ToSharedRef())
					// We only want to set a parent window on this dock area, if we need to have title area content
					// embedded within it.  SDockingArea assumes that if it has a parent window set, then it needs to have
					// title area content 
					.ParentWindow(bEmbedTitleAreaContent ? ParentWindow : TSharedPtr<SWindow>())
					// Never manage these windows, even if a parent window is set.  The owner will take care of
					// destroying these windows.
					.ShouldManageParentWindow(false);

				// Restore content
				if (!bCanOutputBeNullptr)
				{
					RestoreSplitterContent(NodeAsArea.ToSharedRef(), NewDockAreaWidget.ToSharedRef(), ParentWindow, OutSidebarTabs);
				}
				else
				{
					RestoreSplitterContent(DockingNodes, NewDockAreaWidget.ToSharedRef());
				}
			}
		}
		
		if (NewDockAreaWidget && !NodeAsArea->ActivePanelDrawerTab.TabId.TabType.IsNone())
		{
			TSharedPtr<SDockTab> InvokedTab;
			if (CanInvokeInPanelDrawer(NodeAsArea->ActivePanelDrawerTab.TabId, NewDockAreaWidget, InvokedTab, false))
			{
				TSharedPtr<SDockingArea> OtherDockingArea = InvokedTab->GetDockArea();

				if (!OtherDockingArea || OtherDockingArea->GetPanelDrawerHostedTab() != InvokedTab)
				{
					// Restore Panel Drawer active tab
					TSharedRef<UE::Slate::Private::FPanelDrawerData> PanelDrawerData = MakeShared<UE::Slate::Private::FPanelDrawerData>();
					PanelDrawerData->Size = NodeAsArea->ActivePanelDrawerTab.Size;
					PanelDrawerData->HostedTab = MoveTemp(InvokedTab);
					NewDockAreaWidget->SetPanelDrawerHiddenActiveTab(MoveTemp(PanelDrawerData));
				}
			}
		}

		return NewDockAreaWidget;
	}
	else if ( NodeAsSplitter.IsValid() ) 
	{
		TArray<TSharedRef<SDockingNode>> DockingNodes;
		if (CanRestoreSplitterContent(DockingNodes, NodeAsSplitter.ToSharedRef(), ParentWindow, OutSidebarTabs, OutputCanBeNullptr))
		{
			TSharedRef<SDockingSplitter> NewSplitterWidget = SNew( SDockingSplitter, NodeAsSplitter.ToSharedRef() );
			NewSplitterWidget->SetSizeCoefficient(LayoutNode->GetSizeCoefficient());
			// Restore content
			if (!bCanOutputBeNullptr)
			{
				RestoreSplitterContent(NodeAsSplitter.ToSharedRef(), NewSplitterWidget, ParentWindow, OutSidebarTabs);
			}
			else
			{
				RestoreSplitterContent(DockingNodes, NewSplitterWidget);
			}
			return NewSplitterWidget;
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		ensureMsgf( false, TEXT("Unexpected node type") );
		TSharedRef<SDockingTabStack> NewStackWidget = SNew(SDockingTabStack, FTabManager::NewStack());
		NewStackWidget->OpenTab(SpawnTab(FName(NAME_None), ParentWindow, bCanOutputBeNullptr).ToSharedRef());
		return NewStackWidget;
	}
}

bool FTabManager::CanRestoreSplitterContent(TArray<TSharedRef<SDockingNode>>& DockingNodes, const TSharedRef<FSplitter>& SplitterNode, const TSharedPtr<SWindow>& ParentWindow, FSidebarTabLists& OutSidebarTabs, const EOutputCanBeNullptr OutputCanBeNullptr)
{
	if (OutputCanBeNullptr == EOutputCanBeNullptr::Never)
	{
		return true;
	}
	DockingNodes.Empty();
	// Restore the contents of this splitter.
	for ( const TSharedRef<FLayoutNode>& ThisChildNode : SplitterNode->GetChildNodes() )
	{
		const bool bEmbedTitleAreaContent = false;
		const TSharedPtr<SDockingNode> ThisChildNodeWidget = RestoreArea_Helper(ThisChildNode, ParentWindow, bEmbedTitleAreaContent, OutSidebarTabs, OutputCanBeNullptr);
		if (ThisChildNodeWidget)
		{
			const TSharedRef<SDockingNode> ThisChildNodeWidgetRef = StaticCastSharedRef<SDockingNode>(ThisChildNodeWidget->AsShared());
			DockingNodes.Add(ThisChildNodeWidgetRef);
		}
	}
	return (DockingNodes.Num() > 0);
}

void FTabManager::RestoreSplitterContent( const TArray<TSharedRef<SDockingNode>>& DockingNodes, const TSharedRef<SDockingSplitter>& SplitterWidget)
{
	for (const TSharedRef<SDockingNode>& DockingNode : DockingNodes)
	{
		SplitterWidget->AddChildNode(DockingNode, INDEX_NONE);
	}
}

void FTabManager::RestoreSplitterContent(const TSharedRef<FSplitter>& SplitterNode, const TSharedRef<SDockingSplitter>& SplitterWidget, const TSharedPtr<SWindow>& ParentWindow, FSidebarTabLists& OutSidebarTabs)
{
	// Restore the contents of this splitter.
	for ( const TSharedRef<FLayoutNode>& ThisChildNode : SplitterNode->GetChildNodes() )
	{
		const bool bEmbedTitleAreaContent = false;
		TSharedPtr<SDockingNode> ThisChildNodeWidget = RestoreArea_Helper(ThisChildNode, ParentWindow, bEmbedTitleAreaContent, OutSidebarTabs);
		check(ThisChildNodeWidget.IsValid());
		if (ThisChildNodeWidget)
		{
			const TSharedRef<SDockingNode> ThisChildNodeWidgetRef = StaticCastSharedRef<SDockingNode>(ThisChildNodeWidget->AsShared());
			SplitterWidget->AddChildNode( ThisChildNodeWidgetRef, INDEX_NONE );
		}
	}
}

bool FTabManager::HasTabSpawner(FName TabId) const
{
	// Look for a spawner in this tab manager.
	const TSharedRef<FTabSpawnerEntry>* Spawner = TabSpawner.Find(TabId);
	if (Spawner == nullptr)
	{
		Spawner = NomadTabSpawner->Find(TabId);
	}

	return Spawner != nullptr;
}

TSharedRef<FNamePermissionList>& FTabManager::GetTabPermissionList()
{
	return TabPermissionList;
}

bool FTabManager::IsValidTabForSpawning( const FTab& SomeTab ) const
{
	if (!IsAllowedTab(SomeTab.TabId))
	{
		return false;
	}

	// Nomad tabs being restored from layouts should not be spawned if the nomad tab is already spawned.
	TSharedRef<FTabSpawnerEntry>* NomadSpawner = NomadTabSpawner->Find( SomeTab.TabId.TabType );
	return ( !NomadSpawner || !NomadSpawner->Get().IsSoleTabInstanceSpawned() || NomadSpawner->Get().OnFindTabToReuse.IsBound() );
}

bool FTabManager::IsAllowedTab(const FTabId& TabId) const
{
	bool bAllowed = true;

	// If we are in read-only mode, make sure this tab doesn't want to be hidden
	if(bReadOnly)
	{
		TOptional<ETabReadOnlyBehavior> TabReadOnlyBehavior = GetTabReadOnlyBehavior(TabId);

		if(TabReadOnlyBehavior.IsSet())
		{
			bAllowed &= (TabReadOnlyBehavior.GetValue() != ETabReadOnlyBehavior::Hidden);
		}
	}
	
	bAllowed &= IsAllowedTabType(TabId.TabType);
	
	return bAllowed;
}

TOptional<ETabReadOnlyBehavior> FTabManager::GetTabReadOnlyBehavior(const FTabId& TabId) const
{
	if (const TSharedPtr<const FTabSpawnerEntry> Spawner = FindTabSpawnerFor(TabId.TabType))
	{
		return Spawner->ReadOnlyBehavior;
	}
	return TOptional<ETabReadOnlyBehavior>();
}

bool FTabManager::IsAllowedTabType(const FName TabType) const
{
	const bool bIsAllowed = TabType == NAME_None || TabPermissionList->PassesFilter(TabType);
	if (!bIsAllowed)
	{
		UE_LOGF(LogSlate, Verbose, "Disallowed Tab: %ls", *TabType.ToString());
	}
	return bIsAllowed;
}

bool FTabManager::IsTabAllowedInSidebar(const FTabId TabId) const
{
	if (const TSharedPtr<const FTabSpawnerEntry> Spawner = FindTabSpawnerFor(TabId.TabType))
	{
		return Spawner->CanSidebarTab();
	}

	return false;
}

void FTabManager::ToggleSidebarOpenTabs(const TSet<FTabId>& InExceptions)
{
	bool bHasSidebarTabs = !TemporarilySidebaredTabs.IsEmpty();

	// Check if TemporarilySidebaredTabs only contains InExceptions
	if (bHasSidebarTabs)
	{
		bool bHasOnlyExceptions = true;

		for (const TWeakPtr<SDockTab>& WeakTab : TemporarilySidebaredTabs)
		{
			if (TSharedPtr<SDockTab> Tab = WeakTab.Pin())
			{
				const FTabId TabId = Tab->GetLayoutIdentifier();
				if (!InExceptions.Contains(TabId))
				{
					bHasOnlyExceptions = false;
					break;
				}
			}
		}

		if (bHasOnlyExceptions)
		{
			bHasSidebarTabs = false;
		}
	}

	if (!bHasSidebarTabs)
	{
		// Sidebar opened tabs not in a sidebar already
		for (int32 AreaIndex = 0; AreaIndex < DockAreas.Num(); ++AreaIndex)
		{
			TSharedPtr<SDockingArea> SomeDockArea = DockAreas[AreaIndex].Pin();

			if (SomeDockArea.IsValid() && SomeDockArea->CanHaveSidebar())
			{
				TArray<TSharedRef<SDockTab>> AllTabs = SomeDockArea->GetAllChildTabs();
				for (TSharedRef<SDockTab>& Tab : AllTabs)
				{
					const FTabId TabId = Tab->GetLayoutIdentifier();
					if (!InExceptions.Contains(TabId)
						&& IsTabAllowedInSidebar(Tab->GetLayoutIdentifier())
						&& !SomeDockArea->IsTabInSidebar(Tab)
						&& Tab->GetParentDockTabStack()->CanMoveTabToSideBar(Tab))
					{
						Tab->GetParentDockTabStack()->MoveTabToSidebar(Tab);
						TemporarilySidebaredTabs.Add(Tab);
					}
				}

			}
		}
	}
	else
	{
		for (auto It = TemporarilySidebaredTabs.CreateIterator(); It; ++It)
		{
			if (TSharedPtr<SDockTab> Tab = It->Pin())
			{
				const FTabId TabId = Tab->GetLayoutIdentifier();
				if (!InExceptions.Contains(TabId))
				{
					Tab->GetParentDockTabStack()->GetDockArea()->RestoreTabFromSidebar(Tab.ToSharedRef());
					It.RemoveCurrent();
				}
			}
			else
			{
				It.RemoveCurrent();
			}
		}
	}

	OnToggleTemporarilySidebaredTabs.Broadcast(!bHasSidebarTabs, InExceptions);
}

TSharedPtr<SDockTab> FTabManager::SpawnTab(const FTabId& TabId, const TSharedPtr<SWindow>& ParentWindow, const bool bCanOutputBeNullptr)
{
	TSharedPtr<SDockTab> NewTabWidget;

	// Whether or not the spawner overrode the ability for the tab to even spawn.  This is not a failure case.
	bool bSpawningAllowedBySpawner = true;
	// Do we know how to spawn such a tab?
	TSharedPtr<FTabSpawnerEntry> Spawner = FindTabSpawnerFor(TabId.TabType);
	if ( Spawner.IsValid() )
	{
		if (Spawner->CanSpawnTab.IsBound())
		{
			bSpawningAllowedBySpawner = Spawner->CanSpawnTab.Execute(FSpawnTabArgs(ParentWindow, TabId));
		}

		if (bSpawningAllowedBySpawner && (!Spawner->SpawnedTabPtr.IsValid() || Spawner->OnFindTabToReuse.IsBound()))
		{
			NewTabWidget = Spawner->OnSpawnTab.Execute(FSpawnTabArgs(ParentWindow, TabId));

			if(PendingMainNonClosableTab && NewTabWidget == PendingMainNonClosableTab)
			{
				PendingMainNonClosableTab = nullptr;
				MainNonCloseableTabID = TabId;
			}

			if (FGlobalTabmanager::Get()->GetShouldUseMiddleEllipsisForDockTabLabel())
			{
				NewTabWidget->SetTabLabelOverflowPolicy(ETextOverflowPolicy::MiddleEllipsis);
			}

			NewTabWidget->SetLayoutIdentifier(TabId);

			const FText Label = GetTabLabelBasedOnSpawner(Spawner);

			NewTabWidget->ProvideDefaultLabel(Label);
			NewTabWidget->ProvideDefaultIcon(Spawner->GetIcon().GetIcon());
			NewTabWidget->SetIsTabNameHidden(Spawner->IsTabNameHidden());

			// The spawner tracks that last tab it spawned
			Spawner->SpawnedTabPtr = NewTabWidget;
		} 
		else
		{
			// If we got here, somehow there is two entries spawning the same tab.  This is now allowed so just ignore it.
			bSpawningAllowedBySpawner = false;
		}
	}

	// The tab was allowed to be spawned but failed for some reason
	if (bSpawningAllowedBySpawner && !NewTabWidget.IsValid())
	{
		// We don't know how to spawn this tab. 2 alternatives:
		// 1) Make a dummy tab so that things aren't entirely broken (previous versions of UE did this in all cases).
		// 2) Do not open the widget and return nullptr, but keep the unknown widget saved in the layout. E.g., applied when calling RestoreFrom() from MainFrameModule.

		FString StringToDisplay = GetTabLabelBasedOnSpawner(Spawner).ToString();

		if (StringToDisplay.IsEmpty() && !(Spawner.IsValid() && Spawner->IsTabNameHidden()))
		{
			StringToDisplay = FString("Unknown");
		}
		// If an output must be generated, create an "unrecognized tab" and log it
		if (!bCanOutputBeNullptr)
		{
			UE_LOGF(LogSlate, Log,
				"The tab \"%ls\" attempted to spawn in layout '%ls' but failed for some reason. An \"unrecognized tab\" will be returned instead.", *StringToDisplay, *ActiveLayoutName.ToString()
			);

			NewTabWidget = SNew(SDockTab)
				.Label( TabId.ToText() )
				.ShouldAutosize( false )
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text( NSLOCTEXT("TabManagement", "Unrecognized", "unrecognized tab") )
					]
				];

			const FTabId UnrecognizedId(FName("Unrecognized"), ETabIdFlags::None);
			NewTabWidget->SetLayoutIdentifier(UnrecognizedId);
		}
		// If we can return nullptr, log it
		else
		{
			UE_LOGF(LogSlate, Log,
				"The tab \"%ls\" attempted to spawn in layout '%ls' but failed for some reason. It will not be displayed.", *StringToDisplay, *ActiveLayoutName.ToString()
			);
		}
	}

	if (NewTabWidget.IsValid())
	{
		NewTabWidget->SetTabManager(SharedThis(this));
	}

	return NewTabWidget;
}

TSharedPtr<SDockTab> FTabManager::FindExistingLiveTab( const FTabId& TabId ) const
{
	for ( int32 AreaIndex = 0; AreaIndex < DockAreas.Num(); ++AreaIndex )
	{
		const TSharedPtr<SDockingArea> SomeDockArea = DockAreas[ AreaIndex ].Pin();
		if ( SomeDockArea.IsValid() )
		{
			TArray< TSharedRef<SDockTab> > ChildTabs = SomeDockArea->GetAllChildTabs();
			ChildTabs.Append(SomeDockArea->GetAllSidebarTabs());
			for (int32 ChildTabIndex=0; ChildTabIndex < ChildTabs.Num(); ++ChildTabIndex)
			{
				if ( TabId == ChildTabs[ChildTabIndex]->GetLayoutIdentifier() )
				{
					return ChildTabs[ChildTabIndex];
				}
			}

			if (TSharedPtr<SDockTab> Tab = SomeDockArea->GetPanelDrawerSystemHostedTab(TabId))
			{
				return Tab;
			}
		}
	}

	if (TabId.InstanceId == INDEX_NONE)
	{
		if (const TSharedRef<SDockTab>* TabPtr = PanelDrawerActiveHiddenTabs.Find(TabId))
		{
			return *TabPtr;
		}
	}

	return TSharedPtr<SDockTab>();
}

TSharedPtr<SDockTab> FTabManager::TryOpenTabInPanelDrawer(const FTabId& TabId, const TSharedPtr<SWindow>& ParentWindow, bool bForceInDrawerPanel)
{
	return InvokeTabInPanelDrawer(TabId, false, ParentWindow, bForceInDrawerPanel);
}

TSharedPtr<SDockTab> FTabManager::TryToggleTabInPanelDrawer(const FTabId& TabId, const TSharedPtr<SWindow>& ParentWindow, bool bForceInDrawerPanel)
{
	return InvokeTabInPanelDrawer(TabId, true, ParentWindow, bForceInDrawerPanel);
}

TSharedPtr<SDockTab> FTabManager::InvokeTabInPanelDrawer(const FTabId& TabId, bool bToggleIfActiveInDrawer, const TSharedPtr<SWindow>& ParentWindow, bool bForceInDrawerPanel)
{
	TSharedPtr<SDockingArea> DockingArea = GetDockingAreaForPanelDrawer(ParentWindow);
	if ( DockingArea && DockingArea->HasPanelDrawer())
	{
		TSharedPtr<SDockTab> InvokedTab;
		if (CanInvokeInPanelDrawer(TabId, DockingArea, InvokedTab, bForceInDrawerPanel))
		{
			if (DockingArea->IsPanelDrawerOpen() && DockingArea->GetPanelDrawerHostedTab() == InvokedTab)
			{
				if (bToggleIfActiveInDrawer)
				{
					// Close the open panel
					if (TSharedPtr<SDockingArea> OtherDockingArea = InvokedTab->GetDockArea())
					{
						OtherDockingArea->ClosePanelDrawer();
					}
					else
					{
						ensure(false);
						DockingArea->ClosePanelDrawer();
					}
				}
				else
				{
					InvokeTab_DrawAttentionToTab(InvokedTab);
				}

				return InvokedTab;
			}

			if (TSharedPtr<SDockingArea> OtherDockingArea = InvokedTab->GetDockArea())
			{
				if (OtherDockingArea->GetPanelDrawerHostedTab() == InvokedTab)
				{
					OtherDockingArea->ClosePanelDrawerForTransfer();
				}
			}

			if (bForceInDrawerPanel)
			{
				InvokedTab->RemoveTabFromParent_Internal();
			}

			SetTabInPanelDrawer(InvokedTab, DockingArea, ParentWindow);
			return InvokedTab;
		}
	}

	// If no drawer is available fall be back to a normal tab
	return TryInvokeTab(TabId);
}

bool FTabManager::CanInvokeInPanelDrawer(const FTabId& TabId, const TSharedPtr<SDockingArea>& DockingArea, TSharedPtr<SDockTab>& OutTab, bool bForceInPanelDrawer)
{
	if (!CVarPanelDrawerToggle.GetValueOnGameThread())
	{
		return false;
	}

	if (!InvokeTab_CanInvokeTab(TabId))
	{
		return {};
	}

	TSharedPtr<SDockTab> ExistingTab = InvokeTab_FindOrReuseExistingTab(TabId);

	if (ExistingTab.IsValid())
	{
		OutTab = ExistingTab;

		// Only support nomad tab for now
		if (ExistingTab->GetTabRole() == ETabRole::NomadTab)
		{
			bool bShouldOpenDrawer = ExistingTab->GetTabManagerPtr()->RemoveFromHiddenPanelDrawerTabs(ExistingTab);

			if (TSharedPtr<SDockingArea> OtherDockingArea = ExistingTab->GetDockArea())
			{
				if (OtherDockingArea->GetPanelDrawerHostedTab() == ExistingTab)
				{
					bShouldOpenDrawer = true;
				}
			}

			bShouldOpenDrawer |= bForceInPanelDrawer;

			return bShouldOpenDrawer;
		}
		else
		{
			return false;
		}
	}


	const TSharedPtr<SDockTab> NewTab = SpawnTab(TabId, GetPrivateApi().GetParentWindow());
	OutTab = NewTab;
	if (NewTab.IsValid() && NewTab->GetTabRole() == ETabRole::NomadTab)
	{
		return true;
	}

	return false;
}

void FTabManager::SetTabInPanelDrawer(const TSharedPtr<SDockTab>& InTab, const TSharedPtr<SDockingArea>& TargetDockingArea, const TSharedPtr<SWindow>& InWindow)
{
	if (InTab && TargetDockingArea)
	{
		RemoveFromHiddenPanelDrawerTabs(InTab);

		if (InTab->GetTabRole() == ETabRole::NomadTab)
		{
			TSharedRef<FGlobalTabmanager> TargetManager = FGlobalTabmanager::Get();
			TSharedPtr<SDockingArea> MajorDockingArea = TargetManager->GetDockingAreaForPanelDrawer(InWindow);
			if (!MajorDockingArea)
			{
				MajorDockingArea = TargetManager->GetDockingAreaForPanelDrawer(
					TargetManager->GetMajorTabForTabManager(TargetDockingArea->GetTabManager())->GetParentWindow()
					);
			}

			if (!MajorDockingArea)
			{
				MajorDockingArea = TargetManager->GetDockingAreaForPanelDrawer(TargetManager->GetRootWindow());
			}

			// Shouldn't happen but just in case
			if (!ensure(MajorDockingArea))
			{
				return;
			}

			TSharedPtr<UE::Slate::Private::SPanelDrawerArea> TargetPanelDrawer = TargetDockingArea->GetPanelDrawerArea();
			check(TargetPanelDrawer);
			MajorDockingArea->SetPanelDrawerArea(TargetPanelDrawer.ToSharedRef());
			MajorDockingArea->HostTabIntoPanelDrawer(InTab.ToSharedRef());
		}
		else
		{
			TargetDockingArea->HostTabIntoPanelDrawer(InTab.ToSharedRef());
		}
	}
}

void FTabManager::OnPanelDrawerStateChanged() const
{
	OnPanelDrawerStateChangedDelegate.Broadcast(*this);
}

bool FTabManager::RemoveFromHiddenPanelDrawerTabs(TSharedPtr<SDockTab> ExistingTab)
{
	bool bWasRemoved = false;
	if (TSharedPtr<SDockingArea> DockingArea = ExistingTab->GetDockArea())
	{
		bWasRemoved |= DockingArea->RemoveHiddenInactivePanelDrawerTab(ExistingTab);
	}

	if (TSharedPtr<FTabManager> TabManager = ExistingTab->GetTabManagerPtr())
	{
		bWasRemoved |=  TabManager->PanelDrawerActiveHiddenTabs.Remove(ExistingTab->GetLayoutIdentifier()) > 0;
	}

	return bWasRemoved;
}

void FTabManager::HandleClosingAreaPanelDrawerData(const TSharedPtr<SDockingArea>& ClosingDockingArea)
{
	if (ClosingDockingArea)
	{
		TMap<FTabId, TSharedRef<SDockTab>> PanelDrawerAliveTabs = ClosingDockingArea->GetPanelDrawerKeepAliveTabs();
		for (TPair<FTabId, TSharedRef<SDockTab>>& Pair : PanelDrawerAliveTabs)
		{
			Pair.Value->SetTabManager(AsShared());
			Pair.Value->SetParentDockingArea({});
		}

		ClosingDockingArea->CleanPanelDrawer();
		PanelDrawerActiveHiddenTabs.Append(MoveTemp(PanelDrawerAliveTabs));
	}
}

FTabManager::~FTabManager()
{
	ClearPendingLayoutSave();
}

TSharedPtr<SDockTab> FTabManager::FindLastTabInWindow(TSharedPtr<SWindow> Window) const
{
	if ( Window.IsValid() )
	{
		for ( int32 AreaIndex = 0; AreaIndex < DockAreas.Num(); ++AreaIndex )
		{
			const TSharedPtr<SDockingArea> SomeDockArea = DockAreas[AreaIndex].Pin();
			if ( SomeDockArea.IsValid() )
			{
				if ( SomeDockArea->GetParentWindow() == Window )
				{
					TArray< TSharedRef<SDockTab> > ChildTabs = SomeDockArea->GetAllChildTabs();
					if ( ChildTabs.Num() > 0 )
					{
						return ChildTabs[ChildTabs.Num() - 1];
					}
				}
			}
		}
	}

	return TSharedPtr<SDockTab>();
}

TSharedPtr<class SDockingTabStack> FTabManager::FindTabInLiveAreas( const FTabMatcher& TabMatcher ) const
{
	for ( int32 AreaIndex = 0; AreaIndex < DockAreas.Num(); ++AreaIndex )
	{
		const TSharedPtr<SDockingArea> SomeDockArea = DockAreas[ AreaIndex ].Pin();
		if (SomeDockArea.IsValid())
		{
			TSharedPtr<SDockingTabStack> TabFoundHere = FindTabInLiveArea(TabMatcher, SomeDockArea.ToSharedRef());
			if ( TabFoundHere.IsValid() )
			{
				return TabFoundHere;
			}
		}
	}

	return TSharedPtr<SDockingTabStack>();
}

TSharedPtr<class SDockingTabStack> FTabManager::FindTabInLiveArea( const FTabMatcher& TabMatcher, const TSharedRef<SDockingArea>& InArea )
{
	TArray< TSharedRef<SDockingTabStack> > AllTabStacks;
	GetAllStacks(InArea, AllTabStacks);

	for (int32 StackIndex = 0; StackIndex < AllTabStacks.Num(); ++StackIndex)
	{
		if (AllTabStacks[StackIndex]->HasTab(TabMatcher))
		{
			return AllTabStacks[StackIndex];
		}		
	}

	return TSharedPtr<SDockingTabStack>();
}

FVector2D FTabManager::GetDefaultTabWindowSize(const FTabId& TabId)
{
	FVector2D WindowSize = FTabManager::FallbackWindowSize;
	FVector2D* DefaultTabSize = FTabManager::DefaultTabWindowSizeMap.Find(TabId);

	if (DefaultTabSize != nullptr)
	{
		WindowSize = *DefaultTabSize;
	}

	return WindowSize;
}

bool FTabManager::HasAnyTabWithTabId( const TSharedRef<FLayoutNode>& SomeNode, const FName& InTabTypeToMatch ) const
{
	return HasAnyMatchingTabs(SomeNode,
		[this, InTabTypeToMatch](const FTab& Candidate)
		{
			return this->IsValidTabForSpawning(Candidate) && Candidate.TabId.TabType == InTabTypeToMatch;
		});
}

TSharedPtr<FTabManager::FArea> FTabManager::GetAreaFromInitialLayoutWithTabType( const FTabId& InTabIdToMatch ) const
{
	const TSharedPtr<FTabManager::FLayout> InitialLayoutSP = FGlobalTabmanager::Get()->GetInitialLayoutSP();
	if (InitialLayoutSP.IsValid())
	{
		for (const TSharedRef<FArea>& Area : InitialLayoutSP->Areas)
		{
			if (HasAnyTabWithTabId(Area, InTabIdToMatch.TabType))
			{
				return Area.ToSharedPtr();
			}
		}
	}
	return nullptr;
}

TSharedRef<FTabManager::FArea> FTabManager::GetAreaForTabId(const FTabId& TabId)
{
	if (const TSharedPtr<FArea> AreaFromInitiallyLoadedLayout = FGlobalTabmanager::Get()->GetAreaFromInitialLayoutWithTabType(TabId))
	{
		/* we must reuse positions from the initial layout for positionally specified floating windows. If we don't
		* do this then any persisted floating windows load in a big cluster in the middle on top of one another */
		if ( AreaFromInitiallyLoadedLayout->DefinesPositionallySpecifiedFloatingWindow() )
		{
			return AreaFromInitiallyLoadedLayout.ToSharedRef();
		}
	}
	return NewArea( GetDefaultTabWindowSize(TabId) );
}

FText FTabManager::GetTabLabelBasedOnSpawner(const TSharedPtr<FTabSpawnerEntry>& InSpawnerEntry) const
{
	if (!InSpawnerEntry.IsValid())
	{
		return FText::GetEmpty();
	}

	FText Label = FText::GetEmpty();

	if (!InSpawnerEntry->IsTabNameHidden())
	{
		Label = InSpawnerEntry->GetDisplayName().IsEmpty() ? FText::FromName(InSpawnerEntry->TabType) : InSpawnerEntry->GetDisplayName();
	}

	return Label;
}

void FGlobalTabmanager::SetInitialLayoutSP(TSharedPtr<FTabManager::FLayout> InLayout)
{
	InitialLayoutSP = InLayout;
}

TSharedPtr<FTabManager::FLayout> FGlobalTabmanager::GetInitialLayoutSP()
{
	return InitialLayoutSP;
}

bool FTabManager::HasAnyMatchingTabs( const TSharedRef<FTabManager::FLayoutNode>& SomeNode, const TFunctionRef<bool(const FTab& Candidate)>& Matcher )
{
	TSharedPtr<FTabManager::FSplitter> AsSplitter = SomeNode->AsSplitter();
	TSharedPtr<FTabManager::FStack> AsStack = SomeNode->AsStack();

	if ( AsStack.IsValid() )
	{
		return INDEX_NONE != AsStack->Tabs.IndexOfByPredicate(Matcher);
	}
	else
	{
		ensure( AsSplitter.IsValid() );
		// Do any of the child nodes have open tabs?
		for (const TSharedRef<FTabManager::FLayoutNode>& ChildNode : AsSplitter->GetChildNodes())
		{
			if ( HasAnyMatchingTabs(ChildNode, Matcher) )
			{
				return true;
			}
		}
		return false;
	}
}

bool FTabManager::HasValidOpenTabs( const TSharedRef<FTabManager::FLayoutNode>& SomeNode ) const
{
	// Search for valid and open tabs
	return HasAnyMatchingTabs(SomeNode,
		[this](const FTab& Candidate)
		{
				return this->IsValidTabForSpawning(Candidate) && Candidate.TabState == ETabState::OpenedTab;
		});
}

bool FTabManager::HasValidTabs( const TSharedRef<FTabManager::FLayoutNode>& SomeNode ) const
{
	// Search for valid tabs that can be spawned
	return HasAnyMatchingTabs(SomeNode,
		[this](const FTab& Candidate)
		{
			return this->IsValidTabForSpawning(Candidate);
		});
}

void FTabManager::SetTabsTo(const TSharedRef<FTabManager::FLayoutNode>& SomeNode, const ETabState::Type NewTabState, const ETabState::Type OriginalTabState) const
{
	// Set particular tab to desired NewTabState
	TSharedPtr<FTabManager::FStack> AsStack = SomeNode->AsStack();
	if (AsStack.IsValid())
	{
		TArray<FTab>& Tabs = AsStack->Tabs;
		for (int32 TabIndex = 0; TabIndex < Tabs.Num(); ++TabIndex)
		{
			if (Tabs[TabIndex].TabState == OriginalTabState)
			{
				Tabs[TabIndex].TabState = NewTabState;
			}
		}
	}
	// Recursively set all tabs to desired NewTabState
	else
	{
		TSharedPtr<FTabManager::FSplitter> AsSplitter = SomeNode->AsSplitter();
		ensure(AsSplitter.IsValid());
		for (int32 ChildIndex = 0; ChildIndex < AsSplitter->ChildNodes.Num(); ++ChildIndex)
		{
			SetTabsTo(AsSplitter->ChildNodes[ChildIndex], NewTabState, OriginalTabState);
		}
	}
}

void FTabManager::OnTabForegrounded( const TSharedPtr<SDockTab>& NewForegroundTab, const TSharedPtr<SDockTab>& BackgroundedTab )
{
	// Do nothing.
}

void FTabManager::OnTabRelocated( const TSharedRef<SDockTab>& RelocatedTab, const TSharedPtr<SWindow>& NewOwnerWindow )
{
	RelocatedTab->NotifyTabRelocated();

	CleanupPointerArray(DockAreas);
	RemoveTabFromCollapsedAreas( FTabMatcher( RelocatedTab->GetLayoutIdentifier() ) );
	for (int32 DockAreaIndex=0; DockAreaIndex < DockAreas.Num(); ++DockAreaIndex)
	{
		DockAreas[DockAreaIndex].Pin()->OnTabFoundNewHome( RelocatedTab, NewOwnerWindow.ToSharedRef() );
	}

	FGlobalTabmanager::Get()->UpdateMainMenu(RelocatedTab, true);

	UpdateStats();

	RequestSavePersistentLayout();

	if (TSharedPtr<FTabManager> NewTabManager = RelocatedTab->GetTabManagerPtr())
	{
		NewTabManager->RequestSavePersistentLayout();
	}
}

void FTabManager::OnTabOpening( const TSharedRef<SDockTab>& TabBeingOpened )
{
	UpdateStats();

	RequestSavePersistentLayout();
}

void FTabManager::OnTabClosing( const TSharedRef<SDockTab>& TabBeingClosed )
{
	RequestSavePersistentLayout();
}

void FTabManager::OnTabManagerClosing()
{
	CleanupPointerArray(DockAreas);

	// Gather the persistent layout and allow a custom handler to persist it
	SavePersistentLayout();

	for (int32 DockAreaIndex=0; DockAreaIndex < DockAreas.Num(); ++DockAreaIndex)
	{
		TSharedRef<SDockingArea> ChildDockArea = DockAreas[DockAreaIndex].Pin().ToSharedRef();
		TSharedPtr<SWindow> DockAreaWindow = ChildDockArea->GetParentWindow();
		if (DockAreaWindow.IsValid())
		{
			DockAreaWindow->RequestDestroyWindow();
		}
	}
}

bool FTabManager::CanCloseManager( const TSet< TSharedRef<SDockTab> >& TabsToIgnore )
{
	CleanupPointerArray(DockAreas);

	bool bCanCloseManager = true;

	for (int32 DockAreaIndex=0; bCanCloseManager && DockAreaIndex < DockAreas.Num(); ++DockAreaIndex)
		{
		TSharedPtr<SDockingArea> SomeArea = DockAreas[DockAreaIndex].Pin();
		TArray< TSharedRef<SDockTab> > AreasTabs = SomeArea.IsValid() ? SomeArea->GetAllChildTabs() : TArray< TSharedRef<SDockTab> >();
		
		for (int32 TabIndex=0; bCanCloseManager && TabIndex < AreasTabs.Num(); ++TabIndex)	
			{
			bCanCloseManager =
				TabsToIgnore.Contains( AreasTabs[TabIndex] ) ||
				AreasTabs[TabIndex]->GetTabRole() != ETabRole::MajorTab ||
				AreasTabs[TabIndex]->CanCloseTab(/** Ignore locked tabs */ true);
		}		
	}

	return bCanCloseManager;
}

void FTabManager::GetAllStacks( const TSharedRef<SDockingArea>& InDockArea, TArray< TSharedRef<SDockingTabStack> >& OutTabStacks )
{
	TArray< TSharedRef<SDockingNode> > AllNodes = InDockArea->GetChildNodesRecursively();
	for (int32 NodeIndex=0; NodeIndex < AllNodes.Num(); ++NodeIndex)
	{	
		if ( AllNodes[NodeIndex]->GetNodeType() == SDockingNode::DockTabStack )
		{
			OutTabStacks.Add( StaticCastSharedRef<SDockingTabStack>( AllNodes[NodeIndex] ) );
		}
	}
}

TSharedPtr<FTabManager::FStack> FTabManager::FindTabUnderNode( const FTabMatcher& Matcher, const TSharedRef<FTabManager::FLayoutNode>& NodeToSearchUnder )
{
	TSharedPtr<FTabManager::FStack> NodeAsStack = NodeToSearchUnder->AsStack();
	TSharedPtr<FTabManager::FSplitter> NodeAsSplitter = NodeToSearchUnder->AsSplitter();

	if (NodeAsStack.IsValid())
	{
		const int32 TabIndex = NodeAsStack->Tabs.IndexOfByPredicate(Matcher);
		if (TabIndex != INDEX_NONE)
		{
			return NodeAsStack;
		}
		else
		{
			return TSharedPtr<FTabManager::FStack>();
		}
	}
	else
	{
		ensure( NodeAsSplitter.IsValid() );
		TSharedPtr<FTabManager::FStack> StackWithTab;
		for ( const TSharedRef<FTabManager::FLayoutNode>& ChildNode : NodeAsSplitter->GetChildNodes() )
		{
			StackWithTab = FindTabUnderNode( Matcher, ChildNode );
		}

			return StackWithTab;
		}
}


TSharedPtr<FTabSpawnerEntry> FTabManager::FindTabSpawnerFor(FName TabId)
{
	// Look for a spawner in this tab manager.
	TSharedRef<FTabSpawnerEntry>* Spawner = TabSpawner.Find(TabId);
	if (Spawner == nullptr)
	{
		Spawner = NomadTabSpawner->Find(TabId);
	}

	return (Spawner != nullptr)
		? TSharedPtr<FTabSpawnerEntry>(*Spawner)
		: TSharedPtr<FTabSpawnerEntry>();
}

const TSharedPtr<const FTabSpawnerEntry> FTabManager::FindTabSpawnerFor(FName TabId) const
{
	// Look for a spawner in this tab manager.
	const TSharedRef<FTabSpawnerEntry>* Spawner = TabSpawner.Find(TabId);
	if (Spawner == nullptr)
	{
		Spawner = NomadTabSpawner->Find(TabId);
	}

	return (Spawner != nullptr)
		? TSharedPtr<const FTabSpawnerEntry>(*Spawner)
		: TSharedPtr<const FTabSpawnerEntry>();
}

bool FTabManager::HasSidebaredTabs() const
{
	return !TemporarilySidebaredTabs.IsEmpty();
}

int32 FTabManager::FindTabInCollapsedAreas( const FTabMatcher& Matcher )
{
	for ( int32 CollapsedDockAreaIndex=0; CollapsedDockAreaIndex < CollapsedDockAreas.Num(); ++CollapsedDockAreaIndex )
	{
		TSharedPtr<FTabManager::FStack> StackWithMatchingTab = FindTabUnderNode(Matcher, CollapsedDockAreas[CollapsedDockAreaIndex]);
		if (StackWithMatchingTab.IsValid())
		{
			return CollapsedDockAreaIndex;
		}
	}	

	return INDEX_NONE;
}

void FTabManager::RemoveTabFromCollapsedAreas( const FTabMatcher& Matcher )
{
	for ( int32 CollapsedDockAreaIndex=0; CollapsedDockAreaIndex < CollapsedDockAreas.Num(); ++CollapsedDockAreaIndex )
	{
		const TSharedRef<FTabManager::FArea>& DockArea = CollapsedDockAreas[CollapsedDockAreaIndex];

		TSharedPtr<FTabManager::FStack> StackWithMatchingTab;
		do
		{
			StackWithMatchingTab = FindTabUnderNode(Matcher, DockArea);

			if (StackWithMatchingTab.IsValid())
			{
				const int32 TabIndex = StackWithMatchingTab->Tabs.IndexOfByPredicate(Matcher);
				if ( ensure(TabIndex != INDEX_NONE) )
				{
					StackWithMatchingTab->Tabs.RemoveAt(TabIndex);
				}
			}
		}
		while ( StackWithMatchingTab.IsValid() );
	}	
}

void FTabManager::UpdateStats()
{
	StaticCastSharedRef<FTabManager>(FGlobalTabmanager::Get())->UpdateStats();
}

TSharedPtr<SDockingArea> FTabManager::GetDockingAreaForWindow(const TSharedRef<SWindow>& InWindow) const
{
	for (const TWeakPtr<SDockingArea>& WeakDockArea : DockAreas)
	{
		if (TSharedPtr<SDockingArea> DockArea = WeakDockArea.Pin())
		{
			if (DockArea->GetParentWindow() == InWindow)
			{
				return DockArea;
			}
		}
	}

	return {};
}

TSharedPtr<SDockingArea> FTabManager::GetDockingAreaForPanelDrawer(const TSharedPtr<SWindow>& InWindow) const
{
	TSharedPtr<SDockingArea> PrimaryArea;
	for (TWeakPtr<SDockingArea> LiveArea : DockAreas)
	{
		if (TSharedPtr<SDockingArea> LiveDockingArea = LiveArea.Pin())
		{
			// Found the primary area (SubTabmanager primary area never own their windows)
			TSharedPtr<SWindow> ParentWindow = LiveDockingArea->GetParentWindow();
			if (ParentWindow == InWindow)
			{
				return LiveDockingArea;
			}

			// Found the primary area (SubTabmanager primary area never own their windows)
			if (!ParentWindow)
			{
				ensure(!PrimaryArea);
				PrimaryArea = MoveTemp(LiveDockingArea);
			}
		}
	}

	return PrimaryArea;
}

void FTabManager::GetRecordableStats( int32& OutTabCount, TArray<TSharedPtr<SWindow>>& OutUniqueParentWindows ) const
{
	OutTabCount = 0;
	for (auto AreaIter = DockAreas.CreateConstIterator(); AreaIter; ++AreaIter)
	{
		TSharedPtr<SDockingArea> DockingArea = AreaIter->Pin();
		if (DockingArea.IsValid())
		{
			TSharedPtr<SWindow> ParentWindow = DockingArea->GetParentWindow();
			if (ParentWindow.IsValid())
			{
				OutUniqueParentWindows.AddUnique(ParentWindow);
			}

			TArray< TSharedRef<SDockingTabStack> > OutTabStacks;
			GetAllStacks(DockingArea.ToSharedRef(), OutTabStacks);
			for (auto StackIter = OutTabStacks.CreateConstIterator(); StackIter; ++StackIter)
			{
				OutTabCount += (*StackIter)->GetNumTabs();
			}
		}
	}
}

void FTabManager::FixLayoutLoadingPrimaryArea(const TSharedRef<FArea>& InPrimaryArea)
{
	FName LevelEditorTabId = FName("LevelEditor");
	FName HomeScreenTabId = FName("HomeScreen");
	const bool bIsHomeScreenEnabled = UE::Editor::HomeScreen::IsHomeScreenEnabled();

	for (const TSharedRef<FLayoutNode>& Node : InPrimaryArea->GetChildNodes())
	{
		if (const TSharedPtr<FStack>& StackNode = Node->AsStack())
		{
			// Get the LevelEditor tab
			FTab* LevelEditorTabPtr = StackNode->Tabs.FindByPredicate([LevelEditorTabId] (const FTab& InTab)
				{
					return InTab.TabId.TabType == LevelEditorTabId;
				});

			if (!LevelEditorTabPtr)
			{
				// LevelEditor tabs is not part of this PrimaryArea, could be that we are restoring the internal tabs of the LevelEditor area so the tab won't be here but the area is still the PrimaryArea
				continue;
			}

			// Get the actual LevelEditor tab since the pointer may point to other tab later since we are moving tabs
			const FTab LevelEditorTab = *LevelEditorTabPtr;
			const FTab NewHomeScreenTab = FTab(FTabId(HomeScreenTabId), ETabState::OpenedTab);
			const int32 LevelEditorExpectedIndex = bIsHomeScreenEnabled ? /** Always second tab with HomeScreen */ 1 : /** Always first without HomeScreen */ 0;

			// Exit immediately if the position is already correct, either HomeScreen first and LevelEditor second if the HomeScreen is enabled or LevelEditor first if not
			if (bIsHomeScreenEnabled && StackNode->Tabs.IndexOfByKey(LevelEditorTab) == LevelEditorExpectedIndex && StackNode->Tabs.IndexOfByKey(NewHomeScreenTab) == 0)
			{
				return;
			}

			if (!bIsHomeScreenEnabled && StackNode->Tabs.IndexOfByKey(LevelEditorTab) == LevelEditorExpectedIndex)
			{
				return;
			}

			// Remove all HomeScreen tab from the current stack and insert a new one in the correct position.
			StackNode->Tabs.RemoveAll([HomeScreenTabId] (const FTab& InTab)
				{
					return InTab.TabId.TabType == HomeScreenTabId;
				});

			if (bIsHomeScreenEnabled)
			{
				// Insert the HomeScreen tab as the first tab
				StackNode->Tabs.Insert(NewHomeScreenTab, 0);
			}

			const int32 LevelEditorIndex = StackNode->Tabs.IndexOfByKey(LevelEditorTab);

			// Do not touch the LevelEditor position if already correct, the order should be HomeScreen->LevelEditor->Other tabs... or LevelEditor->Other tabs... if the HomeScreen is not enabled
			if (LevelEditorIndex != LevelEditorExpectedIndex)
			{
				// Re-Insert the LevelEditor as the first or second tab
				StackNode->Tabs.RemoveAt(LevelEditorIndex);
				StackNode->Tabs.Insert(LevelEditorTab, LevelEditorExpectedIndex);
			}

			// Do not continue since we already fixed the area we wanted to contain the HomeScreen and the LevelEditor next to each other.
			return;
		}
	}
}

const TSharedRef<FGlobalTabmanager>& FGlobalTabmanager::Get()
{
	static const TSharedRef<FGlobalTabmanager> Instance = FGlobalTabmanager::New();
	// @todo: Never Destroy the Global Tab Manager because it has hooks into a bunch of different modules.
	//        All those modules are unloaded first, so unbinding the delegates will cause a problem.
	static const TSharedRef<FGlobalTabmanager>* NeverDestroyGlobalTabManager = new TSharedRef<FGlobalTabmanager>( Instance );
	return Instance;
}

bool FGlobalTabmanager::GetShouldUseMiddleEllipsisForDockTabLabel() const
{
	return bShouldUseMiddleEllipsisForDockTabLabel;
}

void FGlobalTabmanager::SetShouldUseMiddleEllipsisForDockTabLabel(const bool bInShouldUseMiddleEllipsis)
{
	bShouldUseMiddleEllipsisForDockTabLabel = bInShouldUseMiddleEllipsis;
}

TArray<TSharedPtr<SWindow>> FGlobalTabmanager::GetDockableWindows() const
{
	TArray<TSharedPtr<SWindow>> Windows;
	Windows.Reserve(DockAreas.Num());
	for (const TWeakPtr<SDockingArea>& DockingAreaWeak : DockAreas)
	{
		if (const TSharedPtr<SDockingArea> DockArea = DockingAreaWeak.Pin())
		{
			if (TSharedPtr<SWindow> Window = DockArea->GetParentWindow())
			{
				Windows.Add(Window);
			}
		}
	}
	return Windows;
}

FGlobalTabmanager::FWindowMajorTabsInfo FGlobalTabmanager::GetDockableWindowMajorTabsInfo(const TSharedRef<SWindow>& InWindow) const
{
	FWindowMajorTabsInfo TabsInfo;
	TabsInfo.Window = InWindow;
	if (const TSharedPtr<SDockingArea> DockingArea = GetDockingAreaForWindow(InWindow))
	{
		TArray<TSharedRef<SDockingTabStack>> TabStacks;
		GetAllStacks(DockingArea.ToSharedRef(), TabStacks);

		for (const TSharedRef<SDockingTabStack>& TabStack : TabStacks)
		{
			TSlotlessChildren<SDockTab> Tabs = TabStack->GetTabs();
			for (int32 Index = 0; Index < Tabs.Num(); Index++)
			{
				TSharedRef<SDockTab> Tab = StaticCastSharedRef<SDockTab>(Tabs.GetChildAt(Index));
				TabsInfo.DockTabs.Add(Tab);

				if (Tab->IsForeground())
				{
					TabsInfo.ActiveTab = Tab;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "Could not find docking area for window %ls", *InWindow->GetTitle().ToString())
	}
	return TabsInfo;
}

FDelegateHandle FGlobalTabmanager::OnActiveTabChanged_Subscribe( const FOnActiveTabChanged::FDelegate& InDelegate )
{
	return OnActiveTabChanged.Add( InDelegate );
}

void FGlobalTabmanager::OnActiveTabChanged_Unsubscribe( FDelegateHandle Handle )
{
	OnActiveTabChanged.Remove( Handle );
}

FDelegateHandle FGlobalTabmanager::OnTabForegrounded_Subscribe(const FOnActiveTabChanged::FDelegate& InDelegate)
{
	return TabForegrounded.Add(InDelegate);
}

void FGlobalTabmanager::OnTabForegrounded_Unsubscribe(FDelegateHandle Handle)
{
	TabForegrounded.Remove(Handle);
}

TSharedPtr<class SDockTab> FGlobalTabmanager::GetActiveTab() const
{
	return ActiveTabPtr.Pin();
}

bool FGlobalTabmanager::CanSetAsActiveTab(const TSharedPtr<SDockTab>& Tab)
{
	// Setting NULL wipes out the active tab; always apply that change.
	// Major tabs are ignored for the purposes of active-tab tracking. We do not care about their
	return !Tab.IsValid() || Tab->GetVisualTabRole() != ETabRole::MajorTab;
}

void FGlobalTabmanager::SetActiveTab( const TSharedPtr<class SDockTab>& NewActiveTab )
{
	const bool bShouldApplyChange = CanSetAsActiveTab(NewActiveTab);
	
	TSharedPtr<SDockTab> CurrentlyActiveTab = GetActiveTab();

	if (bShouldApplyChange && (CurrentlyActiveTab != NewActiveTab))
	{
		if (NewActiveTab.IsValid())
		{
			NewActiveTab->UpdateActivationTime();
		}

		OnActiveTabChanged.Broadcast( CurrentlyActiveTab, NewActiveTab );
		ActiveTabPtr = NewActiveTab;
	}	
}

FTabSpawnerEntry& FGlobalTabmanager::RegisterNomadTabSpawner(const FName TabId, const FOnSpawnTab& OnSpawnTab, const FCanSpawnTab& CanSpawnTab)
{
	// Sanity check
	ensure(!IsLegacyTabType(TabId));

	LLM_SCOPE_BYTAG(UI_Slate);

	// Remove TabId if it was previously loaded. This allows re-loading the Editor UI layout without restarting the whole Editor (Window->Load Layout)
	if (NomadTabSpawner->Contains(TabId))
	{
		UnregisterNomadTabSpawner(TabId);
	}

	// (Re)create and return NewSpawnerEntry
	TSharedRef<FTabSpawnerEntry> NewSpawnerEntry = MakeShareable(new FTabSpawnerEntry(TabId, OnSpawnTab, CanSpawnTab));
	NomadTabSpawner->Add(TabId, NewSpawnerEntry);
	return NewSpawnerEntry.Get();
}

void FGlobalTabmanager::UnregisterNomadTabSpawner( const FName TabId )
{
	const int32 NumRemoved = NomadTabSpawner->Remove(TabId);
}

void FGlobalTabmanager::SetApplicationTitle( const FText& InAppTitle )
{
	AppTitle = InAppTitle;

	for (int32 DockAreaIndex=0; DockAreaIndex < DockAreas.Num(); ++DockAreaIndex)
	{
		if (DockAreas[DockAreaIndex].IsValid())
		{
			TSharedPtr<SWindow> ParentWindow = DockAreas[DockAreaIndex].Pin()->GetParentWindow();
			if (ParentWindow.IsValid() && ParentWindow == FGlobalTabmanager::Get()->GetRootWindow())
			{
				ParentWindow->SetTitle( AppTitle );
			}
		}
	}
}

const FText& FGlobalTabmanager::GetApplicationTitle() const
{
	return AppTitle;
}

bool FGlobalTabmanager::CanCloseManager( const TSet< TSharedRef<SDockTab> >& TabsToIgnore )
{
	bool bCanCloseManager = FTabManager::CanCloseManager(TabsToIgnore);

	for( int32 ManagerIndex=0; bCanCloseManager && ManagerIndex < SubTabManagers.Num(); ++ManagerIndex )
	{
		TSharedPtr<FTabManager> SubManager = SubTabManagers[ManagerIndex].TabManager.Pin();
		if (SubManager.IsValid())
		{
			bCanCloseManager = SubManager->CanCloseManager(TabsToIgnore);
		}
	}

	return bCanCloseManager;
}

TSharedPtr<SDockTab> FGlobalTabmanager::GetMajorTabForTabManager(const TSharedRef<FTabManager>& ChildManager)
{
	const int32 MajorTabIndex = SubTabManagers.IndexOfByPredicate(FindByManager(ChildManager));
	if ( MajorTabIndex != INDEX_NONE )
	{
		return SubTabManagers[MajorTabIndex].MajorTab.Pin();
	}

	return TSharedPtr<SDockTab>();
}

TSharedPtr<FTabManager> FGlobalTabmanager::GetTabManagerForMajorTab(const TSharedPtr<SDockTab> DockTab) const
{
	const int32 Index = SubTabManagers.IndexOfByPredicate(FindByTab(DockTab.ToSharedRef()));
	if (Index != INDEX_NONE)
	{
		return SubTabManagers[Index].TabManager.Pin();
	}

	return nullptr;
}

TSharedPtr<FTabManager> FGlobalTabmanager::GetSubTabManagerForWindow(const TSharedRef<SWindow>& InWindow) const
{
	TSharedRef<SWindow> SelectedWindow = UE::Slate::Private::GetTabManagerTopWindow(InWindow);

	for (const FSubTabManager& SubTabManager : SubTabManagers)
	{
		if (TSharedPtr<SDockTab> Tab = SubTabManager.MajorTab.Pin())
		{
			if (Tab->IsForeground())
			{
				if (TSharedPtr<FTabManager> TabManager = SubTabManager.TabManager.Pin())
				{
					// Test the major tab and the manager areas in case they own some other windows
					if (Tab->GetParentWindow() == SelectedWindow || TabManager->GetDockingAreaForWindow(SelectedWindow))
					{
						return TabManager;
					}
				}
			}
		}
	}

	return {};
}

void FGlobalTabmanager::DrawAttentionToTabManager( const TSharedRef<FTabManager>& ChildManager )
{
	TSharedPtr<SDockTab> Tab = GetMajorTabForTabManager(ChildManager);
	if ( Tab.IsValid() )
	{
		this->DrawAttention(Tab.ToSharedRef());

		// #HACK VREDITOR
		if ( ProxyTabManager.IsValid() )
		{
			if( ProxyTabManager->IsTabSupported( Tab->GetLayoutIdentifier() ) )
			{
				ProxyTabManager->DrawAttention(Tab.ToSharedRef());
			}
		}
	}
}

TSharedRef<FTabManager> FGlobalTabmanager::NewTabManager( const TSharedRef<SDockTab>& InOwnerTab )
{
	struct {
		bool operator()(const FSubTabManager& InItem) const
		{
			return !InItem.MajorTab.IsValid();
		}
	} ShouldRemove;

	SubTabManagers.RemoveAll( ShouldRemove );

	const TSharedRef<FTabManager> NewTabManager = FTabManager::New( InOwnerTab, NomadTabSpawner );
	SubTabManagers.Add( FSubTabManager(InOwnerTab, NewTabManager) );

	UpdateStats();

	return NewTabManager;
}

void FGlobalTabmanager::UpdateMainMenu(const TSharedRef<SDockTab>& ForTab, bool const bForce)
{
	TSharedPtr<FTabManager> TabManager = ForTab->GetTabManagerPtr();
	if(TabManager == AsShared())
	{
		const int32 TabIndex = SubTabManagers.IndexOfByPredicate(FindByTab(ForTab));
		if (TabIndex != INDEX_NONE)
		{
			TabManager = SubTabManagers[TabIndex].TabManager.Pin();
		}
	}
	TabManager->UpdateMainMenu(ForTab, bForce);
}

void FGlobalTabmanager::SaveAllVisualState()
{
	this->SavePersistentLayout();

	for( int32 ManagerIndex=0; ManagerIndex < SubTabManagers.Num(); ++ManagerIndex )
	{
		const TSharedPtr<FTabManager> SubManagerTab = SubTabManagers[ManagerIndex].TabManager.Pin();		
		if (SubManagerTab.IsValid())
		{
			SubManagerTab->SavePersistentLayout();
		}
	}
}

void FGlobalTabmanager::SetRootWindow( const TSharedRef<SWindow> InRootWindow )
{
	RootWindowPtr = InRootWindow;
}

TSharedPtr<SWindow> FGlobalTabmanager::GetRootWindow() const
{
	return RootWindowPtr.Pin();
}

void FGlobalTabmanager::AddLegacyTabType(FName InLegacyTabType, FName InNewTabType)
{
	ensure(!TabSpawner.Contains(InLegacyTabType));
	ensure(!NomadTabSpawner->Contains(InLegacyTabType));

	LegacyTabTypeRedirectionMap.Add(InLegacyTabType, InNewTabType);
}

bool FGlobalTabmanager::IsLegacyTabType(FName InTabType) const
{
	return LegacyTabTypeRedirectionMap.Contains(InTabType);
}

FName FGlobalTabmanager::GetTabTypeForPotentiallyLegacyTab(FName InTabType) const
{
	const FName* NewTabType = LegacyTabTypeRedirectionMap.Find(InTabType);
	return NewTabType ? *NewTabType : InTabType;
}

void FGlobalTabmanager::OnTabForegrounded( const TSharedPtr<SDockTab>& NewForegroundTab, const TSharedPtr<SDockTab>& BackgroundedTab )
{
	FTabAndManagerForDrawer ForegroundTabAndManager;
	ForegroundTabAndManager.Key = NewForegroundTab;
	if (NewForegroundTab.IsValid())
	{
		// Show any child windows associated with the Major Tab that got foregrounded.
		const int32 ForegroundedTabIndex = SubTabManagers.IndexOfByPredicate(FindByTab(NewForegroundTab.ToSharedRef()));
		if (ForegroundedTabIndex != INDEX_NONE)
		{
			TSharedPtr<FTabManager> ForegroundTabManager = SubTabManagers[ForegroundedTabIndex].TabManager.Pin();
			ForegroundTabManager->GetPrivateApi().ShowWindows();
			ForegroundTabAndManager.Value = ForegroundTabManager;
		}

		NewForegroundTab->UpdateActivationTime();
	}
	
	FTabAndManagerForDrawer BackgroundedTabAndSubManager;
	BackgroundedTabAndSubManager.Key = BackgroundedTab;
	if (BackgroundedTab.IsValid())
	{
		// Hide any child windows associated with the Major Tab that got backgrounded.
		const int32 BackgroundedTabIndex = SubTabManagers.IndexOfByPredicate(FindByTab(BackgroundedTab.ToSharedRef()));
		if (BackgroundedTabIndex != INDEX_NONE)
		{
			TSharedPtr<FTabManager> BackgroundedTabManager = SubTabManagers[BackgroundedTabIndex].TabManager.Pin();
			BackgroundedTabManager->GetPrivateApi().HideWindows();
			BackgroundedTabAndSubManager.Value = BackgroundedTabManager;
		}
	}

	RelocatePanelDrawerNomadTab(MoveTemp(ForegroundTabAndManager), MoveTemp(BackgroundedTabAndSubManager));

	TabForegrounded.Broadcast(NewForegroundTab, BackgroundedTab);
}

void FGlobalTabmanager::RelocatePanelDrawerNomadTab(FTabAndManagerForDrawer ForegroundTabAndSubManager, FTabAndManagerForDrawer BackgroundedTabAndSubManager)

{
	
	SDockTab* BackgroundedTab = BackgroundedTabAndSubManager.Key.Get();
	FTabManager* BackgroundedTabManager = BackgroundedTabAndSubManager.Value.Get();

	// The panel drawer only support Major tab a the moment so don't bother with any other type of tab.
	if (BackgroundedTabManager && BackgroundedTab && BackgroundedTab->GetTabRole() == ETabRole::MajorTab)
	{
		if (TSharedPtr<SDockingArea> BackgroundTabbDockingArea = BackgroundedTab->GetDockArea())
		{
			BackgroundTabbDockingArea->SetPanelDrawerArea({});
		}
	}

	SDockTab* ForegroundTab = ForegroundTabAndSubManager.Key.Get();
	FTabManager* ForegroundTabManager = ForegroundTabAndSubManager.Value.Get();
	if (ForegroundTab && ForegroundTabManager)
	{
		if (TSharedPtr<SDockingArea> Primary = ForegroundTabManager->GetDockingAreaForPanelDrawer(ForegroundTab->GetParentWindow()))
		{
			// Don't try to forward a panel drawer from the tab nmanager to a tab if it doesn't have a panel drawer.
			if (Primary->HasPanelDrawer())
			{
				if (TSharedPtr<SDockingArea> ForegroundTabDockingArea = ForegroundTab->GetDockArea())
				{
					ForegroundTabDockingArea->SetPanelDrawerArea(Primary->GetPanelDrawerArea());
				}
			}
		}
	}
}

void FGlobalTabmanager::OnTabRelocated( const TSharedRef<SDockTab>& RelocatedTab, const TSharedPtr<SWindow>& NewOwnerWindow )
{
	// Handle transferring the drawer
	if ( RelocatedTab->GetTabRole() == ETabRole::MajorTab || RelocatedTab->GetTabRole() == ETabRole::NomadTab )
	{
		LastMajorDockWindow = NewOwnerWindow;
	}

	if (NewOwnerWindow.IsValid())
	{
		const int32 RelocatedManagerIndex = SubTabManagers.IndexOfByPredicate(FindByTab(RelocatedTab));
		if (RelocatedManagerIndex != INDEX_NONE)
		{
			const TSharedRef<FTabManager>& RelocatedManager = SubTabManagers[RelocatedManagerIndex].TabManager.Pin().ToSharedRef();

			// Reparent any DockAreas hanging out in a child window.
			// We do not support native window re-parenting, so destroy old windows and re-create new ones in their place that are properly parented.
			// Move the old DockAreas into new windows.
			const TArray< TWeakPtr<SDockingArea> >& LiveDockAreas = RelocatedManager->GetPrivateApi().GetLiveDockAreas();
			for (int32 DockAreaIndex=0; DockAreaIndex < LiveDockAreas.Num(); ++DockAreaIndex)
			{
				const TSharedRef<SDockingArea>& ChildDockArea = LiveDockAreas[ DockAreaIndex ].Pin().ToSharedRef();
				const TSharedPtr<SWindow> OldChildWindow = ChildDockArea->GetParentWindow();
				if ( OldChildWindow.IsValid() )
				{
					TSharedRef<SWindow> NewChildWindow = SNew(SWindow)
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
					// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
					// and for multi-view rendering to function properly
					.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
					.AutoCenter(EAutoCenter::None)
					.ScreenPosition(OldChildWindow->GetPositionInScreen())
					.ClientSize(OldChildWindow->GetSizeInScreen())
					.SupportsMinimize(false)
					.SupportsMaximize(false)
					.CreateTitleBar(false)
					.AdjustInitialSizeAndPositionForDPIScale(false)
					[
						ChildDockArea
					];

					ChildDockArea->SetParentWindow(NewChildWindow);

					FSlateApplication::Get().AddWindowAsNativeChild( NewChildWindow, NewOwnerWindow.ToSharedRef() );

					FSlateApplication::Get().RequestDestroyWindow( OldChildWindow.ToSharedRef() );
				}
			}
		}
#if WITH_EDITOR
		// When a tab is relocated we need to let the content know that the dpi scale window where the tab now resides may have changed
		FSlateApplication::Get().OnWindowDPIScaleChanged().Broadcast(NewOwnerWindow.ToSharedRef());
#endif
	}

	FTabManager::OnTabRelocated( RelocatedTab, NewOwnerWindow );
}

void FGlobalTabmanager::OnTabClosing( const TSharedRef<SDockTab>& TabBeingClosed )
{
	FTabManager::OnTabClosing(TabBeingClosed);

	// Is this a major tab that contained a Sub TabManager?
	// If so, need to properly close the sub tab manager
	const int32 TabManagerBeingClosedIndex = SubTabManagers.IndexOfByPredicate(FindByTab(TabBeingClosed));
	if (TabManagerBeingClosedIndex != INDEX_NONE)
	{
		const TSharedRef<FTabManager>& TabManagerBeingClosed = SubTabManagers[TabManagerBeingClosedIndex].TabManager.Pin().ToSharedRef();
		TabManagerBeingClosed->GetPrivateApi().OnTabManagerClosing();
	}
}

void FGlobalTabmanager::OnTabManagerClosing()
{
	for( int32 ManagerIndex=0; ManagerIndex < SubTabManagers.Num(); ++ManagerIndex )
	{
		TSharedPtr<SDockTab> SubManagerTab = SubTabManagers[ManagerIndex].MajorTab.Pin();
		if (SubManagerTab.IsValid())
		{
			SubManagerTab->RemoveTabFromParent();
		}
		
	}
}

void FGlobalTabmanager::UpdateStats()
{
	// Get all the tabs and windows in the global manager's own areas
	int32 AllTabsCount = 0;
	TArray<TSharedPtr<SWindow>> ParentWindows;

	GetRecordableStats(AllTabsCount, ParentWindows);

	// Add in all the tabs and windows in the sub-managers
	for (auto ManagerIter = SubTabManagers.CreateConstIterator(); ManagerIter; ++ManagerIter)
	{
		if (ManagerIter->TabManager.IsValid())
		{
			int32 TabsCount = 0;
			ManagerIter->TabManager.Pin()->GetRecordableStats(TabsCount, ParentWindows);

			AllTabsCount += TabsCount;
		}
	}

	// Keep a running maximum of the tab and window counts
	AllTabsMaxCount = FMath::Max(AllTabsMaxCount, AllTabsCount);
	AllAreasWindowMaxCount = FMath::Max(AllAreasWindowMaxCount, ParentWindows.Num());
}

void FGlobalTabmanager::OpenUnmanagedTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab)
{
	if ( ProxyTabManager.IsValid() && ProxyTabManager->IsTabSupported( UnmanagedTab->GetLayoutIdentifier() ) )
	{
		ProxyTabManager->OpenUnmanagedTab(PlaceholderId, SearchPreference, UnmanagedTab);
	}
	else
	{
		FTabManager::OpenUnmanagedTab(PlaceholderId, SearchPreference, UnmanagedTab);
	}
}

void FGlobalTabmanager::FinishRestore()
{
	for (FSubTabManager& SubManagerInfo : SubTabManagers)
	{
		if (TSharedPtr<FTabManager> Manager = SubManagerInfo.TabManager.Pin())
		{
			Manager->UpdateMainMenu(nullptr, false);

			if (TSharedPtr<SDockTab> Tab = SubManagerInfo.MajorTab.Pin())
			{
				/**
				 * Grab the primary area and restore its PanelDrawer.
				 * When restoring the via global manager they don't restore themselves right away
				 */
				if (TSharedPtr<SDockingArea> PrimaryDockingArea = Manager->GetDockingAreaForPanelDrawer({}))
				{
					if (!PrimaryDockingArea->IsPanelDrawerOpen())
					{
						PrimaryDockingArea->RestorePanelDrawerArea();
					}
				}
			}
		}
	}
}

void FGlobalTabmanager::SetCanSavePersistentLayouts(bool bInCanSavePersistentLayouts)
{
	bCanSavePersistentLayouts = bInCanSavePersistentLayouts;
}

bool FGlobalTabmanager::CanSavePersistentLayouts() const
{
	return bCanSavePersistentLayouts;
}

void FGlobalTabmanager::SetProxyTabManager(TSharedPtr<FProxyTabmanager> InProxyTabManager)
{
	ProxyTabManager = InProxyTabManager;
}

bool FProxyTabmanager::IsTabSupported( const FTabId TabId ) const
{
	bool bIsTabSupported = true;
	if( OnIsTabSupported.IsBound() )
	{
		OnIsTabSupported.Broadcast( TabId, /* In/Out */ bIsTabSupported );
	}

	return bIsTabSupported;
}

void FProxyTabmanager::OpenUnmanagedTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab)
{
	TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin();
	if (ensure(ParentWindowPtr.IsValid()))
	{
		const TSharedPtr<FArea> Area = FGlobalTabmanager::Get()->GetAreaFromInitialLayoutWithTabType(UnmanagedTab->GetLayoutIdentifier());
		const TSharedRef<FArea> NewAreaForTab =  Area.IsValid() ? Area.ToSharedRef() : NewPrimaryArea();

		NewAreaForTab
			->Split
			(
				FTabManager::NewStack()
				->AddTab( UnmanagedTab->GetLayoutIdentifier(), ETabState::OpenedTab )
			);

		if (TSharedPtr<SDockingArea> DockingArea = RestoreArea(NewAreaForTab, ParentWindowPtr))
		{
			ParentWindowPtr->SetContent(StaticCastSharedRef<SDockingArea>(DockingArea->AsShared()));
			if (DockingArea->GetAllChildTabs().Num() > 0)
			{
				const TSharedPtr<SDockTab> NewlyOpenedTab = DockingArea->GetAllChildTabs()[0];
				check(NewlyOpenedTab.IsValid());

				NewlyOpenedTab->GetParent()->GetParentDockTabStack()->OpenTab(UnmanagedTab);
				NewlyOpenedTab->RequestCloseTab();

				MainNonCloseableTabID = UnmanagedTab->GetLayoutIdentifier();

				OnTabOpened.Broadcast(UnmanagedTab);
			}
		}
	}
}

void FProxyTabmanager::DrawAttention(const TSharedRef<SDockTab>& TabToHighlight)
{
	FTabManager::DrawAttention(TabToHighlight);

	OnAttentionDrawnToTab.Broadcast(TabToHighlight);
}


void FProxyTabmanager::SetParentWindow(TSharedRef<SWindow> InParentWindow)
{
	ParentWindow = InParentWindow;
}

#undef LOCTEXT_NAMESPACE
