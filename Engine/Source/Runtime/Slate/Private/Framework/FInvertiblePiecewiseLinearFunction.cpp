// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/FInvertiblePiecewiseLinearFunction.h"

FInvertiblePointSlopeFunction::FInvertiblePointSlopeFunction(FVector2f Point, float Slope)
	: Point(Point)
	, Slope(Slope)
{}

float FInvertiblePointSlopeFunction::SolveForY(float X) const
{
	const float M = Slope;
	const float B = Point.Y - (M * Point.X);
	// y = m*x + b
	return M * X + B;
}

float FInvertiblePointSlopeFunction::SolveForX(float Y) const
{
	const float M = 1 / Slope;
	const float B = Point.X - (M * Point.Y);
	// x = m*y + b
	return M * Y + B;
}

FInvertiblePiecewiseLinearFunction::FInvertiblePiecewiseLinearFunction()
	: SubFunctions{{{0.f,0.f}, 1.f}}
{}

FInvertiblePiecewiseLinearFunction::FInvertiblePiecewiseLinearFunction(const TArray<FVector2f>& FixedPoints)
{
	if (FixedPoints.Num() <= 1)
	{
		SubFunctions.Emplace(FVector2f{0.f,0.f}, 1.f);
		return;
	}
	
	float PrevSlope = 0;
	for(int32 Index = 0; (Index + 1) < FixedPoints.Num(); ++Index)
	{
		const FVector2f& Begin = FixedPoints[Index]; 
		const FVector2f& End = FixedPoints[Index + 1];

		// duplicate points are allowed but ignored
		if(Begin == End)
		{
			continue;
		}
			
		// for invertibility, both x and y must be increasing
		check(Begin.X <= End.X && Begin.Y <= End.Y);
			
		const float Slope = (End.Y - Begin.Y) / (End.X - Begin.X);
			
		// Only add a subfunction if it changes the slope
		if (PrevSlope != Slope)
		{
			SubFunctions.Emplace(Begin, Slope);
		}
		PrevSlope = Slope;
	}
}

float FInvertiblePiecewiseLinearFunction::SolveForY(float X) const
{
	return FindSubFunctionAtX(X).SolveForY(X);
}

float FInvertiblePiecewiseLinearFunction::SolveForX(float Y) const
{
	return FindSubFunctionAtY(Y).SolveForX(Y);
}

const FInvertiblePiecewiseLinearFunction::SubFunction& FInvertiblePiecewiseLinearFunction::FindSubFunctionAtX(float X) const
{
	if(X < SubFunctions[0].Point.X)
	{
		return SubFunctions[0];
	}
		
	int32 Start = 0;
	int32 End = SubFunctions.Num() - 1;
		
	// Binary search for a subFunction with the correct bounds
	while (Start <= End)
	{
		const int Mid = (Start + End) / 2;
		// If K is found
		if (SubFunctions[Mid].Point.X == X)
		{
			return SubFunctions[Mid];
		}
		else if (SubFunctions[Mid].Point.X < X)
		{
			Start = Mid + 1;
		}
		else
		{
			End = Mid - 1;
		}
	}

	return SubFunctions[End];
}

const FInvertiblePiecewiseLinearFunction::SubFunction& FInvertiblePiecewiseLinearFunction::FindSubFunctionAtY(float Y) const
{
	if(Y < SubFunctions[0].Point.Y)
	{
		return SubFunctions[0];
	}
		
	int Start = 0;
	int End = SubFunctions.Num() - 1;
		
	// Binary search for a subFunction with the correct bounds
	while (Start <= End) {
		const int Mid = (Start + End) / 2;
		// If K is found
		if (SubFunctions[Mid].Point.Y == Y)
			return SubFunctions[Mid];
		else if (SubFunctions[Mid].Point.Y < Y)
			Start = Mid + 1;
		else
			End = Mid - 1;
	}
		
	return SubFunctions[End];
}
