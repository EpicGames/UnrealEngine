// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

struct FInvertiblePointSlopeFunction
{
	FInvertiblePointSlopeFunction(FVector2f Point, float Slope);

	float SolveForY(float X) const;
	float SolveForX(float Y) const;

	FVector2f Point;
	float Slope;
};

class FInvertiblePiecewiseLinearFunction
{
public:
	FInvertiblePiecewiseLinearFunction();

	// construct using a list of fixed points that will be interpolated between
	explicit FInvertiblePiecewiseLinearFunction(const TArray<FVector2f>& FixedPoints);

	float SolveForY(float X) const;
	float SolveForX(float Y) const;

private:
	using SubFunction = FInvertiblePointSlopeFunction;
	const SubFunction& FindSubFunctionAtX(float X) const;
	const SubFunction& FindSubFunctionAtY(float Y) const;

	TArray<SubFunction> SubFunctions;
};
