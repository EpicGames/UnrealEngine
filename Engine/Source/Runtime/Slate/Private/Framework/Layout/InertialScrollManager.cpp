// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Layout/InertialScrollManager.h"
#include "HAL/IConsoleManager.h"

float FrictionCoefficient = 2.0f;
FAutoConsoleVariableRef CVarFrictionCoefficient(
	TEXT("Slate.InertialScroll.FrictionCoefficient"),
	FrictionCoefficient,
	TEXT("This is the percentage of velocity loss per second."),
	ECVF_Default);

float StaticVelocityDrag = 100;
FAutoConsoleVariableRef CVarStaticVelocityDrag(
	TEXT("Slate.InertialScroll.StaticVelocityDrag"),
	StaticVelocityDrag,
	TEXT("This is a constant amount of velocity lost per second."),
	ECVF_Default);

FInertialScrollManager::FInertialScrollManager(double InSampleTimeout)
	: ScrollVelocity(0.f)
	, SampleTimeout(InSampleTimeout)
{
}

void FInertialScrollManager::AddScrollSample(float Delta, double CurrentTime)
{
	ScrollSamples.Emplace(Delta, CurrentTime);

	float Total = 0;
	double OldestTime = 0;
	for ( int32 VelIdx = ScrollSamples.Num() - 1; VelIdx >= 0; --VelIdx )
	{
		const double SampleTime = ScrollSamples[VelIdx].Time;
		const float SampleDelta = ScrollSamples[VelIdx].Delta;
		if ( CurrentTime - SampleTime > SampleTimeout )
		{
			ScrollSamples.RemoveAt(VelIdx);
		}
		else
		{
			if ( SampleTime < OldestTime || OldestTime == 0 )
			{
				OldestTime = SampleTime;
			}

			Total += SampleDelta;
		}
	}

	// Set the current velocity to the average of the previous recent samples
	const double Duration = (OldestTime > 0) ? (CurrentTime - OldestTime) : 0;
	if ( Duration > 0 )
	{
		ScrollVelocity = Total / Duration;
	}
	else
	{
		ScrollVelocity = 0;
	}
}	

void FInertialScrollManager::UpdateScrollVelocity(const float InDeltaTime)
{
	const float VelocityLostPerSecond = ScrollVelocity > 0 ? StaticVelocityDrag : -StaticVelocityDrag;
	const float DeltaVelocity = FrictionCoefficient * ScrollVelocity * InDeltaTime + VelocityLostPerSecond * InDeltaTime;

	if ( ScrollVelocity > 0 )
	{
		ScrollVelocity = FMath::Max<float>(0.f, ScrollVelocity - DeltaVelocity);
	}
	else if ( ScrollVelocity < 0 )
	{
		ScrollVelocity = FMath::Min<float>(0.f, ScrollVelocity - DeltaVelocity);
	}
}

void FInertialScrollManager::ClearScrollVelocity(bool bInShouldStopScrollNow)
{
	bShouldStopScrollNow = ScrollVelocity != 0 ? bInShouldStopScrollNow : false;
	ScrollVelocity = 0;
}
