// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Layout/Overscroll.h"

float FOverscroll::Looseness(50.0f);
float FOverscroll::OvershootLooseMax(100.0f);
float FOverscroll::OvershootBounceRate(1500.0f);

FOverscroll::FOverscroll()
	: OverscrollAmount(0.0f)
{
}

float FOverscroll::ScrollBy(const FGeometry& AllottedGeometry, float LocalDeltaScroll)
{
	const float ScreenDeltaScroll = LocalDeltaScroll / AllottedGeometry.Scale;

	const float ValueBeforeDeltaApplied = OverscrollAmount;
	OverscrollAmount += ScreenDeltaScroll;

	// Don't allow an interaction to change from positive <-> negative overscroll
	const bool bCrossedOverscrollBoundary = FMath::Sign(ValueBeforeDeltaApplied) != FMath::Sign(OverscrollAmount);
	if ( bCrossedOverscrollBoundary && ValueBeforeDeltaApplied != 0.f )
	{
		OverscrollAmount = 0.f;
	}

	return ValueBeforeDeltaApplied - OverscrollAmount;
}

float FOverscroll::GetOverscroll(const FGeometry& AllottedGeometry) const
{
	// We apply the resistance when observers attempt to read the overscroll.
	// First we calculate the origin shift for the looseness factor, by default,
	// straight up Log would be too tight, so we scale the result, the problem is that shifts
	// the inflection point of the log function, so we subtract the shift amount to correct it.
	const float OriginShift = FOverscroll::Looseness * FMath::Loge(FOverscroll::Looseness);

	// Scale the log of the current overscroll and shift the origin so that the overscroll the resistance is continuous.
	const float AbsElasticOverscroll = FOverscroll::Looseness * FMath::Loge(FMath::Abs(OverscrollAmount) + FOverscroll::Looseness) - OriginShift;

	const float AbsOverscroll = FMath::Sign(OverscrollAmount) * AbsElasticOverscroll;
	return AbsOverscroll * AllottedGeometry.Scale;
}

void FOverscroll::UpdateOverscroll(float InDeltaTime)
{
	const float PullForce = FMath::Abs(OverscrollAmount) + 1.0f;
	const float EasedDelta = FOverscroll::OvershootBounceRate * InDeltaTime * FMath::Max(1.0f, PullForce / FOverscroll::OvershootLooseMax);

	if ( OverscrollAmount > 0 )
	{
		OverscrollAmount = FMath::Max(0.0f, OverscrollAmount - EasedDelta);
	}
	else
	{
		OverscrollAmount = FMath::Min(0.0f, OverscrollAmount + EasedDelta);
	}
}

bool FOverscroll::ShouldApplyOverscroll(const bool bIsAtStartOfList, const bool bIsAtEndOfList, const float ScrollDelta) const
{
	const bool bShouldApplyOverscroll =
		// We can scroll past the edge of the list only if we are at the edge
		( bIsAtStartOfList && ScrollDelta < 0 ) || ( bIsAtEndOfList && ScrollDelta > 0 ) ||
		// ... or if we are already past the edge and are scrolling in the opposite direction.
		( OverscrollAmount > 0 && ScrollDelta < 0 ) || ( OverscrollAmount < 0 && ScrollDelta > 0 );

	return bShouldApplyOverscroll;
}

void FOverscroll::ResetOverscroll()
{
	OverscrollAmount = 0.0f;
}
