// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Layout/ScrollyZoomy.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWidget.h"
#include "Framework/Application/SlateApplication.h"


/* FScrollyZoomy structors
 *****************************************************************************/

FScrollyZoomy::FScrollyZoomy(const bool InUseInertialScrolling)
	: AmountScrolledWhileRightMouseDown(0.0f)
	, bShowSoftwareCursor(false)
	, SoftwareCursorPosition(FVector2f::ZeroVector)
	, UseInertialScrolling(InUseInertialScrolling)
{ }


/* FScrollyZoomy interface
 *****************************************************************************/

void FScrollyZoomy::Tick(const float DeltaTime, IScrollableZoomable& ScrollableZoomable)
{
	if (!IsRightClickScrolling())
	{
		FVector2f ScrollBy = FVector2f::ZeroVector;

		this->HorizontalIntertia.UpdateScrollVelocity(DeltaTime);
		const float HorizontalScrollVelocity = this->HorizontalIntertia.GetScrollVelocity();

		if (HorizontalScrollVelocity != 0.f)
		{
			ScrollBy.X = HorizontalScrollVelocity * DeltaTime;
		}

		this->VerticalIntertia.UpdateScrollVelocity(DeltaTime);
		const float VerticalScrollVelocity = this->VerticalIntertia.GetScrollVelocity();

		if (VerticalScrollVelocity != 0.f)
		{
			ScrollBy.Y = VerticalScrollVelocity * DeltaTime;
		}

		if (ScrollBy != FVector2f::ZeroVector)
		{
			ScrollableZoomable.ScrollBy(FVector2D(ScrollBy));
		}
	}
}


FReply FScrollyZoomy::OnMouseButtonDown(const FPointerEvent& MouseEvent)
{
	// @todo: Should we only clear on RMB down?  Also see other uses of this in the code base
	this->HorizontalIntertia.ClearScrollVelocity();
	this->VerticalIntertia.ClearScrollVelocity();

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		AmountScrolledWhileRightMouseDown = 0;

		// NOTE: We don't bother capturing the mouse, unless the user starts dragging a few pixels (see the
		// mouse move handling here.)  This is important so that the item row has a chance to select
		// items when the right mouse button is released.  Just keep in mind that you might not get
		// an OnMouseButtonUp event for the right mouse button if the user moves off of the table before
		// they reach our scroll threshold

		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FReply FScrollyZoomy::OnMouseButtonUp(const TSharedRef<SWidget> MyWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		AmountScrolledWhileRightMouseDown = 0;

		FReply Reply = FReply::Handled().ReleaseMouseCapture();
		bShowSoftwareCursor = false;

		// If we have mouse capture, snap the mouse back to the closest location that is within the panel's bounds
		if (MyWidget->HasMouseCapture())
		{
			FSlateRect PanelScreenSpaceRect = MyGeometry.GetLayoutBoundingRect();
			FVector2f CursorPosition = MyGeometry.LocalToAbsolute(SoftwareCursorPosition);

			FIntPoint BestPositionInPanel(
				FMath::RoundToInt(FMath::Clamp(CursorPosition.X, PanelScreenSpaceRect.Left, PanelScreenSpaceRect.Right)),
				FMath::RoundToInt(FMath::Clamp(CursorPosition.Y, PanelScreenSpaceRect.Top, PanelScreenSpaceRect.Bottom))
				);

			Reply.SetMousePos(BestPositionInPanel);
		}

		if (!UseInertialScrolling)
		{
			HorizontalIntertia.ClearScrollVelocity();
			VerticalIntertia.ClearScrollVelocity();
		}

		return Reply;
	}

	return FReply::Unhandled();
}


FReply FScrollyZoomy::OnMouseMove(const TSharedRef<SWidget> MyWidget, IScrollableZoomable& ScrollableZoomable, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		// If scrolling with the right mouse button, we need to remember how much we scrolled.
		// If we did not scroll at all, we will bring up the context menu when the mouse is released.
		AmountScrolledWhileRightMouseDown += FMath::Abs(MouseEvent.GetCursorDelta().X) + FMath::Abs(MouseEvent.GetCursorDelta().Y);

		// Has the mouse moved far enough with the right mouse button held down to start capturing
		// the mouse and dragging the view?
		if (IsRightClickScrolling())
		{
			this->HorizontalIntertia.AddScrollSample(MouseEvent.GetCursorDelta().X, FPlatformTime::Seconds());
			this->VerticalIntertia.AddScrollSample(MouseEvent.GetCursorDelta().Y, FPlatformTime::Seconds());
			const bool DidScroll = ScrollableZoomable.ScrollBy(MouseEvent.GetCursorDelta());

			FReply Reply = FReply::Handled();

			// Capture the mouse if we need to
			if (MyWidget->HasMouseCapture() == false)
			{
				Reply.CaptureMouse(MyWidget).UseHighPrecisionMouseMovement(MyWidget);
				SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
				bShowSoftwareCursor = true;
			}

			// Check if the mouse has moved.
			if (DidScroll)
			{
				SoftwareCursorPosition += MouseEvent.GetCursorDelta();
			}

			return Reply;
		}
	}

	return FReply::Unhandled();
}


void FScrollyZoomy::OnMouseLeave(const TSharedRef<SWidget> MyWidget, const FPointerEvent& MouseEvent)
{
	if (MyWidget->HasMouseCapture() == false)
	{
		// No longer scrolling (unless we have mouse capture)
		AmountScrolledWhileRightMouseDown = 0;
	}
}


FReply FScrollyZoomy::OnMouseWheel(const FPointerEvent& MouseEvent, IScrollableZoomable& ScrollableZoomable)
{
	// @todo: Inertial zoom support!
	const bool DidZoom = ScrollableZoomable.ZoomBy(MouseEvent.GetWheelDelta());

	if (DidZoom)
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FCursorReply FScrollyZoomy::OnCursorQuery() const
{
	if (IsRightClickScrolling())
	{
		// We hide the native cursor as we'll be drawing the software EMouseCursor::GrabHandClosed cursor
		return FCursorReply::Cursor(EMouseCursor::None);
	}

	return FCursorReply::Unhandled();
}


bool FScrollyZoomy::IsRightClickScrolling() const
{
	return (AmountScrolledWhileRightMouseDown >= FSlateApplication::Get().GetDragTriggerDistance());
}


bool FScrollyZoomy::NeedsSoftwareCursor() const
{
	return bShowSoftwareCursor;
}


UE::Slate::FDeprecateVector2DResult FScrollyZoomy::GetSoftwareCursorPosition() const
{
	return SoftwareCursorPosition;
}


int32 FScrollyZoomy::PaintSoftwareCursorIfNeeded(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (bShowSoftwareCursor)
	{
		const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_Grab"));

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(Brush->ImageSize, FSlateLayoutTransform(SoftwareCursorPosition - (Brush->ImageSize / 2))),
			Brush
		);
	}

	return LayerId;
}
