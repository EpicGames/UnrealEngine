// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MetaData/DriverMetaData.h"
#include "Framework/MetaData/DriverIdMetaData.h"

TSharedRef<ISlateMetaData> FDriverMetaData::Id(FName InTag)
{
	return MakeShareable(new FDriverIdMetaData(MoveTemp(InTag)));
}
