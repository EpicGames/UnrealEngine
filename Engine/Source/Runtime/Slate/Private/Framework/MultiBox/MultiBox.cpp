// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/MultiBox.h"

#include "SClippingVerticalBox.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/ToolMenuBase.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Framework/MultiBox/SMenuEntryBlock.h"
#include "Framework/MultiBox/SMenuSeparatorBlock.h"
#include "Framework/MultiBox/SMenuSpacerBlock.h"
#include "Framework/MultiBox/MultiBoxCustomization.h"
#include "Framework/MultiBox/SClippingHorizontalBox.h"
#include "Framework/MultiBox/SWidgetBlock.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
#include "HAL/PlatformMath.h"

#include "Framework/Commands/UICommandDragDropOp.h"
#include "Framework/MultiBox/SUniformToolbarPanel.h"
#include "Styling/ToolBarStyle.h"
#include "Misc/CoreMiscDefines.h"
#include "SPrioritizedWrapBox.h"

#define LOCTEXT_NAMESPACE "MultiBox"

namespace UE
{
namespace MultiBoxUtils
{

/**
 * Returns a path-like text containing the sub-menu(s) location of the block before it was pulled, flatten and added to its top-level multibox. Searchable
 * blocks are added to their parent(s) multibox to support recursive searching. When a user performs a search in a top-level menu, all sub-menu items are
 * also in the top menu (but hidden) and the matching ones are made visible. When one or more items of a given sub-menu match, a hierarchy tip appears
 * to let the user know from which sub-menu the item(s) came from.
 */
static FText GetSearchHierarchyInfoText(const TArray<FText>& SearchTextHierarchyComponents)
{
	FText HierarchyInfoText = FText::GetEmpty();
	if (!SearchTextHierarchyComponents.IsEmpty())
	{
		HierarchyInfoText = SearchTextHierarchyComponents[0];
		for (int32 Index = 1; Index < SearchTextHierarchyComponents.Num() - 1; ++Index)
		{
			HierarchyInfoText = FText::Format(INVTEXT("{0}/{1}"), HierarchyInfoText, SearchTextHierarchyComponents[Index]);
		}
	}

	return HierarchyInfoText;
}

/** Returns the searchable text displayed to the user for this search text hierarchy. This is the text normally displayed by the widget in its own menu. */
static FText GetBlockWidgetDisplayText(const TArray<FText>& SearchTextHierarchyComponents)
{
	return SearchTextHierarchyComponents.IsEmpty() ? FText::GetEmpty() : SearchTextHierarchyComponents.Last();
}

/** Joins the text components into a single string for debugging purpose. */
static FString ToString(const TArray<FText>& SearchTextHierarchyComponents)
{
	FString Pathname;
	if (!SearchTextHierarchyComponents.IsEmpty())
	{
		Pathname = SearchTextHierarchyComponents[0].ToString();
		for (int32 Index = 1; Index < SearchTextHierarchyComponents.Num(); ++Index)
		{
			Pathname.AppendChar(TEXT('/'));
			Pathname.Append(SearchTextHierarchyComponents[Index].ToString());
		}
	}

	return Pathname;
}

} // namespace MultiBoxUtils
} // namespace UE

TAttribute<bool> FMultiBoxSettings::UseSmallToolBarIcons;
TAttribute<bool> FMultiBoxSettings::DisplayMultiboxHooks;
FMultiBoxSettings::FConstructToolTip FMultiBoxSettings::ToolTipConstructor = FConstructToolTip::CreateStatic( &FMultiBoxSettings::ConstructDefaultToolTip );
TAttribute<int> FMultiBoxSettings::MenuSearchFieldVisibilityThreshold;

FMultiBoxSettings::FMultiBoxSettings()
{
	ResetToolTipConstructor();
}

TSharedRef< SToolTip > FMultiBoxSettings::ConstructDefaultToolTip( const TAttribute<FText>& ToolTipText, const TSharedPtr<SWidget>& OverrideContent, const TSharedPtr<const FUICommandInfo>& Action, bool ShowActionShortcut )
{
	struct Local
	{
		/** Appends the key binding to the end of the provided ToolTip */
		static FText AppendKeyBindingToToolTip(const TAttribute<FText> ToolTip, TSharedPtr< const FUICommandInfo> Command)
		{
			if (Command.IsValid() && (Command->GetFirstValidChord()->IsValidChord()))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ToolTipDescription"), ToolTip.Get());
				Args.Add(TEXT("Keybinding"), Command->GetInputText());
				return FText::Format(NSLOCTEXT("ToolBar", "ToolTip + Keybinding", "{ToolTipDescription} ({Keybinding})"), Args);
			}
			else
			{
				return ToolTip.Get();
			}
		}
	};

	if ( OverrideContent.IsValid() )
	{
		return SNew( SToolTip )
		[
			OverrideContent.ToSharedRef()
		];
	}

	TAttribute<FText> ActualToolTip = ToolTipText;
	if (Action.IsValid())
	{
		ActualToolTip = TAttribute< FText >::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::AppendKeyBindingToToolTip, ToolTipText, Action));
	}

	return SNew( SToolTip ).Text(ActualToolTip);
}

void FMultiBoxSettings::ResetToolTipConstructor()
{
	ToolTipConstructor = FConstructToolTip::CreateStatic( &FMultiBoxSettings::ConstructDefaultToolTip );
}

const FMultiBoxCustomization FMultiBoxCustomization::None( NAME_None );


TSharedRef< SWidget > SMultiBlockBaseWidget::AsWidget()
{
	return this->AsShared();
}

TSharedRef< const SWidget > SMultiBlockBaseWidget::AsWidget() const
{
	return this->AsShared();
}

void SMultiBlockBaseWidget::SetOwnerMultiBoxWidget(TSharedRef< SMultiBoxWidget > InOwnerMultiBoxWidget)
{
	OwnerMultiBoxWidget = InOwnerMultiBoxWidget;
}

void SMultiBlockBaseWidget::SetMultiBlock(TSharedRef<const FMultiBlock> InMultiBlock)
{
	MultiBlock = InMultiBlock;
}

void SMultiBlockBaseWidget::SetOptionsBlockWidget(TSharedPtr<SWidget> InOptionsBlock)
{
	OptionsBlockWidget = InOptionsBlock;
}


void SMultiBlockBaseWidget::SetMultiBlockLocation(EMultiBlockLocation::Type InLocation, bool bInSectionContainsIcons)
{
	Location = InLocation;
	bSectionContainsIcons = bInSectionContainsIcons;
}

EMultiBlockLocation::Type SMultiBlockBaseWidget::GetMultiBlockLocation()
{
	return Location;
}

EMultiBlockLocation::Type SMultiBlockBaseWidget::GetMultiBlockLocation() const
{
	return Location;
}

void SMultiBlockBaseWidget::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		OwnerMultiBoxWidget.Pin()->OnCustomCommandDragEnter( MultiBlock.ToSharedRef(), MyGeometry, DragDropEvent );
	}
}

FReply SMultiBlockBaseWidget::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		OwnerMultiBoxWidget.Pin()->OnCustomCommandDragged( MultiBlock.ToSharedRef(), MyGeometry, DragDropEvent );
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SMultiBlockBaseWidget::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		OwnerMultiBoxWidget.Pin()->OnCustomCommandDropped();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SMultiBlockBaseWidget::IsInEditMode() const
{
	if (OwnerMultiBoxWidget.IsValid())
	{
		return OwnerMultiBoxWidget.Pin()->GetMultiBox()->IsInEditMode();
	}

	return false;
}

/**
 * Creates a MultiBlock widget for this MultiBlock
 *
 * @param	InOwnerMultiBoxWidget	The widget that will own the new MultiBlock widget
 * @param	InLocation				The location information for the MultiBlock widget
 *
 * @return  MultiBlock widget object
 */
TSharedRef<IMultiBlockBaseWidget> FMultiBlock::MakeWidget(TSharedRef<SMultiBoxWidget> InOwnerMultiBoxWidget, EMultiBlockLocation::Type InLocation, bool bSectionContainsIcons, TSharedPtr<SWidget> OptionsBlockWidget) const
{
	TSharedRef<IMultiBlockBaseWidget> NewMultiBlockWidget = ConstructWidget();

	// Tell the widget about its parent MultiBox widget
	NewMultiBlockWidget->SetOwnerMultiBoxWidget( InOwnerMultiBoxWidget );

	// Assign ourselves to the MultiBlock widget
	NewMultiBlockWidget->SetMultiBlock(AsShared());

	NewMultiBlockWidget->SetOptionsBlockWidget(OptionsBlockWidget);

	// Pass location information to widget.
	NewMultiBlockWidget->SetMultiBlockLocation(InLocation, bSectionContainsIcons);

	// Work out what style the widget should be using
	const ISlateStyle* const StyleSet = InOwnerMultiBoxWidget->GetStyleSet();
	FName StyleName = (StyleNameOverride != NAME_None) ? StyleNameOverride : InOwnerMultiBoxWidget->GetStyleName();

	// Build up the widget
	NewMultiBlockWidget->BuildMultiBlockWidget(StyleSet, StyleName);

	return NewMultiBlockWidget;
}

void FMultiBlock::SetSearchable( bool InSearchable )
{
	bSearchable = InSearchable;
}

bool FMultiBlock::GetSearchable() const
{
	return bSearchable;
}

TAttribute<EVisibility> FMultiBlock::GetVisibilityOverride() const
{
	return VisibilityOverride;
}

void FMultiBlock::SetVisibilityOverride(TAttribute<EVisibility> InVisibilityOverride)
{
	VisibilityOverride = InVisibilityOverride;
}

const TAttribute<FMenuEntryResizeParams>& FMultiBlock::GetResizeParams() const
{
	return ResizeParams;
}

void FMultiBlock::SetResizeParams(const TAttribute<FMenuEntryResizeParams>& InResizeParams)
{
	ResizeParams = InResizeParams;
}

/**
 * Constructor
 *
 * @param	InType	Type of MultiBox
 * @param	bInShouldCloseWindowAfterMenuSelection	Sets whether or not the window that contains this multibox should be destroyed after the user clicks on a menu item in this box
 */
FMultiBox::FMultiBox(const EMultiBoxType InType, FMultiBoxCustomization InCustomization, const bool bInShouldCloseWindowAfterMenuSelection)
	: bHasSearchWidget(false)
	, bIsFocusable(true)
	, CommandLists()
	, Blocks()
	, StyleSet( &FCoreStyle::Get() )
	, StyleName( "ToolBar" )
	, Type( InType )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
	, LastSelectedCommandIndex( INDEX_NONE )
{

	if (InType == EMultiBoxType::SlimHorizontalToolBar ||  InType == EMultiBoxType::SlimHorizontalUniformToolBar)
	{
		StyleName = "SlimToolBar";
	}
}

FMultiBox::~FMultiBox()
{
	if (UToolMenuBase* ToolMenu = GetToolMenu())
	{
		ToolMenu->OnMenuDestroyed();
	}
}

TSharedRef<FMultiBox> FMultiBox::Create( const EMultiBoxType InType, FMultiBoxCustomization InCustomization, const bool bInShouldCloseWindowAfterMenuSelection )
{
	TSharedRef<FMultiBox> NewBox = MakeShareable( new FMultiBox( InType, InCustomization, bInShouldCloseWindowAfterMenuSelection ) );

	return NewBox;
}

/**
 * Adds a MultiBlock to this MultiBox, to the end of the list
 */
void FMultiBox::AddMultiBlock( TSharedRef< const FMultiBlock > InBlock )
{
	checkSlow( !Blocks.Contains( InBlock ) );

	if( InBlock->GetActionList().IsValid() )
	{
		CommandLists.AddUnique( InBlock->GetActionList() );
	}

	Blocks.Add( InBlock );
}

void FMultiBox::AddMultiBlockToFront(TSharedRef< const FMultiBlock > InBlock)
{
	checkSlow(!Blocks.Contains(InBlock));

	if (InBlock->GetActionList().IsValid())
	{
		CommandLists.AddUnique(InBlock->GetActionList());
	}

	Blocks.Insert(InBlock, 0);
}

void FMultiBox::RemoveCustomMultiBlock( TSharedRef< const FMultiBlock> InBlock )
{
	if( IsCustomizable() )
	{
		int32 Index = Blocks.Find( InBlock );

		// Remove the block from the visual list
		if( Index != INDEX_NONE )
		{
			Blocks.RemoveAt( Index );
		}

	}
}

void FMultiBox::InsertCustomMultiBlock( TSharedRef<const FMultiBlock> InBlock, int32 Index )
{
	if (IsCustomizable() && ensure(InBlock->GetExtensionHook() != NAME_None))
	{
		int32 ExistingIndex = Blocks.Find( InBlock );

		FName DestinationBlockName = NAME_None;
		FName DestinationSectionName = NAME_None;
		if (Blocks.IsValidIndex(Index))
		{
			DestinationBlockName = Blocks[Index]->GetExtensionHook();

			int32 DestinationSectionEndIndex = INDEX_NONE;
			int32 DestinationSectionIndex = GetSectionEditBounds(Index, DestinationSectionEndIndex);
			if (Blocks.IsValidIndex(DestinationSectionIndex))
			{
				DestinationSectionName = Blocks[DestinationSectionIndex]->GetExtensionHook();
			}
		}

		if (InBlock->IsPartOfHeading())
		{
			if (InBlock->GetExtensionHook() == DestinationSectionName)
			{
				return;
			}

			if (ExistingIndex != INDEX_NONE)
			{
				int32 SourceSectionEndIndex = INDEX_NONE;
				int32 SourceSectionIndex = GetSectionEditBounds(ExistingIndex, SourceSectionEndIndex);
				if (SourceSectionIndex != INDEX_NONE && SourceSectionEndIndex != INDEX_NONE)
				{
					bool bHadSeparator = Blocks[SourceSectionIndex]->IsSeparator();

					TArray< TSharedRef< const FMultiBlock > > BlocksToMove;
					BlocksToMove.Reset(SourceSectionEndIndex - SourceSectionIndex + 1);
					for (int32 BlockIdx = SourceSectionIndex; BlockIdx < SourceSectionEndIndex; ++BlockIdx)
					{
						BlocksToMove.Add(Blocks[BlockIdx]);
					}

					Blocks.RemoveAt(SourceSectionIndex, SourceSectionEndIndex - SourceSectionIndex, EAllowShrinking::No);

					if (Index > SourceSectionIndex)
					{
						Index -= BlocksToMove.Num();
					}

					if (Index == 0)
					{
						// Add missing separator for next section
						if (Blocks.Num() > 0 && (Blocks[0]->GetType() == EMultiBlockType::Heading))
						{
							BlocksToMove.Add(MakeShareable(new FMenuSeparatorBlock(Blocks[0]->GetExtensionHook(), /* bInIsPartOfHeading=*/ true)));
						}
					}
					else
					{
						// Add separator to beginning of section
						if (BlocksToMove.Num() > 0 && (BlocksToMove[0]->GetType() == EMultiBlockType::Heading))
						{
							BlocksToMove.Insert(MakeShareable(new FMenuSeparatorBlock(BlocksToMove[0]->GetExtensionHook(), /* bInIsPartOfHeading=*/ true)), 0);
						}
					}

					Blocks.Insert(BlocksToMove, Index);

					// Menus do not start with separators, remove separator if one exists
					if (Blocks.Num() > 0 && Blocks[0]->IsSeparator())
					{
						Blocks.RemoveAt(0, EAllowShrinking::No);
					}

					if (UToolMenuBase* ToolMenu = GetToolMenu())
					{
						ToolMenu->UpdateMenuCustomizationFromMultibox(SharedThis(this));
					}
				}
			}
		}
		else
		{
			if (ExistingIndex != INDEX_NONE)
			{
				Blocks.RemoveAt(ExistingIndex);
				if (ExistingIndex < Index)
				{
					--Index;
				}
			}

			Blocks.Insert(InBlock, Index);

			if (UToolMenuBase* ToolMenu = GetToolMenu())
			{
				ToolMenu->UpdateMenuCustomizationFromMultibox(SharedThis(this));
			}
		}
	}
}

/**
 * Creates a MultiBox widget for this MultiBox
 *
 * @return  MultiBox widget object
 */
TSharedRef< SMultiBoxWidget > FMultiBox::MakeWidget( bool bSearchable, FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride /* = nullptr */, TAttribute<float> InMaxHeight )
{	
	TSharedRef< SMultiBoxWidget > NewMultiBoxWidget =
		SNew( SMultiBoxWidget );

	// Set whether this box should be searched
	NewMultiBoxWidget->SetSearchable( bSearchable );

	// Assign ourselves to the MultiBox widget
	NewMultiBoxWidget->SetMultiBox( AsShared() );

	// Set the maximum height the MultiBox widget should be
	NewMultiBoxWidget->SetMaxHeight( InMaxHeight );

	if( (InMakeMultiBoxBuilderOverride != nullptr) && (InMakeMultiBoxBuilderOverride->IsBound()) )
	{
		TSharedRef<FMultiBox> ThisMultiBox = AsShared();
		InMakeMultiBoxBuilderOverride->Execute( ThisMultiBox, NewMultiBoxWidget );
	}
	else
	{
		// Build up the widget
		NewMultiBoxWidget->BuildMultiBoxWidget();
	}
	
	return NewMultiBoxWidget;
}

bool FMultiBox::IsCustomizable() const
{
	if (UToolMenuBase* ToolMenu = GetToolMenu())
	{
		return ToolMenu->IsEditing();
	}

	return false;
}

int32 FMultiBox::GetLastSelectedCommandIndex() const
{
	return LastSelectedCommandIndex;
}

void FMultiBox::SetLastSelectedCommandIndex(const int32 InLastSelectedCommandIndex)
{
	LastSelectedCommandIndex = InLastSelectedCommandIndex;
}

FName FMultiBox::GetCustomizationName() const
{
	return NAME_None;
}

TSharedPtr<FMultiBlock> FMultiBox::MakeMultiBlockFromCommand( TSharedPtr<const FUICommandInfo> CommandInfo, bool bCommandMustBeBound ) const
{
	TSharedPtr<FMultiBlock> NewBlock;

	// Find the command list that processes this command
	TSharedPtr<const FUICommandList> CommandList;

	for (int32 CommandListIndex = 0; CommandListIndex < CommandLists.Num(); ++CommandListIndex )
	{
		TSharedPtr<const FUICommandList> TestCommandList = CommandLists[CommandListIndex];
		if( TestCommandList->GetActionForCommand( CommandInfo.ToSharedRef() ) != NULL )
		{
			CommandList = TestCommandList;
			break;
		}
	}

	
	if( !bCommandMustBeBound && !CommandList.IsValid() && CommandLists.Num() > 0 )
	{
		// The first command list is the main command list and other are commandlists added from extension points
		// Use the main command list if one was not found
		CommandList = CommandLists[0];
	}

	if( !bCommandMustBeBound || CommandList.IsValid() )
	{
		// Only toolbars and menu buttons are supported currently
		switch ( Type )
		{
		case EMultiBoxType::ToolBar:
		case EMultiBoxType::UniformToolBar:
			{
				NewBlock = MakeShareable( new FToolBarButtonBlock( CommandInfo, CommandList ) );
			}
			break;
		case EMultiBoxType::Menu:
			{
				NewBlock = MakeShareable( new FMenuEntryBlock( NAME_None, CommandInfo, CommandList ) );
			}
			break;
		}
	}

	return NewBlock;

}

TSharedPtr<const FMultiBlock> FMultiBox::FindBlockFromNameAndType(const FName InName, const EMultiBlockType InType) const
{
	for (const auto& Block : Blocks)
	{
		if (Block->GetExtensionHook() == InName && Block->GetType() == InType)
		{
			return Block;
		}
	}

	return nullptr;
}

int32 FMultiBox::GetSectionEditBounds(const int32 Index, int32& OutSectionEndIndex) const
{
	// Only used by edit mode, identifies sections by heading blocks
	if (!IsInEditMode())
	{
		return INDEX_NONE;
	}

	int32 SectionBeginIndex = INDEX_NONE;
	for (int32 BlockIdx = Index; BlockIdx >= 0; --BlockIdx)
	{
		if (Blocks[BlockIdx]->GetType() == EMultiBlockType::Heading)
		{
			if (BlockIdx > 0 && Blocks[BlockIdx - 1]->IsSeparator() && Blocks[BlockIdx]->GetExtensionHook() == Blocks[BlockIdx - 1]->GetExtensionHook())
			{
				SectionBeginIndex = BlockIdx - 1;
			}
			else
			{
				SectionBeginIndex = BlockIdx;
			}
			break;
		}
	}

	OutSectionEndIndex = Blocks.Num();
	for (int32 BlockIdx = Index + 1; BlockIdx < Blocks.Num(); ++BlockIdx)
	{
		if (Blocks[BlockIdx]->GetType() == EMultiBlockType::Heading)
		{
			if (BlockIdx > 0 && Blocks[BlockIdx - 1]->IsSeparator() && Blocks[BlockIdx]->GetExtensionHook() == Blocks[BlockIdx - 1]->GetExtensionHook())
			{
				OutSectionEndIndex = BlockIdx - 1;
			}
			else
			{
				OutSectionEndIndex = BlockIdx;
			}
			break;
		}
	}

	return SectionBeginIndex;
}

UToolMenuBase* FMultiBox::GetToolMenu() const
{
	return WeakToolMenu.Get();
}

bool FMultiBox::IsInEditMode() const
{
	UToolMenuBase* ToolMenu = GetToolMenu();
	return ToolMenu && ToolMenu->IsEditing();
}

void SMultiBoxWidget::Construct( const FArguments& InArgs )
{
	LinkedBoxManager = MakeShared<FLinkedBoxManager>();
	SetContentScale(InArgs._ContentScale);
}

TSharedRef<ITableRow> SMultiBoxWidget::GenerateTiles(TSharedPtr<SWidget> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow< TSharedPtr<SWidget> >, OwnerTable)
		[
			Item.ToSharedRef()
		];
}

float SMultiBoxWidget::GetItemWidth() const
{
	float MaxItemWidth = 0;
	for (int32 i = 0; i < TileViewWidgets.Num(); ++i)
	{
		MaxItemWidth = FMath::Max(TileViewWidgets[i]->GetDesiredSize().X, MaxItemWidth);
	}
	return MaxItemWidth;
}

float SMultiBoxWidget::GetItemHeight() const
{
	float MaxItemHeight = 0;
	for (int32 i = 0; i < TileViewWidgets.Num(); ++i)
	{
		MaxItemHeight = FMath::Max(TileViewWidgets[i]->GetDesiredSize().Y, MaxItemHeight);
	}
	return MaxItemHeight;
}

bool SMultiBoxWidget::IsBlockBeingDragged( TSharedPtr<const FMultiBlock> Block ) const
{
	if( DragPreview.PreviewBlock.IsValid() )
	{
		return DragPreview.PreviewBlock->GetActualBlock() == Block;
	}

	return false;
}

EVisibility SMultiBoxWidget::GetCustomizationBorderDragVisibility(const FName InBlockName, const EMultiBlockType InBlockType, bool& bOutInsertAfter) const
{
	bOutInsertAfter = false;

	if (DragPreview.PreviewBlock.IsValid())
	{
		const TArray< TSharedRef< const FMultiBlock > >& Blocks = MultiBox->GetBlocks();
		if (Blocks.IsValidIndex(DragPreview.InsertIndex))
		{
			if (InBlockName != NAME_None)
			{
				const TSharedRef< const FMultiBlock >& DropDestination = Blocks[DragPreview.InsertIndex];
				if (DropDestination->GetExtensionHook() == InBlockName && DropDestination->GetType() == InBlockType)
				{
					return EVisibility::Visible;
				}
			}
		}
		else if (Blocks.Num() == DragPreview.InsertIndex)
		{
			if (Blocks.Num() > 0 && Blocks.Last()->GetExtensionHook() == InBlockName && Blocks.Last()->GetType() == InBlockType)
			{
				bOutInsertAfter = true;
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

void SMultiBoxWidget::AddBlockWidget(const FMultiBlock& Block, TSharedPtr<SHorizontalBox> HorizontalBox, TSharedPtr<SVerticalBox> VerticalBox, TSharedPtr<SPanel> Panel, EMultiBlockLocation::Type InLocation, bool bSectionContainsIcons, TSharedPtr<const FToolBarComboButtonBlock> OptionsBlock)
{
	check( MultiBox.IsValid() );

	// Skip Separators for Uniform Tool Bars
	if ( Block.IsSeparator()  && (MultiBox->GetType() == EMultiBoxType::UniformToolBar || MultiBox->GetType() == EMultiBoxType::SlimHorizontalUniformToolBar))
	{
		return;
	}

	bool bDisplayExtensionHooks = FMultiBoxSettings::DisplayMultiboxHooks.Get() && Block.GetExtensionHook() != NAME_None;

	TSharedPtr<SWidget> OptionsWidget;
	if (OptionsBlock)
	{
		OptionsWidget = OptionsBlock->MakeWidget(SharedThis(this), EMultiBlockLocation::None, bSectionContainsIcons, nullptr)->AsWidget();
	}

	TSharedRef<SWidget> BlockWidget = Block.MakeWidget(SharedThis(this), InLocation, bSectionContainsIcons, OptionsWidget)->AsWidget();

	if (FName TutorialHighlightName = Block.GetTutorialHighlightName(); !TutorialHighlightName.IsNone())
	{
		BlockWidget->AddMetadata(MakeShared<FTagMetaData>(TutorialHighlightName));
	}

	// If the block being added is one of the searchable flatten blocks from one of the sub-menus. (Second pass when building the multibox widget)
	TSharedRef<const FMultiBlock> BlockRef = Block.AsShared();
	if (TSharedPtr<FFlattenSearchableBlockInfo>* FlattenBlockInfo = FlattenSearchableBlocks.Find(BlockRef))
	{
		// Wrap the block widget to display its ancestor menu as a tip to the user.
		TSharedRef<SWidget> FlattenWidget = SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.Padding(FMargin(5, 0))
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSPLambda(this, [this, BlockRef]()
				{
					if (const TSharedPtr<FFlattenSearchableBlockInfo>* FoundBlockInfo = FlattenSearchableBlocks.Find(BlockRef))
					{
						return (*FoundBlockInfo)->HierarchyTipVisibility;
					}
					return EVisibility::Collapsed;
				})))
				[
					SNew(STextBlock)
					.Text(UE::MultiBoxUtils::GetSearchHierarchyInfoText((*FlattenBlockInfo)->SearchableTextHierarchyComponents))
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				BlockWidget
			];

		(*FlattenBlockInfo)->Widget = FlattenWidget;

		// Remember that this widget was pulled from a sub-menus into this menu and therefore has a special status.
		FlattenSearchableWidgets.Emplace(BlockWidget, *FlattenBlockInfo);

		// This widgets will only be visible if it matches a search result.
		FlattenWidget->SetVisibility(EVisibility::Collapsed);

		// The widget corresponding to the block was created above and stored along with its searchable text but the text doesn't contain the ancestors texts.
		if (TArray<FText>* SearchableTextHierarchy = MultiBoxWidgets.Find(BlockWidget))
		{
			// Update the widget searchable texts hierarchy to include the ancestors.
			*SearchableTextHierarchy = (*FlattenBlockInfo)->SearchableTextHierarchyComponents;
		}

		BlockWidget = FlattenWidget;
	}

	const ISlateStyle* const StyleSet = MultiBox->GetStyleSet();

	TSharedPtr<SWidget> FinalWidget;

	if (MultiBox->ModifyBlockWidgetAfterMake.IsBound())
	{
		FinalWidget = MultiBox->ModifyBlockWidgetAfterMake.Execute(SharedThis(this), Block, BlockWidget);
	}
	else
	{
		FinalWidget = BlockWidget;
	}

	TSharedRef<SWidget> FinalWidgetWithHook = SNullWidget::NullWidget;

	if (bDisplayExtensionHooks)
	{
		FinalWidgetWithHook =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Visibility(bDisplayExtensionHooks ? EVisibility::Visible : EVisibility::Collapsed)
				.ColorAndOpacity(StyleSet->GetColor("MultiboxHookColor"))
				.Text(FText::FromName(Block.GetExtensionHook()))
			]
			+ SVerticalBox::Slot()
			[
				FinalWidget.ToSharedRef()
			];
	}
	else
	{
		FinalWidgetWithHook = FinalWidget.ToSharedRef();
	}

	switch (MultiBox->GetType())
	{
	case EMultiBoxType::MenuBar:
	case EMultiBoxType::ToolBar:
	case EMultiBoxType::SlimHorizontalToolBar:
		{
			FMenuEntryStyleParams StyleParams;
			const bool bOverride = Block.GetAlignmentOverrides(StyleParams);

			{
				SHorizontalBox::FScopedWidgetSlotArguments NewSlot = HorizontalBox->AddSlot();

				NewSlot.Padding(0.f)
				[
					FinalWidgetWithHook
				];

				FinalMultiBoxWidgets.Add(FinalWidgetWithHook.ToSharedPtr(), BlockRef.ToSharedPtr());

				NewSlot.AutoWidth();

				if (bOverride)
				{
					NewSlot
					.HAlign(StyleParams.HorizontalAlignment)
					.VAlign(StyleParams.VerticalAlignment.Get(VAlign_Fill));

					if (StyleParams.SizeRule.IsSet())
					{
						FSizeParam::ESizeRule SizeRule = StyleParams.SizeRule.GetValue();
						if (SizeRule == FSizeParam::SizeRule_Stretch)
						{
							NewSlot.FillWidth(StyleParams.FillSize.Get(1.0f));
						}
						else if (SizeRule == FSizeParam::SizeRule_StretchContent)
						{
							NewSlot.FillContentWidth(
								StyleParams.FillSize.Get(1.0f),
								StyleParams.FillSizeMin.Get(StyleParams.FillSize.Get(1.0f)));
						}
					}

					if (StyleParams.MinimumSize.IsSet() || StyleParams.MinimumSize.IsBound())
					{
						NewSlot.MinWidth(StyleParams.MinimumSize);
					}

					if (StyleParams.MaximumSize.IsSet() || StyleParams.MaximumSize.IsBound())
					{
						NewSlot.MaxWidth(StyleParams.MaximumSize);
					}
				}
			}
		}
		break;
	case EMultiBoxType::SlimWrappingToolBar:
		{
			FMenuEntryStyleParams StyleParams;
			const bool bOverride = Block.GetAlignmentOverrides(StyleParams);

			{
				static const FName PrioritizedWrapBoxTypeName("SPrioritizedWrapBox");
			
				// SPrioritizedWrapBox isn't public, so is stored as an SPanel in the header. We need to check it before casting
				if (Panel->GetType() != PrioritizedWrapBoxTypeName)
				{
					break;
				}

				TSharedPtr<SPrioritizedWrapBox> PrioritizedWrapBox = StaticCastSharedPtr<SPrioritizedWrapBox>(Panel);
				if (!PrioritizedWrapBox.IsValid())
				{
					break;
				}

				const FMenuEntryResizeParams ResizeParams = Block.GetResizeParams().Get();

				TAttribute<bool> AllowWrapping = ResizeParams.Wrapping.Allow;
				if (!AllowWrapping.IsSet()
					&& StyleSet->HasWidgetStyle<FToolBarStyle>(MultiBox->GetStyleName()))
				{
					const FToolBarStyle& Style = StyleSet->GetWidgetStyle<FToolBarStyle>(MultiBox->GetStyleName());
					AllowWrapping = Style.bAllowWrappingDefault;
				}

				SPrioritizedWrapBox::FScopedWidgetSlotArguments NewSlot = PrioritizedWrapBox->AddSlot();
				NewSlot.AllowWrapping(AllowWrapping);
				NewSlot.WrapPriority(ResizeParams.Wrapping.Priority.IsSet() ? ResizeParams.Wrapping.Priority : FMenuEntryResizeParams::FWrappingParams::DefaultPriority);
				NewSlot.WrapMode(ResizeParams.Wrapping.Mode.IsSet() ? ResizeParams.Wrapping.Mode : FMenuEntryResizeParams::FWrappingParams::DefaultMode);
				NewSlot.VerticalOverflowBehavior(ResizeParams.Wrapping.VerticalOverflowBehavior.Get(FMenuEntryResizeParams::FWrappingParams::DefaultVerticalOverflowBehavior));
				NewSlot.ForceNewLine(ResizeParams.Wrapping.ForceNewLine.Get(FMenuEntryResizeParams::FWrappingParams::DefaultForceNewLine));
				NewSlot.ExcludeIfFirstOrLast(Block.IsSeparator());

				NewSlot.Padding(0.f)
				[
					FinalWidgetWithHook
				];

				FinalMultiBoxWidgets.Add(FinalWidgetWithHook.ToSharedPtr(), BlockRef.ToSharedPtr());

				NewSlot.SizeParam(FAuto());

				if (bOverride)
				{
					NewSlot
					.HAlign(StyleParams.HorizontalAlignment)
					.VAlign(StyleParams.VerticalAlignment.Get(VAlign_Fill));

					if (StyleParams.SizeRule.IsSet())
					{
						FSizeParam::ESizeRule SizeRule = StyleParams.SizeRule.GetValue();
						if (SizeRule == FSizeParam::SizeRule_Stretch)
						{
							NewSlot.SizeParam(FStretch(StyleParams.FillSize.Get(1.0f)));
						}
						else if (SizeRule == FSizeParam::SizeRule_StretchContent)
						{
							NewSlot.SizeParam(FStretchContent(
								StyleParams.FillSize.Get(1.0f),
								StyleParams.FillSizeMin.Get(StyleParams.FillSize.Get(1.0f))));
						}
					}

					if (StyleParams.MinimumSize.IsSet() || StyleParams.MinimumSize.IsBound())
					{
						NewSlot.MinSize(StyleParams.MinimumSize);
					}

					if (StyleParams.MaximumSize.IsSet() || StyleParams.MaximumSize.IsBound())
					{
						NewSlot.MaxSize(StyleParams.MaximumSize);
					}
				}
			}
		}
		break;
	case EMultiBoxType::VerticalToolBar:
		{
			if (UniformToolbarPanel.IsValid())
			{
				UniformToolbarPanel->AddSlot()
				[
					FinalWidgetWithHook
				];
			}
			else
			{
				VerticalBox->AddSlot()
				.AutoHeight()
				[
					FinalWidgetWithHook
				];
			}

			FinalMultiBoxWidgets.Add(FinalWidgetWithHook.ToSharedPtr(), BlockRef.ToSharedPtr());
		}
		break;
	case EMultiBoxType::UniformToolBar:
	case EMultiBoxType::SlimHorizontalUniformToolBar:
		{
			UniformToolbarPanel->AddSlot()
			[
				FinalWidgetWithHook
			];

			FinalMultiBoxWidgets.Add(FinalWidgetWithHook.ToSharedPtr(), BlockRef.ToSharedPtr());
		}
		break;
	case EMultiBoxType::ButtonRow:
		{
			TileViewWidgets.Add( FinalWidget.ToSharedRef() );

			FinalMultiBoxWidgets.Add(FinalWidget, BlockRef.ToSharedPtr());
		}
		break;
	case EMultiBoxType::Menu:
		{
			VerticalBox->AddSlot()
			.AutoHeight()
			.Padding( 0.0f, 0.0f, 0.0f, 0.0f )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Visibility(bDisplayExtensionHooks ? EVisibility::Visible : EVisibility::Collapsed)
					.ColorAndOpacity(StyleSet->GetColor("MultiboxHookColor"))
					.Text(FText::FromName(Block.GetExtensionHook()))
				]
				+SHorizontalBox::Slot()
				[
					FinalWidget.ToSharedRef()
				]
			];

			FinalMultiBoxWidgets.Add(FinalWidget, BlockRef.ToSharedPtr());
		}
		break;
	}
}

void SMultiBoxWidget::SetSearchable(bool InSearchable)
{
	bSearchable = InSearchable;
}

bool SMultiBoxWidget::GetSearchable() const
{
	return bSearchable;
}

/** Creates the SearchTextWidget if the MultiBox has requested one */
void SMultiBoxWidget::CreateSearchTextWidget()
{
	if (!MultiBox->bHasSearchWidget || !bSearchable)
	{
		return;
	}

	SearchTextWidget =
		SNew(SSearchBox)
			.HintText(LOCTEXT("SearchHintStartTyping", "Start typing to search"))
			.SelectAllTextWhenFocused(false)
			.OnTextChanged(this, &SMultiBoxWidget::OnFilterTextChanged)
			.OnTextCommitted(this, &SMultiBoxWidget::OnFilterTextCommitted);

	TSharedRef<SBox> SearchBox =
		SNew(SBox)
			.Padding(FMargin(8, 0, 8, 0))
			.Visibility(this, &SMultiBoxWidget::GetSearchBoxVisibility)
			[
				SearchTextWidget.ToSharedRef()
			];

	FMenuEntryStyleParams StyleParams;
	StyleParams.bNoIndent = false;

	TSharedRef<FWidgetBlock> NewWidgetBlock = MakeShared<FWidgetBlock>(SearchBox, FText::GetEmpty(), FText(), StyleParams);
	NewWidgetBlock->SetSearchable(false);

	MultiBox->AddMultiBlockToFront(NewWidgetBlock);
}

EVisibility SMultiBoxWidget::GetSearchBoxVisibility() const
{
	const bool bShow = !SearchText.IsEmpty() || ShouldShowMenuSearchField();
	return bShow ? EVisibility::Visible : EVisibility::Collapsed;
}

/** Called when the SearchText changes */
void SMultiBoxWidget::OnFilterTextChanged(const FText& InFilterText)
{
	const bool bInitialSearch = SearchText.IsEmpty();
	// Set SearchText here, so the SBox wrapping our SearchTextWidget will be visible when we attempt to focus the
	// SearchTextWidget. Otherwise, setting focus will fail because the parent is invisible and therefore the
	// SearchTextWidget is too, and Slate refuses to focus invisible widgets.
	SearchText = InFilterText;

	// Activate the searchbox if it was empty and we are putting text in it for the first time.
	// This is for IME keyboards only because they don't go through the OnKeyChar route.
	if (bSearchable && bInitialSearch && !InFilterText.IsEmpty()
		&& !FSlateApplication::Get().HasUserFocusedDescendants(SearchTextWidget.ToSharedRef(), 0))
	{
		if (SearchTextWidget.IsValid() && SearchBlockWidget.IsValid())
		{
			// Make the search box visible and focused.
			SearchBlockWidget->SetVisibility(EVisibility::Visible);
			FSlateApplication::Get().SetUserFocus(0, SearchTextWidget);
		}
	}

	FilterMultiBoxEntries();
}

void SMultiBoxWidget::OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::Type::OnCleared)
	{
		SearchText = InFilterText;
		FilterMultiBoxEntries();
	}
}

/**
 * Builds this MultiBox widget up from the MultiBox associated with it
 */
void SMultiBoxWidget::BuildMultiBoxWidget()
{
	check( MultiBox.IsValid() );

	// Grab the list of blocks, early out if there's nothing to fill the widget with
	const TArray< TSharedRef< const FMultiBlock > >& Blocks = MultiBox->GetBlocks();
	if ( Blocks.Num() == 0 )
	{
		// Clear content if there was any
		if (ChildSlot.Num() > 0 && ChildSlot.GetWidget() != SNullWidget::NullWidget)
		{
			ChildSlot
			[
				SNullWidget::NullWidget
			];
		}

		return;
	}

	CreateSearchTextWidget();

	// Select background brush based on the type of multibox.
	const ISlateStyle* const StyleSet = MultiBox->GetStyleSet();
	const FName& StyleName = MultiBox->GetStyleName();

	const FSlateBrush* BackgroundBrush = nullptr;
	FMargin BackgroundPadding;
	if (StyleSet->HasWidgetStyle<FToolBarStyle>(StyleName))
	{
		const FToolBarStyle& Style = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

		BackgroundBrush = &Style.BackgroundBrush;
		BackgroundPadding = Style.BackgroundPadding;
	}
	else
	{
		BackgroundBrush = StyleSet->GetOptionalBrush(StyleName, ".Background", FStyleDefaults::GetNoBrush());
	}

	// Create a box panel that the various multiblocks will resides within
	// @todo Slate MultiBox: Expose margins and other useful bits
	TSharedPtr<SVerticalBox> VerticalBox;
	TSharedPtr<SWidget> MainWidget;
	TSharedPtr<SHorizontalBox> HorizontalBox;
	TSharedPtr<SPanel> Panel;

	/** The current row of buttons for if the multibox type is a button row */
	TSharedPtr<SHorizontalBox> ButtonRow;

	TSharedPtr< STileView< TSharedPtr<SWidget> > > TileView;

	switch (MultiBox->GetType())
	{
	case EMultiBoxType::MenuBar:
		MainWidget = HorizontalBox = SNew(SHorizontalBox);
		break;
	case EMultiBoxType::ToolBar:
	case EMultiBoxType::SlimHorizontalToolBar:
		{
			MainWidget = HorizontalBox = ClippedHorizontalBox =
				SNew(SClippingHorizontalBox)
					.AllowWrapButton(MultiBox->bAllowWrapButton)
					.OnWrapButtonClicked(FOnGetContent::CreateSP(this, &SMultiBoxWidget::OnWrapButtonClicked))
					.OnGetWidgetResizeParams(FOnGetWidgetResizeParams::CreateSPLambda(this,
						[this](const TSharedRef<SWidget>& InWidget) -> FMenuEntryResizeParams
						{
							for (const TPair<TSharedPtr<SWidget>, TSharedPtr<const FMultiBlock>>& Pair : FinalMultiBoxWidgets)
							{
								if (InWidget == Pair.Key)
								{
									return Pair.Value->GetResizeParams().Get();
								}
							}

							return FMenuEntryResizeParams();
						}
					))
					.IsFocusable(MultiBox->bIsFocusable)
					.StyleSet(StyleSet)
					.StyleName(StyleName);
		}
		break;
	case EMultiBoxType::SlimWrappingToolBar:
		{
			const FToolBarStyle& Style = StyleSet->GetWidgetStyle<FToolBarStyle>(MultiBox->GetStyleName());
			const float LineHeight = Style.IconSize.Y + Style.ButtonStyle.NormalPadding.Top + Style.ButtonStyle.NormalPadding.Bottom; 
			MainWidget = Panel = SNew(SPrioritizedWrapBox)
				.MinLineHeight(LineHeight)
				.LinePadding(Style.BackgroundPadding.Bottom + Style.BackgroundPadding.Top)
				.GroupedWrapping(true);
			break;
		}
	case EMultiBoxType::VerticalToolBar:
		{
			MainWidget = VerticalBox = ClippedVerticalBox = SNew(SClippingVerticalBox)
				.OnWrapButtonClicked(FOnGetContent::CreateSP(this, &SMultiBoxWidget::OnWrapButtonClicked))
				.IsFocusable(MultiBox->bIsFocusable)
				.SelectedIndex(TAttribute<int32>::Create(TAttribute<int32>::FGetter::CreateSPLambda(this, [this]()
				{
					return MultiBox->GetLastSelectedCommandIndex();
				})))
				.StyleSet(StyleSet)
				.StyleName(StyleName);
		}
		break;
	case EMultiBoxType::UniformToolBar:
		{
			MainWidget = VerticalBox = SNew (SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(FMargin(0.f, 2.f))
			.AutoHeight()
			[
				SAssignNew(UniformToolbarPanel, SUniformWrapPanel)
				.HAlign(HAlign_Left)
				.MinDesiredSlotWidth(50.f)
				.MinDesiredSlotHeight(43.f)
				.MaxDesiredSlotWidth(50.f)
				.MaxDesiredSlotHeight(43.f)
				.SlotPadding(FMargin(2.f, 1.f))
			];
		}
		break;
		// @TODO: ~Move hardcodes into styling file
	case EMultiBoxType::SlimHorizontalUniformToolBar:
		{
			const FToolBarStyle& Style = StyleSet->GetWidgetStyle<FToolBarStyle>(MultiBox->GetStyleName());
			
			MainWidget = VerticalBox = SNew (SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(0.f)
			.AutoHeight()
			[
				SAssignNew(UniformToolbarPanel, SUniformWrapPanel)
				.HAlign(HAlign_Fill)
				.MinDesiredSlotWidth(Style.UniformBlockWidth)
				.MinDesiredSlotHeight(Style.UniformBlockHeight)
				.MaxDesiredSlotWidth(Style.UniformBlockWidth)
				.MaxDesiredSlotHeight(Style.UniformBlockHeight)
				.NumColumnsOverride(Style.NumColumns)
				.SlotPadding(0.f)	
			];
		}
		break;
	case EMultiBoxType::ButtonRow:
		{
			MainWidget = TileView = SNew(STileView< TSharedPtr<SWidget> >)
				.OnGenerateTile(this, &SMultiBoxWidget::GenerateTiles)
				.ListItemsSource(&TileViewWidgets)
				.ItemWidth(this, &SMultiBoxWidget::GetItemWidth)
				.ItemHeight(this, &SMultiBoxWidget::GetItemHeight)
				.SelectionMode(ESelectionMode::None);
		}
		break;
	case EMultiBoxType::Menu:
		{
			if (MaxHeight.IsSet())
			{
				MainWidget = SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.MaxHeight(MaxHeight)
					[
						// wrap menu content in a scrollbox to support vertical scrolling if needed
						SNew(SScrollBox)
						+ SScrollBox::Slot()
						[
							SAssignNew(VerticalBox, SVerticalBox)
						]
					];
			}
			else
			{
				// wrap menu content in a scrollbox to support vertical scrolling if needed
				MainWidget = SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						SAssignNew(VerticalBox, SVerticalBox)
					];
			}
		}
		break;
	}
	
	bool bInsideGroup = false;

	// Start building up the actual UI from each block in this MultiBox
	bool bSectionContainsIcons = false;
	int32 NextMenuSeparator = INDEX_NONE;

	int32 ConsecutiveSeparatorCount = 0;

	for( int32 Index = 0; Index < Blocks.Num(); Index++ )
	{
		// If we've passed the last menu separator, scan for the next one (the end of the list is also considered a menu separator for the purposes of this index)
		if (NextMenuSeparator < Index)
		{
			bSectionContainsIcons = false;
			for (++NextMenuSeparator; NextMenuSeparator < Blocks.Num(); ++NextMenuSeparator)
			{
				const FMultiBlock& TestBlock = *Blocks[NextMenuSeparator];
				if (!bSectionContainsIcons && TestBlock.HasIcon())
				{
					bSectionContainsIcons = true;
				}

				if (TestBlock.IsSeparator())
				{
					break;
				}
			}
		}

		const FMultiBlock& Block = *Blocks[Index];
		EMultiBlockLocation::Type Location = EMultiBlockLocation::None;
	
		if (Block.IsSeparator())
		{
			++ConsecutiveSeparatorCount;
			// Skip consecutive separators. Only draw one separator no matter how many are submitted.
			if (ConsecutiveSeparatorCount > 1)
			{
				continue;
			}
		}
		else
		{
			ConsecutiveSeparatorCount = 0;
		}

		TSharedPtr<const FMultiBlock> NextBlock = nullptr;

		if (Blocks.IsValidIndex(Index + 1))
		{
			NextBlock = Blocks[Index + 1];
		}

		// Determine the location of the current block, used for group styling information
		{
			// Check if we are a start or end block
			if (Block.IsGroupStartBlock())
			{
				bInsideGroup = true;
			}
			else if (Block.IsGroupEndBlock())
			{
				bInsideGroup = false;
			}

			// Check if we are next to a start or end block
			bool bIsNextToStartBlock = false;
			bool bIsNextToEndBlock = false;
			if (NextBlock)
			{
				if ( NextBlock->IsGroupEndBlock() )
				{
					bIsNextToEndBlock = true;
				}
			}
			if (Index > 0)
			{
				const FMultiBlock& PrevBlock = *Blocks[Index - 1];
				if ( PrevBlock.IsGroupStartBlock() )
				{
					bIsNextToStartBlock = true;
				}
			}

			// determine location
			if (bInsideGroup)
			{
				// assume we are in the middle of a group
				Location = EMultiBlockLocation::Middle;

				// We are the start of a group
				if (bIsNextToStartBlock && !bIsNextToEndBlock)
				{
					Location = EMultiBlockLocation::Start;
				}
				// we are the end of a group
				else if (!bIsNextToStartBlock && bIsNextToEndBlock)
				{
					Location = EMultiBlockLocation::End;
				}
				// we are the only block in a group
				else if (bIsNextToStartBlock && bIsNextToEndBlock)
				{
					Location = EMultiBlockLocation::None;
				}
			}
		}

		TSharedPtr<const FToolBarComboButtonBlock> OptionsBlock;
		if (NextBlock && NextBlock->GetType() == EMultiBlockType::ToolBarComboButton)
		{
			TSharedPtr<const FToolBarComboButtonBlock> NextToolBarComboButtonBlock = StaticCastSharedPtr<const FToolBarComboButtonBlock>(NextBlock);
			if (NextToolBarComboButtonBlock->IsSimpleComboBox())
			{
				// Apply a special treatment to simple combo boxes as they represent options for the previous button
				OptionsBlock = NextToolBarComboButtonBlock;
				// Skip over options blocks. They are not added directly
				++Index;
			}
		}

		if( DragPreview.IsValid() && DragPreview.InsertIndex == Index )
		{
			// Add the drag preview before if we have it. This block shows where the custom block will be 
			// added if the user drops it
			AddBlockWidget(*DragPreview.PreviewBlock, HorizontalBox, VerticalBox, Panel, EMultiBlockLocation::None, bSectionContainsIcons, OptionsBlock);
		}
		
		// Do not add a block if it is being dragged
		if( !IsBlockBeingDragged( Blocks[Index] ) )
		{
			AddBlockWidget(Block, HorizontalBox, VerticalBox, Panel, Location, bSectionContainsIcons, OptionsBlock);
		}

		const bool bIsSearchWidgetBlock = Index == 0 && MultiBox->bHasSearchWidget;
		if (bIsSearchWidgetBlock && NextBlock.IsValid())
		{
			const bool bNextBlockIsSeparatorLike = NextBlock->IsSeparator() || NextBlock->IsGroupStartBlock() || NextBlock->IsPartOfHeading();

			// If the search widget is the first block and the next block isn't a separator or a group/section start, we should add padding below the search block
			if (!bNextBlockIsSeparatorLike)
			{
				constexpr bool bIsPartOfHeading = true;
				TSharedPtr<FMenuSpacerBlock> SpacerBlock = MakeShared<FMenuSpacerBlock>(Block.GetExtensionHook(), bIsPartOfHeading);

				// Note that the search widget might be collapsed, so the visibility of this spacer needs to be dynamic (like the search box itself)
				SpacerBlock->SetVisibilityOverride(TAttribute<EVisibility>(this, &SMultiBoxWidget::GetSearchBoxVisibility));

				AddBlockWidget(*SpacerBlock, HorizontalBox, VerticalBox, Panel, EMultiBlockLocation::None, bSectionContainsIcons, OptionsBlock);
			}
		}
	}

	// The first loop above added the multibox blocks and discovered/collected all sub-menus blocks. This second loops adds all sub-menu blocks discovered in the first pass.
	for (const TPair<TSharedPtr<const FMultiBlock>, TSharedPtr<FFlattenSearchableBlockInfo>>& Pair : FlattenSearchableBlocks)
	{
		// Do not add a block if it is being dragged
		if (!IsBlockBeingDragged(Pair.Key))
		{
			TSharedPtr<const FToolBarComboButtonBlock> OptionsBlock;
			AddBlockWidget(*Pair.Key, HorizontalBox, VerticalBox, Panel, EMultiBlockLocation::None, /*bSectionContainsIcons*/false, OptionsBlock);
		}
	}

	// Add the wrap button as the final block
	if (ClippedHorizontalBox.IsValid())
	{
		ClippedHorizontalBox->AddWrapButton();
	}
	
	if (ClippedVerticalBox.IsValid())
	{
		ClippedVerticalBox->AddWrapButton();
	}

	FMargin BorderPadding(0);
	const FSlateBrush* BorderBrush = FStyleDefaults::GetNoBrush();
	FSlateColor BorderForegroundColor = FSlateColor::UseForeground();

	// Setup the root border widget
	TSharedPtr<SWidget> RootBorder;

	switch (MultiBox->GetType())
	{
	case EMultiBoxType::Menu:
		{
			BorderPadding = FMargin(0, 8, 0, 8);
		}
		break;
	default:
		{ 
			BorderBrush = BackgroundBrush;
			BorderPadding = BackgroundPadding;
			BorderForegroundColor = FCoreStyle::Get().GetSlateColor("DefaultForeground");
		}
		break;
	}

	RootBorder =
		SNew(SBorder)
		.Padding(BorderPadding)
		.BorderImage(BorderBrush)
		.ForegroundColor(BorderForegroundColor)
		// Assign the box panel as the child
		[
			MainWidget.ToSharedRef()
		];

	// Prevent tool-tips spawned by child widgets from drawing on top of our main widget
	RootBorder->EnableToolTipForceField( true );

	ChildSlot
	[
		RootBorder.ToSharedRef()
	];

	// Save pointers to horizontal/vertical/panel box so that we can insert more blocks later if necessary (e.g. FlattenSubMenusRecursive)
	MainHorizontalBox = HorizontalBox;
	MainVerticalBox = VerticalBox;
	MainPanelBox = Panel;
}

void SMultiBoxWidget::UpdateMultiBoxWidget(TSharedRef<FMultiBox> InMultiBox)
{
	SetMultiBox(InMultiBox);
	BuildMultiBoxWidget();
}

TSharedRef<SWidget> SMultiBoxWidget::OnWrapButtonClicked()
{
	FMenuBuilder MenuBuilder(true, MultiBox->GetLastCommandList(), TSharedPtr<FExtender>(), false, GetStyleSet());

	if (ClippedVerticalBox)
	{
		MenuBuilder.SetCheckBoxStyle("ClippingVerticalBox.Check");
	}

	const TArray<TWeakPtr<SWidget>> ClippedWidgets = ClippedHorizontalBox ? ClippedHorizontalBox->GetClippedWidgets()
																		  : ClippedVerticalBox->GetClippedWidgets();

	for (const TWeakPtr<SWidget>& WeakClippedWidget : ClippedWidgets)
	{
		const TSharedPtr<SWidget> ClippedWidget = WeakClippedWidget.Pin();
		if (!ClippedWidget)
		{
			continue;
		}

		// Find the block of the clipped widget and add it to the menu.
		for (const TPair<TSharedPtr<SWidget>, TSharedPtr<const FMultiBlock>>& Pair : FinalMultiBoxWidgets)
		{
			if (ClippedWidget == Pair.Key)
			{
				Pair.Value->CreateMenuEntry(MenuBuilder);
				
				// Entries that are followed by simple combo boxes have both widgets merged together to create a single button with an "options" dropdown.
				// When overflowing, the primary button of this pair needs to handle placing this consumed child.
				const TArray<TSharedRef<const FMultiBlock>>& Blocks = GetMultiBox()->GetBlocks();
				int32 NextBlockIndex = Blocks.Find(Pair.Value.ToSharedRef()) + 1;
				if (NextBlockIndex < Blocks.Num())
				{
					const TSharedRef<const FMultiBlock>& NextBlock = Blocks[NextBlockIndex];
					if (NextBlock->GetType() == EMultiBlockType::ToolBarComboButton)
					{
						TSharedRef<const FToolBarComboButtonBlock> NextToolBarComboButtonBlock = StaticCastSharedRef<const FToolBarComboButtonBlock>(NextBlock);
						if (NextToolBarComboButtonBlock->IsSimpleComboBox())
						{
							NextBlock->CreateMenuEntry(MenuBuilder);
						}
					}
				}
				
				break;
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

void SMultiBoxWidget::UpdateDropAreaPreviewBlock( TSharedRef<const FMultiBlock> MultiBlock, TSharedPtr<FUICommandDragDropOp> DragDropContent, const FGeometry& DragAreaGeometry, const FVector2D& DragPos )
{
	const FName BlockName = DragDropContent->ItemName;
	const EMultiBlockType BlockType = DragDropContent->BlockType;
	FName OriginMultiBox = DragDropContent->OriginMultiBox;

	FVector2D LocalDragPos = DragAreaGeometry.AbsoluteToLocal( DragPos );

	FVector2D DrawSize = DragAreaGeometry.GetDrawSize();

	bool bIsDraggingSection = DragDropContent->bIsDraggingSection;

	bool bAddedNewBlock = false;
	bool bValidCommand = true;
	if (!DragPreview.IsSameBlockAs(BlockName, BlockType))
	{
		TSharedPtr<const FMultiBlock> ExistingBlock = MultiBox->FindBlockFromNameAndType(BlockName, BlockType);
		// Check that the command does not already exist and that we can create it or that we are dragging an existing block in this box
		if( !ExistingBlock.IsValid() || ( ExistingBlock.IsValid() && OriginMultiBox == MultiBox->GetCustomizationName() ) )
		{
			TSharedPtr<const FMultiBlock> NewBlock = ExistingBlock;

			if( NewBlock.IsValid() )
			{
				DragPreview.Reset();
				DragPreview.BlockName = BlockName;
				DragPreview.BlockType = BlockType;
				DragPreview.PreviewBlock = 
					MakeShareable(
						new FDropPreviewBlock( 
							NewBlock.ToSharedRef(), 
							NewBlock->MakeWidget( SharedThis(this), EMultiBlockLocation::None, NewBlock->HasIcon(),nullptr ) )
					);

				bAddedNewBlock = true;
			}
		}
		else
		{
			// this command cannot be dropped here
			bValidCommand = false;
		}
	}

	if( bValidCommand )
	{
		// determine whether or not to insert before or after
		bool bInsertBefore = false;
		if( MultiBox->GetType() == EMultiBoxType::ToolBar )
		{
			DragPreview.InsertOrientation  = EOrientation::Orient_Horizontal;
			if( LocalDragPos.X < DrawSize.X / 2 )
			{
				// Insert before horizontally
				bInsertBefore = true;
			}
			else
			{
				// Insert after horizontally
				bInsertBefore = false;
			}
		}
		else 
		{
			DragPreview.InsertOrientation  = EOrientation::Orient_Vertical;
			if( LocalDragPos.Y < DrawSize.Y / 2 )
			{
				// Insert before vertically
				bInsertBefore = true;
			}
			else
			{
				// Insert after vertically
				bInsertBefore = false;
			}
		}

		int32 CurrentIndex = DragPreview.InsertIndex;
		DragPreview.InsertIndex = INDEX_NONE;
		// Find the index of the multiblock being dragged over. This is where we will insert the new block
		if( DragPreview.PreviewBlock.IsValid() )
		{
			const TArray< TSharedRef< const FMultiBlock > >& Blocks = MultiBox->GetBlocks();
			int32 HoverIndex = Blocks.IndexOfByKey(MultiBlock);
			int32 HoverSectionEndIndex = INDEX_NONE;
			int32 HoverSectionBeginIndex = MultiBox->GetSectionEditBounds(HoverIndex, HoverSectionEndIndex);

			if (bIsDraggingSection)
			{
				// Hovering over final block means insert at end of list
				if ((HoverIndex == Blocks.Num() - 1) && Blocks.Num() > 0)
				{
					DragPreview.InsertIndex = Blocks.Num();
				}
				else if (Blocks.IsValidIndex(HoverSectionBeginIndex))
				{
					DragPreview.InsertIndex = HoverSectionBeginIndex;
				}
			}
			else if (HoverIndex != INDEX_NONE)
			{
				if (MultiBlock->IsPartOfHeading())
				{
					if (MultiBlock->IsSeparator())
					{
						// Move insert index above separator of heading
						DragPreview.InsertIndex = HoverIndex;
					}
					else
					{
						// Move insert index after heading
						DragPreview.InsertIndex = HoverIndex + 1;
					}
				}
				else
				{
					if (bInsertBefore)
					{
						DragPreview.InsertIndex = HoverIndex;
					}
					else
					{
						DragPreview.InsertIndex = HoverIndex + 1;
					}
				}
			}
		}
	}
}

EVisibility SMultiBoxWidget::GetCustomizationVisibility( TWeakPtr<const FMultiBlock> BlockWeakPtr, TWeakPtr<SWidget> BlockWidgetWeakPtr ) const
{
	if( MultiBox->IsInEditMode() && BlockWidgetWeakPtr.IsValid() && BlockWeakPtr.IsValid() && (!DragPreview.PreviewBlock.IsValid() || BlockWeakPtr.Pin() != DragPreview.PreviewBlock->GetActualBlock() ) )
	{
		// If in edit mode and this is not the block being dragged, the customization widget should be visible if the default block beging customized would have been visible
		return BlockWeakPtr.Pin()->GetAction().IsValid() && BlockWidgetWeakPtr.Pin()->GetVisibility() == EVisibility::Visible ? EVisibility::Visible : EVisibility::Collapsed;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

void SMultiBoxWidget::OnCustomCommandDragEnter( TSharedRef<const FMultiBlock> MultiBlock, const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if( MultiBlock != DragPreview.PreviewBlock && MultiBox->IsInEditMode() )
	{
		TSharedPtr<FUICommandDragDropOp> DragDropContent = StaticCastSharedPtr<FUICommandDragDropOp>( DragDropEvent.GetOperation() );

		UpdateDropAreaPreviewBlock( MultiBlock, DragDropContent, MyGeometry, DragDropEvent.GetScreenSpacePosition() );
	}
}

void SMultiBoxWidget::OnCustomCommandDragged( TSharedRef<const FMultiBlock> MultiBlock, const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if( MultiBlock != DragPreview.PreviewBlock && MultiBox->IsInEditMode() )
	{
		TSharedPtr<FUICommandDragDropOp> DragDropContent = StaticCastSharedPtr<FUICommandDragDropOp>( DragDropEvent.GetOperation() );

		UpdateDropAreaPreviewBlock( MultiBlock, DragDropContent, MyGeometry, DragDropEvent.GetScreenSpacePosition() );
	}
}

void SMultiBoxWidget::OnCustomCommandDropped()
{
	if( DragPreview.IsValid() )
	{	

		// Check that the command does not already exist and that we can create it or that we are dragging an exisiting block in this box
		TSharedPtr<const FMultiBlock> Block = MultiBox->FindBlockFromNameAndType(DragPreview.BlockName, DragPreview.BlockType);
		if(Block.IsValid())
		{
			if (Block->IsSeparator() && Block->IsPartOfHeading())
			{
				TSharedPtr<const FMultiBlock> HeadingBlock = MultiBox->FindBlockFromNameAndType(DragPreview.BlockName, EMultiBlockType::Heading);
				if (HeadingBlock.IsValid())
				{
					Block = HeadingBlock;
				}
			}

			MultiBox->InsertCustomMultiBlock( Block.ToSharedRef(), DragPreview.InsertIndex );
		}

		DragPreview.Reset();

		BuildMultiBoxWidget();
	}
}

void SMultiBoxWidget::OnDropExternal()
{
	// The command was not dropped in this widget
	if( DragPreview.IsValid() )
	{
		DragPreview.Reset();

		BuildMultiBoxWidget();
	}
}

FReply SMultiBoxWidget::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() && MultiBox->IsInEditMode() )
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SMultiBoxWidget::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		OnCustomCommandDropped();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SMultiBoxWidget::SupportsKeyboardFocus() const
{
	return MultiBox->bIsFocusable;
}

FReply SMultiBoxWidget::FocusNextWidget(EUINavigation NavigationType)
{
	TSharedPtr<SWidget> FocusWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	if(FocusWidget.IsValid())
	{
		FWidgetPath FocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( FocusWidget.ToSharedRef(), FocusPath );
		if (FocusPath.IsValid())
		{
			FWeakWidgetPath WeakFocusPath = FocusPath;
			FWidgetPath NextFocusPath = WeakFocusPath.ToNextFocusedPath(NavigationType);
			if ( NextFocusPath.Widgets.Num() > 0 )
			{
				return FReply::Handled().SetUserFocus(NextFocusPath.Widgets.Last().Widget, EFocusCause::Navigation);
			}
		}
	}

	return FReply::Unhandled();
}

FReply SMultiBoxWidget::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// Mouse DOWN on an element in the search result may fires OnFocusReceived if the mouse down event is not handled.
	// If the search is cleared, the widget layout changes and this usually prevents the mouse UP to reach the selected
	// widget which in turn doesn't trigger the selected item action.
	if (InFocusEvent.GetCause() != EFocusCause::Mouse)
	{
		ResetSearch();
	}

	if (SearchTextWidget.IsValid())
	{
		SearchTextWidget->EnableTextInputMethodContext();
	}

	if (InFocusEvent.GetCause() == EFocusCause::Navigation)
	{
		// forward focus to children
		return FocusNextWidget( EUINavigation::Next );
	}

	return FReply::Unhandled();
}

void SMultiBoxWidget::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	if (!NewWidgetPath.IsValid())
	{
		return;
	}

	if (!ShouldShowMenuSearchField())
	{
		return;
	}

	if (SearchTextWidget)
	{
		// We need to figure out if we're on the current focus path but after the last window, and thus
		// live in the most-expanded-to submenu. If that's the case, we should enable our search field to
		// signify to the user that any keyboard input will go there. If not, we disable our search field
		// (and another menu will enable its search field instead).
		const TSharedRef<SWindow> DeepestWindow = NewWidgetPath.GetDeepestWindow();
		const FWidgetPath WidgetPathToThis = NewWidgetPath.GetPathDownTo(AsShared());
		const bool bIsChildOfDeepestWindow = WidgetPathToThis.IsValid() && WidgetPathToThis.ContainsWidget(&DeepestWindow.Get());
		SearchTextWidget->SetEnabled(bIsChildOfDeepestWindow);
	}
}

FReply SMultiBoxWidget::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent )
{
	SCompoundWidget::OnKeyDown( MyGeometry, KeyEvent );

	// allow use of up and down keys to transfer focus/hover state
	if( KeyEvent.GetKey() == EKeys::Up )
	{
		return FocusNextWidget( EUINavigation::Previous );
	}
	else if( KeyEvent.GetKey() == EKeys::Down )
	{
		return FocusNextWidget( EUINavigation::Next );
	}

	return FReply::Unhandled();
}

FReply SMultiBoxWidget::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	FReply Reply = FReply::Unhandled();

	if (bSearchable && SearchText.IsEmpty())
	{
		// Check for special characters
		const TCHAR Character = InCharacterEvent.GetCharacter();
		BeginSearch(Character);
		Reply = FReply::Handled();
	}

	return Reply;
}

void SMultiBoxWidget::BeginSearch(const TCHAR InChar)
{
	// Certain characters are not allowed
	bool bIsCharAllowed = true;
	{
		if (InChar <= 0x1F)
		{
			bIsCharAllowed = false;
		}
	}

	if (bIsCharAllowed)
	{
		FString NewSearchText;
		NewSearchText += InChar;

		if (SearchTextWidget.IsValid() && SearchBlockWidget.IsValid())
		{
			// Make the search box visible and focused.
			SearchBlockWidget->SetVisibility(EVisibility::Visible);
			FSlateApplication::Get().SetUserFocus(0, SearchTextWidget);

			SearchTextWidget->SetText(FText::FromString(NewSearchText));
		}
	}
}

void SMultiBoxWidget::ResetSearch()
{
	// Empty search text
	if (SearchTextWidget.IsValid())
	{
		SearchTextWidget->SetText(FText::GetEmpty());
	}
}

void SMultiBoxWidget::FlattenSubMenusRecursive(uint32 MaxRecursionLevels)
{
	// Only flatten once (should happen the first time menu search is invoked).
	// If we reached MaxRecursionLevels of 0, then don't continue flattening sub-menus to prevent infinite recursion.
	if (bDidFlattenSearchableBlocks || !GetSearchable() || MaxRecursionLevels == 0)
	{
		return;
	}

	bDidFlattenSearchableBlocks = true;

	// Do a first pass and find/process all blocks that are submenus. This pass ends up being recursive.
	for (auto It = MultiBoxWidgets.CreateConstIterator(); It; ++It)
	{
		const TSharedPtr<SWidget>& BlockWidget = It.Key();
		const TArray<FText> SearchableTextHierarchy = It.Value();

		// Skip non-submenu blocks
		if (BlockWidget->GetTypeAsString() != TEXT("SMenuEntryBlock"))
		{
			continue;
		}

		// If the block widget being added displays a searchable text.
		const FText& ThisBlockDisplayText = UE::MultiBoxUtils::GetBlockWidgetDisplayText(SearchableTextHierarchy);
		if (!ThisBlockDisplayText.IsEmpty())
		{
			// If the underlying block is searchable and the search is allowed to walk down sub-menus recursively.
			TSharedPtr<SMenuEntryBlock> MenuEntryBlockWidget = StaticCastSharedPtr<SMenuEntryBlock>(BlockWidget);
			TSharedPtr<const FMultiBlock> Block = MenuEntryBlockWidget->GetBlock();
			if (!Block->GetSearchable() || Block->GetType() != EMultiBlockType::MenuEntry || !static_cast<const FMenuEntryBlock&>(*Block.Get()).IsSubMenu() || !static_cast<const FMenuEntryBlock&>(*Block.Get()).IsRecursivelySearchable())
			{
				continue;
			}

			// Expand the block widget into its sub-menu (as if the user clicked on the block widget to expand it).
			TSharedRef<SWidget> SubMenuWidget = StaticCastSharedPtr<SMenuEntryBlock>(BlockWidget)->MakeNewMenuWidget();

			// If the sub-menu widget is a multibox widget.
			if (SubMenuWidget->GetTypeAsString() == TEXT("SMultiBoxWidget"))
			{
				// Scan this expanded menu widgets.
				TSharedRef<SMultiBoxWidget> SubMenuMultiboxWidget = StaticCastSharedRef<SMultiBoxWidget>(SubMenuWidget);

				// Force all the sub-menus contained, in turn, by this submenu to be flattened so that we can flatten them into our searchable hierarchy.
				// Each time we recurse, we should decrease MaxRecursionLevels to prevent infinite recursion (e.g., in dynamic sub-menus which could go on forever).
				SubMenuMultiboxWidget->FlattenSubMenusRecursive(MaxRecursionLevels - 1);

				for (const TPair<TSharedPtr<SWidget>, TArray<FText>>& Pair : SubMenuMultiboxWidget->MultiBoxWidgets)
				{
					const TSharedPtr<SWidget>& ChildBlockWidget = Pair.Key;     // The widget.
					const TArray<FText>& ChildBlockSearchTextHierarchy = Pair.Value;  // The text displayed by the widget to the user along with all ancestor menus. Ex. "["Shapes", "Cube"]
					const FText& ChildBlockDisplayText = UE::MultiBoxUtils::GetBlockWidgetDisplayText(ChildBlockSearchTextHierarchy); // The text displayed by the widget to the user. Ex "Cube"

					// Skip if the widget has no searcheable text or if this is the 'search' text entry widget.
					if (ChildBlockDisplayText.IsEmpty() || ChildBlockWidget == SubMenuMultiboxWidget->GetSearchTextWidget())
					{
						continue; // Skip over, not searchable.
					}

					TSharedPtr<const FMultiBlock> ChildBlock = StaticCastSharedPtr<SMultiBlockBaseWidget>(ChildBlockWidget)->GetBlock();
					if (ChildBlock->GetSearchable())
					{
						// The algorithm to flatten the structure is recursive. It walks down to create/expand the tree and then walk up to collect and flatten the children blocks that have a displayable text.
						// For example if the blocks were organized as below, A would create B, which would create C1 and C2, then B would collect and merge it children C1 and C2 as B/C1 and B/C2, then A would
						// collect its children B, B/C1, B/C2 as A/B, A/B/C1 and A/B/C2.
						// A
						// |- B
						//    |- C1
						//    |- C2
						TSharedPtr<FFlattenSearchableBlockInfo> FlattenSearchableBlockInfo = MakeShared<FFlattenSearchableBlockInfo>();
						FlattenSearchableBlockInfo->SearchableTextHierarchyComponents.Add(ThisBlockDisplayText);             // If the multibox was adding the block displayed as "A" in the example, this would be "A".
						FlattenSearchableBlockInfo->SearchableTextHierarchyComponents.Append(ChildBlockSearchTextHierarchy); // If the multibox was adding the block displayed as "A" and visiting the child ["B", "C1"], that would generate ["A", "B", "C1"] path.

						// The widget corresponding to this flatten block is not added to this multibox yet, preserve the hierarchy information to update the widget searchable hierarchy once the block is added and the widget created.
						FlattenSearchableBlocks.Emplace(ChildBlock, MoveTemp(FlattenSearchableBlockInfo));
					}
				}
			}
		}
	}

	// The first loop above added the multibox blocks and discovered/collected all sub-menus blocks. This second loops adds all sub-menu blocks discovered in the first pass.
	for (const TPair<TSharedPtr<const FMultiBlock>, TSharedPtr<FFlattenSearchableBlockInfo>>& Pair : FlattenSearchableBlocks)
	{
		// Do not add a block if it is being dragged
		if (!IsBlockBeingDragged(Pair.Key))
		{
			TSharedPtr<const FToolBarComboButtonBlock> OptionsBlock;
			AddBlockWidget(*Pair.Key, MainHorizontalBox, MainVerticalBox, MainPanelBox, EMultiBlockLocation::None, /*bSectionContainsIcons*/false, OptionsBlock);
		}
	}
}

bool SMultiBoxWidget::ShouldShowMenuSearchField() const
{
	const int SearchFieldTreshold = FMultiBoxSettings::MenuSearchFieldVisibilityThreshold.Get();

	const TArray<TSharedRef<const FMultiBlock>> Blocks = MultiBox->GetBlocks();

	int NumUserCountableEntries = 0;
	for (const TSharedRef<const FMultiBlock>& Block : Blocks)
	{
		// Don't count headers and separators.
		const EMultiBlockType BlockType = Block->GetType();
		if (BlockType == EMultiBlockType::Heading
		 || BlockType == EMultiBlockType::Separator)
		{
			continue;
		}

		++NumUserCountableEntries;
	}

	// Subtract 1 to account for the search field widget. After that widget is created,
	// it's added at the start of the menu regardless if it's visible or not.
	NumUserCountableEntries = FMath::Max<int>(0, NumUserCountableEntries-1);

	return NumUserCountableEntries >= SearchFieldTreshold;
}

void SMultiBoxWidget::FilterMultiBoxEntries()
{
	VisibleFlattenHierarchyTips.Empty();

	if (SearchText.IsEmpty())
	{
		for (auto It = MultiBoxWidgets.CreateConstIterator(); It; ++It)
		{
			It.Key()->SetVisibility(EVisibility::Visible);
		}

		// Hide the sub-menus widgets that were made visible by searching this multi-box hierarchy.
		for (TPair<TSharedPtr<const FMultiBlock>, TSharedPtr<FFlattenSearchableBlockInfo>>& Pair: FlattenSearchableBlocks)
		{
			Pair.Value->Widget->SetVisibility(EVisibility::Collapsed);
			Pair.Value->HierarchyTipVisibility = EVisibility::Collapsed;
		}

		// Return focus to parent widget
		FSlateApplication::Get().SetUserFocus(0, SharedThis(this));

		return;
	}

	// Index 5 levels of nested sub-menus (this doesn't do anything if we already flattened)
	FlattenSubMenusRecursive(/*MaxRecursionLevels*/ 5);

	for(auto It = MultiBoxWidgets.CreateConstIterator(); It; ++It)
	{
		FText DisplayText = UE::MultiBoxUtils::GetBlockWidgetDisplayText(It.Value());
		const TSharedPtr<SWidget>& Widget = It.Key();

		// Skip the search widget itself when scanning for searchable items.
		if (Widget == SearchBlockWidget)
		{
			continue;
		}

		// Non-labeled elements should not be visible when searching
		if (DisplayText.IsEmpty())
		{
			Widget->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			// Does the widget label match the searched text?
			EVisibility WidgetVisibility = DisplayText.ToString().Contains(SearchText.ToString()) ? EVisibility::Visible : EVisibility::Collapsed;

			// If this widget is one of the sub-menu item that was flatten to support recursive search
			if (TSharedPtr<FFlattenSearchableBlockInfo>* FlattenBlockInfo = FlattenSearchableWidgets.Find(Widget))
			{
				(*FlattenBlockInfo)->Widget->SetVisibility(WidgetVisibility);
				(*FlattenBlockInfo)->HierarchyTipVisibility = WidgetVisibility;

				if (WidgetVisibility == EVisibility::Visible)
				{
					// Check if a item in the the same menu already enabled the 'hierarchy tip'. Because they are added in order, every next one with the same parent will be below each other. For example if
					// the user searches for 'C' and the multibox has expendable menus "Cinematics" containing "Camera X" and "Shapes" containing "Cube", "Cone" and "Cylinder", we want the search result to look like:
					// 
					//    - Cinematics > (This is an expandable menu that matches the search)
					// ##Shapes##        (This is the hierarchy tip enabled once for all matching items in the expandable 'Shapes' menu)
					//    - Cube
					//    - Cone
					//    - Cylinder
					// ##Cinematics##    (This is the hierarchy tip enabled once for the single matching item in the expandable 'Cinematics' menu)
					//    - Camera X
					//
					bool bHierarchyTipAlreadyVisible = false;
					VisibleFlattenHierarchyTips.FindOrAdd(UE::MultiBoxUtils::GetSearchHierarchyInfoText((*FlattenBlockInfo)->SearchableTextHierarchyComponents).ToString(), &bHierarchyTipAlreadyVisible);
					if (bHierarchyTipAlreadyVisible)
					{
						(*FlattenBlockInfo)->HierarchyTipVisibility = EVisibility::Collapsed;
					}
				}
			}
			else // This widget is a normal item in this menu.
			{
				Widget->SetVisibility(WidgetVisibility);
			}
		}
	}
}

FText SMultiBoxWidget::GetSearchText() const
{
	return SearchText;
}

TSharedPtr<SWidget> SMultiBoxWidget::GetSearchTextWidget()
{
	return SearchTextWidget;
}

void SMultiBoxWidget::SetSearchBlockWidget(TSharedPtr<SWidget> InWidget)
{
	SearchBlockWidget = InWidget;
}

void SMultiBoxWidget::AddSearchElement( TSharedPtr<SWidget> BlockWidget, FText BlockDisplayText )
{
	AddElement(BlockWidget, BlockDisplayText, true);
}

void SMultiBoxWidget::AddElement(TSharedPtr<SWidget> BlockWidget, FText BlockDisplayText, bool bInSearchable)
{
	 // Non-Searchable widgets shouldn't have search text
	if (!bInSearchable)
	{
		BlockDisplayText = FText::GetEmpty();
	}

	MultiBoxWidgets.Add(BlockWidget, TArray<FText>{BlockDisplayText});
}


bool SMultiBoxWidget::OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent)
{
	// tooltips on multibox widgets are not supported outside of the editor or programs
	return !GIsEditor && !FGenericPlatformProperties::IsProgram();
}

void SMultiBoxWidget::SetSummonedMenuTime(double InSummonedMenuTime)
{
	SummonedMenuTime = InSummonedMenuTime;
}

double SMultiBoxWidget::GetSummonedMenuTime() const
{
	return SummonedMenuTime;
}

#undef LOCTEXT_NAMESPACE
