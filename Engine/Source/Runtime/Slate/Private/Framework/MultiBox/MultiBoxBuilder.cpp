// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/SHeadingBlock.h"
#include "Framework/MultiBox/SMenuEntryBlock.h"
#include "Framework/MultiBox/SMenuSeparatorBlock.h"
#include "Framework/MultiBox/SToolBarSeparatorBlock.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
#include "Framework/MultiBox/SToolBarStackButtonBlock.h"
#include "Framework/MultiBox/SEditableTextBlock.h"
#include "Framework/MultiBox/SButtonRowBlock.h"
#include "Framework/MultiBox/SWidgetBlock.h"
#include "Framework/MultiBox/SGroupMarkerBlock.h"
#include "Styling/ToolBarStyle.h"

namespace UE::Slate::Private
{
	/** Utility to set a Target attribute to the given Source attribute, if the target is not already set. */
	template <typename AttributeType>
	static void ApplyUnsetAttributeOverride(TAttribute<AttributeType>& InTargetAttribute, const TAttribute<AttributeType>& InSourceAttribute)
	{
		if (!InTargetAttribute.IsSet())
		{
			if (InSourceAttribute.IsSet())
			{
				// Target is not set, and source is, so override Target with Source
				InTargetAttribute = InSourceAttribute;
			}
		}
	}

	/** Utility to set a Target TOptional to the given Source TOptional, if the target is not already set. */
	template <typename OptionalType>
	static void ApplyUnsetOptionalOverride(TOptional<OptionalType>& InTargetOptional, const TOptional<OptionalType>& InSourceOptional)
	{
		if (!InTargetOptional.IsSet() && InSourceOptional.IsSet())
		{
			// Target is not set, and source is, so override Target with Source
			InTargetOptional = InSourceOptional;
		}
	}

	/** Applies section-level overrides to the given block, if any. Currently only affects Resize parameters. */
	static void ApplySectionOverridesToBlock(
		const TSharedRef<FMultiBlock>& InMultiBlock,
		const TAttribute<FMenuEntryResizeParams>& InSectionResizeParams)
	{
		// If per-entry resize params aren't set, and section params are, inject those of the section
		if (InSectionResizeParams.IsSet())
		{
			TAttribute<FMenuEntryResizeParams> BlockResizeParamsAttribute = InMultiBlock->GetResizeParams();
			if (!BlockResizeParamsAttribute.IsSet())
			{
				BlockResizeParamsAttribute = InSectionResizeParams;
			}
			else if (!BlockResizeParamsAttribute.IsBound())
			{
				const FMenuEntryResizeParams& SectionResizeParams = InSectionResizeParams.Get();

				FMenuEntryResizeParams BlockResizeParams = BlockResizeParamsAttribute.Get();
				ApplyUnsetAttributeOverride(BlockResizeParams.Wrapping.Allow, SectionResizeParams.Wrapping.Allow);
				ApplyUnsetAttributeOverride(BlockResizeParams.Wrapping.Mode, SectionResizeParams.Wrapping.Mode);
				ApplyUnsetAttributeOverride(BlockResizeParams.Wrapping.Priority, SectionResizeParams.Wrapping.Priority);
				ApplyUnsetOptionalOverride(BlockResizeParams.Wrapping.VerticalOverflowBehavior, SectionResizeParams.Wrapping.VerticalOverflowBehavior);
				ApplyUnsetOptionalOverride(BlockResizeParams.Wrapping.ForceNewLine, SectionResizeParams.Wrapping.ForceNewLine);

				BlockResizeParamsAttribute = BlockResizeParams;
			}
			else
			{
				// Intercept and inject wrapping properties
				TAttribute<FMenuEntryResizeParams> NewResizeParams(
					TAttribute<FMenuEntryResizeParams>::Create(
						TAttribute<FMenuEntryResizeParams>::FGetter::CreateLambda([BlockResizeParamsAttribute, SectionResizeParams = InSectionResizeParams.Get()]
						{
							FMenuEntryResizeParams ResizeParams = BlockResizeParamsAttribute.Get();
							ApplyUnsetAttributeOverride(ResizeParams.Wrapping.Allow, SectionResizeParams.Wrapping.Allow);
							ApplyUnsetAttributeOverride(ResizeParams.Wrapping.Mode, SectionResizeParams.Wrapping.Mode);
							ApplyUnsetAttributeOverride(ResizeParams.Wrapping.Priority, SectionResizeParams.Wrapping.Priority);
							ApplyUnsetOptionalOverride(ResizeParams.Wrapping.VerticalOverflowBehavior, SectionResizeParams.Wrapping.VerticalOverflowBehavior);
							ApplyUnsetOptionalOverride(ResizeParams.Wrapping.ForceNewLine, SectionResizeParams.Wrapping.ForceNewLine);

							return ResizeParams;
						})));

				BlockResizeParamsAttribute = NewResizeParams;
			}

			InMultiBlock->SetResizeParams(BlockResizeParamsAttribute);
		}
	}
}

FMultiBoxBuilder::FMultiBoxBuilder( const EMultiBoxType InType, FMultiBoxCustomization InCustomization, const bool bInShouldCloseWindowAfterMenuSelection, const TSharedPtr< const FUICommandList >& InCommandList, TSharedPtr<FExtender> InExtender, FName InTutorialHighlightName, FName InMenuName )
	: MultiBox( FMultiBox::Create( InType, (InMenuName != NAME_None) ? FMultiBoxCustomization::AllowCustomization(InMenuName) : InCustomization, bInShouldCloseWindowAfterMenuSelection ) )
	, CommandListStack()
	, TutorialHighlightName(InTutorialHighlightName)
	, MenuName(InMenuName)
    , CheckBoxStyle(NAME_None)
	, bExtendersEnabled(true)
{
	CommandListStack.Push( InCommandList );
	ExtenderStack.Push(InExtender);
}

void FMultiBoxBuilder::SetCheckBoxStyle(FName InCheckBoxStyle)
{
	this->CheckBoxStyle = InCheckBoxStyle;
}


void FMultiBoxBuilder::AddEditableText( const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, const TAttribute< FText >& InTextAttribute, const FOnTextCommitted& InOnTextCommitted, const FOnTextChanged& InOnTextChanged, bool bInReadOnly )
{
	MultiBox->AddMultiBlock( MakeShareable( new FEditableTextBlock( InLabel, InToolTip, InIcon, InTextAttribute, bInReadOnly, InOnTextCommitted, InOnTextChanged ) ) );
}

void FMultiBoxBuilder::AddVerifiedEditableText(const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, const TAttribute< FText >& InTextAttribute, const FOnVerifyTextChanged& InOnVerifyTextChanged, const FOnTextCommitted& InOnTextCommitted /*= FOnTextCommitted()*/, const FOnTextChanged& InOnTextChanged /*= FOnTextChanged()*/, bool bInReadOnly /*= false*/)
{
	MultiBox->AddMultiBlock(MakeShareable(new FEditableTextBlock(InLabel, InToolTip, InIcon, InTextAttribute, bInReadOnly, InOnTextCommitted, InOnTextChanged, InOnVerifyTextChanged)));
}

void FMultiBoxBuilder::PushCommandList( const TSharedRef< const FUICommandList > CommandList )
{
	CommandListStack.Push( CommandList );
}

void FMultiBoxBuilder::SetLastSelectedCommandIndex(int32 InLastSelectedCommandIndex)
{
	MultiBox->SetLastSelectedCommandIndex( InLastSelectedCommandIndex );			
}

void FMultiBoxBuilder::PopCommandList()
{
	// Never allowed to pop the last command-list!  This command-list was set when the multibox was first created and is canonical.
	if( ensure( CommandListStack.Num() > 1 ) )
	{
		CommandListStack.Pop();
	}
}

TSharedPtr<const FUICommandList> FMultiBoxBuilder::GetTopCommandList()
{
	return (CommandListStack.Num() > 0) ? CommandListStack.Top() : TSharedPtr<const FUICommandList>(NULL);
}

void FMultiBoxBuilder::PushExtender( TSharedRef< FExtender > InExtender )
{
	ExtenderStack.Push( InExtender );
}

void FMultiBoxBuilder::PopExtender()
{
	// Never allowed to pop the last extender! This extender was set when the multibox was first created and is canonical.
	if( ensure( ExtenderStack.Num() > 1 ) )
	{
		ExtenderStack.Pop();
	}
}

const ISlateStyle* FMultiBoxBuilder::GetStyleSet() const 
{ 
	return MultiBox->GetStyleSet();
}

const FName& FMultiBoxBuilder::GetStyleName() const 
{ 
	return MultiBox->GetStyleName();
}

void FMultiBoxBuilder::SetStyle( const ISlateStyle* InStyleSet, const FName& InStyleName ) 
{ 
	MultiBox->SetStyle( InStyleSet, InStyleName ); 
}

FMultiBoxCustomization FMultiBoxBuilder::GetCustomization() const
{
	return FMultiBoxCustomization( MultiBox->GetCustomizationName() ); 
}

TSharedRef< class SWidget > FMultiBoxBuilder::MakeWidget( FMultiBox::FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride)
{
	return MultiBox->MakeWidget( false, InMakeMultiBoxBuilderOverride );
}

TSharedRef< class FMultiBox > FMultiBoxBuilder::GetMultiBox()
{
	return MultiBox;
}

/** Helper function to generate unique widget-identifying names given various bits of information */
static FName GenerateTutorialIdentifierName(FName InContainerName, FName InElementName, const TSharedPtr< const FUICommandInfo > InCommand, int32 InIndex)
{
	FString BaseName;
	if(InContainerName != NAME_None)
	{
		BaseName = InContainerName.ToString() + TEXT(".");
	}

	if(InElementName != NAME_None)
	{
		return FName(*(BaseName + InElementName.ToString()));
	}
	else if(InCommand.IsValid() && InCommand->GetCommandName() != NAME_None)
	{
		return FName(*(BaseName + InCommand->GetCommandName().ToString()));
	}
	else
	{
		// default to index if no other info is available
		const FString IndexedName = FString::Printf(TEXT("MultiboxWidget%d"), InIndex);
		return FName(*(BaseName + IndexedName));
	}
}

FBaseMenuBuilder::FBaseMenuBuilder( const EMultiBoxType InType, const bool bInShouldCloseWindowAfterMenuSelection, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, TSharedPtr<FExtender> InExtender, const ISlateStyle* InStyleSet, FName InTutorialHighlightName, FName InMenuName )
	: FMultiBoxBuilder( InType, FMultiBoxCustomization::None, bInShouldCloseWindowAfterMenuSelection, InCommandList, InExtender, InTutorialHighlightName, InMenuName )
	, bCloseSelfOnly( bInCloseSelfOnly )
{
	MultiBox->SetStyle(InStyleSet, "Menu");
}

void FBaseMenuBuilder::AddMenuEntry( const TSharedPtr< const FUICommandInfo > InCommand, FName InExtensionHook, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const FSlateIcon& InIconOverride, FName InTutorialHighlightName, const TAttribute<EVisibility>& InVisibility )
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);
	
	// The command must be valid
	check( InCommand.IsValid() );
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( InExtensionHook, InCommand, CommandListStack.Last(), InLabelOverride, InToolTipOverride, InIconOverride, bCloseSelfOnly );
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, InCommand, MultiBox->GetBlocks().Num()));
	NewMenuEntryBlock->SetCheckBoxStyle(CheckBoxStyle);

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );

	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FBaseMenuBuilder::AddMenuEntry( const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FSlateIcon& InIcon, const FUIAction& InAction, FName InExtensionHook, const EUserInterfaceActionType UserInterfaceActionType, FName InTutorialHighlightName, const TAttribute<FText>& InInputBindingOverride, const TAttribute<EVisibility>& InVisibility)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);
	
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>(InExtensionHook, InLabel, InToolTip, InIcon, InAction, UserInterfaceActionType, bCloseSelfOnly, bShouldCloseWindowAfterMenuSelection, InInputBindingOverride);
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
	
	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FBaseMenuBuilder::AddMenuEntry( const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const FName& InExtensionHook, const TAttribute<FText>& InToolTip, const EUserInterfaceActionType UserInterfaceActionType, FName InTutorialHighlightName, const TAttribute<EVisibility>& InVisibility )
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( InExtensionHook, UIAction, Contents, InToolTip, UserInterfaceActionType, bCloseSelfOnly );
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );

	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FBaseMenuBuilder::AddMenuEntry(const FMenuEntryParams& InMenuEntryParams)
{
	ApplySectionBeginning();

	ApplyHook(InMenuEntryParams.ExtensionHook, EExtensionHook::Before);
	
	TSharedPtr< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>(InMenuEntryParams);
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InMenuEntryParams.TutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));

	if (InMenuEntryParams.Visibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InMenuEntryParams.Visibility);
	}
	
	MultiBox->AddMultiBlock(NewMenuEntryBlock.ToSharedRef());

	ApplyHook(InMenuEntryParams.ExtensionHook, EExtensionHook::After);
}

TSharedRef< class SWidget > FMenuBuilder::MakeWidget( FMultiBox::FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride)
{
	return MakeWidget(InMakeMultiBoxBuilderOverride, 1000);
}

TSharedRef< class SWidget > FMenuBuilder::MakeWidget( FMultiBox::FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride, uint32 MaxHeight)
{
	TAttribute<float> MaxHeightAttribute;
	if (MaxHeight < INT_MAX)
	{
		MaxHeightAttribute.Set((float)MaxHeight);
	}
	return MultiBox->MakeWidget(bSearchable, InMakeMultiBoxBuilderOverride, MaxHeightAttribute);
}

void FMenuBuilder::BeginSection(FName InExtensionHook, const TAttribute< FText >& InHeadingText, const TAttribute<EVisibility>& InVisibility, const TAttribute<FMenuEntryResizeParams>& InResizeParams)
{
	check(CurrentSectionExtensionHook == NAME_None && !bSectionNeedsToBeApplied);

	ApplyHook(InExtensionHook, EExtensionHook::Before);
	
	// Do not actually apply the section header, because if this section is ended immediately
	// then nothing ever gets created, preventing empty sections from ever appearing
	bSectionNeedsToBeApplied = true;
	CurrentSectionExtensionHook = InExtensionHook;
	CurrentSectionHeadingText = InHeadingText.Get();
	CurrentSectionVisibility = InVisibility;
	CurrentSectionResizeParams = InResizeParams;

	// Do apply the section beginning if we are in developer "show me all the hooks" mode
	if (FMultiBoxSettings::DisplayMultiboxHooks.Get())
	{
		ApplySectionBeginning();
	}

	ApplyHook(InExtensionHook, EExtensionHook::First);
}

void FMenuBuilder::EndSection()
{
	FName SectionExtensionHook = CurrentSectionExtensionHook;
	CurrentSectionExtensionHook = NAME_None;
	bSectionNeedsToBeApplied = false;
	CurrentSectionHeadingText = FText::GetEmpty();
	CurrentSectionVisibility = EVisibility::Visible;
	CurrentSectionResizeParams = TAttribute<FMenuEntryResizeParams>();

	ApplyHook(SectionExtensionHook, EExtensionHook::After);
}

void FMenuBuilder::AddMenuSeparator(FName InExtensionHook, const TAttribute<EVisibility>& InVisibility)
{
	AddSeparator(InExtensionHook, InVisibility);
}

void FMenuBuilder::AddSeparator(FName InExtensionHook /*= NAME_None*/, const TAttribute<EVisibility>& InVisibility)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	// Never add a menu separate as the first item, even if we were asked to
	if (MultiBox->GetBlocks().Num() > 0 || FMultiBoxSettings::DisplayMultiboxHooks.Get())
	{
		TSharedRef< FMenuSeparatorBlock > NewMenuSeparatorBlock(new FMenuSeparatorBlock(InExtensionHook, /* bInIsPartOfHeading=*/ false));

		if (InVisibility.IsSet())
		{
			NewMenuSeparatorBlock->SetVisibilityOverride(InVisibility);
		}

		MultiBox->AddMultiBlock(NewMenuSeparatorBlock);
	}

	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FMenuBuilder::AddSubMenu( const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InSubMenu, const FUIAction& InUIAction, FName InExtensionHook, const EUserInterfaceActionType InUserInterfaceActionType, const bool bInOpenSubMenuOnClick, const FSlateIcon& InIcon, const bool bInShouldCloseWindowAfterMenuSelection /*= true*/, const TAttribute<EVisibility>& InVisibility, const TAttribute<FText>& InInputBindingOverride )
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( InExtensionHook, InMenuLabel, InToolTip, InSubMenu, ExtenderStack.Top(), bIsSubMenu, bInOpenSubMenuOnClick, InIcon, InUIAction, InUserInterfaceActionType, bCloseSelfOnly, bInShouldCloseWindowAfterMenuSelection, CommandListStack.Last(), InInputBindingOverride);
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddSubMenu( const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InSubMenu, const bool bInOpenSubMenuOnClick /*= false*/, const FSlateIcon& InIcon /*= FSlateIcon()*/, const bool bInShouldCloseWindowAfterMenuSelection /*= true*/, FName InExtensionHook /*=NAME_None*/, FName InTutorialHighlightName /*= NAME_None*/, const TAttribute<EVisibility>& InVisibility)
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( InExtensionHook, InMenuLabel, InToolTip, InSubMenu, ExtenderStack.Top(), bIsSubMenu, bInOpenSubMenuOnClick, CommandListStack.Last(), bCloseSelfOnly, InIcon, bInShouldCloseWindowAfterMenuSelection );
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);
	NewMenuEntryBlock->SetCheckBoxStyle(CheckBoxStyle);

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddSubMenu( const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InSubMenu, const bool bInOpenSubMenuOnClick /*= false*/, const bool bInShouldCloseWindowAfterMenuSelection /*= true*/, const TAttribute<EVisibility>& InVisibility )
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( NAME_None, Contents, InSubMenu, ExtenderStack.Top(), bIsSubMenu, bInOpenSubMenuOnClick, CommandListStack.Last(), bCloseSelfOnly, bInShouldCloseWindowAfterMenuSelection );
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddSubMenu( const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InSubMenu, const bool bInShouldCloseWindowAfterMenuSelection /*= true*/, const TAttribute<EVisibility>& InVisibility )
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( NAME_None, UIAction, Contents, InSubMenu, ExtenderStack.Top(), bIsSubMenu, CommandListStack.Last(), bCloseSelfOnly, bInShouldCloseWindowAfterMenuSelection );
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);

	if (InVisibility.IsSet())
	{
		NewMenuEntryBlock->SetVisibilityOverride(InVisibility);
	}

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddWrapperSubMenu( const FText& InMenuLabel, const FText& InToolTip, const FOnGetContent& InSubMenu, const FSlateIcon& InIcon )
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( NAME_None, InMenuLabel, InToolTip, InSubMenu, ExtenderStack.Top(), bIsSubMenu, false, CommandListStack.Last(), bCloseSelfOnly, InIcon );
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddWrapperSubMenu( const FText& InMenuLabel, const FText& InToolTip, const FOnGetContent& InSubMenu, const FSlateIcon& InIcon, const FUIAction& UIAction )
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( NAME_None, UIAction, InMenuLabel, InToolTip, InSubMenu, ExtenderStack.Top(), bIsSubMenu, false, bCloseSelfOnly, InIcon );
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddWrapperSubMenu( const FText& InMenuLabel, const FText& InToolTip, const TSharedPtr<SWidget>& InSubMenu, const FSlateIcon& InIcon )
{
	ApplySectionBeginning();

	const bool bIsSubMenu = true;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock = MakeShared<FMenuEntryBlock>( NAME_None, InMenuLabel, InToolTip, InSubMenu, ExtenderStack.Top(), bIsSubMenu, false, CommandListStack.Last(), bCloseSelfOnly, InIcon );
	NewMenuEntryBlock->SetRecursivelySearchable(bRecursivelySearchable);

	MultiBox->AddMultiBlock( NewMenuEntryBlock );
}

void FMenuBuilder::AddWidget( TSharedRef<SWidget> InWidget, const FText& Label, bool bNoIndent, bool bInSearchable, const TAttribute<FText>& InToolTipText )
{
	FMenuEntryStyleParams StyleParams;
	StyleParams.bNoIndent = bNoIndent;

	return AddWidget(InWidget, Label, StyleParams, bInSearchable, InToolTipText);
}

void FMenuBuilder::AddWidget(const TSharedRef<SWidget>& InWidget, const FText& InLabel, const FMenuEntryStyleParams& InStyleParams, const bool bInSearchable, const TAttribute<FText>& InToolTipText, const TAttribute<FSlateIcon>& InIcon, const TAttribute<EVisibility>& InVisibility)
{
	AddWidget(InWidget, InLabel, InStyleParams, FMenuEntryResizeParams(), bInSearchable, InToolTipText, InIcon, InVisibility);
}

void FMenuBuilder::AddWidget(const TSharedRef<SWidget>& InWidget, const FText& InLabel, const FMenuEntryStyleParams& InStyleParams, const TAttribute<FMenuEntryResizeParams>& InResizeParams, const bool bInSearchable, const TAttribute<FText>& InToolTipText, const TAttribute<FSlateIcon>& InIcon, const TAttribute<EVisibility>& InVisibility)
{
	ApplySectionBeginning();

	TSharedRef<FWidgetBlock> NewWidgetBlock = MakeShared<FWidgetBlock>(InWidget, InLabel, InToolTipText, InStyleParams, InIcon);
	NewWidgetBlock->SetSearchable( bInSearchable );
	NewWidgetBlock->SetResizeParams(InResizeParams);

	if (InVisibility.IsSet())
	{
		NewWidgetBlock->SetVisibilityOverride(InVisibility);
	}

	UE::Slate::Private::ApplySectionOverridesToBlock(NewWidgetBlock, CurrentSectionResizeParams);

	MultiBox->AddMultiBlock( NewWidgetBlock );
}

void FMenuBuilder::AddSearchWidget()
{
	MultiBox->bHasSearchWidget = true;
}

void FMenuBuilder::ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition)
{
	if (ExtendersEnabled())
	{
		// this is a virtual function to get a properly typed "this" pointer
		auto& Extender = ExtenderStack.Top();
		if (InExtensionHook != NAME_None && Extender.IsValid())
		{
			if (!MultiBox->IsInEditMode())
			{
				Extender->Apply(InExtensionHook, HookPosition, *this);
			}
		}
	}
}

void FMenuBuilder::ApplySectionBeginning()
{
	if (bSectionNeedsToBeApplied)
	{
		if (!CurrentSectionHeadingText.IsEmpty())
		{
			TSharedRef<FHeadingBlock> HeadingBlock = MakeShared<FHeadingBlock>(CurrentSectionExtensionHook, CurrentSectionHeadingText);
			HeadingBlock->SetResizeParams(CurrentSectionResizeParams);

			if (CurrentSectionVisibility.IsSet())
			{
				HeadingBlock->SetVisibilityOverride(CurrentSectionVisibility);
			}

			MultiBox->AddMultiBlock(HeadingBlock);
		}
		bSectionNeedsToBeApplied = false;
		CurrentSectionHeadingText = FText::GetEmpty();
	}
}

void FMenuBarBuilder::AddPullDownMenu(const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InPullDownMenu, FName InExtensionHook, FName InTutorialHighlightName)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	const bool bIsSubMenu = false;
	const bool bOpenSubMenuOnClick = false;
	// Pulldown menus always close all menus not just themselves
	const bool bShouldCloseSelfOnly = false;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock(new FMenuEntryBlock(InExtensionHook, InMenuLabel, InToolTip, InPullDownMenu, ExtenderStack.Top(), bIsSubMenu, bOpenSubMenuOnClick, CommandListStack.Last(), bShouldCloseSelfOnly));
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));

	MultiBox->AddMultiBlock(NewMenuEntryBlock);

	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FMenuBarBuilder::AddPullDownMenu(const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FOnGetContent& InMenuContentGenerator, FName InExtensionHook, FName InTutorialHighlightName)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	const bool bIsSubMenu = false;
	const bool bOpenSubMenuOnClick = false;
	// Pulldown menus always close all menus not just themselves
	const bool bShouldCloseSelfOnly = false;
	TSharedRef< FMenuEntryBlock > NewMenuEntryBlock(new FMenuEntryBlock(InExtensionHook, InMenuLabel, InToolTip, InMenuContentGenerator, ExtenderStack.Top(), bIsSubMenu, bOpenSubMenuOnClick, CommandListStack.Last(), bShouldCloseSelfOnly));
	NewMenuEntryBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));

	MultiBox->AddMultiBlock(NewMenuEntryBlock);

	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FMenuBarBuilder::ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition)
{
	if (ExtendersEnabled())
	{
		// this is a virtual function to get a properly typed "this" pointer
		auto& Extender = ExtenderStack.Top();
		if (InExtensionHook != NAME_None && Extender.IsValid())
		{
			Extender->Apply(InExtensionHook, HookPosition, *this);
		}
	}
}

void FToolBarBuilder::SetIsFocusable(bool bInIsFocusable)
{
	bIsFocusable = bInIsFocusable; 
	MultiBox->bIsFocusable = bIsFocusable;
}

void FToolBarBuilder::SetAllowWrapButton(const TOptional<bool>& bInAllowWrapButton) const
{
	MultiBox->bAllowWrapButton = bInAllowWrapButton;
}

TSharedPtr<FToolBarButtonBlock> FToolBarBuilder::AddToolBarButton(
	const TSharedPtr<const FUICommandInfo> InCommand,
	FName InExtensionHook,
	const TAttribute<FText>& InLabelOverride,
	const TAttribute<FText>& InToolTipOverride,
	const TAttribute<FSlateIcon>& InIconOverride,
	FName InTutorialHighlightName,
	FNewMenuDelegate InCustomMenuDelegate,
	TAttribute<EVisibility> InVisibilityOverride,
	TAttribute<FText> InToolbarLabelOverride,
	const TAttribute<FMenuEntryResizeParams>& InResizeParams
)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	TSharedRef<FToolBarButtonBlock> NewToolBarButtonBlock(new FToolBarButtonBlock(
		InCommand.ToSharedRef(), CommandListStack.Last(), InLabelOverride, InToolTipOverride, InIconOverride, InToolbarLabelOverride
	));

	if ( LabelVisibility.IsSet() )
	{
		NewToolBarButtonBlock->SetLabelVisibility( LabelVisibility.GetValue() );
	}

	NewToolBarButtonBlock->SetIsFocusable(bIsFocusable);
	NewToolBarButtonBlock->SetForceSmallIcons(bForceSmallIcons);
	NewToolBarButtonBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, InCommand, MultiBox->GetBlocks().Num()));
	NewToolBarButtonBlock->SetStyleNameOverride(CurrentStyleOverride);
	NewToolBarButtonBlock->SetCustomMenuDelegate(InCustomMenuDelegate);
	NewToolBarButtonBlock->SetVisibilityOverride(InVisibilityOverride);
	NewToolBarButtonBlock->SetResizeParams(InResizeParams);

	UE::Slate::Private::ApplySectionOverridesToBlock(NewToolBarButtonBlock, CurrentSectionResizeParams);

	MultiBox->AddMultiBlock( NewToolBarButtonBlock );

	ApplyHook(InExtensionHook, EExtensionHook::After);
	
	return NewToolBarButtonBlock;
}

void FToolBarBuilder::AddToolBarButton(const FButtonArgs& Args)
{
	ApplySectionBeginning();

	ApplyHook(Args.ExtensionHook, EExtensionHook::Before);

	const bool bHasUserInteractionType = Args.UserInterfaceActionType != EUserInterfaceActionType::None;

	const TSharedPtr<FToolBarButtonBlock> NewToolBarButtonBlock(
		bHasUserInteractionType ? new FToolBarButtonBlock(
			Args.LabelOverride, Args.ToolTipOverride, Args.IconOverride, Args.Action, Args.UserInterfaceActionType, Args.ToolbarLabelOverride
		)
								: new FToolBarButtonBlock(
									Args.Command.ToSharedRef(),
									CommandListStack.Last(),
									Args.LabelOverride,
									Args.ToolTipOverride,
									Args.IconOverride,
									Args.ToolbarLabelOverride
								)
	);

	if ( LabelVisibility.IsSet() )
	{
		NewToolBarButtonBlock->SetLabelVisibility( LabelVisibility.GetValue() );
	}

	NewToolBarButtonBlock->SetIsFocusable(bIsFocusable);
	NewToolBarButtonBlock->SetForceSmallIcons(bForceSmallIcons);
	NewToolBarButtonBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, Args.TutorialHighlightName, Args.Command, MultiBox->GetBlocks().Num()));
	NewToolBarButtonBlock->SetStyleNameOverride(CurrentStyleOverride);
	NewToolBarButtonBlock->SetCustomMenuDelegate(Args.CustomMenuDelegate);
	NewToolBarButtonBlock->SetOnGetMenuContent(Args.OnGetMenuContent);
	NewToolBarButtonBlock->SetGetDecoratedButtonDelegate( Args.GetDecoratedButtonDelegate );
	
	if (Args.VisibilityOverride.IsSet())
	{
		NewToolBarButtonBlock->SetVisibilityOverride(Args.VisibilityOverride);
	}	

	if (bHasUserInteractionType)
	{
		NewToolBarButtonBlock->SetStyleNameOverride(CurrentStyleOverride);
		NewToolBarButtonBlock->SetCustomMenuDelegate(Args.CustomMenuDelegate);
	}

	MultiBox->AddMultiBlock( NewToolBarButtonBlock.ToSharedRef() );

	ApplyHook(Args.ExtensionHook, EExtensionHook::After);
}

TSharedPtr<FToolBarButtonBlock> FToolBarBuilder::AddToolBarButton(
	const FUIAction& InAction,
	FName InExtensionHook,
	const TAttribute<FText>& InLabelOverride,
	const TAttribute<FText>& InToolTipOverride,
	const TAttribute<FSlateIcon>& InIconOverride,
	const EUserInterfaceActionType UserInterfaceActionType,
	FName InTutorialHighlightName,
	TAttribute<EVisibility> InVisibilityOverride,
	TAttribute<FText> InToolbarLabelOverride,
	const TAttribute<FMenuEntryResizeParams>& InResizeParams
)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	TSharedRef<FToolBarButtonBlock> NewToolBarButtonBlock(new FToolBarButtonBlock(
		InLabelOverride, InToolTipOverride, InIconOverride, InAction, UserInterfaceActionType, InToolbarLabelOverride
	));

	if ( LabelVisibility.IsSet() )
	{
		NewToolBarButtonBlock->SetLabelVisibility( LabelVisibility.GetValue() );
	}

	NewToolBarButtonBlock->SetIsFocusable(bIsFocusable);
	NewToolBarButtonBlock->SetForceSmallIcons(bForceSmallIcons);
	NewToolBarButtonBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));
	NewToolBarButtonBlock->SetStyleNameOverride(CurrentStyleOverride);
	NewToolBarButtonBlock->SetVisibilityOverride(InVisibilityOverride);
	NewToolBarButtonBlock->SetResizeParams(InResizeParams);

	UE::Slate::Private::ApplySectionOverridesToBlock(NewToolBarButtonBlock, CurrentSectionResizeParams);

	MultiBox->AddMultiBlock( NewToolBarButtonBlock );

	ApplyHook(InExtensionHook, EExtensionHook::After);
	
	return NewToolBarButtonBlock;
}

TSharedPtr<FToolBarComboButtonBlock> FToolBarBuilder::AddComboButton(
	const FUIAction& InAction,
	const FOnGetContent& InMenuContentGenerator,
	const TAttribute<FText>& InLabelOverride,
	const TAttribute<FText>& InToolTipOverride,
	const TAttribute<FSlateIcon>& InIconOverride,
	bool bInSimpleComboBox,
	FName InTutorialHighlightName,
	TAttribute<EVisibility> InVisibilityOverride,
	TAttribute<FText> InToolbarLabelOverride,
	TAttribute<EMenuPlacement> InPlacementOverride,
	const EUserInterfaceActionType InUserInterfaceActionType,
	const TAttribute<FMenuEntryResizeParams>& InResizeParams
)
{
	ApplySectionBeginning();

	TSharedRef<FToolBarComboButtonBlock> NewToolBarComboButtonBlock(new FToolBarComboButtonBlock(
		InAction, InMenuContentGenerator, InLabelOverride, InToolTipOverride, InIconOverride, bInSimpleComboBox, InToolbarLabelOverride, InPlacementOverride, InUserInterfaceActionType
	));

	if ( LabelVisibility.IsSet() )
	{
		NewToolBarComboButtonBlock->SetLabelVisibility( LabelVisibility.GetValue() );
	}

	NewToolBarComboButtonBlock->SetForceSmallIcons(bForceSmallIcons);
	NewToolBarComboButtonBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, nullptr, MultiBox->GetBlocks().Num()));
	NewToolBarComboButtonBlock->SetStyleNameOverride(CurrentStyleOverride);
	NewToolBarComboButtonBlock->SetVisibilityOverride(InVisibilityOverride);
	NewToolBarComboButtonBlock->SetResizeParams(InResizeParams);

	UE::Slate::Private::ApplySectionOverridesToBlock(NewToolBarComboButtonBlock, CurrentSectionResizeParams);

	MultiBox->AddMultiBlock( NewToolBarComboButtonBlock );

	return NewToolBarComboButtonBlock;
}

void FToolBarBuilder::AddToolbarStackButton(const TSharedPtr< const FUICommandInfo > InCommand, FName InTutorialHighlightName)
{
	ApplySectionBeginning();

	TSharedRef< FToolBarStackButtonBlock > NewToolBarStackButtonBlock(new FToolBarStackButtonBlock(InCommand.ToSharedRef(), CommandListStack.Last()));

	if (LabelVisibility.IsSet())
	{
		NewToolBarStackButtonBlock->SetLabelVisibility(LabelVisibility.GetValue());
	}

	NewToolBarStackButtonBlock->SetForceSmallIcons(bForceSmallIcons);
	NewToolBarStackButtonBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(TutorialHighlightName, InTutorialHighlightName, InCommand, MultiBox->GetBlocks().Num()));
	NewToolBarStackButtonBlock->SetStyleNameOverride(CurrentStyleOverride);

	MultiBox->AddMultiBlock(NewToolBarStackButtonBlock);
}

void FToolBarBuilder::AddToolBarWidget( TSharedRef<SWidget> InWidget, const TAttribute<FText>& InLabel, FName InTutorialHighlightName, bool bSearchable, const TAttribute<FText>& InToolTip)
{
	ApplySectionBeginning();

	const FToolBarStyle& ToolBarStyle = GetStyleSet()->GetWidgetStyle<FToolBarStyle>(GetStyleName());

	TSharedRef<SWidget> ChildWidget = InWidget;
	InWidget = 
		SNew( SVerticalBox )
		.AddMetaData<FTagMetaData>(FTagMetaData(InTutorialHighlightName))
		.ToolTipText(InToolTip)

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign( HAlign_Center )
		[
			ChildWidget
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(ToolBarStyle.LabelPadding)
		.HAlign( HAlign_Center )
		[
			SNew( STextBlock )
			.Visibility_Lambda( [LabelVisibilityCopy = LabelVisibility, ChildWidget] () -> EVisibility {
				if (FMultiBoxSettings::UseSmallToolBarIcons.Get())
					return EVisibility::Collapsed;

				if (LabelVisibilityCopy.IsSet())
				{
					return LabelVisibilityCopy.GetValue();
				}

				return ChildWidget->GetVisibility();
			})
			.IsEnabled_Lambda( [ChildWidget] () -> bool { return ChildWidget->IsEnabled(); } )
			.Text( InLabel )
			.TextStyle(&ToolBarStyle.LabelStyle)	// Smaller font for tool tip labels
		] ;

	static const TAttribute<FText> EmptyTextAttribute;

	FMenuEntryStyleParams StyleParams;
	StyleParams.bNoIndent = true;

	TSharedRef<FWidgetBlock> NewWidgetBlock = MakeShared<FWidgetBlock>(InWidget, FText::GetEmpty(), EmptyTextAttribute, StyleParams);
	MultiBox->AddMultiBlock( NewWidgetBlock );
	NewWidgetBlock->SetSearchable(bSearchable);
}

void FToolBarBuilder::AddWidget(
	TSharedRef<SWidget> InWidget,
	FName InTutorialHighlightName,
	bool bSearchable,
	EHorizontalAlignment Alignment,
	FNewMenuDelegate InCustomMenuDelegate,
	TAttribute<EVisibility> InVisibilityOverride
)
{
	FMenuEntryStyleParams StyleParams;
	StyleParams.HorizontalAlignment = Alignment;

	AddWidgetInternal(InWidget, StyleParams, InTutorialHighlightName, bSearchable, InCustomMenuDelegate, InVisibilityOverride);
}

void FToolBarBuilder::AddWidget(
	TSharedRef<SWidget> InWidget,
	const FMenuEntryStyleParams& InStyleParams,
	FName InTutorialHighlightName,
	bool bInSearchable,
	FNewMenuDelegate InCustomMenuDelegate,
	TAttribute<EVisibility> InVisibilityOverride,
	const TAttribute<FMenuEntryResizeParams>& InResizeParams
)
{
	FMenuEntryStyleParams StyleParams = InStyleParams;
	if (!StyleParams.VerticalAlignment.IsSet())
	{
		StyleParams.VerticalAlignment = VAlign_Center; // Default to centered, versions prior to 5.5 decide alignment based on label presence
	}

	AddWidgetInternal(
		InWidget, StyleParams, InTutorialHighlightName, bInSearchable, InCustomMenuDelegate, InVisibilityOverride, InResizeParams
	);
}

void FToolBarBuilder::AddWidgetInternal(
	TSharedRef<SWidget> InWidget,
	const FMenuEntryStyleParams& InStyleParams,
	FName InTutorialHighlightName,
	bool bInSearchable,
	FNewMenuDelegate InCustomMenuDelegate,
	TAttribute<EVisibility> InVisibilityOverride,
	const TAttribute<FMenuEntryResizeParams>& InResizeParams
)
{
	ApplySectionBeginning();

	TSharedRef<SWidget> ChildWidget = InWidget;
	InWidget =
		SNew( SBox )
		.WidthOverride(InStyleParams.DesiredWidthOverride)
		.HeightOverride(InStyleParams.DesiredHeightOverride)
		.AddMetaData<FTagMetaData>(FTagMetaData(InTutorialHighlightName))
		[
			ChildWidget
		];

	FMenuEntryStyleParams StyleParams = InStyleParams;
	StyleParams.bNoIndent = true;

	static const TAttribute<FText> EmptyTextAttribute;
	TSharedRef<FWidgetBlock> NewWidgetBlock = MakeShared<FWidgetBlock>(InWidget, FText::GetEmpty(), EmptyTextAttribute, StyleParams);
	MultiBox->AddMultiBlock( NewWidgetBlock );
	NewWidgetBlock->SetSearchable(bInSearchable);
	NewWidgetBlock->SetCustomMenuDelegate(InCustomMenuDelegate);
	NewWidgetBlock->SetVisibilityOverride(InVisibilityOverride);
	NewWidgetBlock->SetResizeParams(InResizeParams);

	UE::Slate::Private::ApplySectionOverridesToBlock(NewWidgetBlock, CurrentSectionResizeParams);
}

void FToolBarBuilder::AddSeparator(
	FName InExtensionHook, TAttribute<EVisibility> InVisibilityOverride, const TAttribute<FMenuEntryResizeParams>& InResizeParams
)
{
	ApplySectionBeginning();

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	TSharedRef<FToolBarSeparatorBlock> NewSeparatorBlock = MakeShared<FToolBarSeparatorBlock>(InExtensionHook);
	NewSeparatorBlock->SetStyleNameOverride(CurrentStyleOverride);
	NewSeparatorBlock->SetVisibilityOverride(InVisibilityOverride);
	NewSeparatorBlock->SetResizeParams(InResizeParams);

	UE::Slate::Private::ApplySectionOverridesToBlock(NewSeparatorBlock, CurrentSectionResizeParams);

	MultiBox->AddMultiBlock(NewSeparatorBlock);

	ApplyHook(InExtensionHook, EExtensionHook::After);
}

void FToolBarBuilder::BeginSection(FName InExtensionHook, bool bInSectionShouldHaveSeparator, const TAttribute<FMenuEntryResizeParams>& InResizeParams)
{
	checkf(CurrentSectionExtensionHook == NAME_None && !bSectionNeedsToBeApplied, TEXT("Did you forget to call EndSection()?"));

	ApplyHook(InExtensionHook, EExtensionHook::Before);

	// Do not actually apply the section header, because if this section is ended immediately
	// then nothing ever gets created, preventing empty sections from ever appearing
	bSectionNeedsToBeApplied = true;
	bSectionShouldHaveSeparator = bInSectionShouldHaveSeparator;
	CurrentSectionExtensionHook = InExtensionHook;
	CurrentSectionResizeParams = InResizeParams;

	// Do apply the section beginning if we are in developer "show me all the hooks" mode
	if (FMultiBoxSettings::DisplayMultiboxHooks.Get())
	{
		ApplySectionBeginning();
	}
	
	ApplyHook(InExtensionHook, EExtensionHook::First);
}

void FToolBarBuilder::EndSection()
{
	FName SectionExtensionHook = CurrentSectionExtensionHook;
	CurrentSectionExtensionHook = NAME_None;
	CurrentSectionResizeParams = TAttribute<FMenuEntryResizeParams>();
	bSectionNeedsToBeApplied = false;

	ApplyHook(SectionExtensionHook, EExtensionHook::After);
}

void FToolBarBuilder::ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition)
{
	if (ExtendersEnabled())
	{
		// this is a virtual function to get a properly typed "this" pointer
		auto& Extender = ExtenderStack.Top();
		if (InExtensionHook != NAME_None && Extender.IsValid())
		{
			Extender->Apply(InExtensionHook, HookPosition, *this);
		}
	}
}

void FToolBarBuilder::ApplySectionBeginning()
{
	if (bSectionNeedsToBeApplied)
	{
		if (bSectionShouldHaveSeparator
			&& (MultiBox->GetBlocks().Num() > 0 || FMultiBoxSettings::DisplayMultiboxHooks.Get()))
		{
			TSharedRef<FToolBarSeparatorBlock> NewSeparatorBlock = MakeShared<FToolBarSeparatorBlock>(CurrentSectionExtensionHook);
			NewSeparatorBlock->SetStyleNameOverride(CurrentStyleOverride);
			NewSeparatorBlock->SetResizeParams(CurrentSectionResizeParams);

			MultiBox->AddMultiBlock(NewSeparatorBlock);
		}
		bSectionNeedsToBeApplied = false;
	}
}

void FToolBarBuilder::EndBlockGroup()
{
	ApplySectionBeginning();

	TSharedRef<FGroupEndBlock> NewGroupEndBlock( new FGroupEndBlock() );
	NewGroupEndBlock->SetStyleNameOverride(CurrentStyleOverride);
	
	MultiBox->AddMultiBlock( NewGroupEndBlock );
}

void FToolBarBuilder::BeginStyleOverride(FName StyleOverrideName)
{
	CurrentStyleOverride = StyleOverrideName;
}

void FToolBarBuilder::EndStyleOverride()
{
	CurrentStyleOverride = NAME_None;
}

void FToolBarBuilder::BeginBlockGroup()
{
	ApplySectionBeginning();

	TSharedRef< FGroupStartBlock > NewGroupStartBlock( new FGroupStartBlock() );
	NewGroupStartBlock->SetStyleNameOverride(CurrentStyleOverride);

	MultiBox->AddMultiBlock( NewGroupStartBlock );
}

void FButtonRowBuilder::AddButton( const TSharedPtr< const FUICommandInfo > InCommand, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const FSlateIcon& InIconOverride )
{
	ApplySectionBeginning();

	TSharedRef< FButtonRowBlock > NewButtonRowBlock( new FButtonRowBlock( InCommand.ToSharedRef(), CommandListStack.Last(), InLabelOverride, InToolTipOverride, InIconOverride ) );

	MultiBox->AddMultiBlock( NewButtonRowBlock );
}

void FButtonRowBuilder::AddButton( const FText& InLabel, const FText& InToolTip, const FUIAction& UIAction, const FSlateIcon& InIcon, const EUserInterfaceActionType UserInterfaceActionType )
{
	ApplySectionBeginning();

	TSharedRef< FButtonRowBlock > NewButtonRowBlock( new FButtonRowBlock( InLabel, InToolTip, InIcon, UIAction, UserInterfaceActionType ) );

	MultiBox->AddMultiBlock( NewButtonRowBlock );
}

FSlimHorizontalUniformToolBarBuilder::FSlimHorizontalUniformToolBarBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender, const bool InForceSmallIcons)
	: FToolBarBuilder(EMultiBoxType::SlimHorizontalUniformToolBar, InCommandList, InCustomization, InExtender, InForceSmallIcons)
{
	const FToolBarStyle& ToolBarStyle = GetStyleSet()->GetWidgetStyle<FToolBarStyle>(GetStyleName());
}

void FSlimHorizontalUniformToolBarBuilder::AddToolBarButton(const FButtonArgs& ButtonArgs)
{
 	ApplySectionBeginning();
	ApplyHook(ButtonArgs.ExtensionHook, EExtensionHook::Before);

//	ButtonArgs.BorderBrushName = "SlimPaletteToolBar.PaletteButtonBorderBrush";
	const TSharedPtr< FToolBarButtonBlock > NewHorizontalToolBarButtonBlock( new FToolBarButtonBlock( ButtonArgs) );
    InitializeToolBarButtonBlock(StaticCastSharedPtr<FToolBarButtonBlock>(NewHorizontalToolBarButtonBlock), ButtonArgs);
}
void FToolBarBuilder::InitializeToolBarButtonBlock(TSharedPtr<FToolBarButtonBlock> ToolBarButtonBlock, const FButtonArgs& ButtonArgs)
{
	if ( LabelVisibility.IsSet() )
	{
		ToolBarButtonBlock->SetLabelVisibility( LabelVisibility.GetValue() );
	}

	ToolBarButtonBlock->SetBorderBrushName(ButtonArgs.BorderBrushName);
	ToolBarButtonBlock->SetIsFocusable(bIsFocusable);
	ToolBarButtonBlock->SetForceSmallIcons(bForceSmallIcons);
	ToolBarButtonBlock->SetTutorialHighlightName(GenerateTutorialIdentifierName(
	TutorialHighlightName, ButtonArgs.TutorialHighlightName, ButtonArgs.Command, MultiBox->GetBlocks().Num()));
	ToolBarButtonBlock->SetCustomMenuDelegate(ButtonArgs.CustomMenuDelegate);
	ToolBarButtonBlock->SetOnGetMenuContent(ButtonArgs.OnGetMenuContent);
	ToolBarButtonBlock->SetStyleNameOverride(CurrentStyleOverride);

	MultiBox->AddMultiBlock( ToolBarButtonBlock.ToSharedRef() );

	ApplyHook(ButtonArgs.ExtensionHook, EExtensionHook::After);
}
