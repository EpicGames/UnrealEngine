// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/MultiBoxCustomization.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/Commands/UICommandDragDropOp.h"
#include "Widgets/Colors/SColorBlock.h"

void SCustomToolbarPreviewWidget::Construct( const FArguments& InArgs )
{
	Content = InArgs._Content.Widget;
}

void SCustomToolbarPreviewWidget::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	ChildSlot
	[
		SNew( SBorder )
		.Padding(0.f)
		.BorderImage( FCoreStyle::Get().GetBrush("NoBorder") )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			Content.ToSharedRef()
		]
	];

	// Add this widget to the search list of the multibox and hide it
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), FText::GetEmpty(), MultiBlock->GetSearchable());

}

TSharedRef< class IMultiBlockBaseWidget > FDropPreviewBlock::ConstructWidget() const
{
	return
		SNew( SCustomToolbarPreviewWidget )
		.Visibility( EVisibility::Hidden )
		.Content()
		[
			ActualWidget->AsWidget()
		];

}
