// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBox.h"

class SCustomToolbarPreviewWidget : public SMultiBlockBaseWidget
{
public:
	SLATE_BEGIN_ARGS( SCustomToolbarPreviewWidget ) {}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;
private:
	TSharedPtr<SWidget> Content;
};

/**
 * Arbitrary Widget MultiBlock
 */
class FDropPreviewBlock
	: public FMultiBlock
{

public:

	FDropPreviewBlock( TSharedRef<const FMultiBlock> InActualBlock, TSharedRef<IMultiBlockBaseWidget> InActualWidget ) 
		: FMultiBlock( NULL, NULL )
		, ActualBlock( InActualBlock )
		, ActualWidget( InActualWidget )
	{
	}

	/** FMultiBlock interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;
	virtual bool HasIcon() const override { return GetActualBlock()->HasIcon(); }

	TSharedRef<const FMultiBlock> GetActualBlock() const { return ActualBlock.ToSharedRef(); }

private:
	TSharedPtr<const FMultiBlock> ActualBlock;
	TSharedPtr<IMultiBlockBaseWidget> ActualWidget;

};

