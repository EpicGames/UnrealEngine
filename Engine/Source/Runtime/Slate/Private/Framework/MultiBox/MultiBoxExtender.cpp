// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"


namespace EExtensionType
{
	enum Type
	{
		/** Menu bar extension (FMenuBarBuilder) */
		MenuBar,

		/** Menu extension (FMenuBuilder) */
		Menu,

		/** Toolbar extension (FToolBarBuilder) */
		ToolBar,
	};
}


class FExtensionBase
{

public:

	/** Virtual destructor needed, so that our members get cleaned up properly */
	virtual ~FExtensionBase()
	{
	}

	/** @return Returns the type of extension object.  Implement this in derived classes */
	virtual EExtensionType::Type GetType() const = 0;

	/** The ID of the extension point. */
	FName Hook;

	/** Where to hook in relation to the extension point */
	EExtensionHook::Position HookPosition;

	/** Command list to use for the actions being added to the UI */
	TSharedPtr< FUICommandList > CommandList;

};



class FMenuBarExtension : public FExtensionBase
{

public:

	/** FExtensionBase implementation */
	virtual EExtensionType::Type GetType() const override
	{
		return EExtensionType::MenuBar;
	}

	/** Called to populate the menu bar */
	FMenuBarExtensionDelegate MenuBarExtensionDelegate;
};



class FMenuExtension : public FExtensionBase
{

public:

	/** FExtensionBase implementation */
	virtual EExtensionType::Type GetType() const override
	{
		return EExtensionType::Menu;
	}

	/** Called to populate the menu */
	FMenuExtensionDelegate MenuExtensionDelegate;
};



class FToolBarExtension : public FExtensionBase
{

public:

	/** FExtensionBase implementation */
	virtual EExtensionType::Type GetType() const override
	{
		return EExtensionType::ToolBar;
	}

	/** Called to populate the menu */
	FToolBarExtensionDelegate ToolBarExtensionDelegate;
};



TSharedRef< const FExtensionBase > FExtender::AddMenuBarExtension( FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr< FUICommandList >& CommandList, const FMenuBarExtensionDelegate& MenuBarExtensionDelegate )
{
	TSharedRef< FMenuBarExtension > MenuBarExtension( new FMenuBarExtension );
	MenuBarExtension->Hook = ExtensionHook;
	MenuBarExtension->HookPosition = HookPosition;
	MenuBarExtension->CommandList = CommandList;
	MenuBarExtension->MenuBarExtensionDelegate = MenuBarExtensionDelegate;
	Extensions.Add( MenuBarExtension );

	return MenuBarExtension;
}


TSharedRef< const FExtensionBase > FExtender::AddMenuExtension( FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr< FUICommandList >& CommandList, const FMenuExtensionDelegate& MenuExtensionDelegate )
{
	TSharedRef< FMenuExtension > MenuExtension( new FMenuExtension );
	MenuExtension->Hook = ExtensionHook;
	MenuExtension->HookPosition = HookPosition;
	MenuExtension->CommandList = CommandList;
	MenuExtension->MenuExtensionDelegate = MenuExtensionDelegate;
	Extensions.Add( MenuExtension );

	return MenuExtension;
}


TSharedRef< const FExtensionBase > FExtender::AddToolBarExtension( FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr< FUICommandList >& CommandList, const FToolBarExtensionDelegate& ToolBarExtensionDelegate )
{
	TSharedRef< FToolBarExtension > ToolBarExtension( new FToolBarExtension );
	ToolBarExtension->Hook = ExtensionHook;
	ToolBarExtension->HookPosition = HookPosition;
	ToolBarExtension->CommandList = CommandList;
	ToolBarExtension->ToolBarExtensionDelegate = ToolBarExtensionDelegate;
	Extensions.Add( ToolBarExtension );

	return ToolBarExtension;
}


void FExtender::RemoveExtension( const TSharedRef< const FExtensionBase >& Extension )
{
	Extensions.Remove( Extension );
}


void FExtender::Apply( FName ExtensionHook, EExtensionHook::Position HookPosition, FMenuBarBuilder& MenuBarBuilder ) const
{
	for( auto ExtensionIt( Extensions.CreateConstIterator() ); ExtensionIt; ++ExtensionIt )
	{
		const auto& Extension = *ExtensionIt;
		if( Extension->GetType() == EExtensionType::MenuBar && Extension->Hook == ExtensionHook && Extension->HookPosition == HookPosition )
		{
			const auto& MenuBarExtension = StaticCastSharedPtr< const FMenuBarExtension >( Extension );

			if ( Extension->CommandList.IsValid() )
			{
				// Push the command list needed for this extension's menu items
				MenuBarBuilder.PushCommandList( Extension->CommandList.ToSharedRef() );
			}

			// Extend the menu!
			MenuBarExtension->MenuBarExtensionDelegate.ExecuteIfBound( MenuBarBuilder );

			if ( Extension->CommandList.IsValid() )
			{
				// Restore the original command list
				MenuBarBuilder.PopCommandList();
			}
		}
	}
}


void FExtender::Apply( FName ExtensionHook, EExtensionHook::Position HookPosition, FMenuBuilder& MenuBuilder ) const
{
	for( auto ExtensionIt( Extensions.CreateConstIterator() ); ExtensionIt; ++ExtensionIt )
	{
		const auto& Extension = *ExtensionIt;
		if( Extension->GetType() == EExtensionType::Menu && Extension->Hook == ExtensionHook && Extension->HookPosition == HookPosition )
		{
			const auto& MenuExtension = StaticCastSharedPtr< const FMenuExtension >( Extension );

			if ( Extension->CommandList.IsValid() )
			{
				// Push the command list needed for this extension's menu items
				MenuBuilder.PushCommandList( Extension->CommandList.ToSharedRef() );
			}

			// Extend the menu!
			MenuExtension->MenuExtensionDelegate.ExecuteIfBound( MenuBuilder );

			if ( Extension->CommandList.IsValid() )
			{
				// Restore the original command list
				MenuBuilder.PopCommandList();
			}
		}
	}
}


void FExtender::Apply( FName ExtensionHook, EExtensionHook::Position HookPosition, FToolBarBuilder& ToolBarBuilder ) const
{
	for( auto ExtensionIt( Extensions.CreateConstIterator() ); ExtensionIt; ++ExtensionIt )
	{
		const auto& Extension = *ExtensionIt;
		if( Extension->GetType() == EExtensionType::ToolBar && Extension->Hook == ExtensionHook && Extension->HookPosition == HookPosition )
		{
			const auto& ToolBarExtension = StaticCastSharedPtr< const FToolBarExtension >( Extension );

			if ( Extension->CommandList.IsValid() )
			{
				// Push the command list needed for this extension's menu items
				ToolBarBuilder.PushCommandList( Extension->CommandList.ToSharedRef() );
			}

			// Extend the menu!
			ToolBarExtension->ToolBarExtensionDelegate.ExecuteIfBound( ToolBarBuilder );

			if ( Extension->CommandList.IsValid() )
			{
				// Restore the original command list
				ToolBarBuilder.PopCommandList();
			}
		}
	}
}


TSharedRef<FExtender> FExtender::Combine(const TArray< TSharedPtr<FExtender> >& Extenders)
{
	TSharedRef<FExtender> OutExtender = MakeShared<FExtender>();
	for (int32 i = 0; i < Extenders.Num(); ++i)
	{
		OutExtender->Extensions.Append(Extenders[i]->Extensions);
	}
	return OutExtender;
}
