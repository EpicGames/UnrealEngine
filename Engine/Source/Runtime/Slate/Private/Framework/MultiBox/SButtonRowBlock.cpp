// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SButtonRowBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"



FButtonRowBlock::FButtonRowBlock( const TSharedPtr< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const FSlateIcon& InIconOverride )
	: FMultiBlock( InCommand, InCommandList, NAME_None, EMultiBlockType::ButtonRow )
	, LabelOverride( InLabelOverride )
	, ToolTipOverride( InToolTipOverride )
	, IconOverride( InIconOverride )
{
}

FButtonRowBlock::FButtonRowBlock( const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FSlateIcon& InIcon, const FUIAction& UIAction, const EUserInterfaceActionType InUserInterfaceActionType )
	: FMultiBlock( UIAction, NAME_None, EMultiBlockType::ButtonRow )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, IconOverride( InIcon )
	, UserInterfaceActionTypeOverride( InUserInterfaceActionType )
{

}

bool FButtonRowBlock::HasIcon() const
{
	const FSlateIcon& ActualIcon = !IconOverride.IsSet() && GetAction().IsValid() ? GetAction()->GetIcon() : IconOverride;

	if (ActualIcon.IsSet())
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();

		return IconBrush->GetResourceName() != NAME_None;
	}

	return false;
}

/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FButtonRowBlock::ConstructWidget() const
{
	return SNew( SButtonRowBlock )
			.Cursor(EMouseCursor::Default);
}



/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SButtonRowBlock::Construct( const FArguments& InArgs )
{
	this->SetForegroundColor( TAttribute<FSlateColor>::Create( TAttribute<FSlateColor>::FGetter::CreateRaw( this, &SButtonRowBlock::InvertOnHover ) ) );
}


/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SButtonRowBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	TSharedRef< const FMultiBox > MultiBox( OwnerMultiBoxWidget.Pin()->GetMultiBox() );
	
	TSharedRef< const FButtonRowBlock > ButtonRowBlock = StaticCastSharedRef< const FButtonRowBlock >( MultiBlock.ToSharedRef() );

	// Allow the block to override the action's label and tool tip string, if desired
	TAttribute<FText> ActualLabel = ButtonRowBlock->GetAction()->GetLabel();
	if (ButtonRowBlock->LabelOverride.IsSet())
	{
		ActualLabel = ButtonRowBlock->LabelOverride;
	}
	TAttribute<FText> ActualToolTip = ButtonRowBlock->GetAction()->GetDescription();
	if (ButtonRowBlock->ToolTipOverride.IsSet())
	{
		ActualToolTip = ButtonRowBlock->ToolTipOverride;
	}

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), ActualLabel.Get(), MultiBlock->GetSearchable());

	// Allow the block to override the button icon, too
	const FSlateIcon& ActualIcon = !ButtonRowBlock->IconOverride.IsSet() && ButtonRowBlock->GetAction().IsValid() ? ButtonRowBlock->GetAction()->GetIcon() : ButtonRowBlock->IconOverride;

	// If we were supplied an image than go ahead and use that, otherwise we use a null widget
	TSharedRef< SWidget > IconWidget = SNullWidget::NullWidget;
	TSharedRef< SWidget > SmallIconWidget = SNullWidget::NullWidget;
	if( ActualIcon.IsSet() )
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		const FSlateBrush* SmallIconBrush = ActualIcon.GetSmallIcon();

		if( IconBrush->GetResourceName() != NAME_None )
		{
			IconWidget =
				SNew( SImage )
				.Visibility(this, &SButtonRowBlock::GetIconVisibility, false)
				.Image( IconBrush );
		}
		if( SmallIconBrush->GetResourceName() != NAME_None )
		{
			SmallIconWidget =
				SNew( SImage )
				.Visibility(this, &SButtonRowBlock::GetIconVisibility, true)
				.Image( SmallIconBrush );
		}
	}

	// Create the content for our button
	TSharedRef< SWidget > ButtonContent =
		SNew( SVerticalBox )

			// Icon image
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 1.0f )
			// Center the icon horizontally, so that large labels don't stretch out the artwork
			.HAlign( HAlign_Center )
			[
				IconWidget
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 1.0f )
			.HAlign( HAlign_Center )
			[
				SmallIconWidget
			]

			// Label text
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 1.0f )
			.HAlign( HAlign_Center )	// Center the label text horizontally
			[
				SNew( STextBlock )
					.Text( ActualLabel )
					.Visibility( this, &SButtonRowBlock::GetIconVisibility, false )

					// Smaller font for tool tip labels
					.TextStyle( StyleSet, ISlateStyle::Join( StyleName, ".Label" ) )
			]
		;


	// What type of UI should we create for this block?
	const EUserInterfaceActionType UserInterfaceType = ButtonRowBlock->GetAction().IsValid() ? ButtonRowBlock->GetAction()->GetUserInterfaceType() : ButtonRowBlock->UserInterfaceActionTypeOverride;
	if( UserInterfaceType == EUserInterfaceActionType::Button )
	{
		ChildSlot
		[
			// Create a button
			SNew( SButton )

				// Use the tool bar item style for this button
				.ButtonStyle( StyleSet, ISlateStyle::Join( StyleName, ".Button" ) )

				// Pass along the block's tool-tip string
				.ToolTipText( ActualToolTip )

				.ContentPadding( 0.0f )

				.ForegroundColor( FSlateColor::UseForeground() )

				[
					ButtonContent
				]

				// Bind the button's "on clicked" event to our object's method for this
				.OnClicked( this, &SButtonRowBlock::OnClicked )
		];
	}
	else if( ensure( UserInterfaceType == EUserInterfaceActionType::ToggleButton ) )
	{
		ChildSlot
		[
			// Create a check box
 			SAssignNew( ToggleButton, SCheckBox )
 
 				// Use the tool bar style for this check box
				.Style(StyleSet, ISlateStyle::Join( StyleName, ".ToggleButton" ))
 
 				// Pass along the block's tool-tip string
 				.ToolTipText( ActualToolTip )
 
 				// Put some space between the content and the checkbox border
 				.Padding( 2.0f )
		
				[
					ButtonContent
				]

				// Bind the button's "on checked" event to our object's method for this
				.OnCheckStateChanged( this, &SButtonRowBlock::OnCheckStateChanged )

				// Bind the check box's "checked" state to our user interface action
				.IsChecked( this, &SButtonRowBlock::OnIsChecked )
		];
	}


	// Bind our widget's enabled state to whether or not our action can execute
	SetEnabled( TAttribute< bool >( this, &SButtonRowBlock::IsEnabled ) );

	// Bind our widget's visible state to whether or not the button should be visible
	SetVisibility( TAttribute<EVisibility>(this, &SButtonRowBlock::GetVisibility) );
}



/**
 * Called by Slate when this button is clicked
 */
FReply SButtonRowBlock::OnClicked()
{
	// Button was clicked, so trigger the action!
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	ActionList->ExecuteAction( MultiBlock->GetAction().ToSharedRef() );

	TSharedRef< const FMultiBox > MultiBox( OwnerMultiBoxWidget.Pin()->GetMultiBox() );

	// If this is a context menu, then we'll also dismiss the window after the user clicked on the item
	if( MultiBox->ShouldCloseWindowAfterMenuSelection() )
	{
		TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow( AsShared() ).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow( ParentContextMenuWindow );
	}

	return FReply::Handled();
}



/**
 * Called by Slate when this check box button is toggled
 */
void SButtonRowBlock::OnCheckStateChanged( const ECheckBoxState NewCheckedState )
{
	OnClicked();
}

/**
 * Called by slate to determine if this button should appear checked
 *
 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
 */
ECheckBoxState SButtonRowBlock::OnIsChecked() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		CheckState = ActionList->GetCheckState( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		CheckState = DirectActions.GetCheckState();
	}

	return CheckState;
}

/**
 * Called by Slate to determine if this button is enabled
 * 
 * @return True if the menu entry is enabled, false otherwise
 */
bool SButtonRowBlock::IsEnabled() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	bool bEnabled = true;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		bEnabled = ActionList->CanExecuteAction( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		bEnabled = DirectActions.CanExecute();
	}

	return bEnabled;
}


/**
 * Called by Slate to determine if this button is visible
 *
 * @return EVisibility::Visible or EVisibility::Collapsed, depending on if the button should be displayed
 */
EVisibility SButtonRowBlock::GetVisibility() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	if (ActionList.IsValid() && Action.IsValid())
	{
		return ActionList->GetVisibility( Action.ToSharedRef() );
	}
	else
	{
		return DirectActions.IsVisible();
	}
}

EVisibility SButtonRowBlock::GetIconVisibility(bool bIsASmallIcon) const
{
	return (FMultiBoxSettings::UseSmallToolBarIcons.Get() ^ bIsASmallIcon) ? EVisibility::Collapsed : EVisibility::Visible;
}


FSlateColor SButtonRowBlock::InvertOnHover() const
{
	if ( this->IsHovered() || (ToggleButton.IsValid() && ToggleButton->IsChecked()) )
	{
		return FLinearColor::Black;
	}
	else
	{
		return FSlateColor::UseForeground();
	}
}
