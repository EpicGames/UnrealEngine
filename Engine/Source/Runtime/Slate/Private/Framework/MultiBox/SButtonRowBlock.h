// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

class SCheckBox;

/**
 * Button Row MultiBlock
 */
class FButtonRowBlock
	: public FMultiBlock
{

public:

	/**
	 * Constructor
	 *
	 * @param	InCommand					The command associated with this tool bar button
	 * @param	InCommandList				The list of commands that are mapped to delegates so that we know what to execute for this button
	 * @param	InLabelOverride				Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride			Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InOverride					Optional icon to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 */
	FButtonRowBlock( const TSharedPtr< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const FSlateIcon& InIconOverride = FSlateIcon() );

	/**
	 * Constructor
	 *
	 * @param	InLabel					The button label to display
	 * @param	InToolTip				The tooltip for the button
	 * @param	InIcon					The icon to use
	 * @param	UIAction				Action to execute when the button is clicked or when state should be checked
	 * @param	UserInterfaceActionType	The style of button to display
	 */
	FButtonRowBlock( const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FSlateIcon& InIcon, const FUIAction& UIAction, const EUserInterfaceActionType InUserInterfaceActionType );

	/** FMultiBlock interface */
	virtual bool HasIcon() const override;

private:

	/**
	 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
	 *
	 * @return  MultiBlock widget object
	 */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;
private:

	// Friend our corresponding widget class
	friend class SButtonRowBlock;

	/** Optional overridden text label for this tool bar button.  If not set, then the action's label will be used instead. */
	TAttribute<FText> LabelOverride;

	/** Optional overridden tool tip for this tool bar button.  If not set, then the action's tool tip will be used instead. */
	TAttribute<FText> ToolTipOverride;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	FSlateIcon IconOverride;

	/** In the case where a command is not bound, the user interface action type to use.  If a command is bound, we
	    simply use the action type associated with that command. */
	EUserInterfaceActionType UserInterfaceActionTypeOverride;
};



/**
 * Tool bar button MultiBlock widget
 */
class SButtonRowBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SButtonRowBlock ){}
	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );


protected:


	/**
	 * Called by Slate when this tool bar button's button is clicked
	 */
	SLATE_API FReply OnClicked();


	/**
	 * Called by Slate when this tool bar check box button is toggled
	 */
	SLATE_API void OnCheckStateChanged( const ECheckBoxState NewCheckedState );

	/**
	 * Called by slate to determine if this button should appear checked
	 *
	 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
	 */
	SLATE_API ECheckBoxState OnIsChecked() const;

	/**
	 * Called by Slate to determine if this button is enabled
	 * 
	 * @return True if the menu entry is enabled, false otherwise
	 */
	SLATE_API bool IsEnabled() const;

	/**
	 * Called by Slate to determine if this button is visible
	 *
	 * @return EVisibility::Visible or EVisibility::Collapsed, depending on if the button should be displayed
	 */
	SLATE_API EVisibility GetVisibility() const;
private:
	
	/** For check toggle buttons, this is the widget that gets toggled. */
	TSharedPtr<SCheckBox> ToggleButton;

	/** Called by Slate to determine whether icons/labels are visible */
	EVisibility GetIconVisibility(bool bIsASmallIcon) const;

	/** @return inverted foreground color when this widget is hovered. */
	FSlateColor InvertOnHover() const;
};
