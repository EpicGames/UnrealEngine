// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SClippingHorizontalBox.h"

#include "Framework/Application/SlateApplication.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSeparator.h"

namespace UE::Slate
{

void PrioritizedResize(
	float InAllottedWidth,
	float InWrapButtonWidth,
	const FMargin& InWrapButtonPadding,
	int32 InWrapButtonIndex,
	TArray<FClippingInfo>& InOutClippingInfos,
	TOptional<float>& OutWrapButtonX
)
{
	double WidthOfAllChildren = 0.0;
	double NonClippingWidgetWidths = 0.0;
	for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
	{
		WidthOfAllChildren += Info.Width;

		// Find the combined width of widgets that should never clip.
		if (!Info.ResizeParams.AllowClipping.Get(FMenuEntryResizeParams::DefaultAllowClipping))
		{
			NonClippingWidgetWidths += Info.Width;
		}
	}

	// Early out if we don't need to clip.
	// Round to avoid adding a wrap button if the contents are a subpixel larger than the box.
	if (const bool bNeedsClipping = FMath::CeilToInt(WidthOfAllChildren - UE_KINDA_SMALL_NUMBER)
								  > FMath::CeilToInt(InAllottedWidth - UE_KINDA_SMALL_NUMBER);
		!bNeedsClipping)
	{
		return;
	}

	// Propagate the resize parameter's visibility overflow to the clipping info.
	for (UE::Slate::FClippingInfo& Info : InOutClippingInfos)
	{
		Info.bAppearsInOverflow = Info.ResizeParams.VisibleInOverflow.Get(FMenuEntryResizeParams::DefaultVisibleInOverflow);
	}

	const double PaddedWidthOfWrapButton = InWrapButtonWidth + InWrapButtonPadding.Left + InWrapButtonPadding.Right;

	bool bNeedsWrapButton = false;
	// Clip widgets in priority order.
	{
		// Construct a priority-sorted array of pointers into the clipping infos. Work on this copy
		// to not mess up the original sorting of the input widgets.
		TArray<UE::Slate::FClippingInfo*> PrioritySortedClippingInfos;
		for (UE::Slate::FClippingInfo& Info : InOutClippingInfos)
		{
			PrioritySortedClippingInfos.Add(&Info);
		}
		PrioritySortedClippingInfos.StableSort(
			[](const UE::Slate::FClippingInfo& A, const UE::Slate::FClippingInfo& B) -> bool
			{
				return A.ResizeParams.ClippingPriority.Get(FMenuEntryResizeParams::DefaultClippingPriority)
					 > B.ResizeParams.ClippingPriority.Get(FMenuEntryResizeParams::DefaultClippingPriority);
			}
		);

		// Iterate our arranged children in resize-priority order, accumulate their width, and start removing widgets
		// once their combined width is larger than our allotted width. Start counting at the width required by
		// non-clipping widgets to make sure we start clipping clippable widgets immediately once non-clippable widths
		// and the accumulator sum up to our allotted width.
		//
		// If we find that we need a wrap button, restart the clipping loop and include the width of the wrap button in
		// the initial accumulator width.
		for (int32 NumClippingAttemptsLeft = 1; NumClippingAttemptsLeft > 0; --NumClippingAttemptsLeft)
		{
			double WidthAccumulator = NonClippingWidgetWidths;
			// If we need a wrap button, clip widgets as if the wrap button is already taking up space.
			if (bNeedsWrapButton)
			{
				WidthAccumulator += PaddedWidthOfWrapButton;
			}

			for (int32 InfoIndex = 0; InfoIndex < PrioritySortedClippingInfos.Num(); ++InfoIndex)
			{
				UE::Slate::FClippingInfo& Info = *PrioritySortedClippingInfos[InfoIndex];

				// Don't clip non-clipping widgets and also don't count their width, we've already done that above.
				if (!Info.ResizeParams.AllowClipping.Get(FMenuEntryResizeParams::DefaultAllowClipping))
				{
					continue;
				}

				WidthAccumulator += Info.Width;

				// We are wider than our allotted width, so mark this widget for clipping.
				if (WidthAccumulator > InAllottedWidth)
				{
					Info.bWasClipped = true;

					// If this widget appears in an overflow menu, but we haven't taken the wrap button width
					// into account, we need to restart the clipping loop and do it over again while taking the wrap button width into account.
					if (Info.bAppearsInOverflow && !bNeedsWrapButton)
					{
						bNeedsWrapButton = true;

						++NumClippingAttemptsLeft;
						break;
					}
				}
			}
		}
	}

	// Sort blocks by X position. Use stable sort to keep the original order if priorities are the same.
	InOutClippingInfos.StableSort(
		[](const UE::Slate::FClippingInfo& A, const UE::Slate::FClippingInfo& B) -> bool
		{
			return A.X < B.X;
		}
	);

	// Iterate widgets left to right and if clipped widgets are found, translate all subsequent widgets to the left to compensate.
	double WidthOfRemovedWidgets = 0.0;
	for (UE::Slate::FClippingInfo& Info : InOutClippingInfos)
	{
		if (Info.bWasClipped)
		{
			WidthOfRemovedWidgets += Info.Width;
			continue;
		}
		Info.X -= WidthOfRemovedWidgets;
	}

	// Here we want to EXPAND again to fill up the whole post-clip toolbar if possible. The reason is we've reduced
	// the width of the toolbar in "integer steps", i.e. by clipping whole widgets, and in general we'll have
	// some slack space left.
	//
	// An example of widgets to expand are variable-sized spacers that were previously shrunk to make things fit.
	{
		// Count our children that can stretch that were not clipped above.
		const int32 NumStretchWidgets = InOutClippingInfos
											.FilterByPredicate(
												[](const UE::Slate::FClippingInfo& Info)
												{
													return Info.bIsStretchable && !Info.bWasClipped;
												}
											)
											.Num();

		if (NumStretchWidgets > 0)
		{
			// Distribute our extra space to our stretching children.
			const double ExtraSpace = InAllottedWidth - WidthOfAllChildren
									- (bNeedsWrapButton ? PaddedWidthOfWrapButton : 0.0) + WidthOfRemovedWidgets;
			const double ExtraSpacePerStretchChild = ExtraSpace / NumStretchWidgets;

			if (ExtraSpace > 0.0)
			{
				double StretchSpaceAdded = 0.0;
				for (UE::Slate::FClippingInfo& Info : InOutClippingInfos)
				{
					if (Info.bWasClipped)
					{
						continue;
					}

					Info.X += StretchSpaceAdded;
					if (Info.bIsStretchable)
					{
						Info.Width += ExtraSpacePerStretchChild;
						StretchSpaceAdded += ExtraSpacePerStretchChild;
					}
				}
			}
		}
	}

	// Add the wrap button. Move widgets to make space for it and then position it.
	const bool bHasSpaceForWrapButton = InWrapButtonWidth <= InAllottedWidth;
	if (bNeedsWrapButton && bHasSpaceForWrapButton)
	{
		// Use an array that excludes clipped widgets for sanity while indexing. Each element points
		// into InOutClippingInfos, so we don't need to consolidate the arrays later.
		TArray<FClippingInfo*> NonClippedInfos;
		for (FClippingInfo& Info : InOutClippingInfos)
		{
			if (!Info.bWasClipped)
			{
				NonClippedInfos.Add(&Info);
			}
		}

		const bool bIsIndexingFromLeft = InWrapButtonIndex >= 0;

		const int32 StolenIndex = FMath::Clamp(
			bIsIndexingFromLeft ? InWrapButtonIndex : NonClippedInfos.Num() + InWrapButtonIndex, 0, NonClippedInfos.Num() - 1
		);

		// Sum up the width of widgets to the left of the wrap button index.
		double WidthOfWidgetsBeforeStolenIndex = 0.0;
		for (int32 Index = 0; Index < StolenIndex; ++Index)
		{
			WidthOfWidgetsBeforeStolenIndex += NonClippedInfos[Index]->Width;
		}

		OutWrapButtonX = 0.0;
		int32 IndexFromWhichToPushToTheRight = 0;
		// If we're indexing from the left, we want to add the wrap button to the left of the widget originally at the stolen index.
		if (bIsIndexingFromLeft)
		{
			OutWrapButtonX = WidthOfWidgetsBeforeStolenIndex + InWrapButtonPadding.Left;
			IndexFromWhichToPushToTheRight = StolenIndex;
		}
		// If we're indexing from the right, we want to add the wrap button to the right of the widget originally at the stolen index.
		else
		{
			const double WidthOfWidgetAtStolenIndex =
				StolenIndex < NonClippedInfos.Num() ? NonClippedInfos[StolenIndex]->Width : 0.0;

			OutWrapButtonX = WidthOfWidgetsBeforeStolenIndex + WidthOfWidgetAtStolenIndex + InWrapButtonPadding.Left;
			IndexFromWhichToPushToTheRight = StolenIndex + 1;
		}

		// Special case: if the wrap button is set to appear at the right-most index (WrapButtonIndex = -1), then always
		// position it to the right, allowing additional space to appear to the left of the wrap button.
		if (InWrapButtonIndex == -1)
		{
			OutWrapButtonX = InAllottedWidth - InWrapButtonWidth - InWrapButtonPadding.Right;
		}

		for (int32 Index = IndexFromWhichToPushToTheRight; !NonClippedInfos.IsEmpty() && Index < NonClippedInfos.Num();
			 ++Index)
		{
			NonClippedInfos[Index]->X += PaddedWidthOfWrapButton;
		}
	}
}

} // namespace UE::Slate

void SClippingHorizontalBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// If WrapButton hasn't been initialized, that means AddWrapButton() hasn't
	// been called and this method isn't going to behave properly
	check(WrapButton.IsValid());

	SHorizontalBox::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Clear our previously clipped widgets. We'll repopulate them below.
	ClippedWidgets.Reset();

	// Build clipping info for all children.
	TArray<UE::Slate::FClippingInfo> ClippingInfos;
	for (int i = 0; i < ArrangedChildren.Num() - 1 /* Skip the wrap button. */; ++i)
	{
		const FArrangedWidget& Child = ArrangedChildren[i];

		UE::Slate::FClippingInfo& Info = ClippingInfos.AddDefaulted_GetRef();

		Info.Widget = Child.Widget;
		if (OnGetWidgetResizeParams.IsBound())
		{
			Info.ResizeParams = OnGetWidgetResizeParams.Execute(Child.Widget);
		}
		Info.X = Child.Geometry.GetLocalPositionAtCoordinates(FVector2f::ZeroVector).X;
		Info.Width = Child.Geometry.GetLocalSize().X;

		// Set Info.bIsStretchable by finding if the slot of Info.Widget uses SizeRule_StretchContent.
		for (int32 ChildrenIndex = Children.Num() - 1; ChildrenIndex >= 0; --ChildrenIndex)
		{
			if (Info.Widget == Children[ChildrenIndex].GetWidget())
			{
				if (Children[ChildrenIndex].GetSizeRule() == FSizeParam::SizeRule_Stretch
					|| Children[ChildrenIndex].GetSizeRule() == FSizeParam::SizeRule_StretchContent)
				{
					Info.bIsStretchable = true;
					break;
				}
			}
		}
	}

	const float InAllottedWidth = AllottedGeometry.GetLocalSize().X;
	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);
	TOptional<float> WrapButtonX;

	if (bAllowWrapButton)
	{
		UE::Slate::PrioritizedResize(
			InAllottedWidth, WrapButtonWidth, ToolBarStyle.WrapButtonStyle.Padding, WrapButtonIndex, ClippingInfos, WrapButtonX
		);
	}
	else
	{
		// We don't care about WrapButton stuff here, so use placeholder values.
		UE::Slate::PrioritizedResize(
			InAllottedWidth, 0.0, FMargin(0.0f), 0, ClippingInfos, WrapButtonX
		);
	}

	const int32 NumClippedWidgets = ClippingInfos
										.FilterByPredicate(
											[](const UE::Slate::FClippingInfo& Info)
											{
												return Info.bWasClipped;
											}
										)
										.Num();

	if (NumClippedWidgets == 0)
	{
		// None of the children are being clipped, so remove the wrap button and early out.
		ArrangedChildren.Remove(ArrangedChildren.Num() - 1);

		return;
	}

	for (const UE::Slate::FClippingInfo& Info : ClippingInfos)
	{
		if (Info.bWasClipped && Info.bAppearsInOverflow)
		{
			ClippedWidgets.Add(Info.Widget.ToWeakPtr());
		}
	}

	// Position all children using the clipping information.
	for (int32 Index = 0; Index < ArrangedChildren.Num() - 1; ++Index)
	{
		const UE::Slate::FClippingInfo* const Info = ClippingInfos.FindByPredicate(
			[Widget = ArrangedChildren[Index].Widget](const UE::Slate::FClippingInfo& Info)
			{
				return Info.Widget == Widget;
			}
		);

		check(Info);

		// Remove the widget if it was clipped.
		if (Info->bWasClipped)
		{
			ArrangedChildren.Remove(Index);
			--Index;
			continue;
		}

		FGeometry& Geometry = ArrangedChildren[Index].Geometry;
		const FVector2D WrapButtonSize = FVector2D(Info->Width, Geometry.GetLocalSize().Y);
		Geometry = AllottedGeometry.MakeChild(WrapButtonSize, FSlateLayoutTransform(FVector2f(Info->X, 0)));
	}
	
	// Position or remove the wrap button
	if (const float* WrapX = WrapButtonX.GetPtrOrNull())
	{
		const FMargin& Padding = ToolBarStyle.WrapButtonStyle.Padding;  
	
		FGeometry& WrapButtonGeometry = ArrangedChildren[ArrangedChildren.Num() - 1].Geometry;
		FVector2f WrapButtonSize(WrapButtonWidth, WrapButtonGeometry.GetLocalSize().Y);
		FVector2f WrapButtonPosition(*WrapX, 0.0f);
		
		// Allow configured padding to break the widget out of its normal bounds
		if (Padding.Left < 0.f)
		{
			WrapButtonSize.X -= Padding.Left;
			WrapButtonPosition.X -= Padding.Left;
		}
		
		if (Padding.Right < 0.f)
		{
			WrapButtonSize.X -= Padding.Right;
		}
		
		if (Padding.Top < 0.f)
		{
			WrapButtonSize.Y -= Padding.Top;
			WrapButtonPosition.Y += Padding.Top;	
		}
		
		if (Padding.Bottom < 0.f)
		{
			WrapButtonSize.Y -= Padding.Bottom;	
		}
		
		WrapButtonGeometry = AllottedGeometry.MakeChild(WrapButtonSize, FSlateLayoutTransform(WrapButtonPosition));
	}
	else
	{
		ArrangedChildren.Remove(ArrangedChildren.Num() - 1);
	}
}

int32 SClippingHorizontalBox::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	return SHorizontalBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FVector2D SClippingHorizontalBox::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D Size = SBoxPanel::ComputeDesiredSize(LayoutScaleMultiplier);
	{
		// If the wrap button isn't being shown, subtract it's size from the total desired size
		const SBoxPanel::FSlot& Child = Children[Children.Num() - 1];
		const FVector2D& ChildDesiredSize = Child.GetWidget()->GetDesiredSize();
		Size.X -= ChildDesiredSize.X;
	}
	return Size;
}

void SClippingHorizontalBox::Construct( const FArguments& InArgs )
{
	OnWrapButtonClicked = InArgs._OnWrapButtonClicked;
	StyleSet = InArgs._StyleSet;
	StyleName = InArgs._StyleName;
	bIsFocusable = InArgs._IsFocusable;
	OnGetWidgetResizeParams = InArgs._OnGetWidgetResizeParams;

	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);
	bAllowWrapButton = InArgs._AllowWrapButton.Get(ToolBarStyle.bAllowWrapButton);
}

void SClippingHorizontalBox::AddWrapButton()
{
	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

	// Construct the wrap button used in toolbars and menubars
	// Always allow this to be focusable to prevent the menu from collapsing during interaction
	
	const FWrapButtonStyle& WrapButtonStyle = ToolBarStyle.WrapButtonStyle;
	
	const FComboButtonStyle* ComboButtonStyle = WrapButtonStyle.ComboButtonStyle.GetPtrOrNull();
	const FButtonStyle* ButtonStyle = nullptr;
	if (!ComboButtonStyle)
	{
		ComboButtonStyle = &FAppStyle::Get().GetWidgetStyle< FComboButtonStyle >( "ComboButton" );
		ButtonStyle = &ToolBarStyle.ButtonStyle;
	}
	
	WrapButton = 
		SNew( SComboButton )
		.HasDownArrow( WrapButtonStyle.bHasDownArrow )
		.ComboButtonStyle(ComboButtonStyle)
		.ButtonStyle(ButtonStyle)
		.ToolTipText( NSLOCTEXT("Slate", "ExpandToolbar", "Click to expand toolbar") )
		.OnGetMenuContent( OnWrapButtonClicked )
		.Cursor( EMouseCursor::Default )
		.OnMenuOpenChanged(this, &SClippingHorizontalBox::OnWrapButtonOpenChanged)
		.IsFocusable(true)
		.ButtonContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(&WrapButtonStyle.ExpandBrush)
		];

	TSharedRef<SWidget> RootWidget = WrapButton.ToSharedRef();

	if (WrapButtonStyle.bIncludeSeparator)
	{
		TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);
		
		if (WrapButtonStyle.WrapButtonIndex != 0) // Not at the start
		{
			// Insert a separator before
			Box->AddSlot()
			.AutoWidth()
			.Padding(WrapButtonStyle.SeparatorPadding.Get(ToolBarStyle.SeparatorPadding))
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Thickness(WrapButtonStyle.SeparatorThickness.Get(ToolBarStyle.SeparatorThickness))
				.SeparatorImage(&WrapButtonStyle.SeparatorBrush.Get(ToolBarStyle.SeparatorBrush))
			];	
		}
		
		Box->AddSlot()
		.AutoWidth()
		[
			RootWidget
		];
		
		if (WrapButtonStyle.WrapButtonIndex != -1) // Not at the end 
		{
			// Insert a separator after
			Box->AddSlot()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Thickness(WrapButtonStyle.SeparatorThickness.Get(ToolBarStyle.SeparatorThickness))
				.SeparatorImage(&WrapButtonStyle.SeparatorBrush.Get(ToolBarStyle.SeparatorBrush))
			];
		}
	
		RootWidget = Box;
	}

	// Perform a prepass to get a valid DesiredSize value below
	RootWidget->SlatePrepass(1.0f);
	WrapButtonWidth = bAllowWrapButton ? RootWidget->GetDesiredSize().X : 0.0f;
	WrapButtonIndex = WrapButtonStyle.WrapButtonIndex;

	// Add the wrap button
	AddSlot()
	.FillWidth(0.0f) // Effectively makes this widget 0 width, so it exists as a slot/child, but isn't considered for layout
	.Padding(0.f)
	[
		RootWidget
	];
}

void SClippingHorizontalBox::OnWrapButtonOpenChanged(bool bIsOpen)
{
	if (bIsOpen && !WrapButtonOpenTimer.IsValid())
	{
		WrapButtonOpenTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SClippingHorizontalBox::UpdateWrapButtonStatus));
	}
	else if(!bIsOpen && WrapButtonOpenTimer.IsValid())
	{
		UnRegisterActiveTimer(WrapButtonOpenTimer.ToSharedRef());
		WrapButtonOpenTimer.Reset();
	}
}

EActiveTimerReturnType SClippingHorizontalBox::UpdateWrapButtonStatus(double CurrentTime, float DeltaTime)
{
	if (!WrapButton->IsOpen())
	{
		WrapButton->SetIsOpen(false);
		WrapButtonOpenTimer.Reset();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}
