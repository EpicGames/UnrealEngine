// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/SlateDelegates.h"
#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class SComboButton;

DECLARE_DELEGATE_RetVal_OneParam(FMenuEntryResizeParams, FOnGetWidgetResizeParams, const TSharedRef<SWidget>&);

namespace UE::Slate
{

struct FClippingInfo
{
	TSharedPtr<SWidget> Widget;
	FMenuEntryResizeParams ResizeParams;
	double X = 0.0;
	double Width = 0.0;
	bool bIsStretchable = false;
	bool bAppearsInOverflow = false;
	bool bWasClipped = false;
};

void PrioritizedResize(
	float InAllottedWidth,
	float InWrapButtonWidth,
	const FMargin& InWrapButtonPadding,
	int32 InWrapButtonIndex,
	TArray<FClippingInfo>& InOutClippingInfos,
	TOptional<float>& OutWrapButtonX
);

}; // namespace UE::Slate


template <>
struct TWidgetTypeTraits<class SClippingHorizontalBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Specialized control for handling the clipping of toolbars and menubars */
class SClippingHorizontalBox : public SHorizontalBox
{
public:
	SLATE_BEGIN_ARGS(SClippingHorizontalBox) 
		: _AllowWrapButton(true)
		, _StyleSet(&FCoreStyle::Get())
		, _StyleName(NAME_None)
		, _IsFocusable(true) { }
		SLATE_ARGUMENT(TOptional<bool>, AllowWrapButton)
		SLATE_ARGUMENT(FOnGetContent, OnWrapButtonClicked)
		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		SLATE_ARGUMENT(FName, StyleName)
		SLATE_ARGUMENT(bool, IsFocusable)
		SLATE_ARGUMENT(FOnGetWidgetResizeParams, OnGetWidgetResizeParams)
	SLATE_END_ARGS()

	/** SWidget interface */
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	/** SPanel interface */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	/** Construct this widget */
	void Construct( const FArguments& InArgs );

	/** Adds the wrap button */
	void AddWrapButton();

	TArray<TWeakPtr<SWidget>> GetClippedWidgets() const
	{
		return ClippedWidgets;
	}

private:
	void OnWrapButtonOpenChanged(bool bIsOpen);
	EActiveTimerReturnType UpdateWrapButtonStatus(double CurrentTime, float DeltaTime);

	/** The button that is displayed when a toolbar or menubar is clipped */
	TSharedPtr<SComboButton> WrapButton;

	/** Whether or not to (ever) produce a wrap button */
	bool bAllowWrapButton = true;

	/** Callback for when the wrap button is clicked */
	FOnGetContent OnWrapButtonClicked;

	TSharedPtr<FActiveTimerHandle> WrapButtonOpenTimer;

	/** Can the wrap button be focused? */
	bool bIsFocusable = true;

	float WrapButtonWidth = 0.0f; // Fixed width, initialized after button creation
	int32 WrapButtonIndex = -1; // 0 is left-most index, -1 is right-most index (Python style).

	/** The style to use */
	const ISlateStyle* StyleSet = nullptr;
	FName StyleName;

	FOnGetWidgetResizeParams OnGetWidgetResizeParams;

	mutable TArray<TWeakPtr<SWidget>> ClippedWidgets;
};
