// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SClippingVerticalBox.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/StyleColors.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ClippingVerticalBox"

void SClippingVerticalBox::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	// If WrapButton hasn't been initialized, that means AddWrapButton() hasn't 
	// been called and this method isn't going to behave properly
	check(WrapButton.IsValid());

	ClippedWidgets.Empty();
	ClippedIndices.Empty();

	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

	SVerticalBox::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Remove children that are clipped by the allotted geometry
	const int32 NumChildren = ArrangedChildren.Num();
	const int32 OverflowButtonIndex = NumChildren - 1;
	LastToolBarButtonIndex = OverflowButtonIndex - 1;
	const bool bHasLabel = ToolBarStyle.bShowLabels;
	
	const int32 OverflowButtonSize =  ToolBarStyle.ButtonContentMaxWidth;
	
	for (int32 ChildIdx = LastToolBarButtonIndex; ChildIdx >= 0; --ChildIdx)
	{
		const FArrangedWidget& CurrentWidget = ArrangedChildren[ChildIdx];
		const FVector2D CurrentWidgetLocalPosition = CurrentWidget.Geometry.GetLocalPositionAtCoordinates(FVector2D::ZeroVector);

		/* if we're not on the last toolbar button, the button furthest Y should also take into account
		 the height of the overflow button so as not to be positioned over it */
		const int32 CurrentWidgetMaxY = LastToolBarButtonIndex != ChildIdx ?
			CurrentWidgetLocalPosition.Y + CurrentWidget.Geometry.Size.Y + OverflowButtonSize :
			CurrentWidgetLocalPosition.Y + CurrentWidget.Geometry.Size.Y; 
		
		if ( CurrentWidgetMaxY > AllottedGeometry.Size.Y)
		{
			// Insert the clipped widget at the start of the array to keep the array sorted with
			// the top-most clipped widget first and the bottom-most clipped widget last.
			ClippedWidgets.Insert(CurrentWidget.Widget.ToWeakPtr(), 0);
			ClippedIndices.Add(ChildIdx);

			ArrangedChildren.Remove(ChildIdx);
		}
		else if (LastToolBarButtonIndex == ChildIdx)
		{
			ArrangedChildren.Remove(OverflowButtonIndex);	
			return;
		}
	}
	FArrangedWidget& ArrangedButton = ArrangedChildren[ArrangedChildren.Num() - 1];
	FVector2D Size = ArrangedButton.Geometry.GetLocalSize();
	Size.Y = OverflowButtonSize;
	ArrangedButton.Geometry = AllottedGeometry.MakeChild(Size, FSlateLayoutTransform(AllottedGeometry.GetLocalSize() - Size));
}

int32 SClippingVerticalBox::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	return SVerticalBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FVector2D SClippingVerticalBox::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D Size = SBoxPanel::ComputeDesiredSize(LayoutScaleMultiplier);
	{
		// If the wrap button isn't being shown, subtract it's size from the total desired size
		const SBoxPanel::FSlot& Child = Children[Children.Num() - 1];
		const FVector2D& ChildDesiredSize = Child.GetWidget()->GetDesiredSize();
		Size.Y -= ChildDesiredSize.Y;
	}
	return Size;
}

void SClippingVerticalBox::Construct( const FArguments& InArgs )
{
	OnWrapButtonClicked = InArgs._OnWrapButtonClicked;
	StyleSet = InArgs._StyleSet;
	StyleName = InArgs._StyleName;
	bIsFocusable = InArgs._IsFocusable;
	SelectedIndex = InArgs._SelectedIndex;
}

void SClippingVerticalBox::AddWrapButton()
{
	InitializeWrapButton( WrapButton, false );
	InitializeWrapButton( SelectedWrapButton, true );

	AddSlot()
	.Padding( 0.f )
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.Padding(0.f)
		[
			WrapButton.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.Padding(0.f)
		[
			SelectedWrapButton.ToSharedRef()
		]

	];
 
}

void SClippingVerticalBox::InitializeWrapButton( TSharedPtr<SComboButton>& Button, bool bCreateSelectedAppearance  )
{
	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);
	const TSharedRef<SImage> IconWidget =
		SNew( SImage )
		.Visibility(EVisibility::HitTestInvisible)
		.Image( &ToolBarStyle.WrapButtonStyle.ExpandBrush );


	Style = ToolBarStyle.ButtonStyle;
	Style.SetNormalPadding(0 );
	Style.SetPressedPadding( 0 );
	
	SelectedStyle = ToolBarStyle.ButtonStyle;
	SelectedStyle.SetNormalPadding(0 );
	SelectedStyle.SetPressedPadding( 0 );
	SelectedStyle.SetNormal( FSlateRoundedBoxBrush( FStyleColors::Primary, 4.f,
		FLinearColor(0, 0, 0, .8f), 0.5) );

	// Construct the wrap button used in toolbars and menu bars
	// Always allow this to be focusable to prevent the menu from collapsing during interaction
	// clang-format off
	Button =
		SNew( SComboButton )
		.HasDownArrow( false )
		.Visibility_Lambda( [this, bCreateSelectedAppearance] ()
		{
			const bool bShowingSelected = ClippedIndices.Contains(SelectedIndex.Get());
			if ( bCreateSelectedAppearance )
			{
				return bShowingSelected ? EVisibility::Visible : EVisibility::Collapsed;				
			}
			return !bShowingSelected ? EVisibility::Visible : EVisibility::Collapsed;
		})
		.ButtonStyle( bCreateSelectedAppearance ? &SelectedStyle : &Style )
		.ContentPadding( FMargin(2.f, 8.f) )	
		.ToolTipText( NSLOCTEXT("Slate", "ExpandToolbar", "Click to expand toolbar") )
		.OnGetMenuContent_Lambda( [ this ] ()
		{
			TSharedRef<SWidget> Widget = OnWrapButtonClicked.Execute();
			return Widget;
		})
		.Cursor( EMouseCursor::Default )
		.OnMenuOpenChanged(this, &SClippingVerticalBox::OnWrapButtonOpenChanged)
		.IsFocusable(true)
		.ButtonContent()
		[
		SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(  ToolBarStyle.bShowLabels ? ToolBarStyle.IconPaddingWithVisibleLabel : ToolBarStyle.IconPadding )
				.AutoHeight()
				.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
				[
					IconWidget
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Visibility( ToolBarStyle.bShowLabels ? EVisibility::Visible : EVisibility::Collapsed )
					.Text( LOCTEXT("ClippingVerticalBox.Icon.More", "More") )
					.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
				]
			
		];
	// clang-format on
}

void SClippingVerticalBox::OnWrapButtonOpenChanged(bool bIsOpen)
{
	if (bIsOpen && !WrapButtonOpenTimer.IsValid())
	{
		WrapButtonOpenTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SClippingVerticalBox::UpdateWrapButtonStatus));
	}
	else if(!bIsOpen && WrapButtonOpenTimer.IsValid())
	{
		UnRegisterActiveTimer(WrapButtonOpenTimer.ToSharedRef());
		WrapButtonOpenTimer.Reset();
	}
}

EActiveTimerReturnType SClippingVerticalBox::UpdateWrapButtonStatus(double CurrentTime, float DeltaTime)
{
	if (!WrapButton->IsOpen())
	{
		WrapButton->SetIsOpen(false);
		WrapButtonOpenTimer.Reset();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

#undef LOCTEXT_NAMESPACE