// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class SComboButton;

template <>
struct TWidgetTypeTraits<class SClippingVerticalBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Specialized control for handling the clipping of toolbars and menubars */
class SClippingVerticalBox : public SVerticalBox
{
public:
	SLATE_BEGIN_ARGS(SClippingVerticalBox) 
		: _StyleSet(&FCoreStyle::Get())
		, _StyleName(NAME_None)
		, _LabelVisibility(EVisibility::Visible)
		, _IsFocusable(true)
		, _SelectedIndex(INDEX_NONE )
		{ }

		SLATE_ARGUMENT(FOnGetContent, OnWrapButtonClicked)
		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		SLATE_ARGUMENT(FName, StyleName)
		SLATE_ATTRIBUTE( EVisibility, LabelVisibility)
		SLATE_ARGUMENT(bool, IsFocusable)
		SLATE_ATTRIBUTE( int32, SelectedIndex )
	SLATE_END_ARGS()

	/** SWidget interface */
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	/** SPanel interface */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	/** Construct this widget */
	void Construct( const FArguments& InArgs );

	/** Adds the wrap button */
	void AddWrapButton();

	TArray<TWeakPtr<SWidget>> GetClippedWidgets() const
	{
		return ClippedWidgets;
	}

private:
	void OnWrapButtonOpenChanged(bool bIsOpen);
	EActiveTimerReturnType UpdateWrapButtonStatus(double CurrentTime, float DeltaTime);

	/**
	 * Initializes a wrap button that can handle clipped content
	 * 
	 * @param Button the SComboButton that will make up the clipped overflow button
	 * @param bCreateSelectedAppearance if true, the button will have a "selected" appearance to denote that something within it is selected
	 */
	void InitializeWrapButton( TSharedPtr<SComboButton>& Button, bool bCreateSelectedAppearance );
	
private:
	/** The button that is displayed when a toolbar or menubar is clipped and something within it is not selected */
	TSharedPtr<SComboButton> WrapButton;

	/** The button that is displayed when a toolbar or menubar is clipped and something within the clipped content is selected */
	TSharedPtr<SComboButton> SelectedWrapButton;

	/** Callback for when the wrap button is clicked */
	FOnGetContent OnWrapButtonClicked;

	TSharedPtr<FActiveTimerHandle> WrapButtonOpenTimer;

	/** Can the wrap button be focused? */
	bool bIsFocusable = true;

	/** The style to use */
	const ISlateStyle* StyleSet = nullptr;

	/** the button style for the clipped content button when nothing within the clipped content is selected */
	FButtonStyle Style;
	
	/** the button style for the clipped content button when something within the clipped content is selected */
	FButtonStyle SelectedStyle;

	/** the index in the toolbar that is currently selected */
	TAttribute<int32> SelectedIndex = INDEX_NONE;

	/** the last index in the toolbar which is not clipped by the clipped content button */
	mutable int32 LastToolBarButtonIndex = INDEX_NONE;

	FName StyleName;

	mutable TArray<TWeakPtr<SWidget>> ClippedWidgets;
	mutable TArray<int32> ClippedIndices;
};
