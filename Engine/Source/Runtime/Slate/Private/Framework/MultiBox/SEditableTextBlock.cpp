// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SEditableTextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"


FEditableTextBlock::FEditableTextBlock( const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, const TAttribute< FText >& InTextAttribute, bool bInReadOnly, const FOnTextCommitted& InOnTextCommitted, const FOnTextChanged& InOnTextChanged, const FOnVerifyTextChanged& InOnVerifyTextChanged )
	: FMultiBlock( FUIAction(), NAME_None, EMultiBlockType::EditableText ),
	  LabelOverride( InLabel ),
	  ToolTipOverride( InToolTip ),
	  IconOverride( InIcon ),
	  TextAttribute( InTextAttribute ),
	  OnTextCommitted( InOnTextCommitted ),
	  OnTextChanged( InOnTextChanged ),
	  OnVerifyTextChanged( InOnVerifyTextChanged ),
	  bReadOnly( bInReadOnly )
{ }

bool FEditableTextBlock::HasIcon() const
{
	const TSharedPtr< const FUICommandInfo >& UICommand = GetAction();
	const FSlateIcon ActionIcon = UICommand.IsValid() ? UICommand->GetIcon() : FSlateIcon();

	const FSlateIcon& ActualIcon = !IconOverride.IsSet() ? ActionIcon : IconOverride;

	if (ActualIcon.IsSet())
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		return (IconBrush->GetResourceName() != NAME_None);
	}

	return false;
}


TSharedRef<class IMultiBlockBaseWidget> FEditableTextBlock::ConstructWidget() const
{
	return SNew( SEditableTextBlock )
		.Cursor(EMouseCursor::Default);
}


void SEditableTextBlock::BuildMultiBlockWidget( const ISlateStyle* StyleSet, const FName& StyleName )
{
	TSharedPtr< const FMultiBox > MultiBox = OwnerMultiBoxWidget.Pin()->GetMultiBox();
	TSharedPtr< const FEditableTextBlock > EditableTextBlock = StaticCastSharedRef< const FEditableTextBlock >( MultiBlock.ToSharedRef() );


	// Tool tips are optional so if the tool tip override is empty and there is no UI command just use the empty tool tip.
	const TSharedPtr< const FUICommandInfo >& UICommand = EditableTextBlock->GetAction();

	const TAttribute<FText>& Label = !EditableTextBlock->LabelOverride.IsBound() && EditableTextBlock->LabelOverride.Get().IsEmpty() && UICommand.IsValid() ? UICommand->GetLabel() : EditableTextBlock->LabelOverride;
	const TAttribute<FText>& TextBlockToolTip = !EditableTextBlock->ToolTipOverride.IsBound() && EditableTextBlock->ToolTipOverride.Get().IsEmpty() && UICommand.IsValid() ? UICommand->GetDescription() : EditableTextBlock->ToolTipOverride;
	const bool bHasLabel = !Label.Get().IsEmpty();

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), Label.Get(), MultiBlock->GetSearchable());

	// See if the action is valid and if so we will use the actions icon if we dont override it later
	const FSlateIcon ActionIcon = UICommand.IsValid() ? UICommand->GetIcon() : FSlateIcon();

	// Allow the block to override the tool bar icon, too
	const FSlateIcon& ActualIcon = !EditableTextBlock->IconOverride.IsSet() ? ActionIcon : EditableTextBlock->IconOverride;

	// If we were supplied an image than go ahead and use that, otherwise we use a null widget
	TSharedRef< SWidget > IconWidget = SNullWidget::NullWidget;
	if( ActualIcon.IsSet() )
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		if( IconBrush->GetResourceName() != NAME_None )
		{
			IconWidget = SNew( SImage )
				.Image( IconBrush );
		}
	}

	const float MenuIconSize = StyleSet->GetFloat(StyleName, ".MenuIconSize", 16.f);

	ChildSlot
	[
		SNew( SHorizontalBox )
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpacer)
			.Size( FVector2D(MenuIconSize + 3, MenuIconSize) )
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SBox )
			.Visibility(IconWidget != SNullWidget::NullWidget ? EVisibility::Visible : EVisibility::Collapsed)
			.WidthOverride(MenuIconSize + 2 )
			.HeightOverride(MenuIconSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride(MenuIconSize)
				.HeightOverride(MenuIconSize)
				[
					IconWidget
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SBox )
			.Visibility(bHasLabel ? EVisibility::Visible : EVisibility::Collapsed)
			.Padding(FMargin(1, 0, 10, 0))
				.VAlign( VAlign_Center )
				[
					SNew( STextBlock )
					.TextStyle( StyleSet, ISlateStyle::Join(StyleName, ".Label") )
					.Text( Label )
					.ToolTipText( TextBlockToolTip )
				]
		]

		+SHorizontalBox::Slot()
		.HAlign( HAlign_Fill )
		.FillWidth( 1.0f )
		.Padding( FMargin( 2.0f, 1.0f ) )
		[
			SNew( SEditableTextBox )
			.Style( StyleSet, ISlateStyle::Join(StyleName, ".EditableText") )
			.Text( EditableTextBlock->TextAttribute )
			.IsReadOnly( EditableTextBlock->bReadOnly )
			.SelectAllTextWhenFocused( true )
			.RevertTextOnEscape( true )
			.MinDesiredWidth( MultiBoxConstants::EditableTextMinWidth )
			.OnTextChanged( EditableTextBlock->OnTextChanged )
			.OnVerifyTextChanged( EditableTextBlock->OnVerifyTextChanged )
			.OnTextCommitted( EditableTextBlock->OnTextCommitted )
			.ToolTipText( TextBlockToolTip )
		]
	];

	// Bind our widget's enabled state to whether or not our action can execute
	SetEnabled( TAttribute<bool>( this, &SEditableTextBlock::IsEnabled ) );
}


bool SEditableTextBlock::IsEnabled() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	bool bEnabled = true;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		bEnabled = ActionList->CanExecuteAction( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		bEnabled = DirectActions.CanExecute();
	}

	return bEnabled;
}
