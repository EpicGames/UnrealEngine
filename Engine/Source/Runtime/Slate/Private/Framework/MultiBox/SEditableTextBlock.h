// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

/**
 * Editable text block
 */
class FEditableTextBlock
	: public FMultiBlock
{
public:

	/**
	 * Constructor
	 *
	 * @param	InLabel					The label to display in the menu
	 * @param	InToolTip				The tool tip to display when the menu entry is hovered over
	 * @param	InIcon					The icon to display to the left of the label
	 * @param	InTextAttribute			The text string we're editing (often, a delegate will be bound to the attribute)
	 * @param	bInReadOnly				Whether or not the text block should be read only
	 * @param	InOnTextCommitted		Called when the user commits their change to the editable text control
	 * @param	InOnTextChanged			Called when the text is changed interactively
	 * @param	InOnVerifyTextChanged	Called to verify when the text is changed interactively
	 */
	FEditableTextBlock( const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, const TAttribute< FText >& InTextAttribute, bool bInReadOnly, const FOnTextCommitted& InOnTextCommitted, const FOnTextChanged& InOnTextChanged, const FOnVerifyTextChanged& InOnVerifyTextChanged = FOnVerifyTextChanged() );
	
	/** FMultiBlock interface */
	virtual bool HasIcon() const override;

private:

	/**
	 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
	 *
	 * @return  MultiBlock widget object
	 */
	virtual TSharedRef<class IMultiBlockBaseWidget> ConstructWidget() const override;

private:

	// Friend our corresponding widget class
	friend class SEditableTextBlock;

	/** Optional overridden text label for this menu entry.  If not set, then the action's label will be used instead. */
	TAttribute<FText> LabelOverride;

	/** Optional overridden tool tip for this menu entry.  If not set, then the action's tool tip will be used instead. */
	TAttribute<FText> ToolTipOverride;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	FSlateIcon IconOverride;

	/** The text we're editing */
	TAttribute< FText > TextAttribute;

	/** Called when the user commits their change to the editable text control */
	FOnTextCommitted OnTextCommitted;

	/** Called when the text is changed interactively */
	FOnTextChanged OnTextChanged;

	/** Called to verify when the text is changed interactively */
	FOnVerifyTextChanged OnVerifyTextChanged;

	/** If true the text box is read only */
	bool bReadOnly;
};


/**
 * Editable text block widget
 */
class SEditableTextBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SEditableTextBlock ){}
	SLATE_END_ARGS()

	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs ) { }

protected:

	/**
	 * Called by Slate to determine if this widget is enabled
	 * 
	 * @return true if the widget is enabled, false otherwise
	 */
	SLATE_API bool IsEnabled() const;
};
