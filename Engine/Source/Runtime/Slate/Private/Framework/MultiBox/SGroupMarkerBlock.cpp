// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SGroupMarkerBlock.h"

/**
 * Constructor
 */
FGroupStartBlock::FGroupStartBlock()
	: FMultiBlock( NULL, NULL )
{
}

/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FGroupStartBlock::ConstructWidget() const
{
	return SNew( SGroupMarkerBlock );
}

/**
 * Constructor
 */
FGroupEndBlock::FGroupEndBlock()
	: FMultiBlock( NULL, NULL )
{
}

/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FGroupEndBlock::ConstructWidget() const
{
	return SNew( SGroupMarkerBlock );
}

/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it,
 * in this case this is a blank block and therefore provides a null widget.
 */
void SGroupMarkerBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	ChildSlot
	[
		SNullWidget::NullWidget
	];

	// Add this widget to the search list of the multibox and hide it
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), FText::GetEmpty(), MultiBlock->GetSearchable());
}
