// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

/**
 * Group Start MultiBlock
 */
class FGroupStartBlock
	: public FMultiBlock
{
public:
	FGroupStartBlock();

	/** FMultiBlock interface */
	virtual bool IsGroupStartBlock() const override { return true; }
private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;
};

/**
 * Group End MultiBlock
 */
class FGroupEndBlock
	: public FMultiBlock
{

public:
	FGroupEndBlock();

	/** FMultiBlock interface */
	virtual bool IsGroupEndBlock() const override { return true; };
private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;
};

/**
 * Group Marker MultiBlock widget
 */
class SGroupMarkerBlock 
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SGroupMarkerBlock ){}
	SLATE_END_ARGS()

	/** FMultiBlock interface */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs ) {};

private:

};
