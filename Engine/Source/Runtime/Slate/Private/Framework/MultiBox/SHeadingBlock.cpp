// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SHeadingBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSeparator.h"


/**
 * Constructor
 *
 * @param	InHeadingText	Heading text
 */
FHeadingBlock::FHeadingBlock( const FName& InExtensionHook, const FText& InHeadingText )
	: FMultiBlock( NULL, NULL, InExtensionHook, EMultiBlockType::Heading, /* bInIsPartOfHeading=*/ true )
	, HeadingText( InHeadingText )
{
	SetSearchable(false);
}


/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FHeadingBlock::ConstructWidget() const
{
	return SNew( SHeadingBlock );
}


/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SHeadingBlock::Construct( const FArguments& InArgs )
{
}



/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SHeadingBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	TSharedRef< const FHeadingBlock > HeadingBlock = StaticCastSharedRef< const FHeadingBlock >( MultiBlock.ToSharedRef() );

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), FText::GetEmpty(), MultiBlock->GetSearchable());

	ChildSlot
	.Padding(StyleSet->GetMargin(StyleName, ".Heading.Padding"))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew( STextBlock )
			.Text( HeadingBlock->HeadingText.ToUpper() )
			.TextStyle( StyleSet, ISlateStyle::Join( StyleName, ".Heading" ) )
		]

		+ SHorizontalBox::Slot()
		.Padding(FMargin(14.f, 0.f, 0.f, 0.f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(1.0f)
			.SeparatorImage(StyleSet->GetBrush(StyleName, ".Separator") )
		]
	];

	SetVisibility(MultiBlock->GetVisibilityOverride());
}
