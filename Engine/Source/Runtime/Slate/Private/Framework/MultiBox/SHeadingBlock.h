// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

/**
 * Heading MultiBlock
 */
class FHeadingBlock
	: public FMultiBlock
{

public:

	/**
	 * Constructor
	 *
	 * @param	InHeadingText	Heading text
	 */
	FHeadingBlock( const FName& InExtensionHook, const FText& InHeadingText );


private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;


private:

	// Friend our corresponding widget class
	friend class SHeadingBlock;

	/** Text for this heading */
	FText HeadingText;
};




/**
 * Heading MultiBlock widget
 */
class SHeadingBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SHeadingBlock ){}

	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );
};
