// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SMenuEntryBlock.h"
#include "Framework/MultiBox/ToolMenuBase.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SLinkedBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/StyleColors.h"


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TSharedPtr< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const FSlateIcon& InIconOverride, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( InCommand, InCommandList, InExtensionHook, EMultiBlockType::MenuEntry )
	, LabelOverride( InLabelOverride )
	, ToolTipOverride( InToolTipOverride )
	, IconOverride( InIconOverride )
	, bIsSubMenu( false )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( false )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, const FSlateIcon& InIcon, const FUIAction& InUIAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection, TSharedPtr< const FUICommandList > InCommandList, const TAttribute<FText>& InInputBindingOverride)
	: FMultiBlock( InUIAction, InExtensionHook, EMultiBlockType::MenuEntry, /* bInIsPartOfHeading = */ false, InCommandList )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, InputBindingOverride(InInputBindingOverride)
	, IconOverride( InIcon )
	, EntryBuilder( InEntryBuilder )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( bInSubMenuOnClick )
	, UserInterfaceActionType( InUserInterfaceActionType )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FSlateIcon& InIcon, const FUIAction& UIAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection, const TAttribute<FText>& InInputBindingOverride)
	: FMultiBlock( UIAction, InExtensionHook, EMultiBlockType::MenuEntry )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, InputBindingOverride( InInputBindingOverride)
	, IconOverride( InIcon )
	, bIsSubMenu( false )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( false )
	, UserInterfaceActionType( InUserInterfaceActionType )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, const FSlateIcon& InIcon, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( nullptr, InCommandList, InExtensionHook, EMultiBlockType::MenuEntry )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, IconOverride( InIcon )
	, EntryBuilder( InEntryBuilder )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( bInSubMenuOnClick )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FOnGetContent& InMenuBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, const FSlateIcon& InIcon, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( nullptr, InCommandList, InExtensionHook, EMultiBlockType::MenuEntry )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, IconOverride( InIcon )
	, MenuBuilder( InMenuBuilder )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( bInSubMenuOnClick )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TSharedPtr<SWidget>& InEntryWidget, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, const FSlateIcon& InIcon, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( nullptr, InCommandList, InExtensionHook, EMultiBlockType::MenuEntry )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, IconOverride( InIcon )
	, EntryWidget( InEntryWidget )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( bInSubMenuOnClick )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}

FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const TAttribute<FText>& InToolTip, const EUserInterfaceActionType InUserInterfaceActionType, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( UIAction, InExtensionHook, EMultiBlockType::MenuEntry )
	, ToolTipOverride( InToolTip )
	, EntryWidget( Contents )
	, bIsSubMenu( false )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( false )
	, UserInterfaceActionType( InUserInterfaceActionType )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( nullptr, InCommandList, InExtensionHook, EMultiBlockType::MenuEntry )
	, EntryBuilder( InEntryBuilder )
	, EntryWidget( Contents )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( bInSubMenuOnClick )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}


FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( UIAction, InExtensionHook, EMultiBlockType::MenuEntry )
	, EntryBuilder( InEntryBuilder )
	, EntryWidget( Contents )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( false )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}

FMenuEntryBlock::FMenuEntryBlock( const FName& InExtensionHook, const FUIAction& UIAction, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FOnGetContent& InMenuBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, bool bInCloseSelfOnly, const FSlateIcon& InIcon, bool bInShouldCloseWindowAfterMenuSelection)
	: FMultiBlock( UIAction, InExtensionHook, EMultiBlockType::MenuEntry )
	, LabelOverride( InLabel )
	, ToolTipOverride( InToolTip )
	, IconOverride( InIcon )
	, MenuBuilder( InMenuBuilder )
	, bIsSubMenu( bInSubMenu )
	, bIsRecursivelySearchable( true )
	, bOpenSubMenuOnClick( bInSubMenuOnClick )
	, UserInterfaceActionType( EUserInterfaceActionType::Button )
	, bCloseSelfOnly( bInCloseSelfOnly )
	, Extender( InExtender )
	, bShouldCloseWindowAfterMenuSelection( bInShouldCloseWindowAfterMenuSelection )
{
}

FMenuEntryBlock::FMenuEntryBlock(const FMenuEntryParams& InMenuEntryParams)
	: FMultiBlock(InMenuEntryParams)
	, LabelOverride(InMenuEntryParams.LabelOverride)
	, ToolTipOverride(InMenuEntryParams.ToolTipOverride)
	, InputBindingOverride(InMenuEntryParams.InputBindingOverride)
	, IconOverride(InMenuEntryParams.IconOverride)
	, EntryBuilder(InMenuEntryParams.EntryBuilder)
	, MenuBuilder(InMenuEntryParams.MenuBuilder)
	, EntryWidget(InMenuEntryParams.EntryWidget)
	, bIsSubMenu(InMenuEntryParams.bIsSubMenu)
	, bIsRecursivelySearchable(InMenuEntryParams.bIsRecursivelySearchable)
	, bOpenSubMenuOnClick(InMenuEntryParams.bOpenSubMenuOnClick)
	, UserInterfaceActionType(InMenuEntryParams.UserInterfaceActionType)
	, bCloseSelfOnly(InMenuEntryParams.bCloseSelfOnly)
	, Extender(InMenuEntryParams.Extender)
	, bShouldCloseWindowAfterMenuSelection(InMenuEntryParams.bShouldCloseWindowAfterMenuSelection)
{
}

void FMenuEntryBlock::CreateMenuEntry(FMenuBuilder& InMenuBuilder) const
{
	InMenuBuilder.AddSubMenu(LabelOverride.Get(), ToolTipOverride.Get(), EntryBuilder, false, IconOverride);	
}


bool FMenuEntryBlock::HasIcon() const
{
	const FSlateIcon ActionIcon = GetAction().IsValid() ? GetAction()->GetIcon() : FSlateIcon();
	const FSlateIcon& ActualIcon = !IconOverride.IsSet() ? ActionIcon : IconOverride;

	if (ActualIcon.IsSet())
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		return IconBrush->GetResourceName() != NAME_None;
	}

	return false;
}


/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FMenuEntryBlock::ConstructWidget() const
{
	return SNew( SMenuEntryBlock );
}

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SMenuEntryBlock::Construct( const FArguments& InArgs )
{
	// No initial sub-menus should be opened
	/*TimeToSubMenuOpen = 0.0f;*/
	//SubMenuRequestState = Idle;

	MenuBarButtonStyle = nullptr;

	// No images by default
	CheckedImage = nullptr;
	UncheckedImage = nullptr;

	this->SetForegroundColor(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateRaw(this, &SMenuEntryBlock::InvertOnHover)));
}


TSharedRef< SWidget> SMenuEntryBlock::BuildMenuBarWidget( const FMenuEntryBuildParams& InBuildParams )
{
	const TAttribute<FText>& Label = InBuildParams.Label;
	const TAttribute<FText>& EntryToolTip = InBuildParams.ToolTip;

	check( OwnerMultiBoxWidget.IsValid() );

	const ISlateStyle* const StyleSet = InBuildParams.StyleSet;
	const FName& StyleName = InBuildParams.StyleName;

	TSharedPtr< SMenuAnchor > NewMenuAnchor;
	MenuBarButtonStyle = &StyleSet->GetWidgetStyle<FButtonStyle>(ISlateStyle::Join(StyleName, ".Button"));

	// Create a menu bar button within a pop-up anchor
	TSharedRef<SWidget> Widget =
		SAssignNew( NewMenuAnchor, SMenuAnchor )
		.Placement( MenuPlacement_BelowAnchor )
		// When the menu is summoned, this callback will fire to generate content for the menu window
		.OnGetMenuContent( this, &SMenuEntryBlock::MakeNewMenuWidget )
		[
			SNew(SBorder)
			.BorderImage(this, &SMenuEntryBlock::GetMenuBarButtonBorder)
			.Padding(0.0f)
			[
				// Create a button
				SNew(SButton)
				// Use the menu bar item style for this button
				.ButtonStyle(StyleSet, "NoBorder")
				// Pull-down menu bar items always activate on mouse-down, not mouse-up
				.ClickMethod(EButtonClickMethod::MouseDown)
				// Pass along the block's tool-tip string
				.ToolTipText(this, &SMenuEntryBlock::GetFilteredToolTipText, EntryToolTip)
				// Add horizontal padding between the edge of the button and the content.  Also add a bit of vertical
				// padding to push the text down from the top of the menu bar a bit.
				.ContentPadding(StyleSet->GetMargin(StyleName, ".MenuBar.Padding"))
				.ForegroundColor(this, &SMenuEntryBlock::GetMenuBarForegroundColor)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(StyleSet, ISlateStyle::Join(StyleName, ".Label"))
					.Text(Label)
					.HighlightText(OwnerMultiBoxWidget.Pin().Get(), &SMultiBoxWidget::GetSearchText)
				]
				// Bind the button's "on clicked" event to our object's method for this
				.OnClicked(this, &SMenuEntryBlock::OnMenuItemButtonClicked)
			]
		];

	MenuAnchor = NewMenuAnchor;

	return Widget;
}


TSharedRef<SWidget> SMenuEntryBlock::FindTextBlockWidget( TSharedRef<SWidget> Content )
{
	if (Content->GetType() == FName("STextBlock"))
	{
		return Content;
	}

	FChildren* Children = Content->GetChildren();
	int32 NumChildren = Children->Num();

	for( int32 Index = 0; Index < NumChildren; ++Index )
	{
		TSharedRef<SWidget> Found = FindTextBlockWidget( Children->GetChildAt( Index ));
		if (Found != SNullWidget::NullWidget)
		{
			return Found;
		}
	}
	return SNullWidget::NullWidget;
}


FText SMenuEntryBlock::GetFilteredToolTipText( TAttribute<FText> ToolTipText ) const
{
	// If we're part of a menu bar that has a currently open menu, then we suppress our own tool-tip
	// as it will just get in the way
	if( OwnerMultiBoxWidget.Pin()->GetOpenMenu().IsValid() )
	{
		return FText::GetEmpty();
	}

	return ToolTipText.Get();
}

EVisibility SMenuEntryBlock::GetVisibility() const
{
	// Let the visibility override take prescedence here.
	// However, if it returns Visible, let the other methods have a chance to change that.
	if (MultiBlock->GetVisibilityOverride().IsSet())
	{
		const EVisibility OverrideVisibility = MultiBlock->GetVisibilityOverride().Get();
		if (OverrideVisibility != EVisibility::Visible)
		{
			return OverrideVisibility;
		}
	}

	TSharedPtr<const FUICommandList> ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	if (ActionList.IsValid() && Action.IsValid())
	{
		return ActionList->GetVisibility(Action.ToSharedRef());
	}

	// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
	return DirectActions.IsVisible();
}

/**
* A button for a menu entry that has special mouse up handling
*/
class SMenuEntryButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(SMenuEntryButton)
		: _Content()
		, _ButtonStyle(nullptr)
		, _ClickMethod(EButtonClickMethod::DownAndUp)
	{}

	/** Slot for this button's content (optional) */
	SLATE_DEFAULT_SLOT(FArguments, Content)
	/** The style to use */
	SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
	/** Sets the rules to use for determining whether the button was clicked.  This is an advanced setting and generally should be left as the default. */
	SLATE_ARGUMENT(EButtonClickMethod::Type, ClickMethod)
	/** Spacing between button's border and the content. */
	SLATE_ARGUMENT(FMargin, ContentPadding)
	/** Called when the button is clicked */
	SLATE_EVENT(FOnClicked, OnClicked)

	SLATE_END_ARGS()

	enum class EResponseToMouseUp
	{
		Undetermined,
		Handle,
		DoNotHandle,
	};

	void Construct(const FArguments& InArgs)
	{
		SButton::FArguments ButtonArgs;
		ButtonArgs.ButtonStyle(InArgs._ButtonStyle);
		ButtonArgs.ClickMethod(InArgs._ClickMethod);
		ButtonArgs.ToolTip(InArgs._ToolTip);
		ButtonArgs.ContentPadding(InArgs._ContentPadding);
		ButtonArgs.ForegroundColor(FSlateColor::UseStyle());
		ButtonArgs.OnClicked(InArgs._OnClicked)
		[
			InArgs._Content.Widget
		];

		SButton::Construct(ButtonArgs);

		ResponseToMouseUp = EResponseToMouseUp::Undetermined;
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		// On first tick, check mouse cursor position.
		if (ResponseToMouseUp == EResponseToMouseUp::Undetermined)
		{
			FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

			if (AllottedGeometry.IsUnderLocation(CursorPos))
			{
				// button was created under the mouse
				ResponseToMouseUp = EResponseToMouseUp::DoNotHandle;
			}
			else
			{
				// button was NOT created under the mouse
				ResponseToMouseUp = EResponseToMouseUp::Handle;
			}
		}

		SButton::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (ResponseToMouseUp == EResponseToMouseUp::Handle)
		{
			if (IsEnabled())
			{
				Press();
			}
		}

		FReply Reply = SButton::OnMouseButtonUp(MyGeometry, MouseEvent);

		return Reply;
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		if (ResponseToMouseUp == EResponseToMouseUp::DoNotHandle)
		{
			ResponseToMouseUp = EResponseToMouseUp::Handle;
		}

		Release();

		SButton::OnMouseLeave(MouseEvent);
	}

private:
	EResponseToMouseUp ResponseToMouseUp;
};

void FMenuEntryBlock::SetCheckBoxStyle(FName InCheckBoxStyle)
{
	this->CheckBoxStyle = InCheckBoxStyle;
}

TSharedRef< SWidget > SMenuEntryBlock::BuildMenuEntryWidget( const FMenuEntryBuildParams& InBuildParams )
{
	const TAttribute<FText>& Label = InBuildParams.Label;
	const TAttribute<FText>& EntryToolTip = InBuildParams.ToolTip;
	const TAttribute<FText>& InputBinding = InBuildParams.InputBinding;
	const TSharedPtr< const FMenuEntryBlock > MenuEntryBlock = InBuildParams.MenuEntryBlock;
	const TSharedPtr< const FMultiBox > MultiBox = InBuildParams.MultiBox;
	const TSharedPtr< const FUICommandInfo >& UICommand = InBuildParams.UICommand;

	// See if the action is valid and if so we will use the actions icon if we dont override it later
	const FSlateIcon ActionIcon = UICommand.IsValid() ? UICommand->GetIcon() : FSlateIcon();

	// Allow the block to override the tool bar icon, too
	const FSlateIcon& ActualIcon = !MenuEntryBlock->IconOverride.IsSet() ? ActionIcon : MenuEntryBlock->IconOverride;

	check( OwnerMultiBoxWidget.IsValid() );

	const ISlateStyle* const StyleSet = InBuildParams.StyleSet;
	const FName& StyleName = InBuildParams.StyleName;

	// Allow menu item buttons to be triggered on mouse-up events if the menu is configured to be
	// dismissed automatically after clicking.  This preserves the behavior people expect for context
	// menus and pull-down menus
	const EButtonClickMethod::Type ButtonClickMethod =
		MultiBox->ShouldCloseWindowAfterMenuSelection() ?
		EButtonClickMethod::MouseUp : EButtonClickMethod::DownAndUp;

	// If we were supplied an image than go ahead and use that, otherwise we use a null widget
	TSharedRef< SWidget > IconWidget = SNullWidget::NullWidget;
	if( ActualIcon.IsSet() )
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		if( IconBrush->GetResourceName() != NAME_None )
		{
			IconWidget = SNew( SImage )
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
				.Image( IconBrush );
		}
	}

	const float MenuIconSize = StyleSet->GetFloat(StyleName, ".MenuIconSize", 16.f);

	if (bSectionContainsIcons && (IconWidget == SNullWidget::NullWidget))
	{
		// Section should have icons but this entry does not, which is inconsistent with our menu policy (either all or none of menu items in a section should have an icon)
		if (FMultiBoxSettings::DisplayMultiboxHooks.Get())
		{
			IconWidget = SNew(SColorBlock)
				.Color(FColorList::Magenta)
				.Size(FVector2D(MenuIconSize, MenuIconSize))
				.ToolTipText(NSLOCTEXT("SMenuEntryBlock", "MissingIconInMenu", "This menu entry is missing an icon and should be fixed (consistency within each section is required, either every entry in the section has an icon or no entries have an icon)"));
		}
		else
		{
			// This will silently pad the offending items
			// 	IconWidget = SNew(SSpacer)
			// 		.Size(FVector2D(MultiBoxConstants::MenuIconSize, MultiBoxConstants::MenuIconSize));
		}
	}

	// What type of UI should we create for this block?
	EUserInterfaceActionType UserInterfaceType = MenuEntryBlock->UserInterfaceActionType;
	if ( UICommand.IsValid() )
	{
		// If we have a UICommand, then this is specified in the command.
		UserInterfaceType = UICommand->GetUserInterfaceType();
	}
	
	EVisibility CheckBoxVisibility =
		( UserInterfaceType == EUserInterfaceActionType::ToggleButton ||
			UserInterfaceType == EUserInterfaceActionType::RadioButton ||
			UserInterfaceType == EUserInterfaceActionType::Check ) ?
				EVisibility::Visible :
				EVisibility::Collapsed;

	// Collapse (rather than Hide) the checkbox when using a custom menu widget with a button action, otherwise we add additional padding around the user defined widget
	if ((MenuEntryBlock->EntryWidget.IsValid() && UserInterfaceType == EUserInterfaceActionType::Button) ||
		UserInterfaceType == EUserInterfaceActionType::CollapsedButton)
	{
		CheckBoxVisibility = EVisibility::Collapsed;
	}
	else if (MultiBox->IsInEditMode())
	{
		// Hide but keep spacing to avoid confusing user during editing
		CheckBoxVisibility = EVisibility::Collapsed;
	}

	TAttribute<FSlateColor> CheckBoxForegroundColor = FSlateColor::UseStyle();
	FName CheckBoxStyle = ISlateStyle::Join( StyleName, ".CheckBox" );

	if (MenuEntryBlock->CheckBoxStyle != NAME_None)
	{
		CheckBoxStyle = MenuEntryBlock->CheckBoxStyle;
	}
	else if (UserInterfaceType == EUserInterfaceActionType::Check)
	{
		CheckBoxStyle = ISlateStyle::Join( StyleName, ".Check" );
	}
	else if (UserInterfaceType == EUserInterfaceActionType::RadioButton)
	{
		CheckBoxStyle = ISlateStyle::Join( StyleName, ".RadioButton" );
	}


	// If there is custom menu widget, set it
	// If there isn't, create it
	TSharedPtr< SWidget > ButtonContent = MenuEntryBlock->EntryWidget;
	if ( !ButtonContent.IsValid() )
	{
		// Create the content for our button
		ButtonContent = SNew( SHorizontalBox )
		// Whatever we have in the icon area goes first
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2, 0, 6, 0))
		[
			SNew( SBox )
			.Visibility(IconWidget != SNullWidget::NullWidget ? EVisibility::Visible : EVisibility::Collapsed)
			.WidthOverride(MenuIconSize + 2 )
			.HeightOverride(MenuIconSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride(MenuIconSize)
				.HeightOverride(MenuIconSize)
				[
					IconWidget
				]
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth( 1.0f )
		.Padding(FMargin(2, 0, 6, 0))
		.VAlign( VAlign_Center )
		[
			SNew( STextBlock )
			.TextStyle( StyleSet, ISlateStyle::Join( StyleName, ".Label" ) )
			.Text( Label )
			.HighlightText( OwnerMultiBoxWidget.Pin().Get(), &SMultiBoxWidget::GetSearchText )
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(25,0,0,0))
		.VAlign( VAlign_Center )
		.HAlign( HAlign_Right )
		[
			SNew( SBox )
			.Visibility(InputBinding.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
			.Padding(FMargin(16,0,4,0))
			[
				SNew( STextBlock )
				.TextStyle( StyleSet, ISlateStyle::Join( StyleName, ".Keybinding" ) )
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
				.Text( InputBinding )
			]
		];
	}

	// Create a wrapper containing the checkbox, and the generated button content
	TSharedRef< SWidget > CheckBoxAndButtonContent = 
		SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.Padding(FMargin(2, 0, 8, 0))
		.AutoWidth()
		[
			SNew(SLinkedBox, OwnerMultiBoxWidget.Pin()->GetLinkedBoxManager())
			[
				// For check style menus, use an image instead of a CheckBox because it can't really be checked.
				UserInterfaceType == EUserInterfaceActionType::Check
					? StaticCastSharedRef<SWidget>(SNew(SImage)
						.Visibility(CheckBoxVisibility)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SMenuEntryBlock::GetCheckBoxImageBrushFromStyle, &StyleSet->GetWidgetStyle<FCheckBoxStyle>(CheckBoxStyle)))
					: StaticCastSharedRef<SWidget>(SNew(SCheckBox)
						.Visibility(CheckBoxVisibility)
						.ForegroundColor( CheckBoxForegroundColor )
						.IsChecked( this, &SMenuEntryBlock::IsChecked )
						.Style( StyleSet, CheckBoxStyle )
						.OnCheckStateChanged( this, &SMenuEntryBlock::OnCheckStateChanged ))
			]
		]
		+SHorizontalBox::Slot()
		.Padding(FMargin(2, 0, 0, 0))
		[
			ButtonContent.ToSharedRef()
		];

	// Create a menu item button
	TSharedPtr<SWidget> MenuEntryWidget = SNew(SMenuEntryButton)
		// Use the menu item style for this button
		.ButtonStyle( StyleSet, ISlateStyle::Join( StyleName, ".Button" ) )
		// Set our click method for this menu item.  It will be different for pull-down/context menus.
		.ClickMethod( ButtonClickMethod )
		.ContentPadding(StyleSet->GetMargin(StyleName,".Block.Padding"))
		// Pass along the block's tool-tip string
		.ToolTip( FMultiBoxSettings::ToolTipConstructor.Execute(EntryToolTip, nullptr, UICommand, /*ShowActionShortcut=*/ false ) )
		// Bind the button's "on clicked" event to our object's method for this
		.OnClicked(this, &SMenuEntryBlock::OnMenuItemButtonClicked)
		[
			CheckBoxAndButtonContent
		];

	return MenuEntryWidget.ToSharedRef();
}




/**
 * A button for a sub-menu entry that shows its hovered state when the sub-menu is open
 */
class SSubMenuButton : public SButton
{
public:
	SLATE_BEGIN_ARGS( SSubMenuButton )
		: _ShouldAppearHovered( false )
		, _ButtonStyle( nullptr )
		{}
		/** The label to display on the button */
		SLATE_ATTRIBUTE( FText, Label )
		/** Called when the button is clicked */
		SLATE_EVENT( FOnClicked, OnClicked )
		/** Content to put in the button */
		SLATE_DEFAULT_SLOT( FArguments, Content )
		/** Whether or not the button should appear in the hovered state */
		SLATE_ATTRIBUTE( bool, ShouldAppearHovered )
		/** The style to use */
		SLATE_STYLE_ARGUMENT( FButtonStyle, ButtonStyle )
		/** Spacing between button's border and the content. */
		SLATE_ARGUMENT(FMargin, ContentPadding)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		bHovered = false;
		ShouldAppearHovered = InArgs._ShouldAppearHovered;

		SetHover(TAttribute<bool>::CreateSP(this, &SSubMenuButton::HandleShouldAppearHovered));

		SButton::FArguments ButtonArgs;
		ButtonArgs.Text(InArgs._Label);
		ButtonArgs.ForegroundColor( this, &SSubMenuButton::InvertOnHover );
		ButtonArgs.HAlign(HAlign_Fill);
		ButtonArgs.VAlign(VAlign_Fill);
		ButtonArgs.ContentPadding(InArgs._ContentPadding);

		if ( InArgs._ButtonStyle )
		{
			ButtonArgs.ButtonStyle( InArgs._ButtonStyle );
		}
	
		ButtonArgs.OnClicked(InArgs._OnClicked);
		ButtonArgs.ClickMethod(EButtonClickMethod::MouseDown)
		[
			InArgs._Content.Widget
		];

		SButton::Construct( ButtonArgs );
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		bHovered = true;
		SButton::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		bHovered = false;
		SButton::OnMouseLeave(MouseEvent);
	}

private:
	bool HandleShouldAppearHovered() const
	{
		// Submenu widgets which have been opened should remain as if hovered, even if the cursor is outside them
		return bHovered || ShouldAppearHovered.Get();
	}

	FSlateColor InvertOnHover() const
	{
		if ( this->IsHovered() )
		{
			return FStyleColors::ForegroundHover;
		}
		else
		{
			return FSlateColor::UseStyle();
		}
	}

private:
	/** Attribute to indicate if the sub-menu is open or not */
	TAttribute<bool> ShouldAppearHovered;
	/** Keep an internal IsHovered flag*/
	bool bHovered;
};


TSharedRef< SWidget> SMenuEntryBlock::BuildSubMenuWidget( const FMenuEntryBuildParams& InBuildParams )
{
	const TAttribute<FText>& Label = InBuildParams.Label;
	const TAttribute<FText>& EntryToolTip = InBuildParams.ToolTip;
	const TAttribute<FText>& InputBinding = InBuildParams.InputBinding;

	const TSharedPtr< const FMenuEntryBlock > MenuEntryBlock = InBuildParams.MenuEntryBlock;
	const TSharedPtr< const FMultiBox > MultiBox = InBuildParams.MultiBox;
	const TSharedPtr< const FUICommandInfo >& UICommand = InBuildParams.UICommand;
	
	// See if the action is valid and if so we will use the actions icon if we dont override it later
	const FSlateIcon ActionIcon = UICommand.IsValid() ? UICommand->GetIcon() : FSlateIcon();
	
	// Allow the block to override the tool bar icon, too
	const FSlateIcon& ActualIcon = !MenuEntryBlock->IconOverride.IsSet() ? ActionIcon : MenuEntryBlock->IconOverride;

	check( OwnerMultiBoxWidget.IsValid() );

	const ISlateStyle* const StyleSet = InBuildParams.StyleSet;
	const FName& StyleName = InBuildParams.StyleName;

	// Allow menu item buttons to be triggered on mouse-up events if the menu is configured to be
	// dismissed automatically after clicking.  This preserves the behavior people expect for context
	// menus and pull-down menus
	const EButtonClickMethod::Type ButtonClickMethod =
		MultiBox->ShouldCloseWindowAfterMenuSelection() ?
		EButtonClickMethod::MouseUp : EButtonClickMethod::DownAndUp;

	// If we were supplied an image than go ahead and use that, otherwise we use a null widget
	TSharedRef< SWidget > IconWidget = SNullWidget::NullWidget;
	if( ActualIcon.IsSet() )
	{
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		if( IconBrush->GetResourceName() != NAME_None )
		{
			IconWidget =
				SNew( SImage )
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
				.Image( IconBrush );
		}
	}

	// What type of UI should we create for this block?
	EUserInterfaceActionType UserInterfaceType = MenuEntryBlock->UserInterfaceActionType;
	if ( UICommand.IsValid() )
	{
		// If we have a UICommand, then this is specified in the command.
		UserInterfaceType = UICommand->GetUserInterfaceType();
	}
	
	EVisibility CheckBoxVisibility =
		( UserInterfaceType == EUserInterfaceActionType::ToggleButton ||
			UserInterfaceType == EUserInterfaceActionType::RadioButton ||
			UserInterfaceType == EUserInterfaceActionType::Check ) ?
				EVisibility::Visible :
				EVisibility::Collapsed;

	TAttribute<FSlateColor> CheckBoxForegroundColor = FSlateColor::UseForeground();
	FName CheckBoxStyle = ISlateStyle::Join( StyleName, ".CheckBox" );
	if (UserInterfaceType == EUserInterfaceActionType::Check)
	{
		CheckBoxStyle = ISlateStyle::Join( StyleName, ".Check" );
	}
	else if (UserInterfaceType == EUserInterfaceActionType::RadioButton)
	{
		CheckBoxStyle = ISlateStyle::Join( StyleName, ".RadioButton" );
		CheckBoxForegroundColor = TAttribute<FSlateColor>::Create( TAttribute<FSlateColor>::FGetter::CreateRaw( this, &SMenuEntryBlock::TintOnHover ) );
	}

	const float MenuIconSize = StyleSet->GetFloat(StyleName, ".MenuIconSize", 16.f);

	TSharedPtr< SWidget > ButtonContent = MenuEntryBlock->EntryWidget;
	if ( !ButtonContent.IsValid() )
	{
		// Create the content for our button
		ButtonContent = SNew( SHorizontalBox )
		// Whatever we have in the icon area goes first
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2, 0, 6, 0))
		[
			SNew( SBox )
			.Visibility(IconWidget != SNullWidget::NullWidget ? EVisibility::Visible : EVisibility::Collapsed)
			.WidthOverride(MenuIconSize + 2)
			.HeightOverride(MenuIconSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride(MenuIconSize)
				.HeightOverride(MenuIconSize)
				[
					IconWidget
				]
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth( 1.0f )
		.Padding(FMargin(2, 0, 6, 0))
		.VAlign( VAlign_Center )
		[
			SNew( STextBlock )
			.TextStyle( StyleSet, ISlateStyle::Join( StyleName, ".Label" ) )
			.Text( Label )
			.HighlightText( OwnerMultiBoxWidget.Pin().Get(), &SMultiBoxWidget::GetSearchText )
		];
	}

	// Create a wrapper containing the checkbox, and the generated button content
	TSharedRef< SWidget > CheckBoxAndButtonContent =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(2, 0, 8, 0))
		.AutoWidth()
		[
			SNew(SLinkedBox, OwnerMultiBoxWidget.Pin()->GetLinkedBoxManager())
			[
				// For check style menus, use an image instead of a CheckBox because it can't really be checked.
				UserInterfaceType == EUserInterfaceActionType::Check
				? StaticCastSharedRef<SWidget>(SNew(SImage)
					.Visibility(CheckBoxVisibility)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SMenuEntryBlock::GetCheckBoxImageBrushFromStyle, &StyleSet->GetWidgetStyle<FCheckBoxStyle>(CheckBoxStyle)))
				: StaticCastSharedRef<SWidget>(SNew(SCheckBox)
					.ForegroundColor(CheckBoxForegroundColor)
					.Visibility(CheckBoxVisibility)
					.IsChecked(this, &SMenuEntryBlock::IsChecked)
					.Style(StyleSet, CheckBoxStyle)
					.OnCheckStateChanged(this, &SMenuEntryBlock::OnCheckStateChanged))
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(2, 0, 0, 0))
		[
			ButtonContent.ToSharedRef()
		];

	TSharedPtr< SMenuAnchor > NewMenuAnchorPtr;
	TSharedRef< SWidget > Widget = 
	SAssignNew( NewMenuAnchorPtr, SMenuAnchor )
		.Placement( MenuPlacement_MenuRight )
		// When the menu is summoned, this callback will fire to generate content for the menu window
		.OnGetMenuContent( this, &SMenuEntryBlock::MakeNewMenuWidget )
		[
			// Create a button
			SNew( SSubMenuButton )
			// Pass along the block's tool-tip string
			.ToolTip( FMultiBoxSettings::ToolTipConstructor.Execute(EntryToolTip, nullptr, UICommand, /*ShowActionShortcut=*/ false) )
			// Style to use
			.ButtonStyle( &StyleSet->GetWidgetStyle<FButtonStyle>( ISlateStyle::Join( StyleName, ".Button" ) ) )
			.ContentPadding(StyleSet->GetMargin(StyleName, ".Block.Padding"))
			// Allow the button to change its state depending on the state of the submenu
			.ShouldAppearHovered( this, &SMenuEntryBlock::ShouldSubMenuAppearHovered )
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				[
					CheckBoxAndButtonContent
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(25,0,0,0))
				.VAlign( VAlign_Center )
				.HAlign( HAlign_Right )
				[
					SNew( SBox )
					.Visibility(InputBinding.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
					.Padding(FMargin(16,0,4,0))
					[
						SNew( STextBlock )
						.TextStyle( StyleSet, ISlateStyle::Join( StyleName, ".Keybinding" ) )
						.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
						.Text( InputBinding )
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign( VAlign_Center )
				.HAlign( HAlign_Right )
				[
					SNew( SBox )
					.Padding(FMargin(7,0,0,0))
					[
						SNew( SImage )
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image( StyleSet->GetBrush( StyleName, ".SubMenuIndicator" ) )
					]
				]
			]

			// Bind the button's "on clicked" event to our object's method for this
			.OnClicked( this, &SMenuEntryBlock::OnMenuItemButtonClicked )
		];

	MenuAnchor = NewMenuAnchorPtr;

	return Widget;
}

/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SMenuEntryBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	FMenuEntryBuildParams BuildParams;
	TSharedPtr< const FMultiBox > MultiBox = OwnerMultiBoxWidget.Pin()->GetMultiBox();
	TSharedPtr< const FMenuEntryBlock > MenuEntryBlock = StaticCastSharedRef< const FMenuEntryBlock >( MultiBlock.ToSharedRef() );
	BuildParams.MultiBox = MultiBox;
	
	BuildParams.MenuEntryBlock = MenuEntryBlock;
	BuildParams.UICommand = BuildParams.MenuEntryBlock->GetAction();
	BuildParams.StyleSet = StyleSet;
	BuildParams.StyleName = StyleName;

	if (MenuEntryBlock->LabelOverride.IsSet())
	{
		BuildParams.Label = MenuEntryBlock->LabelOverride;
	}
	else
	{
		BuildParams.Label = BuildParams.UICommand.IsValid() ? BuildParams.UICommand->GetLabel() : FText::GetEmpty();
	}

	const bool bIsEditing = MultiBox->IsInEditMode();
	if (bIsEditing)
	{
		// No dynamic labels while editing
		if (BuildParams.Label.IsBound())
		{
			BuildParams.Label = BuildParams.Label.Get();
		}
	}

	// Add this widget to the search list of the multibox
	// If there is a widget already assigned (created early) ensure that it's STextBlock is set up for searching
	TSharedPtr< SWidget > ButtonContent = MenuEntryBlock->EntryWidget;
	if( ButtonContent.IsValid() )
	{
		TSharedRef<SWidget> TextBlock = FindTextBlockWidget(ButtonContent.ToSharedRef());
		if (TextBlock != SNullWidget::NullWidget )
		{
			TSharedRef<STextBlock> TheTextBlock = StaticCastSharedRef<STextBlock>(TextBlock);

			// Bind the search text to the widgets text to highlight
			TAttribute<FText> HighlightText;
			HighlightText.Bind(OwnerMultiBoxWidget.Pin().Get(), &SMultiBoxWidget::GetSearchText);
			TheTextBlock->SetHighlightText(HighlightText);

			OwnerMultiBoxWidget.Pin()->AddElement( this->AsWidget(), TheTextBlock.Get().GetText(), MultiBlock->GetSearchable());
		}
		else
		{
			//UE_LOGF(LogTemp, Warning, "Custom widget block will not be searched, could not find it's displayed STextBlock");
		}
	}
	else
	{
		OwnerMultiBoxWidget.Pin()->AddElement( this->AsWidget(), BuildParams.Label.Get(), MultiBlock->GetSearchable());
	}

	// Tool tips are optional so if the tool tip override is empty and there is no UI command just use the empty tool tip.
	if (MenuEntryBlock->ToolTipOverride.IsSet())
	{
		BuildParams.ToolTip = MenuEntryBlock->ToolTipOverride;
	}
	else
	{
		BuildParams.ToolTip = BuildParams.UICommand.IsValid() ? BuildParams.UICommand->GetDescription() : FText::GetEmpty();
	}

	// Input bindings are optional so if the binding override is empty and there is no UI command just use the empty tool tip.
	if (MenuEntryBlock->InputBindingOverride.IsSet())
	{
		BuildParams.InputBinding = MenuEntryBlock->InputBindingOverride.Get().ToUpper();
	}
	else
	{
		BuildParams.InputBinding = BuildParams.UICommand.IsValid() ? BuildParams.UICommand->GetInputText().ToUpper() : FText::GetEmpty();
	}

	if (bIsEditing)
	{
		// No dynamic tooltips while editing
		if (BuildParams.ToolTip.IsBound())
		{
			BuildParams.ToolTip = BuildParams.ToolTip.Get();
		}
	}

	if( MultiBox->GetType() == EMultiBoxType::Menu )
	{
		if( MenuEntryBlock->bIsSubMenu )
		{
			// This menu entry is actually a submenu that opens a new menu to the right
			ChildSlot.AttachWidget( BuildSubMenuWidget( BuildParams ) );
		}
		else
		{
			// Standard menu entry 
			ChildSlot.AttachWidget( BuildMenuEntryWidget( BuildParams ) );
		}
	}
	else if( ensure( MultiBox->GetType() == EMultiBoxType::MenuBar) )
	{
		// Menu bar items cannot be submenus
		check( !MenuEntryBlock->bIsSubMenu );
		
		ChildSlot
		[ 
			BuildMenuBarWidget( BuildParams ) 
		];
	}

	if (!bIsEditing)
	{
		// Insert named widget if desired
		FName TutorialName = MenuEntryBlock->GetTutorialHighlightName();
		if (TutorialName != NAME_None)
		{
			TSharedRef<SWidget> ChildWidget = ChildSlot.GetWidget();
			ChildWidget->AddMetadata<FTagMetaData>(MakeShared<FTagMetaData>(TutorialName));
		}
	}

	if (bIsEditing)
	{
		SetEnabled(TAttribute<bool>(this, &SMenuEntryBlock::IsEnabledDuringEditMode));
	}
	else
	{
		// Bind our widget's enabled state to whether or not our action can execute
		SetEnabled(TAttribute<bool>(this, &SMenuEntryBlock::IsEnabled));
	}

	if (!bIsEditing)
	{
		// Bind our widget's visible state to whether or not the action should be visible
		SetVisibility(TAttribute<EVisibility>(this, &SMenuEntryBlock::GetVisibility));
	}
}

void SMenuEntryBlock::RequestSubMenuToggle( bool bOpenMenu, const bool bClobber )
{
	// Reset the time before the menu opens
	float TimeToSubMenuOpen = bClobber ? MultiBoxConstants::SubMenuClobberTime : MultiBoxConstants::SubMenuOpenTime;
	//SubMenuRequestState = bOpenMenu ? WantOpen : WantClose;
	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(TimeToSubMenuOpen, FWidgetActiveTimerDelegate::CreateSP(this, &SMenuEntryBlock::UpdateSubMenuState, bOpenMenu));
	}
	//UpdateSubMenuState();
}

void SMenuEntryBlock::CancelPendingSubMenu()
{
	// Reset any pending sub-menu openings
	//SubMenuRequestState = Idle;
	
	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
}

bool SMenuEntryBlock::ShouldSubMenuAppearHovered() const
{
	// The sub-menu entry should appear hovered if the sub-menu is open.  Except in the case that the user is actually interacting with this menu.  
	// In that case we need to show what the user is selecting
	return MenuAnchor.IsValid() && MenuAnchor.Pin()->IsOpen() && !OwnerMultiBoxWidget.Pin()->IsHovered();
}

FReply SMenuEntryBlock::OnMenuItemButtonClicked()
{
	// The button itself was clicked
	const bool bCheckBoxClicked = false;
	OnClicked( bCheckBoxClicked );
	return FReply::Handled();
}

/**
 * Called by Slate when this menu entry's button is clicked
 */
void SMenuEntryBlock::OnClicked( bool bCheckBoxClicked )
{
	// Button was clicked, so trigger the action!
	TSharedRef< const FMenuEntryBlock > MenuEntryBlock = StaticCastSharedRef< const FMenuEntryBlock >( MultiBlock.ToSharedRef() );
	
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();

	TSharedRef< const FMultiBox > MultiBox( OwnerMultiBoxWidget.Pin()->GetMultiBox() );

	// If this is a context menu, then we'll also dismiss the window after the user clicked on the item
	// NOTE: We dismiss the menu stack BEFORE executing the action to allow cases where the action actually starts a new menu stack
	// If we dismiss it before after the action, we would also dismiss the new menu
	const bool ClosingMenu = MultiBox->ShouldCloseWindowAfterMenuSelection() && ( !MenuEntryBlock->bIsSubMenu || ( MenuEntryBlock->bIsSubMenu && MenuEntryBlock->GetDirectActions().IsBound() ) );
	
	// Do not close the menu if we clicked a checkbox
	if( !bCheckBoxClicked )
	{
		if( ClosingMenu )
		{
			if( MenuEntryBlock->bCloseSelfOnly )
			{
				// Close only this menu and its children
				FSlateApplication::Get().DismissMenuByWidget(AsShared());
			}
			else
			{
				// Dismiss the entire menu stack when a button is clicked to close all sub-menus
				FSlateApplication::Get().DismissAllMenus();
			}
		}
	}

	if( ActionList.IsValid() && MultiBlock->GetAction().IsValid() )
	{
		ActionList->ExecuteAction( MultiBlock->GetAction().ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		MenuEntryBlock->GetDirectActions().Execute();
	}

	// If we have a pull-down or sub-menu to summon, then go ahead and do that now
	if( !ClosingMenu && ( MenuEntryBlock->EntryBuilder.IsBound() || MenuEntryBlock->MenuBuilder.IsBound() || MenuEntryBlock->EntryWidget.IsValid() ) )
	{
		// Summon the menu!
		TSharedPtr< SMenuAnchor > PinnedMenuAnchor( MenuAnchor.Pin() );

		// Do not close the menu if its already open
		if( PinnedMenuAnchor.IsValid() && PinnedMenuAnchor->ShouldOpenDueToClick() )
		{
			FWidgetPath WidgetPath;
			FSlateApplication::Get().GeneratePathToWidgetUnchecked(PinnedMenuAnchor->AsShared(), WidgetPath);
			if ( WidgetPath.IsValid() )
			{
				// Don't process clicks that attempt to open sub-menus when the parent is queued for destruction.
				if ( !FSlateApplication::Get().IsWindowInDestroyQueue(WidgetPath.GetWindow()) )
				{
					// Close other open pull-down menus from this menu bar
					OwnerMultiBoxWidget.Pin()->CloseSummonedMenus();

					PinnedMenuAnchor->SetIsOpen(true);

					// Also tell the multibox about this open pull-down menu, so it can be closed later if we need to
					OwnerMultiBoxWidget.Pin()->SetSummonedMenu(PinnedMenuAnchor.ToSharedRef());

					// Make any clicked menu appear as old as possible (so they always pass the clobber min lifetime)
					OwnerMultiBoxWidget.Pin()->SetSummonedMenuTime(0.0);
				}
			}
		}
	}

	// When a menu item is clicked we open the sub-menu instantly or close the entire menu in the case this is an actual menu item.
	CancelPendingSubMenu();
}


/**
 * Called by Slate to determine if this menu entry is enabled
 * 
 * @return True if the menu entry is enabled, false otherwise
 */
bool SMenuEntryBlock::IsEnabled() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	bool bEnabled = true;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		bEnabled = ActionList->CanExecuteAction( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		bEnabled = DirectActions.CanExecute();
	}

	return bEnabled;
}

bool SMenuEntryBlock::IsEnabledDuringEditMode() const
{
	const FName EntryName = MultiBlock->GetExtensionHook();
	if (EntryName != NAME_None && OwnerMultiBoxWidget.IsValid())
	{
		TSharedRef< const FMultiBox > MultiBox = OwnerMultiBoxWidget.Pin()->GetMultiBox();
		if (UToolMenuBase* ToolMenu = MultiBox->GetToolMenu())
		{
			FCustomizedToolMenuHierarchy CustomizationHierarchy = ToolMenu->GetMenuCustomizationHierarchy();
			if (CustomizationHierarchy.Hierarchy.Num() > 0)
			{
				FCustomizedToolMenu FlattenedCustomization = CustomizationHierarchy.GenerateFlattened();

				if (MultiBlock->GetType() == EMultiBlockType::Heading)
				{
					if (FlattenedCustomization.GetSectionVisiblity(EntryName) == ECustomizedToolMenuVisibility::Hidden)
					{
						return false;
					}
				}
				else
				{
					if (FlattenedCustomization.GetEntryVisiblity(EntryName) == ECustomizedToolMenuVisibility::Hidden)
					{
						return false;
					}

					FName SectionName = FlattenedCustomization.GetEntrySectionName(EntryName);
					if (SectionName == NAME_None)
					{
						SectionName = ToolMenu->GetSectionName(EntryName);
					}

					if (SectionName != NAME_None)
					{
						if (FlattenedCustomization.GetSectionVisiblity(SectionName) == ECustomizedToolMenuVisibility::Hidden)
						{
							return false;
						}
					}
				}
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}

/**
 * Called by Slate when this menu entry check box button is toggled
 */
void SMenuEntryBlock::OnCheckStateChanged( const ECheckBoxState NewCheckedState )
{
	// The check box was clicked
	const bool bCheckBoxClicked = true;
	OnClicked( bCheckBoxClicked );
}

/**
 * Called by slate to determine if this menu entry should appear checked
 *
 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
 */
ECheckBoxState SMenuEntryBlock::IsChecked() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		CheckState = ActionList->GetCheckState( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		CheckState = DirectActions.GetCheckState();
	}

	return CheckState;
}

const FSlateBrush* SMenuEntryBlock::OnGetCheckImage() const
{
	return IsChecked() == ECheckBoxState::Checked ? CheckedImage : UncheckedImage;
}

/**
 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
 *
 * @param MyGeometry The Geometry of the widget receiving the event
 * @param MouseEvent Information about the input event
 */
void SMenuEntryBlock::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{

	SMultiBlockBaseWidget::OnMouseEnter( MyGeometry, MouseEvent );

	// Button was clicked, so trigger the action!
	TSharedRef< const FMenuEntryBlock > MenuEntryBlock = StaticCastSharedRef< const FMenuEntryBlock >( MultiBlock.ToSharedRef() );
	
	TSharedPtr< SMultiBoxWidget > PinnedOwnerMultiBoxWidget( OwnerMultiBoxWidget.Pin() );
	check( PinnedOwnerMultiBoxWidget.IsValid() );

	// (Almost) never dismiss another entry's submenu while the cursor is potentially moving toward that menu.  It's
	// not fun to try to keep the mouse in the menu entry bounds while moving towards the actual menu!
	// We will, hoever, dismiss another entry's submenu if the lifetime of the submenu hasn't passed the clobber
	// min lifetime yet. This is so that the user can sweep over multiple entries with submenus, landing on the desired
	// entry without the first submenu being "sticky".
	const TSharedPtr< const SMenuAnchor > OpenedMenuAnchor( PinnedOwnerMultiBoxWidget->GetOpenMenu() );
	const bool bSubMenuAlreadyOpen = ( OpenedMenuAnchor.IsValid() && OpenedMenuAnchor->IsOpen());
	const bool bClobberMinLifetimeElapsed = FPlatformTime::Seconds() > PinnedOwnerMultiBoxWidget->GetSummonedMenuTime() + MultiBoxConstants::SubMenuClobberMinLifetime;
	bool bMouseEnteredTowardSubMenu = false;
	{
		if( bSubMenuAlreadyOpen )
		{
			const FVector2D& SubMenuPosition = OpenedMenuAnchor->GetMenuPosition();
			const bool bIsMenuTowardRight = MouseEvent.GetScreenSpacePosition().X < SubMenuPosition.X;
			const bool bDidMouseEnterTowardRight = MouseEvent.GetCursorDelta().X >= 0.0f;	// NOTE: Intentionally inclusive of zero here.
			bMouseEnteredTowardSubMenu = ( bIsMenuTowardRight == bDidMouseEnterTowardRight );
		}
	}

	// For menu bar entries, we also need to handle mouse enter/leave events, so we can show and hide
	// the pull-down menu appropriately
	if( MenuEntryBlock->EntryBuilder.IsBound() || MenuEntryBlock->MenuBuilder.IsBound() || MenuEntryBlock->EntryWidget.IsValid() )
	{
		// Do we have a different pull-down menu open?
		TSharedPtr< SMenuAnchor > PinnedMenuAnchor( MenuAnchor.Pin() );
		if( MenuEntryBlock->bIsSubMenu )
		{
			if ( MenuEntryBlock->bOpenSubMenuOnClick == false )
			{
				if( PinnedOwnerMultiBoxWidget->GetOpenMenu() != PinnedMenuAnchor )
				{
					const bool bClobber = bSubMenuAlreadyOpen && bMouseEnteredTowardSubMenu && bClobberMinLifetimeElapsed;
					RequestSubMenuToggle( true, bClobber );
				}
			}
		}
		else if( bSubMenuAlreadyOpen && OpenedMenuAnchor != PinnedMenuAnchor )
		{
			// Close other open pull-down menus from this menu bar
			PinnedOwnerMultiBoxWidget->CloseSummonedMenus();

			// Summon the new pull-down menu!
			if( PinnedMenuAnchor.IsValid() )
			{
				PinnedMenuAnchor->SetIsOpen( true );

				// Also tell the multibox about this open pull-down menu, so it can be closed later if we need to
				PinnedOwnerMultiBoxWidget->SetSummonedMenu( PinnedMenuAnchor.ToSharedRef());

				// Make open pull-downs appear old as possible (so they always pass the clobber min lifetime)
				PinnedOwnerMultiBoxWidget->SetSummonedMenuTime(0.0);
			}
		}
		
	}
	else if( bSubMenuAlreadyOpen )
	{
		// Hovering over a menu item that is not a sub-menu, we need to close any sub-menus that are open
		const bool bClobber = bSubMenuAlreadyOpen && bMouseEnteredTowardSubMenu && bClobberMinLifetimeElapsed;
		RequestSubMenuToggle( false, bClobber);
	}
	
}

void SMenuEntryBlock::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SMultiBlockBaseWidget::OnMouseLeave( MouseEvent );

	// Reset any pending sub-menus that may be opening when we stop hovering over it
	CancelPendingSubMenu();
}

FReply SMenuEntryBlock::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent )
{
	SMultiBlockBaseWidget::OnKeyDown(MyGeometry, KeyEvent);

	// allow use of up and down keys to transfer focus
	if(KeyEvent.GetKey() == EKeys::Up || KeyEvent.GetKey() == EKeys::Down)
	{
		// find the next widget to focus
		EUINavigation MoveDirection = (KeyEvent.GetKey() == EKeys::Up)
			? EUINavigation::Previous
			: EUINavigation::Next;
		 
		return SMultiBoxWidget::FocusNextWidget(MoveDirection);
	}

	return FReply::Unhandled();
}

//void SMenuEntryBlock::UpdateSubMenuState()
//{
//	// Check to see if there is a pending sub-menu request
//	if( SubMenuRequestState != Idle )
//	{
//		// Reduce the time until the new menu opens
//		TimeToSubMenuOpen -= FSlateApplication::Get().GetDeltaTime();
//		if( TimeToSubMenuOpen <= 0.0f )
//		{
//			const bool bSubMenuNeedsToOpen = ( SubMenuRequestState == WantOpen );
//			SubMenuRequestState = Idle;
//
//			// The menu should be opened now as our timer is up
//			TSharedRef< const FMenuEntryBlock > MenuEntryBlock = StaticCastSharedRef< const FMenuEntryBlock >( MultiBlock.ToSharedRef() );
//
//			TSharedPtr< SMultiBoxWidget > PinnedOwnerMultiBoxWidget( OwnerMultiBoxWidget.Pin() );
//			check( PinnedOwnerMultiBoxWidget.IsValid() );
//
//			if( bSubMenuNeedsToOpen )
//			{
//				// For menu bar entries, we also need to handle mouse enter/leave events, so we can show and hide
//				// the pull-down menu appropriately
//				check( MenuEntryBlock->EntryBuilder.IsBound() || MenuEntryBlock->MenuBuilder.IsBound() || MenuEntryBlock->EntryWidget.IsValid() );
//
//				// Close other open pull-down menus from this menu bar
//				// Do we have a different pull-down menu open?
//				TSharedPtr< SMenuAnchor > PinnedMenuAnchor( MenuAnchor.Pin() );
//				if( PinnedOwnerMultiBoxWidget->GetOpenMenu() != PinnedMenuAnchor )
//				{
//					PinnedOwnerMultiBoxWidget->CloseSummonedMenus();
//
//					// Summon the new pull-down menu!
//					if( PinnedMenuAnchor.IsValid() )
//					{
//						PinnedMenuAnchor->SetIsOpen( true );
//					}
//
//					// Also tell the multibox about this open pull-down menu, so it can be closed later if we need to
//					PinnedOwnerMultiBoxWidget->SetSummonedMenu( PinnedMenuAnchor.ToSharedRef() );
//				}
//			}
//			else
//			{
//				PinnedOwnerMultiBoxWidget->CloseSummonedMenus();
//			}
//		}
//
//	}
//}

EActiveTimerReturnType SMenuEntryBlock::UpdateSubMenuState(double InCurrentTime, float InDeltaTime, bool bWantsOpen)
{
	//const bool bSubMenuNeedsToOpen = ( SubMenuRequestState == WantOpen );
	//SubMenuRequestState = Idle;

	// The menu should be opened now as our timer is up
	TSharedRef< const FMenuEntryBlock > MenuEntryBlock = StaticCastSharedRef< const FMenuEntryBlock >(MultiBlock.ToSharedRef());

	TSharedPtr< SMultiBoxWidget > PinnedOwnerMultiBoxWidget(OwnerMultiBoxWidget.Pin());
	check(PinnedOwnerMultiBoxWidget.IsValid());

	if (bWantsOpen)
	{
		// For menu bar entries, we also need to handle mouse enter/leave events, so we can show and hide
		// the pull-down menu appropriately
		check(MenuEntryBlock->EntryBuilder.IsBound() || MenuEntryBlock->MenuBuilder.IsBound() || MenuEntryBlock->EntryWidget.IsValid());

		// Close other open pull-down menus from this menu bar
		// Do we have a different pull-down menu open?
		TSharedPtr< SMenuAnchor > PinnedMenuAnchor(MenuAnchor.Pin());
		if (PinnedOwnerMultiBoxWidget->GetOpenMenu() != PinnedMenuAnchor)
		{
			PinnedOwnerMultiBoxWidget->CloseSummonedMenus();

			// Summon the new pull-down menu!
			if (PinnedMenuAnchor.IsValid())
			{
				PinnedMenuAnchor->SetIsOpen(true);
			}

			// Also tell the multibox about this open pull-down menu, so it can be closed later if we need to
			PinnedOwnerMultiBoxWidget->SetSummonedMenu(PinnedMenuAnchor.ToSharedRef());

			// If there was a delay before summoning this menu, use the initial time before the delay to record the menu's lifetime.
			// This makes it so that the new menu will, in turn, require a delay to clobber.
			PinnedOwnerMultiBoxWidget->SetSummonedMenuTime(InCurrentTime - InDeltaTime);
		}
	}
	else
	{
		PinnedOwnerMultiBoxWidget->CloseSummonedMenus();
	}

	return EActiveTimerReturnType::Stop;
}

//void SMenuEntryBlock::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
//{
//	UpdateSubMenuState();
//}

/**
 * Called to create content for a pull-down or sub-menu window when it's summoned by the user
 *
 * @return	The widget content for the new menu
 */
TSharedRef< SWidget > SMenuEntryBlock::MakeNewMenuWidget() const
{
	TSharedRef< const FMenuEntryBlock > MenuEntryBlock = StaticCastSharedRef< const FMenuEntryBlock >( MultiBlock.ToSharedRef() );

	check( OwnerMultiBoxWidget.IsValid() );

	TSharedPtr<SMultiBoxWidget> MultiBoxWidget = OwnerMultiBoxWidget.Pin();
	const ISlateStyle* const StyleSet = MultiBoxWidget->GetStyleSet();

	// Check each of the menu entry creation methods to see which one's been set, then use it to create the entry

	if (MenuEntryBlock->EntryBuilder.IsBound())
	{
		const bool bCloseSelfOnly = false;

		FName SubMenuCustomizationName;
		if (MenuEntryBlock->GetExtensionHook() != NAME_None)
		{
			FName CustomizationName = MultiBoxWidget->GetMultiBox()->GetCustomizationName();
			if (CustomizationName != NAME_None)
			{
				SubMenuCustomizationName = *(CustomizationName.ToString() + TEXT(".") + MenuEntryBlock->GetExtensionHook().ToString());
			}
		}

		FMenuBuilder MenuBuilder(MenuEntryBlock->bShouldCloseWindowAfterMenuSelection,
		                         MultiBlock->GetActionList(),
		                         MenuEntryBlock->Extender,
		                         bCloseSelfOnly,
		                         StyleSet,
		                         /*Searchable*/true, // Best guess default. This could be changed by the EntryBuilder delegate where more context is available.
		                         SubMenuCustomizationName,
		                         MenuEntryBlock->IsRecursivelySearchable()); // Best guess default. This could be changed by the EntryBuilder delegate where more context is available.

		MenuEntryBlock->EntryBuilder.Execute( MenuBuilder );

		return MenuBuilder.MakeWidget();
	}
	else if (MenuEntryBlock->MenuBuilder.IsBound())
	{
		return MenuEntryBlock->MenuBuilder.Execute();
	}
	else if (MenuEntryBlock->EntryWidget.IsValid())
	{
		static constexpr bool bCloseSelfOnly = false;
		FMenuBuilder MenuBuilder(MenuEntryBlock->bShouldCloseWindowAfterMenuSelection, nullptr, TSharedPtr<FExtender>(), bCloseSelfOnly, StyleSet );
		{
			MenuBuilder.AddWidget( MenuEntryBlock->EntryWidget.ToSharedRef(), FText::GetEmpty(), FMenuEntryStyleParams() );
		}

		return MenuBuilder.MakeWidget();
	}
	else
	{
		// No entry creation method was initialized
		check(false);
		return SNullWidget::NullWidget;
	}
}

/**
 * Called to get the appropriate border for Buttons on Menu Bars based on whether or not submenu is open
 *
 * @return	The appropriate border to use
 */

const FSlateBrush* SMenuEntryBlock::GetMenuBarButtonBorder( ) const
{
	TSharedPtr<SMenuAnchor> MenuAnchorSharedPtr = MenuAnchor.Pin();

	if (MenuAnchorSharedPtr.IsValid() && MenuAnchorSharedPtr->IsOpen())
	{
		return &MenuBarButtonStyle->Pressed;
	}
	else if (IsHovered())
	{
		return &MenuBarButtonStyle->Hovered;
	}
	else
	{
		return &MenuBarButtonStyle->Normal;
	}
}

FSlateColor SMenuEntryBlock::GetMenuBarForegroundColor() const
{
	TSharedPtr<SMenuAnchor> MenuAnchorSharedPtr = MenuAnchor.Pin();

	if (MenuAnchorSharedPtr.IsValid() && MenuAnchorSharedPtr->IsOpen())
	{
		return MenuBarButtonStyle->PressedForeground;
	}
	else if (IsHovered())
	{
		return MenuBarButtonStyle->HoveredForeground;
	}
	else
	{
		return MenuBarButtonStyle->NormalForeground;
	}
}

FSlateColor SMenuEntryBlock::TintOnHover() const
{
	if ( this->IsHovered() )
	{
		check( OwnerMultiBoxWidget.IsValid() );

		TSharedPtr<SMultiBoxWidget> MultiBoxWidget = OwnerMultiBoxWidget.Pin();
		const ISlateStyle* const StyleSet = MultiBoxWidget->GetStyleSet();

		static const FName SelectionColorName("SelectionColor");
		return StyleSet->GetSlateColor(SelectionColorName);
	}
	else
	{
		return FSlateColor::UseForeground();
	}
}

FSlateColor SMenuEntryBlock::InvertOnHover() const
{
	if (ShouldSubMenuAppearHovered())
	{
		return FLinearColor::Black;
	}
	else if (this->IsHovered())
	{
		return FSlateColor::UseStyle();
	}
	else
	{
		return FSlateColor::UseStyle();
	}
}


/**
 * Private helper to assign a checkbox image from a given style. Used to create static check boxes
 * so we don't have to literally create a read only checkbox just to show the image for one.
 * 
 * @param Style the style we are extracting the image brushes from.
 * @return the brush associated with the checkbox state for this MenuEntryBlock
 */
const FSlateBrush* SMenuEntryBlock::GetCheckBoxImageBrushFromStyle(const FCheckBoxStyle* Style) const
{
	if (IsHovered()) 
	{
		switch (IsChecked())
		{
			case ECheckBoxState::Checked: return &Style->CheckedHoveredImage;
			case ECheckBoxState::Unchecked: return &Style->UncheckedHoveredImage;
			default: return &Style->UndeterminedHoveredImage;
		}
	}
	else
	{
		switch (IsChecked())
		{
			case ECheckBoxState::Checked: return &Style->CheckedImage;
			case ECheckBoxState::Unchecked: return &Style->UncheckedImage;
			default: return &Style->UndeterminedImage;
		}
	}
}
