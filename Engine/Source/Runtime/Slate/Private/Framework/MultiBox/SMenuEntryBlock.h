// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

class FActiveTimerHandle;

template <>
struct TWidgetTypeTraits<class SMenuEntryBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Menu entry MultiBlock
 */
class FMenuEntryBlock
	: public FMultiBlock
{

public:

	/**
	 * Constructor
	 *
	 * @param	InCommand			The command associated with this menu entry
	 * @param	InCommandList		The list of commands that are mapped to delegates so that we know what to execute when this menu entry is activated
	 * @param	InLabelOverride		Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride	Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride		Optional icon to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 * @param	bInShouldCloseWindowAfterMenuSelection	In the case of a submenu, whether it should close after an item is selected
	 * @param	bInInvertLabelOnHover	Whether to invert the label text's color on hover
	 */
	FMenuEntryBlock( const FName& InExtensionHook, const TSharedPtr< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const FSlateIcon& InIconOverride = FSlateIcon(), bool bInCloseSelfOnly = false, bool bInShouldCloseWindowAfterMenuSelection = true);


	/**
	 * Constructor
	 *
	 * @param	InLabel				The label to display in the menu
	 * @param	InToolTip			The tool tip to display when the menu entry is hovered over
 	 * @param	InEntryBuilder		Menu builder object for the menu to add.
	 * @param	InExtender			The menu extender class to pass down to child menus
	 * @param	bInSubMenu			True if this menu entry spawns a sub-menu instead of activating a command
	 * @param	bInSubMenuOnClick	True if this menu entry spawns a sub-menu only by clicking on it
	 * @param	InCommandList		The list of commands bound to delegates that should be executed for menu entries
	 * @param	InIcon				The icon to display to the left of the label
	 * @param	bInShouldCloseWindowAfterMenuSelection	In the case of a submenu, whether it should close after an item is selected
	 * @param	bInInvertLabelOnHover	Whether to invert the label text's color on hover
	 */
	FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, const FSlateIcon& InIcon = FSlateIcon(), bool bInShouldCloseWindowAfterMenuSelection = true);
	

	/**
	 * Constructor
	 *
	 * @param	InLabel				The label to display in the menu
	 * @param	InToolTip			The tool tip to display when the menu entry is hovered over
 	 * @param	InMenuBuilder		Menu widget object to add.
	 * @param	InExtender			The menu extender class to pass down to child menus
	 * @param	bInSubMenu			True if this menu entry spawns a sub-menu instead of activating a command
	 * @param	bInSubMenuOnClick	True if this menu entry spawns a sub-menu only by clicking on it
	 * @param	InCommandList		The list of commands bound to delegates that should be executed for menu entries
	 * @param	InIcon				The icon to display to the left of the label
	 * @param	bInShouldCloseWindowAfterMenuSelection	In the case of a submenu, whether it should close after an item is selected
	 * @param	bInInvertLabelOnHover	Whether to invert the label text's color on hover
	 */
	FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FOnGetContent& InMenuBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, const FSlateIcon& InIcon = FSlateIcon(), bool bInShouldCloseWindowAfterMenuSelection = true);

	/**
	 * Constructor
	 *
	 * @param	InLabel				The label to display in the menu
	 * @param	InToolTip			The tool tip to display when the menu entry is hovered over
 	 * @param	InEntryWidget		Menu widget object to add.
	 * @param	InExtender			The menu extender class to pass down to child menus
	 * @param	bInSubMenu			True if this menu entry spawns a sub-menu instead of activating a command
	 * @param	bInSubMenuOnClick	True if this menu entry spawns a sub-menu only by clicking on it
	 * @param	InCommandList		The list of commands bound to delegates that should be executed for menu entries
	 * @param	InIcon				The icon to display to the left of the label
	 * @param	bInShouldCloseWindowAfterMenuSelection	In the case of a submenu, whether it should close after an item is selected
	 * @param	bInInvertLabelOnHover	Whether to invert the label text's color on hover
	 */
	FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TSharedPtr<SWidget>& InEntryWidget, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, const FSlateIcon& InIcon = FSlateIcon(), bool bInShouldCloseWindowAfterMenuSelection = true);


	/**
	 * Constructor for use with dynamic menu entries that do not have commands
	 *
	 * @param	InLabel				The label to display in the menu
 	 * @param	InToolTip			The tool tip to display when the menu entry is hovered over
 	 * @param	InIcon				The icon to display to the left of the label
	 * @param	InUIAction			UI action to take when this menu item is clicked as well as to determine if the menu entry can be executed or appears "checked"
	 * @param	InUserInterfaceActionType	Type of interface action
	 * @param	bInShouldCloseWindowAfterMenuSelection	In the case of a submenu, whether it should close after an item is selected
	 * @param	bInInvertLabelOnHover	Whether to invert the label text's color on hover
	 * @param	InCommandList		The list of commands bound to delegates that should be executed for menu entries
	 * @param	InInputBindingOverride		Optional overridden input binding text for this menu entry
	 */
	FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, const FSlateIcon& InIcon, const FUIAction& InUIAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection = true, TSharedPtr< const FUICommandList > InCommandList = nullptr, const TAttribute<FText>& InInputBindingOverride = TAttribute<FText>());

	FMenuEntryBlock( const FName& InExtensionHook, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FSlateIcon& InIcon, const FUIAction& InUIAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection = true, const TAttribute<FText>& InInputBindingOverride = TAttribute<FText>());

	FMenuEntryBlock( const FName& InExtensionHook, const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const TAttribute<FText>& InToolTip, const EUserInterfaceActionType InUserInterfaceActionType, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection = true);

	FMenuEntryBlock( const FName& InExtensionHook, const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection = true);

	FMenuEntryBlock( const FName& InExtensionHook, const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InEntryBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, bool bInShouldCloseWindowAfterMenuSelection = true);

	FMenuEntryBlock( const FName& InExtensionHook, const FUIAction& UIAction, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FOnGetContent& InMenuBuilder, TSharedPtr<FExtender> InExtender, bool bInSubMenu, bool bInSubMenuOnClick, bool bInCloseSelfOnly, const FSlateIcon& InIcon = FSlateIcon(), bool bInShouldCloseWindowAfterMenuSelection = true);

	/** Construct a menu entry given param struct */
	FMenuEntryBlock(const FMenuEntryParams& InMenuEntryParams);

	/** FMultiBlock interface */
	virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const override;
	virtual bool HasIcon() const override;
	
	/** the override for the checkbox style */
	void SetCheckBoxStyle(FName InCheckBoxStyle);

	/** Returns whether this menu entry block opens a sub-menu. */
	bool IsSubMenu() const { return bIsSubMenu; }

	/** Sets whether the menu search algorithm should walk down this menu sub-menus. */
	void SetRecursivelySearchable(bool bInRecursivelySearchable) { bIsRecursivelySearchable = bInRecursivelySearchable; }

	/** Returns whether the menu search algorithm should walk down this menu sub-menus. */
	bool IsRecursivelySearchable() const { return bIsRecursivelySearchable; }

private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;

private:

	// Friend our corresponding widget class
	friend class SMenuEntryBlock;
	friend class FSlateMacMenu;

	/** Optional overridden text label for this menu entry.  If not set, then the action's label will be used instead. */
	TAttribute<FText> LabelOverride;

	/** Optional overridden tool tip for this menu entry.  If not set, then the action's tool tip will be used instead. */
	TAttribute<FText> ToolTipOverride;

	/** Optional overridden input binding text for this menu entry.  If not set, then the UI action's binding will be used if available. */
	TAttribute<FText> InputBindingOverride;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	FSlateIcon IconOverride;

	/** Optional menu entry builder associated with this entry for building sub-menus and pull down menus */
	FNewMenuDelegate EntryBuilder;

	/** Delegate that returns an entire menu */
	FOnGetContent MenuBuilder;

	/** Widget to be added to the menu */
	TSharedPtr<SWidget> EntryWidget;

	/** True if this menu entry opens a sub-menu */
	bool bIsSubMenu;

	/** True if the search algorithm should walk down this menu sub menus. Usually true, unless the menu has circular/infinite expansion (happens in some menus generated on the fly by reflection). */
	bool bIsRecursivelySearchable;

	/** True if this menu entry opens a sub-menu by clicking on it only */
	bool bOpenSubMenuOnClick;

	/** In the case where a command is not bound, the user interface action type to use.  If a command is bound, we
	    simply use the action type associated with that command. */
	EUserInterfaceActionType UserInterfaceActionType;

	/** True if the menu should close itself and all its children or the entire open menu stack */
	bool bCloseSelfOnly;

	/** An extender that this menu entry should pass down to its children, so they get extended properly */
	TSharedPtr<FExtender> Extender;

	/** For submenus, whether the menu should be closed after something is selected */
	bool bShouldCloseWindowAfterMenuSelection;

	/** Whether to invert the label text's color on hover */
	bool bInvertLabelOnHover;

	/** override of the checkbox style */
	FName CheckBoxStyle = NAME_None;
};




/**
 * Menu entry MultiBlock widget
 */
class SMenuEntryBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SMenuEntryBlock ){}

	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Called to create content for a pull-down or sub-menu window when it's summoned by the user
	 *
	 * @return	The widget content for the new menu
	 */
	SLATE_API TSharedRef< SWidget > MakeNewMenuWidget() const;


protected:
	/** Struct for creating menu entry widgets */
	struct FMenuEntryBuildParams
	{
		/** The owning multibox */
		TSharedPtr< const FMultiBox > MultiBox;
		/** Our menu entry block */
		TSharedPtr< const FMenuEntryBlock > MenuEntryBlock;
		/** UI Command associated with the menu entry */
		TSharedPtr< const FUICommandInfo > UICommand;
		/** The text to display */
		TAttribute<FText> Label;
		/** The tooltip to display */
		TAttribute<FText> ToolTip;
		/** The input binding to display */
		TAttribute<FText> InputBinding;
		/** The style set to use */
		const ISlateStyle* StyleSet;
		/** The style name to use */
		FName StyleName;
	};


	/**
	 * Called by Slate when this menu entry's button is clicked
	 */
	SLATE_API FReply OnMenuItemButtonClicked();

	/**
	 * Called when a checkbox in the menu item or the menu item itself is clicked
	 *
	 * @param bCheckBoxClicked	true if a check box was clicked and not the menu item.  We dont close the menu when a check box is clicked
	 */
	SLATE_API void OnClicked( bool bCheckBoxClicked );

	/**
	 * Called by Slate to determine if this menu entry is enabled
	 * 
	 * @return True if the menu entry is enabled, false otherwise
	 */
	SLATE_API bool IsEnabled() const;

	/**
	 * Called by Slate to determine if this menu entry is enabled (during menu editing)
	 * 
	 * @return True if the menu entry is enabled, false otherwise
	 */
	SLATE_API bool IsEnabledDuringEditMode() const;

	/**
	 * Called by Slate when this check box button is toggled in a menu entry
	 */
	SLATE_API void OnCheckStateChanged( const ECheckBoxState NewCheckedState );

	/**
	 * Called by slate to determine if this menu entry should appear checked
	 *
	 * @return true if it should be checked, false if not.
	 */
	SLATE_API ECheckBoxState IsChecked() const;

	/**
	 * In the case that we have an icon to show.  This function is called to get the image that indicates the menu item should appear checked
	 * If we can show an actual check box, this function is not called
	 */
	SLATE_API const FSlateBrush* OnGetCheckImage() const;

	// SWidget interface
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent ) override;
	//virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface


	/**
	 * Called to get the appropriate border for Buttons on Menu Bars based on whether or not submenu is open
	 *
	 * @return	The appropriate border to use
	 */
	SLATE_API const FSlateBrush* GetMenuBarButtonBorder() const;
	SLATE_API FSlateColor GetMenuBarForegroundColor() const;

	/**
	 * Called to create content for a pull-down menu widget
	 *
	 * @param InBuildParams	Parameters for how to build the widget
	 * @return	The widget content for the new menu
	 */
	SLATE_API TSharedRef< SWidget > BuildMenuBarWidget( const FMenuEntryBuildParams& InBuildParams );
	
	/**
	* Finds the STextBlock that gets displayed in the UI
	*
	* @param Content	Widget to check for an STextBlock
	* @return	The STextBlock widget found
	*/
	SLATE_API TSharedRef<SWidget> FindTextBlockWidget(TSharedRef<SWidget> Content);

	/**
	 * Called to create content for a menu entry inside a pull-down, context, or sub-menu
	 *
	 * @param InBuildParams	Parameters for how to build the widget
	 * @return The widget content for the new menu
	 */
	SLATE_API TSharedRef< SWidget > BuildMenuEntryWidget( const FMenuEntryBuildParams& InBuildParams );

	/**
	 * Called to create content for a sub-menu widget (entry in a menu that opens another menu to the right of it)
	 *
	 * @param InBuildParams	Parameters for how to build the widget
	 * @return The widget content for the new menu
	 */
	SLATE_API TSharedRef< SWidget > BuildSubMenuWidget( const FMenuEntryBuildParams& InBuildParams );

	/**
	 * Requests that the sub-menu associated with this widget be toggled on or off.
	 * It does not immediately toggle the menu.  After a set amount of time is passed the menu will toggle
	 *
	 * @param bOpenMenu	true to open the menu, false to close the menu if one is currently open
	 * @param bClobber true if we want to open a menu when another menu is already open
	 */
	SLATE_API void RequestSubMenuToggle( bool bOpenMenu, const bool bClobber );

	/**
	 * Cancels any open requests to toggle a sub-menu       
	 */
	SLATE_API void CancelPendingSubMenu();

	/**
	 * Returns whether or the sub-menu entry should appear hovered.  If the sub-menu is open we will always show the menu as hovered to indicate which sub-menu is open
	 * In the case that the user is interacting in this menu we do not show the menu as hovered because we need to show what the user is actually selecting
	 */
	SLATE_API bool ShouldSubMenuAppearHovered() const;
	
	/**
	 * Called to query the tool tip text for this widget, but will return an empty text for menu bar items
	 * when a menu for that menu bar is already open
	 *
	 * @param	ToolTipText	Tool tip text to display, if possible
	 *
	 * @return	Tool tip text, or an empty text if filtered out
	 */
	SLATE_API FText GetFilteredToolTipText( TAttribute<FText> ToolTipText ) const;


	// Gets the visibility of the menu item
	SLATE_API EVisibility GetVisibility() const;

	/** Get the selection color when the entry is hovered */
	SLATE_API FSlateColor TintOnHover() const;

	/** Get the inverted foreground color when the entry is hovered */
	SLATE_API FSlateColor InvertOnHover() const;

	/** Updates state machine for sub-menu opening logic.  Called in the widget's Tick as well as on demand in some cases. */
	//void UpdateSubMenuState();

private:
	const FSlateBrush* GetCheckBoxImageBrushFromStyle(const FCheckBoxStyle* Style) const;

	/** One-off delayed active timer to update the open/closed state of the sub menu. */
	EActiveTimerReturnType UpdateSubMenuState(double InCurrentTime, float InDeltaTime, bool bWantsOpen);

	/** The handle to the active timer to update the sub-menu state */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** The brush to use when an item should appear checked */
	const FSlateBrush* CheckedImage;
	/** The brush to use when an item should appear unchecked */
	const FSlateBrush* UncheckedImage;
	/** For pull-down or sub-menu entries, this stores a weak reference to the menu anchor widget that we'll use to summon the menu */
	TWeakPtr< SMenuAnchor > MenuAnchor;
	
	const FButtonStyle* MenuBarButtonStyle;
};
