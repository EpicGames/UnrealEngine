// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SMenuSeparatorBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSeparator.h"


/**
 * Constructor
 */
FMenuSeparatorBlock::FMenuSeparatorBlock(const FName& InExtensionHook, bool bInIsPartOfHeading)
	: FMultiBlock( nullptr, nullptr, InExtensionHook, EMultiBlockType::Separator, bInIsPartOfHeading )
{
	SetSearchable(false);
}


/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FMenuSeparatorBlock::ConstructWidget() const
{
	return SNew( SMenuSeparatorBlock );
}



/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SMenuSeparatorBlock::Construct( const FArguments& InArgs )
{
}



/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SMenuSeparatorBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	ChildSlot
	[
		SNew( SVerticalBox )

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(StyleSet->GetMargin(StyleName, ".Separator.Padding"))
		[
			SNew( SSeparator )
			.SeparatorImage(StyleSet->GetBrush(StyleName, ".Separator"))
			.Thickness(1.0f)
		]
	];

	SetVisibility(MultiBlock->GetVisibilityOverride());

	// Add this widget to the search list of the multibox and hide it
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), FText::GetEmpty(), MultiBlock->GetSearchable());
}
