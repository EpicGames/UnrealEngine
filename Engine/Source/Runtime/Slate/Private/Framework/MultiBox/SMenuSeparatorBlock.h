// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

template <>
struct TWidgetTypeTraits<class SMenuSeparatorBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Menu separator MultiBlock
 */
class FMenuSeparatorBlock
	: public FMultiBlock
{

public:

	/**
	 * Constructor
	 */
	FMenuSeparatorBlock(const FName& InExtensionHook, bool bInIsPartOfHeading);


private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;


private:

	// Friend our corresponding widget class
	friend class SMenuSeparatorBlock;

};



/**
 * Menu separator MultiBlock widget
 */
class SMenuSeparatorBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SMenuSeparatorBlock ){}

	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );


private:

};
