// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SMenuSpacerBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSeparator.h"

/**
 * Constructor
 */
FMenuSpacerBlock::FMenuSpacerBlock(
	const FName& InExtensionHook,
	bool bInIsPartOfHeading)
	: FMultiBlock(
		nullptr
		, nullptr
		, InExtensionHook
		, EMultiBlockType::Separator
		, bInIsPartOfHeading)
{
	SetSearchable(false);
}

/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef<class IMultiBlockBaseWidget> FMenuSpacerBlock::ConstructWidget() const
{
	return SNew(SMenuSpacerBlock);
}

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SMenuSpacerBlock::Construct(const FArguments& InArgs)
{
}

/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SMenuSpacerBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	// Unlike the separator, a spacer has no visual representation, so we halve the vertical padding to effectively maintain spacing between visual elements
	static const FMargin BlockPadding = StyleSet->GetMargin(StyleName, ".Separator.Padding") * FMargin(1.0f, 0.5f);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(BlockPadding)
		[
			SNew(SSeparator)
			.SeparatorImage(StyleSet->GetBrush(StyleName, ".Separator"))
			.Visibility(EVisibility::Hidden)
			.Thickness(1.0f)
		]
	];

	// Add this widget to the search list of the multibox and hide it
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), FText::GetEmpty(), MultiBlock->GetSearchable());

	SetVisibility(MultiBlock->GetVisibilityOverride());
}
