// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

template <>
struct TWidgetTypeTraits<class SMenuSpacerBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Menu spacer MultiBlock, a variant of the separator block that has no drawn representation, but consumes space in the menu
 */
class FMenuSpacerBlock
	: public FMultiBlock
{
public:
	/**
	 * Constructor
	 */
	FMenuSpacerBlock(const FName& InExtensionHook, bool bInIsPartOfHeading);

private:
	/** FMultiBlock private interface */
	virtual TSharedRef<class IMultiBlockBaseWidget> ConstructWidget() const override;

private:
	// Friend our corresponding widget class
	friend class SMenuSpacerBlock;
};


/**
 * Menu spacer MultiBlock widget
 */
class SMenuSpacerBlock
	: public SMultiBlockBaseWidget
{
public:
	SLATE_BEGIN_ARGS(SMenuSpacerBlock)
	{}
	SLATE_END_ARGS()

	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);
};
