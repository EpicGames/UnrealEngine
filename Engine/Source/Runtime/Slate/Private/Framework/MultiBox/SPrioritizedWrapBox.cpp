// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SPrioritizedWrapBox.h"

#include "Algo/AllOf.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/LayoutUtils.h"

namespace UE::Slate::PrioritizedWrapBox
{
	uint32 GetTypeHash(const SPrioritizedWrapBox::FSlot& InSlot)
	{
		return HashCombineFast(
			::GetTypeHash(InSlot.GetWrapPriority()),
			::GetTypeHash(InSlot.GetWrapMode()),
			::GetTypeHash(InSlot.GetWidget()));
	}

	class FChildArranger
	{
	private:
		static constexpr EOrientation Orientation = EOrientation::Orient_Horizontal;
		inline static const FVector2f MinimumLineSize = FVector2f(0.001f, 0.001f); // non-zero minimum

		struct FSlotProxy : ::FSlotProxy
		{
			using SlotType = SPrioritizedWrapBox::FSlot;

			FSlotProxy() = default;

			FSlotProxy(const int32 InSlotIndex, const SlotType& InSlot)
				: ::FSlotProxy(InSlotIndex, InSlot)
			{
				WrapMode = InSlot.GetWrapMode();
				VerticalOverflowBehavior = InSlot.GetVerticalOverflowBehavior();
				VerticalExpansionThreshold = InSlot.GetVerticalExpansionThreshold();
				WrapPriority = InSlot.GetWrapPriority();
				bExcludeIfFirstOrLast = InSlot.GetExcludeIfFirstOrLast();
				bForceNewLine = InSlot.GetForceNewLine();
			}

			/** Applies the given slot's values to this proxy. Will return true if any values have changed from those stored. */
			bool UpdateFromSlot(const int32 InSlotIndex, const SlotType& InSlot)
			{
				bool bAnyValueChanged = ::FSlotProxy::UpdateFromSlot<SlotType, true>(InSlotIndex, InSlot);

				bAnyValueChanged = bAnyValueChanged || (WrapMode != InSlot.GetWrapMode());
				WrapMode = InSlot.GetWrapMode();

				bAnyValueChanged = bAnyValueChanged || (VerticalOverflowBehavior != InSlot.GetVerticalOverflowBehavior());
				VerticalOverflowBehavior = InSlot.GetVerticalOverflowBehavior();

				bAnyValueChanged = bAnyValueChanged || (VerticalExpansionThreshold != InSlot.GetVerticalExpansionThreshold());
				VerticalExpansionThreshold = InSlot.GetVerticalExpansionThreshold();

				bAnyValueChanged = bAnyValueChanged || (WrapPriority != InSlot.GetWrapPriority());
				WrapPriority = InSlot.GetWrapPriority();

				bAnyValueChanged = bAnyValueChanged || (bForceNewLine != InSlot.GetForceNewLine());
				bForceNewLine = InSlot.GetForceNewLine();

				bAnyValueChanged = bAnyValueChanged || (bExcludeIfFirstOrLast != InSlot.GetExcludeIfFirstOrLast());
				bExcludeIfFirstOrLast = InSlot.GetExcludeIfFirstOrLast();

				return bAnyValueChanged;
			}

			EWrapMode WrapMode = EWrapMode::Preferred;
			EVerticalOverflowBehavior VerticalOverflowBehavior = EVerticalOverflowBehavior::Default;
			TOptional<float> VerticalExpansionThreshold;
			int32 WrapPriority = INDEX_NONE;
			bool bForceNewLine = false;
			bool bExcludeIfFirstOrLast = false;
		};

		/** Represents a contiguous block of layout elements, which can be either a single slot or sequence of slots. */
		struct FBlock
		{
			/** The original, left-to-right index. */
			int32 SequentialIndex = INDEX_NONE;

			int32 WrapPriority = INDEX_NONE;

			/** Whether the block can wrap. Note that if ForceNewLine is true, it will always move/"wrap" to a new line, */
			bool bCanWrap = true;

			/** Whether to forcibly place this block on a new line. Others can appear to the right of this block, but none to the left (if true). */
			bool bForceNewLine = false;

			/** Based on it's desired size, it was determined this doesn't vertically expand. */
			bool bHasVerticalExpansion = false;

			/** Based on one or more member slots' VerticalBehavior, this could *possibly* expand - the ReserveLength indicates a "probe length". */
			bool bCanVerticallyExpand = false;

			/** The last cached or calculated desired size. */
			FVector2f DesiredSize = FVector2f::Zero();

			/** Size adjusted by the layout algorithm. */
			FVector2f AdjustedSize = FVector2f::Zero();

			/** The minimum acceptable length of this block, which may be the same or less than the ReserveLength. */
			FVector2f::FReal MinLength = 0.0f;

			/** The desired length to reserve, used to test for wrapping. */
			FVector2f::FReal ReserveLength = 0.0f;

			/** The 2D area, calculated from the DesiredSize and maintained when calculating the ReserveLength. */
			FVector2f::FReal Area = 0.0f;

			/** The actual slots that make up this block, either singular or multiple when using grouping. */
			TArray<int32, TInlineAllocator<16>> SlotIndices;

			void ResetToDefault()
			{
				SequentialIndex = INDEX_NONE;
				WrapPriority = INDEX_NONE;
				bCanWrap = true;
				bForceNewLine = false;
				bHasVerticalExpansion = false;
				bCanVerticallyExpand = false;
				DesiredSize = FVector2f::Zero();
				AdjustedSize = FVector2f::Zero();
				ReserveLength = 0.0f;
				MinLength = 0.0f;
				Area = 0.0f;
				SlotIndices.Reset();
			}

			// Min length by default, Max length if expandable
			FVector2f::FReal GetEffectiveLength() const
			{
				return bHasVerticalExpansion ? ReserveLength : MinLength;
			}

			friend bool operator==(const FBlock& InLeft, const FBlock& InRight) { return InLeft.SequentialIndex == InRight.SequentialIndex; }
			friend bool operator==(const uint32& InLeft, const FBlock& InRight) { return InLeft == InRight.SequentialIndex; }
			friend bool operator<(const FBlock& InLeft, const FBlock& InRight) { return InLeft.SequentialIndex < InRight.SequentialIndex; }
		};

		struct FBlockLine
		{
			int32 Index = INDEX_NONE;
			FVector2f::FReal MinLength;
			FVector2f::FReal MaxLength;
			FVector2f::FReal EffectiveLength; // Min length by default, Max length if expandable
			FVector2f::FReal Height;
			TArray<FBlock> Blocks;
		};

	public:
		FChildArranger() = default;

		template <bool UseGroupedWrapping = false>
		FVector2D GetDesiredSize(const SPrioritizedWrapBox& InWidget, const TPanelChildren<SPrioritizedWrapBox::FSlot>& InChildren)
		{
			// This effectively performs a pre-arrangement of the slots
			// based only on the slot's individual desired size and the parent widgets wrapping length.
			// The parents DesiredSize/Geometry isn't always valid, or is a frame behind.

			int32 ParentLineLength = FMath::FloorToInt32(InWidget.GetPaintSpaceGeometry().Size.X);
			const int32 PreferredLineLength = InWidget.GetPreferredSize();

			// A slot that uses Preferred doesn't even attempt wrapping unless the parent size is the same or less than preferred
			const bool bDoPreferredWrapping = NumPreferredWrappingChildren > 0 && ParentLineLength <= PreferredLineLength;

			// A slot that uses Parent always attempts wrapping
			const bool bDoParentWrapping = NumParentWrappingChildren > 0;

			const float MinLineHeight = InWidget.GetMinLineHeight().Get(0.0f);
			float MaxLineHeight = std::numeric_limits<float>::max();

			if (ParentLineLength == 0)
			{
				LastParentLineLength = ParentLineLength = std::numeric_limits<uint16>::max();
				LastParentSize = FVector2f::One();
				MaxLineHeight = MinLineHeight;
			}

			const bool bSlotsHaveChanged = UpdateChildProxies(InChildren);

			if (bSlotsHaveChanged)
			{
				UpdateBlocks(InChildren, MinLineHeight);
			}

			const FVector2f ParentSize = InWidget.GetTickSpaceGeometry().Size;
			const bool bParentSizeChanged = (LastParentSize != ParentSize)
				|| (bDoPreferredWrapping && (PreferredLineLength != LastPreferredLineLength))
				|| (bDoParentWrapping && (ParentLineLength != LastParentLineLength));

			FVector2f DesiredSize = FVector2f::ZeroVector;

			if (bSlotsHaveChanged || bParentSizeChanged)
			{
				int32 NumArrangedBlocks = 0;

				TArray<FBlockLine> Lines;
				Lines.Reserve(Blocks.Num());

				TArray<FBlock> CurrentLineBlocks = Blocks;

				TArray<FBlock> NextLineBlocks;
				NextLineBlocks.Reserve(Blocks.Num());

				const float LinePadding = InWidget.GetLinePadding();
				float LineOffset = 0.0f;

				// We keep chopping the last item off to get the current wrapping candidate
				TConstArrayView<FBlock> SortedBlockView = MakeConstArrayView(WrappableSortedBlockView.GetData(), WrappableSortedBlockView.Num());

				// Re-calculates the line height based on its blocks
				auto UpdateLineHeight = [&](FBlockLine& InLine)
				{
					InLine.Height = 0.0f;
					for (int32 BlockIndex = 0; BlockIndex < InLine.Blocks.Num(); ++BlockIndex)
					{
						InLine.Height = FMath::Max(InLine.Height, InLine.Blocks[BlockIndex].DesiredSize.Y);
					}
				};

				auto AddBlockToLine = [](FBlockLine& InLine, const FBlock& InBlock)
				{
					InLine.MinLength += InBlock.MinLength;
					InLine.MaxLength += InBlock.ReserveLength;
					InLine.EffectiveLength += InBlock.GetEffectiveLength();
					InLine.Height = FMath::Max(InLine.Height, InBlock.DesiredSize.Y);
					InLine.Blocks.Emplace(InBlock);
				};

				auto RemoveBlockFromLine = [&](FBlockLine& InLine, const FBlock& InBlock) -> bool
				{
					if (const int32 FoundIndex = InLine.Blocks.Find(InBlock);
						FoundIndex != INDEX_NONE)
					{
						InLine.EffectiveLength -= InBlock.GetEffectiveLength();
						InLine.MinLength -= InBlock.MinLength;
						InLine.MaxLength -= InBlock.ReserveLength;
						InLine.Blocks.RemoveAt(FoundIndex, EAllowShrinking::No);

						UpdateLineHeight(InLine);

						return true;
					}

					return false;
				};

				auto MakeBlockLine = [&](FBlockLine& InLine, const TConstArrayView<FBlock> InBlocks)
				{
					for (int32 BlockIndex = 0; BlockIndex < InBlocks.Num(); ++BlockIndex)
					{
						const FBlock& CurrentBlock = InBlocks[BlockIndex];
						AddBlockToLine(InLine, CurrentBlock);
					}
				};

				auto HasMoreRemovableBlocks = [&]()
				{
					return !SortedBlockView.IsEmpty();
				};

				auto LineHasMultipleBlocksAndPriorities = [&](const FBlockLine& InLine)
				{
					const bool bHasMoreThanOneBlock = InLine.Blocks.Num() > 1;
					if constexpr (UseGroupedWrapping)
					{
						return bHasMoreThanOneBlock;
					}
					else
					{
						if (bHasMoreThanOneBlock)
						{
							int32 PriorityToMatch = InLine.Blocks[0].WrapPriority;
							return bHasMoreThanOneBlock
								&& !Algo::AllOf(InLine.Blocks, [PriorityToMatch](const FBlock& InBlock)
								{
									return PriorityToMatch == InBlock.WrapPriority;
								});
						}

						return false;
					}					
				};

				int32 LineIndex = 0;
				while (NumArrangedBlocks < Blocks.Num() && HasMoreRemovableBlocks())
				{
					FBlockLine& CurrentLine = Lines.Emplace_GetRef();
					CurrentLine.Index = LineIndex;

					MakeBlockLine(CurrentLine, CurrentLineBlocks);

					int32 NumSortablesUncheckedForForceNewLine = SortedBlockView.Num();
					int32 LastSortableCheckedIndex = 0;

					// Find and move the first "ForceNewLine'able" to the next line, and move all blocks with the same or higher priority.
					while (NumSortablesUncheckedForForceNewLine > 0
						&& LineHasMultipleBlocksAndPriorities(CurrentLine)
						&& HasMoreRemovableBlocks())
					{
						const FBlock& BlockToConsider = SortedBlockView[LastSortableCheckedIndex++];

						const bool bCanRemoveBlock = BlockToConsider.bForceNewLine && BlockToConsider.WrapPriority != CurrentLine.Blocks[0].WrapPriority;

						--NumSortablesUncheckedForForceNewLine;

						if (bCanRemoveBlock)
						{
							FVector2f::FReal EffectiveLengthToRemove = BlockToConsider.GetEffectiveLength();

							FBlock BlockCopy = BlockToConsider;
							if (RemoveBlockFromLine(CurrentLine, BlockToConsider))
							{
								int32 LastForcedPriorityRemoved = BlockCopy.WrapPriority;

								// Don't include 0-length entries, they'll dictate the line height without having any visual content
								if (!FMath::IsNearlyZero(EffectiveLengthToRemove))
								{
									NextLineBlocks.Emplace(BlockCopy);
								}

								// Remove block, and all after it from further consideration for this line
								TConstArrayView<FBlock> RemainingBlockView = SortedBlockView.RightChop(LastSortableCheckedIndex);

								// The block was successfully removed. Now reverse iterate over the remaining blocks and remove those with the same or higher wrapping priority.
								// Note that we don't need to check the priority, it will naturally be the same or higher in the sorted view.
								for (int32 BlockIndex = RemainingBlockView.Num() - 1; BlockIndex >= 0; --BlockIndex)
								{
									const FBlock& EndBlockToConsider = RemainingBlockView[BlockIndex];

									EffectiveLengthToRemove = EndBlockToConsider.GetEffectiveLength();
									BlockCopy = EndBlockToConsider;
									if (RemoveBlockFromLine(CurrentLine, EndBlockToConsider))
									{
										// Don't include 0-length entries, they'll dictate the line height without having any visual content
										if (!FMath::IsNearlyZero(EffectiveLengthToRemove))
										{
											NextLineBlocks.Emplace(BlockCopy);
										}
									}
								}
							}
						}
					}

					int32 NumSortablesUncheckedForExpansion = SortedBlockView.Num();

					// Remove vertical expandables, which rely on the full line length (not min + full combination)
					while (NumSortablesUncheckedForExpansion > 0
						&& CurrentLine.MaxLength > ParentLineLength
						&& LineHasMultipleBlocksAndPriorities(CurrentLine)
						&& HasMoreRemovableBlocks())
					{
						const FBlock& BlockToConsider = SortedBlockView.Last();
						const bool bCanRemoveBlock = BlockToConsider.bCanWrap && BlockToConsider.bCanVerticallyExpand && BlockToConsider.WrapPriority != CurrentLine.Blocks[0].WrapPriority;

						--NumSortablesUncheckedForExpansion;

						if (bCanRemoveBlock)
						{
							bool bShouldRemoveBlock = true;
							const FVector2f::FReal EffectiveLengthToRemove = BlockToConsider.GetEffectiveLength();

							if (BlockToConsider.MinLength != BlockToConsider.ReserveLength)
							{
								const float LineLengthAfterRemovalOfEffective = CurrentLine.MaxLength - EffectiveLengthToRemove;
								if (LineLengthAfterRemovalOfEffective + EffectiveLengthToRemove < ParentLineLength)
								{
									// We're cool
									bShouldRemoveBlock = false;
								}
							}

							if (bShouldRemoveBlock)
							{
								FBlock BlockCopy = BlockToConsider;
								if (RemoveBlockFromLine(CurrentLine, BlockToConsider))
								{
									// Don't include 0-length entries, they'll dictate the line height without having any visual content
									if (!FMath::IsNearlyZero(EffectiveLengthToRemove))
									{
										NextLineBlocks.Emplace(BlockCopy);
									}

									// Remove block from further consideration for this line
									SortedBlockView.LeftChopInline(1);
								}
							}
						}
					}

					// Horizontal, prioritized wrapping
					while (CurrentLine.EffectiveLength > ParentLineLength
						&& LineHasMultipleBlocksAndPriorities(CurrentLine)
						&& HasMoreRemovableBlocks())
					{
						const FBlock& BlockToRemove = SortedBlockView.Last();
						const bool bCanRemoveBlock = BlockToRemove.bCanWrap || BlockToRemove.WrapPriority != CurrentLine.Blocks[0].WrapPriority;

						if (bCanRemoveBlock)
						{
							bool bShouldRemoveBlock = true;
							const FVector2f::FReal EffectiveLengthToRemove = BlockToRemove.GetEffectiveLength();

							// We can remove it, but should we?
							// If we satisfy constraints using its MinSize, then we don't need to remove it
							if (!BlockToRemove.bHasVerticalExpansion && BlockToRemove.MinLength != BlockToRemove.ReserveLength)
							{
								const float LineLengthAfterRemovalOfEffective = CurrentLine.EffectiveLength - EffectiveLengthToRemove;
								if (LineLengthAfterRemovalOfEffective + BlockToRemove.MinLength < ParentLineLength)
								{
									// We're cool
									bShouldRemoveBlock = false;
								}
							}

							if (bShouldRemoveBlock)
							{
								FBlock BlockCopy = BlockToRemove;
								if (RemoveBlockFromLine(CurrentLine, BlockToRemove))
								{
									// Don't include 0-length entries, they'll dictate the line height without having any visual content
									if (!FMath::IsNearlyZero(EffectiveLengthToRemove))
									{
										NextLineBlocks.Emplace(BlockCopy);
									}
								}
							}
						}

						// Remove block from further consideration for this line
						SortedBlockView.LeftChopInline(1);
					}

					// No entries were valid, so no blocks were added - early out
					if (CurrentLine.Blocks.IsEmpty())
					{
						// Empty line, early-out
						break;
					}

					// Remove first and last slots, if necessary
					FBlock& FirstBlockInCurrentLine = CurrentLine.Blocks[0];
					FBlock& LastBlockInCurrentLine = CurrentLine.Blocks.Last();

					int32 FirstSlotIndexInFirstBlock = FirstBlockInCurrentLine.SlotIndices[0];
					int32 LastSlotIndexInLastBlock = LastBlockInCurrentLine.SlotIndices.Last();

					const FSlotProxy& FirstChildProxy = ChildProxies[FirstSlotIndexInFirstBlock];
					const FSlotProxy& LastChildProxy = ChildProxies[LastSlotIndexInLastBlock];

					// Check first and last widget in line for exclude flag
					{
						const bool bIsSameBlock = FirstBlockInCurrentLine == LastBlockInCurrentLine;
						const bool bIsSameSlotIndex = FirstSlotIndexInFirstBlock == LastSlotIndexInLastBlock;

						if (FirstChildProxy.bExcludeIfFirstOrLast)
						{
							FirstBlockInCurrentLine.SlotIndices.Remove(FirstSlotIndexInFirstBlock);
						}

						if (!bIsSameSlotIndex && LastChildProxy.bExcludeIfFirstOrLast)
						{
							LastBlockInCurrentLine.SlotIndices.Remove(LastSlotIndexInLastBlock);
						}

						if (FirstChildProxy.bExcludeIfFirstOrLast
							&& FirstBlockInCurrentLine.SlotIndices.IsEmpty()
							&& RemoveBlockFromLine(CurrentLine, FirstBlockInCurrentLine))
						{
							// Compensate counter
							NumArrangedBlocks++;
						}

						// If it's the same block, it's already removed
						if (LastChildProxy.bExcludeIfFirstOrLast
							&& (!bIsSameBlock
								|| (LastBlockInCurrentLine.SlotIndices.IsEmpty() && RemoveBlockFromLine(CurrentLine, LastBlockInCurrentLine))))
						{
							// Compensate counter
							NumArrangedBlocks++;
						}
					}

					ensureAlways(!CurrentLine.Blocks.IsEmpty());

					// Clamp height to Min
					CurrentLine.Height = FMath::Max(MinLineHeight, CurrentLine.Height);

					// Clamp height to Max
					CurrentLine.Height = FMath::Min(MaxLineHeight, CurrentLine.Height);

					// Constraints satisfied, complete this line...
					DesiredSize.X = FMath::Max(DesiredSize.X, CurrentLine.MaxLength);
					DesiredSize.Y = FMath::Max(DesiredSize.Y, CurrentLine.Height + LineOffset);

					LineOffset += CurrentLine.Height + LinePadding;
					++LineIndex;

					NumArrangedBlocks += CurrentLine.Blocks.Num();

					// ...and setup next
					CurrentLineBlocks = NextLineBlocks;
					NextLineBlocks.Reset();

					CurrentLineBlocks.StableSort();

					SortedBlockView = MakeConstArrayView(WrappableSortedBlockView.GetData(), WrappableSortedBlockView.Num());

					// We've got as far as we can, exit
					if (CurrentLine.MaxLength <= 0)
					{
						break;
					}
				}

				LastPreferredLineLength = PreferredLineLength;
				LastParentLineLength = ParentLineLength;
				LastParentSize = InWidget.GetTickSpaceGeometry().Size;
				LastDesiredSize = DesiredSize;
				LastLinePadding = LinePadding;

				Lines.SetNum(LineIndex);
				BlockLines = Lines;
			}

			return FVector2D(DesiredSize);
		}

		void Arrange(const FGeometry& InAllottedGeometry, FArrangedChildren& InOutArrangedChildren)
		{
			if (BlockLines.IsEmpty())
			{
				return;
			}

			// The actual size available to the widget
			const FVector2f ParentSize = InAllottedGeometry.GetLocalSize();

			// These are re-calculated every time a new line is required
			FVector2f::FReal LineOffset = 0.0f;
			FVector2f LineOffsetXY = FVector2f::ZeroVector;
			float LinePadding = LastLinePadding;

			FVector2f LineSize = ParentSize;
			LineSize.Y = BlockLines[0].Height;

			for (int32 LineIndex = 0; LineIndex < BlockLines.Num(); ++LineIndex)
			{
				const FBlockLine& Line = BlockLines[LineIndex];

				float OffsetX = 0.0f;

				TArray<FSlotProxy> ChildProxiesCurrentLine;
				ChildProxiesCurrentLine.Reserve(ChildProxies.Num());

				for (int32 BlockIndex = 0; BlockIndex < Line.Blocks.Num(); ++BlockIndex)
				{
					const FBlock& Block = Line.Blocks[BlockIndex];

					for (int32 ChildIndex = 0; ChildIndex < Block.SlotIndices.Num(); ++ChildIndex)
					{
						const int32 ChildIndexInSlotOrder = Block.SlotIndices[ChildIndex];

						FSlotProxy& ChildProxy = ChildProxies[ChildIndexInSlotOrder];
						ChildProxy.DesiredSize.Y = FMath::Min(ChildProxy.DesiredSize.Y, Line.Height);
						OffsetX += ChildProxy.DesiredSize.X;

						ChildProxiesCurrentLine.Emplace(ChildProxy);
					}
				}

				LineSize.Y = Line.Height;
				FSlateLayoutTransform LayoutTransform = FSlateLayoutTransform(1.0f, LineOffsetXY);
				FGeometry LineGeometry = InAllottedGeometry.MakeChild(LineSize, LayoutTransform);

				// Unused, required by ArrangeChildrenInStack
				constexpr FVector2f::FReal Offset = 0.0f;

				// By provided a non-const ArrayView, the ArrangedSize of the FSlotProxy is written
				FVector2D ArrangedSize;
				ArrangeChildrenInStack<Orientation>(GSlateFlowDirection, MakeArrayView(ChildProxiesCurrentLine), LineGeometry, InOutArrangedChildren, Offset, true, ArrangedSize);

				for (int32 BlockIndex = 0; BlockIndex < Line.Blocks.Num(); ++BlockIndex)
				{
					const FBlock& Block = Line.Blocks[BlockIndex];

					for (int32 ChildIndex = 0; ChildIndex < Block.SlotIndices.Num(); ++ChildIndex)
					{
						const int32 ChildIndexInSlotOrder = Block.SlotIndices[ChildIndex];
						ChildProxies[ChildIndexInSlotOrder].ArrangedSize = ChildProxiesCurrentLine[ChildIndex].ArrangedSize;
					}
				}

				// Zero out LinePadding if we're the last line
				if (LineIndex == BlockLines.Num() - 1)
				{
					LinePadding = 0.0f;
				}
				// If we're not the last line, setup geometry for the next
				else
				{
					LineOffset += ArrangedSize.Y + LinePadding;
					LineOffsetXY = FVector2f(0.0f, LineOffset);
				}
			}
		}

	private:
		bool UpdateChildProxies(const TPanelChildren<SPrioritizedWrapBox::FSlot>& InChildren)
		{
			bool bAnySlotProxyChanged = false;

			const int32 NumChildren = InChildren.Num();

			NumPreferredWrappingChildren = 0;
			NumParentWrappingChildren = 0;

			// First-time initialization
			if (ChildProxies.IsEmpty())
			{
				ChildProxies.SetNum(NumChildren, EAllowShrinking::No);

				for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
				{
					ChildProxies[ChildIndex] = FSlotProxy(ChildIndex, InChildren[ChildIndex]);

					if (ChildProxies[ChildIndex].WrapMode == EWrapMode::Preferred)
					{
						++NumPreferredWrappingChildren;
					}
					else if (ChildProxies[ChildIndex].WrapMode == EWrapMode::Parent)
					{
						++NumParentWrappingChildren;
					}
				}

				bAnySlotProxyChanged = true;
			}
			else
			{
				for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
				{
					if (ChildProxies.IsValidIndex(ChildIndex))
					{
						if (ChildProxies[ChildIndex].UpdateFromSlot(ChildIndex, InChildren[ChildIndex]))
						{
							bAnySlotProxyChanged = true;
						}

						if (ChildProxies[ChildIndex].WrapMode == EWrapMode::Preferred)
						{
							++NumPreferredWrappingChildren;
						}
						else if (ChildProxies[ChildIndex].WrapMode == EWrapMode::Parent)
						{
							++NumParentWrappingChildren;
						}
					}
				}

				ChildProxies.SetNum(NumChildren, EAllowShrinking::No);
			}

			return bAnySlotProxyChanged;
		}

		void UpdateBlocks(const TPanelChildren<SPrioritizedWrapBox::FSlot>& InChildren, const float InMinLineHeight)
		{
			if (ChildProxies.IsEmpty())
			{
				return;
			}

			Blocks.SetNum(InChildren.Num(), EAllowShrinking::No);
			SortedBlocks.SetNum(InChildren.Num(), EAllowShrinking::No);

			auto GetAreaRetainingWidthForHeight = [](const FVector2f& InOriginalSize, const int32 InNewHeight) -> FVector2f::FReal
			{
				const FVector2f::FReal Area = InOriginalSize.X * InOriginalSize.Y;
				return FMath::CeilToFloat(Area / InNewHeight);
			};

			int32 NumBlocks = 0;
			FBlock* CurrentBlock = &Blocks[NumBlocks];
			CurrentBlock->ResetToDefault();
			CurrentBlock->SequentialIndex = NumBlocks;
			CurrentBlock->WrapPriority = ChildProxies[0].WrapPriority;
			CurrentBlock->bForceNewLine = ChildProxies[0].bForceNewLine;

			NumBlocks++;

			for (int32 ChildIndex = 0; ChildIndex < ChildProxies.Num(); ++ChildIndex)
			{
				const SPrioritizedWrapBox::FSlot& Child = InChildren[ChildIndex];

				// Different wrap priority, add a new block
				if (CurrentBlock->WrapPriority != Child.GetWrapPriority())
				{
					CurrentBlock = &Blocks[NumBlocks];
					CurrentBlock->ResetToDefault();
					CurrentBlock->SequentialIndex = NumBlocks;
					CurrentBlock->WrapPriority = Child.GetWrapPriority();
					CurrentBlock->bForceNewLine = Child.GetForceNewLine();

					NumBlocks++;
				}

				CurrentBlock->SlotIndices.Emplace(ChildIndex);

				TSlotAccessor<SPrioritizedWrapBox::FSlot> SlotAccessor;

				if (SlotAccessor.GetVisibility(Child) == EVisibility::Collapsed)
				{
					continue;
				}

				const FVector2f ChildDesiredSize = SlotAccessor.GetDesiredSize(Child);
				FVector2f ChildArrangedSize = SlotAccessor.GetArrangedSize(Child);

				FVector2f ChildClampedSize = ChildDesiredSize;

				// Min (if specified) or Desired (if not)
				FVector2f::FReal MinDesiredSizeX = ChildDesiredSize.X;

				FVector2f::FReal AppendSizeX = 0.0f;
				FVector2f::FReal PruneSizeX = 0.0f;

				if (const FVector2f::FReal MaxSize = SlotAccessor.GetMaxSize(Child);
					MaxSize > 0)
				{
					ChildClampedSize.X = FMath::Min(MaxSize, ChildClampedSize.X);
				}

				if (const FVector2f::FReal MinSize = SlotAccessor.GetMinSize(Child);
					MinSize > 0)
				{
					ChildClampedSize.X = FMath::Max(MinSize, ChildClampedSize.X); // Equal or greater to min size
					MinDesiredSizeX = FMath::Min(MinSize, ChildClampedSize.X); // Min possible valid size
				}

				// Desired, or Adjusted (if needed)
				FVector2f::FReal OverflowSizeX = ChildClampedSize.X;
				FVector2f::FReal OverflowSizeY = ChildClampedSize.Y;

				const float HorizontalPadding = SlotAccessor.GetPadding(Child).GetTotalSpaceAlong<EOrientation::Orient_Horizontal>();
				const float VerticalPadding = SlotAccessor.GetPadding(Child).GetTotalSpaceAlong<EOrientation::Orient_Vertical>();

				if (Child.GetVerticalOverflowBehavior() == EVerticalOverflowBehavior::ExpandProportional)
				{
					CurrentBlock->bCanVerticallyExpand = true;

					// Default to 1.5x the minimum line height unless specified
					const float VerticalExpansionThreshold = Child.GetVerticalExpansionThreshold().Get(InMinLineHeight * 1.5f);
					if ((ChildClampedSize.Y + VerticalPadding > VerticalExpansionThreshold)
						&& ChildArrangedSize.X > 0.0f)
					{
						FVector2f OriginalSize = ChildClampedSize;

						// While the threshold is used for the test, the target line height is used here
						OverflowSizeX = GetAreaRetainingWidthForHeight(OriginalSize, InMinLineHeight + VerticalPadding) + HorizontalPadding * 2;
						OverflowSizeY = InMinLineHeight;;

						CurrentBlock->bHasVerticalExpansion = true;
					}
				}

				const FVector2f::FReal Area = ChildClampedSize.X * ChildClampedSize.Y;

				CurrentBlock->DesiredSize = FVector2f(CurrentBlock->DesiredSize.X + ChildClampedSize.X, FMath::Max(CurrentBlock->DesiredSize.Y, ChildClampedSize.Y));
				CurrentBlock->AdjustedSize = FVector2f(CurrentBlock->AdjustedSize.X + OverflowSizeX, FMath::Max(CurrentBlock->AdjustedSize.Y, OverflowSizeY));
				CurrentBlock->ReserveLength += OverflowSizeX;
				CurrentBlock->MinLength += MinDesiredSizeX;
				CurrentBlock->bCanWrap = Child.GetAllowWrapping();
				CurrentBlock->bForceNewLine = CurrentBlock->bForceNewLine || Child.GetForceNewLine(); // If -any- child is forced, the block is too
				CurrentBlock->Area += Area;

				FSlotProxy& ChildProxy = ChildProxies[ChildIndex];
				ChildProxy.DesiredSize.X = CurrentBlock->bHasVerticalExpansion ? OverflowSizeX : ChildClampedSize.X;
				ChildProxy.DesiredSize.Y = InMinLineHeight;
			}

			Blocks.SetNum(NumBlocks, EAllowShrinking::No);

			SortedBlocks = Blocks;

			// Needs re-sorting
			SortedBlocks.StableSort([](const FBlock& A, const FBlock& B)
			{
				// A block that's forced to a new line, but can't wrap (to another), is considered to be wrappable
				const bool bACanEverWrap = A.bCanWrap || A.bForceNewLine;
				const bool bBCanEverWrap = B.bCanWrap || B.bForceNewLine;
				
				// Non-wrappable children are always at the start
				if (!bACanEverWrap && !bBCanEverWrap)
				{
					return A.SequentialIndex < B.SequentialIndex;
				}

				// Non-wrappable children come before wrappable ones
				if (!bACanEverWrap) // B is wrappable
				{
					return true;
				}

				if (!bBCanEverWrap) // A is wrappable
				{
					return false;
				}

				// Both are wrappable, with different priorities
				if (A.WrapPriority != B.WrapPriority)
				{
					return A.SequentialIndex < B.SequentialIndex;
				}

				// Same Priority, maintain original order
				return A.SequentialIndex < B.SequentialIndex;
			});

			int32 FirstWrappableBlockIndex = INDEX_NONE;
			for (int32 BlockIndex = 0; BlockIndex < SortedBlocks.Num(); ++BlockIndex)
			{
				const FBlock& SortedBlock = SortedBlocks[BlockIndex];
				if (SortedBlock.bCanWrap || SortedBlock.bForceNewLine)
				{
					FirstWrappableBlockIndex = BlockIndex;
					break;
				}
			}

			if (FirstWrappableBlockIndex == INDEX_NONE)
			{
				// Make Empty
				WrappableSortedBlockView = MakeConstArrayView<FBlock>({ });
			}
			else
			{
				WrappableSortedBlockView = MakeConstArrayView(SortedBlocks.GetData() + FirstWrappableBlockIndex, SortedBlocks.Num() - FirstWrappableBlockIndex);
			}
		}

	private:
		TArray<FSlotProxy> ChildProxies;

		int32 NumPreferredWrappingChildren = 0;
		int32 NumParentWrappingChildren = 0;

		int32 LastPreferredLineLength = INDEX_NONE;
		int32 LastParentLineLength = INDEX_NONE;
		FVector2f LastParentSize = FVector2f::ZeroVector;

		FVector2f LastDesiredSize;
		float LastLinePadding;

		TArray<FBlock> Blocks;

		/** This should always be sorted by WrapPriority, then OriginalIndex (if priorities the same), then bCanWrap (such that all widgets that can't wrap are first). */
		TArray<FBlock> SortedBlocks;

		/** An ArrayView (of SortedChildren) that includes only wrappable children (bCanWrap == true). */
		TConstArrayView<FBlock> WrappableSortedBlockView;

		TArray<FBlockLine> BlockLines;
	};
}

SLATE_IMPLEMENT_WIDGET(SPrioritizedWrapBox)
void SPrioritizedWrapBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer SlotInitializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Slots);
	FSlot::RegisterAttributes(SlotInitializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, PreferredSize, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinLineHeight, EInvalidateWidgetReason::Layout);
}

SPrioritizedWrapBox::FSlot::FSlot()
	: TBasicLayoutWidgetSlot<FSlot>(HAlign_Fill, VAlign_Fill)
	, TResizingWidgetSlotMixin<FSlot>()
	, bAllowWrapping(*this, true)
	, WrapPriority(*this, 0)
	, WrapMode(*this, UE::Slate::PrioritizedWrapBox::EWrapMode::Preferred)
	, VerticalOverflowBehavior(UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior::Default)
	, VerticalExpansionThreshold(0.0f)
	, bExcludeIfFirstOrLast(false)
{
}

void SPrioritizedWrapBox::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	TBasicLayoutWidgetSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
	TResizingWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));

	if (InArgs._AllowWrapping.IsSet())
	{
		bAllowWrapping.Assign(*this, MoveTemp(InArgs._AllowWrapping));
	}

	if (InArgs._WrapPriority.IsSet())
	{
		WrapPriority.Assign(*this, MoveTemp(InArgs._WrapPriority));
	}

	if (InArgs._WrapMode.IsSet())
	{
		WrapMode.Assign(*this, MoveTemp(InArgs._WrapMode));
	}

	VerticalOverflowBehavior = InArgs._VerticalOverflowBehavior;
	VerticalExpansionThreshold = InArgs._VerticalExpansionThreshold;
	bForceNewLine = InArgs._ForceNewLine;
	bExcludeIfFirstOrLast = InArgs._ExcludeIfFirstOrLast;
}

void SPrioritizedWrapBox::FSlot::RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
{
	TBasicLayoutWidgetSlot<FSlot>::RegisterAttributes(AttributeInitializer);
	TResizingWidgetSlotMixin<FSlot>::RegisterAttributesMixin(AttributeInitializer);

	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.AllowWrapping", bAllowWrapping, EInvalidateWidgetReason::Layout);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.WrapPriority", WrapPriority, EInvalidateWidgetReason::Layout);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.WrapMode", WrapMode, EInvalidateWidgetReason::Layout);
}

SPrioritizedWrapBox::SPrioritizedWrapBox()
	: PreferredSize(*this, 100.0f)
	, MinLineHeight(*this, TOptional<float>())
	, LinePadding(0.0f)
	, bGroupedWrapping(false)
	, Slots(this, GET_MEMBER_NAME_CHECKED(SPrioritizedWrapBox, Slots))
	, ChildArranger(MakeUnique<UE::Slate::PrioritizedWrapBox::FChildArranger>())
{
}

SPrioritizedWrapBox::~SPrioritizedWrapBox()
{
}

SPrioritizedWrapBox::FScopedWidgetSlotArguments SPrioritizedWrapBox::AddSlot()
{
	return FScopedWidgetSlotArguments{MakeUnique<FSlot>(), Slots, INDEX_NONE};
}

int32 SPrioritizedWrapBox::RemoveSlot(const TSharedRef<SWidget>& InSlot)
{
	return Slots.Remove(InSlot);
}

void SPrioritizedWrapBox::Construct(const FArguments& InArgs)
{
	PreferredSize.Assign(*this,  InArgs._PreferredSize);
	MinLineHeight.Assign(*this, InArgs._MinLineHeight);
	LinePadding = InArgs._LinePadding;;
	bGroupedWrapping = InArgs._GroupedWrapping;

	Slots.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));

	SetCanTick(false);
	SetClipping(EWidgetClipping::ClipToBounds);
}

FVector2D SPrioritizedWrapBox::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPrioritizedWrapBox::ComputeDesiredSize);

	check(ChildArranger.IsValid());

	return bGroupedWrapping ? ChildArranger->GetDesiredSize<true>(*this, this->Slots) : ChildArranger->GetDesiredSize<false>(*this, this->Slots);
}

void SPrioritizedWrapBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPrioritizedWrapBox::OnArrangeChildren);

	check(ChildArranger.IsValid());

	return ChildArranger->Arrange(AllottedGeometry, ArrangedChildren);
}
