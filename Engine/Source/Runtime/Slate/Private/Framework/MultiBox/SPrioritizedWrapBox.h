// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class SComboButton;

DECLARE_DELEGATE_RetVal_OneParam(FMenuEntryResizeParams, FOnGetWidgetResizeParams, const TSharedRef<SWidget>&);

namespace UE::Slate::PrioritizedWrapBox
{
	class FChildArranger;
}; // namespace UE::Slate


template <>
struct TWidgetTypeTraits<class SPrioritizedWrapBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * An extension of SHorizontalBox that adds wrapping behavior (similar to SWrapBox), primarily for use with MultiBox.
 * Wrapping candidates are determined by priority, so it doesn't necessarily occur sequentially.
 * For example, if the middle of 3 widgets has a higher priority, that middle widget will wrap first.
 */
class SPrioritizedWrapBox : public SPanel
{
	SLATE_DECLARE_WIDGET(SPrioritizedWrapBox, SPanel)

public:
	class FSlot : public TBasicLayoutWidgetSlot<FSlot>, public TResizingWidgetSlotMixin<FSlot>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TBasicLayoutWidgetSlot<FSlot>, TResizingWidgetSlotMixin<FSlot>)
			SLATE_ATTRIBUTE(bool, AllowWrapping) // If set to false, this widget won't be considered for wrapping and will always be in the first line
			SLATE_ATTRIBUTE(int32, WrapPriority)
			SLATE_ATTRIBUTE(UE::Slate::PrioritizedWrapBox::EWrapMode, WrapMode)
			SLATE_ARGUMENT_DEFAULT(UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior, VerticalOverflowBehavior) = UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior::Default;
			SLATE_ARGUMENT(TOptional<float>, VerticalExpansionThreshold) // Vertical expansion will occur when the widget's desired size is at or beyond this value
			SLATE_ARGUMENT_DEFAULT(bool, ForceNewLine) = false; // If true, this will forcibly place this slot on a new line, regardless of the wrapping behavior. This affects all slots after this one with the same or higher wrapping priorities.
			SLATE_ARGUMENT_DEFAULT(bool, ExcludeIfFirstOrLast) = false;
		SLATE_SLOT_END_ARGS()

		FSlot();

	public:
		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);

		static void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer);

		/** If false, this will always remain on the first line. */
		bool GetAllowWrapping() const { return bAllowWrapping.Get(); }

		/** Higher values will wrap first. Default is 0. */
		int32 GetWrapPriority() const { return WrapPriority.Get(); }

		/** Higher values will wrap first. Default is 0. */
		void SetWrapPriority(int32 InPriority) { WrapPriority.Set(*this, InPriority); }

		/** Get the WrapMode, which determines at what reference length the slot wraps. */
		UE::Slate::PrioritizedWrapBox::EWrapMode GetWrapMode() const { return WrapMode.Get(); }

		/** Get the vertical overflow behavior, specifying how a slot, and it's widget should behave when they need to wrap. */
		UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior GetVerticalOverflowBehavior() const { return VerticalOverflowBehavior; }

		/** If the vertical overflow behavior requires expansion, this optionally specifies the height at which the slot should vertically expand. */
		const TOptional<float>& GetVerticalExpansionThreshold() const { return VerticalExpansionThreshold; }

		/**
		 * If true, this will forcibly place this slot on a new line, regardless of the wrapping behavior. This affects all slots after this one with the same or higher wrapping priorities.
		 * When AllowWrapping is false, this will only ever be placed on the line it's forcibly wrapped to.
		 */
		const bool GetForceNewLine() const { return bForceNewLine; }

		/** If, after wrapping, this slot is the first or last in the line, it will be excluded from the result if this is true. */
		bool GetExcludeIfFirstOrLast() const { return bExcludeIfFirstOrLast; }

	private:
		template <typename InObjectType>
		using TSlateSlotAttribute = ::SlateAttributePrivate::TSlateContainedAttribute<InObjectType, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeComparePredicate<>>;

		/** If false, this will always remain on the first line. */
		TSlateSlotAttribute<bool> bAllowWrapping;

		/** Higher values will wrap first. Default is 0. */
		TSlateSlotAttribute<int32> WrapPriority;

		TSlateSlotAttribute<UE::Slate::PrioritizedWrapBox::EWrapMode> WrapMode;

		UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior VerticalOverflowBehavior = UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior::Default;
		TOptional<float> VerticalExpansionThreshold;
		bool bForceNewLine = false;;

		bool bExcludeIfFirstOrLast = false;
	};

	SLATE_BEGIN_ARGS(SPrioritizedWrapBox)
		: _PreferredSize(100.0f)
		, _MinLineHeight(TOptional<float>())
		, _LinePadding(0.0f)
		, _GroupedWrapping(false)
	{ }
		SLATE_SLOT_ARGUMENT(FSlot, Slots)

		/** The preferred size at which wrapping occurs, applicable only when the Slots WrapMode is Preferred  */
		SLATE_ATTRIBUTE(float, PreferredSize)

		/** An optional minimum line height, useful to reduce height variance (which changes when wrapping) */
		SLATE_ATTRIBUTE(TOptional<float>, MinLineHeight)

		/** The padding to add between lines. This only affects spacing between lines, not around each entry */
		SLATE_ARGUMENT(float, LinePadding)

		/** If true, slots with the same wrap priority are treated as a single monolithic element, rather than per-slot */
		SLATE_ARGUMENT(bool, GroupedWrapping)
	SLATE_END_ARGS()

	SPrioritizedWrapBox();
	virtual ~SPrioritizedWrapBox() override;

	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	FScopedWidgetSlotArguments AddSlot();
	int32 RemoveSlot(const TSharedRef<SWidget>& InSlot);

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	//~ End SWidget

	//~ Begin SPanel
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual FChildren* GetChildren() override { return &Slots; }
	//~ End SPanel

	float GetPreferredSize() const { return PreferredSize.Get(); }
	TOptional<float> GetMinLineHeight() const { return MinLineHeight.Get(); }
	float GetLinePadding() const { return LinePadding; }
	bool GetUseGroupedWrapping() const { return bGroupedWrapping; }

private:
	/** How wide or long, dependently of the orientation, this panel should appear to be. Any widgets past this line will be wrapped onto the next line. */
	TSlateAttribute<float> PreferredSize;

	/** An optional minimum line height, useful to reduce height variance (which changes when wrapping) */
	TSlateAttribute<TOptional<float>> MinLineHeight;

	/** The padding to add between lines. This only affects spacing between lines, not around each entry */
	float LinePadding;

	/** If true, slots with the same wrap priority are treated as a single monolithic element, rather than per-slot  */
	bool bGroupedWrapping;

	TPanelChildren<FSlot> Slots;

	TUniquePtr<class UE::Slate::PrioritizedWrapBox::FChildArranger> ChildArranger;
};
