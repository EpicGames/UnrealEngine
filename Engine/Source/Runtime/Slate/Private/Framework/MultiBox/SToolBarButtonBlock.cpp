// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Layout/SSeparator.h"

FToolBarButtonBlock::FToolBarButtonBlock(FButtonArgs ButtonArgs)
	: FMultiBlock(ButtonArgs.Command, ButtonArgs.CommandList, NAME_None, EMultiBlockType::ToolBarButton)
	, LabelOverride(ButtonArgs.LabelOverride)
	, ToolbarLabelOverride(ButtonArgs.ToolbarLabelOverride)
	, ToolTipOverride(ButtonArgs.ToolTipOverride)
	, IconOverride(ButtonArgs.IconOverride)
	, LabelVisibility()
	, UserInterfaceActionType(
		  ButtonArgs.UserInterfaceActionType != EUserInterfaceActionType::None ? ButtonArgs.UserInterfaceActionType
																			   : EUserInterfaceActionType::Button
	  )
	, bIsFocusable(false)
	, bForceSmallIcons(false)
	, GetDecoratedButtonDelegate(ButtonArgs.GetDecoratedButtonDelegate)
{
	if (ButtonArgs.VisibilityOverride.IsSet())
	{
		SetVisibilityOverride(ButtonArgs.VisibilityOverride);
	}
}

FToolBarButtonBlock::FToolBarButtonBlock(
	const TSharedPtr<const FUICommandInfo> InCommand,
	TSharedPtr<const FUICommandList> InCommandList,
	const TAttribute<FText>& InLabelOverride,
	const TAttribute<FText>& InToolTipOverride,
	const TAttribute<FSlateIcon>& InIconOverride,
	TAttribute<FText> InToolbarLabelOverride
)
	: FMultiBlock(InCommand, InCommandList, NAME_None, EMultiBlockType::ToolBarButton)
	, LabelOverride(InLabelOverride)
	, ToolbarLabelOverride(InToolbarLabelOverride)
	, ToolTipOverride(InToolTipOverride)
	, IconOverride(InIconOverride)
	, LabelVisibility()
	, UserInterfaceActionType(EUserInterfaceActionType::Button)
	, bIsFocusable(false)
	, bForceSmallIcons(false)
{
}

FToolBarButtonBlock::FToolBarButtonBlock(
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InToolTip,
	const TAttribute<FSlateIcon>& InIcon,
	const FUIAction& InUIAction,
	const EUserInterfaceActionType InUserInterfaceActionType,
	TAttribute<FText> InToolbarLabelOverride
)
	: FMultiBlock(InUIAction)
	, LabelOverride(InLabel)
	, ToolbarLabelOverride(InToolbarLabelOverride)
	, ToolTipOverride(InToolTip)
	, IconOverride(InIcon)
	, LabelVisibility()
	, UserInterfaceActionType(InUserInterfaceActionType)
	, bIsFocusable(false)
	, bForceSmallIcons(false)
{
}

void FToolBarButtonBlock::SetCustomMenuDelegate(const FNewMenuDelegate& InCustomMenuDelegate)
{
	CustomMenuDelegate = InCustomMenuDelegate;
}

void FToolBarButtonBlock::SetCustomWidget(const TSharedPtr<SWidget>& InCustomWidget)
{
	CustomWidget = InCustomWidget;
}

void FToolBarButtonBlock::SetOnGetMenuContent(const FOnGetContent& InOnGetMenuContent)
{
	OnGetMenuContent = InOnGetMenuContent;
}

void FToolBarButtonBlock::SetGetDecoratedButtonDelegate( const FGetDecoratedButtonDelegate& InGetDecoratedButtonDelegate )
{
	GetDecoratedButtonDelegate = InGetDecoratedButtonDelegate;
}

void FToolBarButtonBlock::CreateMenuEntry(FMenuBuilder& MenuBuilder) const
{
	// Setup Command Context 
	TSharedPtr<const FUICommandInfo> MenuEntryAction = GetAction();
	TSharedPtr<const FUICommandList> MenuEntryActionList = GetActionList();
	bool bHasValidCommand = MenuEntryAction.IsValid() && MenuEntryActionList.IsValid();
	if (bHasValidCommand) 
	{	
		MenuBuilder.PushCommandList(MenuEntryActionList.ToSharedRef());
	}

	if ( CustomMenuDelegate.IsBound() )
	{
		CustomMenuDelegate.Execute(MenuBuilder);
	}
	else if (bHasValidCommand)
	{
		MenuBuilder.AddMenuEntry(MenuEntryAction);
	}
	else if ( LabelOverride.IsSet() )
	{
		const FUIAction& DirectAction = GetDirectActions();
		MenuBuilder.AddMenuEntry( LabelOverride.Get(), ToolTipOverride.Get(), IconOverride.Get(), DirectAction, NAME_None, UserInterfaceActionType);
	}

	if (bHasValidCommand) 
	{	
		MenuBuilder.PopCommandList();
	}
}

bool FToolBarButtonBlock::HasIcon() const
{
	const FSlateIcon ActionIcon = GetAction().IsValid() ? GetAction()->GetIcon() : FSlateIcon();
	const FSlateIcon& ActualIcon = IconOverride.IsSet() ? IconOverride.Get() : ActionIcon;

	if (ActualIcon.IsSet())
	{
		return ActualIcon.GetIcon()->GetResourceName() != NAME_None;
	}

	return false;
}

/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FToolBarButtonBlock::ConstructWidget() const
{
	return SNew( SToolBarButtonBlock )
		.LabelVisibility(LabelVisibility)
		.IsFocusable(bIsFocusable)
		.ForceSmallIcons(bForceSmallIcons)
		.TutorialHighlightName(GetTutorialHighlightName())
		.Cursor(EMouseCursor::Default);
}


/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SToolBarButtonBlock::Construct( const FArguments& InArgs )
{
	LabelVisibilityOverride = InArgs._LabelVisibility;
	bIsFocusable = InArgs._IsFocusable;
	bForceSmallIcons = InArgs._ForceSmallIcons;
	TutorialHighlightName = InArgs._TutorialHighlightName;
}


/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SToolBarButtonBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

	// If override is set use that
	if (LabelVisibilityOverride.IsSet())
	{
		LabelVisibility = LabelVisibilityOverride.GetValue();
	}
	else if (!ToolBarStyle.bShowLabels)
	{
		// Otherwise check the style
		LabelVisibility = EVisibility::Collapsed;
	}
	else
	{
		// Finally if the style doesnt disable labels, use the default
		LabelVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(SharedThis(this), &SToolBarButtonBlock::GetIconVisibility, false));
	}

	struct Local
	{
		/** Appends the key binding to the end of the provided ToolTip */
		static FText AppendKeyBindingToToolTip( const TAttribute<FText> ToolTip, TWeakPtr< const FUICommandInfo> Command )
		{
			TSharedPtr<const FUICommandInfo> CommandPtr = Command.Pin();
			if( CommandPtr.IsValid() && (CommandPtr->GetFirstValidChord()->IsValidChord()) )
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("ToolTipDescription"), ToolTip.Get() );
				Args.Add( TEXT("Keybinding"), CommandPtr->GetInputText() );
				return FText::Format( NSLOCTEXT("ToolBar", "ToolTip + Keybinding", "{ToolTipDescription} ({Keybinding})"), Args );
			}
			else
			{
				return ToolTip.Get();
			}
		}
	};

	TSharedRef<const FMultiBox> MultiBox = OwnerMultiBoxWidget.Pin()->GetMultiBox();

	TSharedRef<const FToolBarButtonBlock> ToolBarButtonBlock = StaticCastSharedRef<const FToolBarButtonBlock>(MultiBlock.ToSharedRef());

	TSharedPtr<const FUICommandInfo> UICommand = ToolBarButtonBlock->GetAction();

	// Allow the block to override the action's label and tool tip string, if desired
	TAttribute<FText> ActualLabel;
	if (ToolBarButtonBlock->ToolbarLabelOverride.IsSet())
	{
		ActualLabel = ToolBarButtonBlock->ToolbarLabelOverride;
	}
	else if (ToolBarButtonBlock->LabelOverride.IsSet())
	{
		ActualLabel = ToolBarButtonBlock->LabelOverride;
	}
	else
	{
		ActualLabel = UICommand.IsValid() ? UICommand->GetLabel() : FText::GetEmpty();
	}

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), ActualLabel.Get(), MultiBlock->GetSearchable());

	TAttribute<FText> ActualToolTip;
	if (ToolBarButtonBlock->ToolTipOverride.IsSet())
	{
		ActualToolTip = ToolBarButtonBlock->ToolTipOverride;
	}
	else
	{
		ActualToolTip = UICommand.IsValid() ? UICommand->GetDescription() : FText::GetEmpty();
	}
	
	// If we were supplied an image than go ahead and use that, otherwise we use a null widget
	TSharedRef<SLayeredImage> IconWidget =
		SNew(SLayeredImage)
		.ColorAndOpacity(this, &SToolBarButtonBlock::GetIconForegroundColor)
		.Visibility(EVisibility::HitTestInvisible)
		.Image(this, &SToolBarButtonBlock::GetIconBrush);

	IconWidget->AddLayer(TAttribute<const FSlateBrush*>(this, &SToolBarButtonBlock::GetOverlayIconBrush));
	const bool bIsSlimHorizontalUniformToolBar = MultiBox->GetType() == EMultiBoxType::SlimHorizontalUniformToolBar;
	const bool bIsSlimWrappingToolBar = MultiBox->GetType() == EMultiBoxType::SlimWrappingToolBar;

	const TSharedRef<STextBlock> TextBlock =
	SNew(STextBlock)
		// Collapse empty labels to prevent them from taking up visible space.
		.Visibility_Lambda(
			[WeakBlock = SharedThis(this).ToWeakPtr(), ActualLabel]() -> EVisibility
			{
				// Check first if the label is empty, and if so collapse it.
				if (ActualLabel.IsSet() && ActualLabel.Get().IsEmpty())
				{
					return EVisibility::Collapsed;
				}
				// Only now check the set override.
				else if (TSharedPtr<SToolBarButtonBlock> Block = WeakBlock.Pin())
				{
					return Block->LabelVisibility.Get();
				}
				else
				{
					return EVisibility::Visible;
				}
			}
		)
		.Text(ActualLabel)
		.TextStyle(&ToolBarStyle.LabelStyle); // Smaller font for tool tip labels

	// Create the content for our button
	TSharedRef<SWidget> ButtonContent = SNullWidget::NullWidget;
	
	if (ToolBarButtonBlock->CustomWidget)
	{
		ButtonContent = ToolBarButtonBlock->CustomWidget.ToSharedRef();
	}
	else if (MultiBox->GetType() == EMultiBoxType::SlimHorizontalToolBar 
		|| bIsSlimHorizontalUniformToolBar
		|| bIsSlimWrappingToolBar)
	{
		const FVector2f IconSize = ToolBarStyle.IconSize;

		if (bIsSlimHorizontalUniformToolBar)
		{
			TextBlock->SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);
			TextBlock->SetVisibility(EVisibility::Visible);
		}

		IconWidget->SetDesiredSizeOverride(FVector2D(IconSize));
		ButtonContent =
			SNew(SHorizontalBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TutorialHighlightName))
			+ SHorizontalBox::Slot()
			.AutoWidth()
		    .Padding(ToolBarStyle.IconPadding)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				IconWidget
			]
			// Label text
		+ (bIsSlimHorizontalUniformToolBar ?
		SHorizontalBox::Slot()
		.Padding(ToolBarStyle.LabelPadding)
		.VAlign(VAlign_Center) :
		SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ToolBarStyle.LabelPadding)
		.VAlign(VAlign_Center))
		.MaxWidth(ToolBarStyle.ButtonTextMaxWidth)
		.MinWidth(ToolBarStyle.ButtonTextMinWidth)
		[ TextBlock ];
	}
	else
	{
		const FMargin IconPadding = !LabelVisibility.IsSet() ? ToolBarStyle.IconPadding :
		(LabelVisibility.Get() == EVisibility::Collapsed ? ToolBarStyle.IconPadding : ToolBarStyle.IconPaddingWithVisibleLabel);

		// Use a delegate rather than static value, to account for the possibility that LabelVisiblity changes
		TAttribute<FMargin> IconPaddingAttribute = TAttribute<FMargin>::Create(
			TAttribute<FMargin>::FGetter::CreateLambda(
				[TextBlock, InitialPadding = IconPadding]() -> FMargin
				{
					FMargin IconPaddingValue = InitialPadding;

					// Icon Padding may use a bottom value appropriate for label separation, rather than for the button bounds,
					// so if the label is empty, we instead use the top padding which will be more appropriate for the button bounds.
					if (TextBlock->GetText().IsEmpty())
					{
						IconPaddingValue.Bottom = InitialPadding.Top;
					}

					return IconPaddingValue;
				}));

		const TSharedRef<SVerticalBox> ContentVBox =
			SNew(SVerticalBox)
			// Icon image
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(IconPaddingAttribute)
			.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
			[
				IconWidget
			]
			// Label text
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(ToolBarStyle.LabelPadding)
			.HAlign(HAlign_Center)	// Center the label text horizontally
			[
				TextBlock
			];

		ButtonContent =
			SNew(SHorizontalBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TutorialHighlightName))
			+ ( MultiBox->GetType() == EMultiBoxType::VerticalToolBar ?
				SHorizontalBox::Slot()
			.MaxWidth(ToolBarStyle.ButtonContentMaxWidth)
			.SizeParam(FStretch())
			.VAlign(VAlign_Center) :
			SHorizontalBox::Slot()
			.FillWidth(ToolBarStyle.ButtonContentFillWidth)
			.VAlign(VAlign_Center))
			[
				ContentVBox
			];
		}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EMultiBlockLocation::Type BlockLocation = GetMultiBlockLocation();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// What type of UI should we create for this block?
	TWeakPtr<const FUICommandInfo> Action = ToolBarButtonBlock->GetAction();
	EUserInterfaceActionType UserInterfaceType = ToolBarButtonBlock->UserInterfaceActionType;
	if ( Action.IsValid() )
	{
		// If we have a UICommand, then this is specified in the command.
		UserInterfaceType = Action.Pin()->GetUserInterfaceType();
	}
	
	if( UserInterfaceType == EUserInterfaceActionType::Button )
	{
		FName BlockStyle = EMultiBlockLocation::ToName(ISlateStyle::Join( StyleName, ".Button" ), BlockLocation);
		const FButtonStyle* ToolbarButtonStyle = BlockLocation == EMultiBlockLocation::None ? &ToolBarStyle.ButtonStyle : &StyleSet->GetWidgetStyle<FButtonStyle>(BlockStyle);

		if (OptionsBlockWidget.IsValid())
		{
			ToolbarButtonStyle = &ToolBarStyle.SettingsButtonStyle;
		}

		ChildSlot
		[
			// Create a button
			SNew(SButton)
			.ContentPadding(0.f)
			.ButtonStyle(ToolbarButtonStyle)
			.IsEnabled(this, &SToolBarButtonBlock::IsEnabled)
			.OnClicked(this, &SToolBarButtonBlock::OnClicked)
			.ToolTip(FMultiBoxSettings::ToolTipConstructor.Execute(ActualToolTip, nullptr, Action.Pin(), /*ShowActionShortcut=*/ true))
			.IsFocusable(bIsFocusable)
			[
				ButtonContent
			]
		];
	}
	else if( ensure( UserInterfaceType == EUserInterfaceActionType::ToggleButton || UserInterfaceType == EUserInterfaceActionType::RadioButton ) )
	{
		FName BlockStyle = EMultiBlockLocation::ToName(ISlateStyle::Join( StyleName, ".ToggleButton" ), BlockLocation);
	
		const FCheckBoxStyle* CheckStyle = BlockLocation == EMultiBlockLocation::None ? &ToolBarStyle.ToggleButton : &StyleSet->GetWidgetStyle<FCheckBoxStyle>(BlockStyle);

		if (OptionsBlockWidget.IsValid())
		{
			CheckStyle = &ToolBarStyle.SettingsToggleButton;
		}

		const TSharedPtr<SWidget> CheckBox = SNew(SCheckBox)
						// Use the tool bar style for this check box
						.Style(CheckStyle)
						.CheckBoxContentUsesAutoWidth(false)
						.IsFocusable(bIsFocusable)
						.ToolTip( FMultiBoxSettings::ToolTipConstructor.Execute( ActualToolTip, nullptr, Action.Pin(), /*ShowActionShortcut=*/ true))		
						.OnCheckStateChanged(this, &SToolBarButtonBlock::OnCheckStateChanged )
						.OnGetMenuContent( ToolBarButtonBlock->OnGetMenuContent )
						.IsChecked(this, &SToolBarButtonBlock::GetCheckState)
						.IsEnabled(this, &SToolBarButtonBlock::IsEnabled)
						[
							ButtonContent
						];

		TSharedRef<SWidget> CheckBoxWidget = CheckBox.ToSharedRef();

		if (!ToolBarButtonBlock->BorderBrushName.Get().IsNone())
		{
			const FSlateBrush* Brush = FAppStyle::GetBrush(ToolBarButtonBlock->BorderBrushName.Get());
			CheckBoxWidget =
				SNew(SBorder)
				.BorderImage(Brush)
				.Padding(2.f)
				[
					CheckBox.ToSharedRef()
				];
		}
		
		if ( ToolBarButtonBlock->GetDecoratedButtonDelegate.IsBound() )
		{
			CheckBoxWidget = ToolBarButtonBlock->GetDecoratedButtonDelegate.Execute( CheckBoxWidget );
		}

		ChildSlot
		[
			CheckBoxWidget
		];
	}

	if (OptionsBlockWidget.IsValid())
	{
		ChildSlot
		.Padding(ToolBarStyle.ComboButtonPadding.Left, 0.0f, ToolBarStyle.ComboButtonPadding.Right, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ButtonBorder, SBorder)
				.Padding(0)
				.BorderImage(this, &SToolBarButtonBlock::GetOptionsBlockLeftBrush)
				.VAlign(VAlign_Center)
				[
					ChildSlot.GetWidget()
				]
			]
			+ SHorizontalBox::Slot()
	
			.AutoWidth()
			[
				SAssignNew(OptionsBorder, SBorder)
				.Padding(0)
				.BorderImage(this, &SToolBarButtonBlock::GetOptionsBlockRightBrush)
				.VAlign(VAlign_Center)
				[
					OptionsBlockWidget.ToSharedRef()
				]
			]
		];
	}
	else
	{
		// Space between buttons. It does not make the buttons larger.
		// Button groups eliminate spacing between buttons.
		ChildSlot.Padding(EMultiBlockLocation::ToHorizontalMargin(MultiBox->GetType(), BlockLocation, ToolBarStyle.ButtonPadding));
	}

	// Bind our widget's visible state to whether or not the button should be visible
	SetVisibility(TAttribute<EVisibility>(this, &SToolBarButtonBlock::GetBlockVisibility));
}

bool FToolBarButtonBlock::GetIsFocusable() const
{
	return bIsFocusable;
}


/**
 * Called by Slate when this tool bar button's button is clicked
 */
FReply SToolBarButtonBlock::OnClicked()
{
	// Button was clicked, so trigger the action!
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();
	
	if( ActionList.IsValid() && Action.IsValid() )
	{
		ActionList->ExecuteAction( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		MultiBlock->GetDirectActions().Execute();
	}

	TSharedRef< const FMultiBox > MultiBox( OwnerMultiBoxWidget.Pin()->GetMultiBox() );

	// If this is a context menu, then we'll also dismiss the window after the user clicked on the item
	const bool ClosingMenu = MultiBox->ShouldCloseWindowAfterMenuSelection();
	if( ClosingMenu )
	{
		FSlateApplication::Get().DismissMenuByWidget(AsShared());
	}

	return FReply::Handled();
}



/**
 * Called by Slate when this tool bar check box button is toggled
 */
void SToolBarButtonBlock::OnCheckStateChanged( const ECheckBoxState NewCheckedState )
{
	OnClicked();
}

/**
 * Called by slate to determine if this button should appear checked
 *
 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
 */
ECheckBoxState SToolBarButtonBlock::GetCheckState() const
{
	TSharedPtr<const FUICommandList> ActionList = MultiBlock->GetActionList();
	TSharedPtr<const FUICommandInfo> Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		CheckState = ActionList->GetCheckState( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		CheckState = DirectActions.GetCheckState();
	}

	return CheckState;
}

/**
 * Called by Slate to determine if this button is enabled
 * 
 * @return True if the menu entry is enabled, false otherwise
 */
bool SToolBarButtonBlock::IsEnabled() const
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	bool bEnabled = true;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		bEnabled = ActionList->CanExecuteAction( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		bEnabled = DirectActions.CanExecute();
	}

	return bEnabled;
}


/**
 * Called by Slate to determine if this button is visible
 *
 * @return EVisibility::Visible or EVisibility::Collapsed, depending on if the button should be displayed
 */
EVisibility SToolBarButtonBlock::GetBlockVisibility() const
{
	// Let the visibility override take prescedence here.
	// However, if it returns Visible, let the other methods have a chance to change that.
	if (MultiBlock->GetVisibilityOverride().IsSet())
	{
		const EVisibility OverrideVisibility = MultiBlock->GetVisibilityOverride().Get();
		if (OverrideVisibility != EVisibility::Visible)
		{
			return OverrideVisibility;
		}
	}

	TSharedPtr<const FUICommandList> ActionList = MultiBlock->GetActionList();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();
	if( ActionList.IsValid() )
	{
		return ActionList->GetVisibility( MultiBlock->GetAction().ToSharedRef() );
	}
	else if(DirectActions.IsActionVisibleDelegate.IsBound())
	{
		return DirectActions.IsActionVisibleDelegate.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SToolBarButtonBlock::GetIconVisibility(bool bIsASmallIcon) const
{
	return ((bForceSmallIcons || FMultiBoxSettings::UseSmallToolBarIcons.Get()) ^ bIsASmallIcon) ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}

const FSlateBrush* SToolBarButtonBlock::GetIconBrush() const
{
	return bForceSmallIcons || FMultiBoxSettings::UseSmallToolBarIcons.Get() ? GetSmallIconBrush() : GetNormalIconBrush();
}

const FSlateBrush* SToolBarButtonBlock::GetOverlayIconBrush() const
{
	TSharedRef<const FToolBarButtonBlock> ToolBarButtonBlock = StaticCastSharedRef<const FToolBarButtonBlock >(MultiBlock.ToSharedRef());

	const FSlateIcon ActionIcon = ToolBarButtonBlock->GetAction().IsValid() ? ToolBarButtonBlock->GetAction()->GetIcon() : FSlateIcon();
	const FSlateIcon& ActualIcon = ToolBarButtonBlock->IconOverride.IsSet() ? ToolBarButtonBlock->IconOverride.Get() : ActionIcon;

	if (ActualIcon.IsSet())
	{
		return ActualIcon.GetOverlayIcon();
	}

	return nullptr;
}

const FSlateBrush* SToolBarButtonBlock::GetNormalIconBrush() const
{
	TSharedRef<const FToolBarButtonBlock> ToolBarButtonBlock = StaticCastSharedRef<const FToolBarButtonBlock >(MultiBlock.ToSharedRef());

	const FSlateIcon ActionIcon = ToolBarButtonBlock->GetAction().IsValid() ? ToolBarButtonBlock->GetAction()->GetIcon() : FSlateIcon();
	const FSlateIcon& ActualIcon = ToolBarButtonBlock->IconOverride.IsSet() ? ToolBarButtonBlock->IconOverride.Get() : ActionIcon;

	if (ActualIcon.IsSet())
	{
		return ActualIcon.GetIcon();
	}
	else
	{
		check(OwnerMultiBoxWidget.IsValid());

		TSharedPtr<SMultiBoxWidget> MultiBoxWidget = OwnerMultiBoxWidget.Pin();
		const ISlateStyle* const StyleSet = MultiBoxWidget->GetStyleSet();

		static const FName IconName("MultiBox.GenericToolBarIcon");
		return StyleSet->GetBrush(IconName);
	}
}

const FSlateBrush* SToolBarButtonBlock::GetSmallIconBrush() const
{
	TSharedRef< const FToolBarButtonBlock > ToolBarButtonBlock = StaticCastSharedRef< const FToolBarButtonBlock >( MultiBlock.ToSharedRef() );
	
	const FSlateIcon ActionIcon = ToolBarButtonBlock->GetAction().IsValid() ? ToolBarButtonBlock->GetAction()->GetIcon() : FSlateIcon();
	const FSlateIcon& ActualIcon = ToolBarButtonBlock->IconOverride.IsSet() ? ToolBarButtonBlock->IconOverride.Get() : ActionIcon;
	
	if( ActualIcon.IsSet() )
	{
		return ActualIcon.GetSmallIcon();
	}
	else
	{
		check( OwnerMultiBoxWidget.IsValid() );

		TSharedPtr<SMultiBoxWidget> MultiBoxWidget = OwnerMultiBoxWidget.Pin();
		const ISlateStyle* const StyleSet = MultiBoxWidget->GetStyleSet();

		static const FName IconName("MultiBox.GenericToolBarIcon.Small" );
		return StyleSet->GetBrush(IconName);
	}
}

FSlateColor SToolBarButtonBlock::GetIconForegroundColor() const
{
	// If any brush has a tint, don't assume it should be subdued
	const FSlateBrush* Brush = GetIconBrush();
	if (Brush && Brush->TintColor != FLinearColor::White)
	{
		return FLinearColor::White;
	}

	return FSlateColor::UseForeground();
}

const FSlateBrush* SToolBarButtonBlock::GetOptionsBlockLeftBrush() const
{
	static const FName ToggledLeft("ToolbarSettingsRegion.LeftToggle");

	if (ButtonBorder->IsHovered())
	{
		static const FName LeftHover("ToolbarSettingsRegion.LeftHover");
		static const FName ToggledLeftHover("ToolbarSettingsRegion.LeftToggleHover");

		return GetCheckState() == ECheckBoxState::Checked ? FAppStyle::Get().GetBrush(ToggledLeftHover) : FAppStyle::Get().GetBrush(LeftHover);
	}
	else if (OptionsBorder->IsHovered())
	{
		static const FName Left("ToolbarSettingsRegion.Left");
		return GetCheckState() == ECheckBoxState::Checked ? FAppStyle::Get().GetBrush(ToggledLeft) : FAppStyle::Get().GetBrush(Left);
	}
	else
	{
		return GetCheckState() == ECheckBoxState::Checked ? FAppStyle::Get().GetBrush(ToggledLeft) : FStyleDefaults::GetNoBrush();
	}

}

const FSlateBrush* SToolBarButtonBlock::GetOptionsBlockRightBrush() const
{
	if (OptionsBorder->IsHovered())
	{
		static const FName RightHover("ToolbarSettingsRegion.RightHover");
		return FAppStyle::Get().GetBrush(RightHover);
	}
	else if (ButtonBorder->IsHovered() || GetCheckState() == ECheckBoxState::Checked)
	{
		static const FName Right("ToolbarSettingsRegion.Right");
		return FAppStyle::Get().GetBrush(Right);
	}
	else
	{
		return FStyleDefaults::GetNoBrush();
	}
}

EVisibility SToolBarButtonBlock::GetOptionsSeparatorVisibility() const
{
	return IsHovered() ? EVisibility::HitTestInvisible : EVisibility::Hidden;
}
