// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SToolBarComboButtonBlock.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"

FToolBarComboButtonBlock::FToolBarComboButtonBlock(
	const FUIAction& InAction,
	const FOnGetContent& InMenuContentGenerator,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InToolTip,
	const TAttribute<FSlateIcon>& InIcon,
	bool bInSimpleComboBox,
	TAttribute<FText> InToolbarLabelOverride,
	TAttribute<EMenuPlacement> InPlacementOverride,
	const EUserInterfaceActionType InUserInterfaceActionType
)
	: FMultiBlock(InAction, NAME_None, EMultiBlockType::ToolBarComboButton)
	, MenuContentGenerator(InMenuContentGenerator)
	, Label(InLabel)
	, ToolbarLabelOverride(InToolbarLabelOverride)
	, ToolTip(InToolTip)
	, Icon(InIcon)
	, PlacementOverride(InPlacementOverride)
	, LabelVisibility()
	, UserInterfaceActionType(InUserInterfaceActionType)
	, bSimpleComboBox(bInSimpleComboBox)
	, bForceSmallIcons(false)
{
}

void FToolBarComboButtonBlock::CreateMenuEntry(FMenuBuilder& MenuBuilder) const
{
	FText EntryLabel = Label.Get();
	if ( EntryLabel.IsEmpty() )
	{
		EntryLabel = NSLOCTEXT("ToolBar", "CustomControlLabel", "Custom Control");
	}

	MenuBuilder.AddWrapperSubMenu(EntryLabel, ToolTip.Get(), MenuContentGenerator, Icon.Get(), GetDirectActions());
}

bool FToolBarComboButtonBlock::HasIcon() const
{
	const FSlateIcon& ActualIcon = Icon.Get();
	return ActualIcon.GetIcon()->GetResourceName() != NAME_None;
}

TSharedRef< class IMultiBlockBaseWidget > FToolBarComboButtonBlock::ConstructWidget() const
{
	return SNew( SToolBarComboButtonBlock )
		.LabelVisibility( LabelVisibility.IsSet() ? LabelVisibility.GetValue() : TOptional< EVisibility >() )
		.Icon(Icon)
		.ForceSmallIcons( bForceSmallIcons )
		.Cursor( EMouseCursor::Default );
}

void SToolBarComboButtonBlock::Construct( const FArguments& InArgs )
{
	LabelVisibilityOverride = InArgs._LabelVisibility;
	Icon = InArgs._Icon;
	bForceSmallIcons = InArgs._ForceSmallIcons;
}

void SToolBarComboButtonBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	TSharedRef< const FMultiBox > MultiBox( OwnerMultiBoxWidget.Pin()->GetMultiBox() );
	
	TSharedRef< const FToolBarComboButtonBlock > ToolBarComboButtonBlock = StaticCastSharedRef< const FToolBarComboButtonBlock >( MultiBlock.ToSharedRef() );

	TSharedPtr<const FUICommandInfo> UICommand = ToolBarComboButtonBlock->GetAction();

	TAttribute<FText> Label;

	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

	// If override is set use that
	TAttribute<EVisibility> LabelVisibility;
	if (LabelVisibilityOverride.IsSet())
	{
		LabelVisibility = LabelVisibilityOverride.GetValue();
	}
	else if (!ToolBarStyle.bShowLabels)
	{
		// Otherwise check the style
		LabelVisibility = EVisibility::Collapsed;
	}
	else
	{
		LabelVisibility = TAttribute< EVisibility >::Create(TAttribute< EVisibility >::FGetter::CreateSP(SharedThis(this), &SToolBarComboButtonBlock::GetIconVisibility, false));
	}

	TSharedRef<SWidget> IconWidget = SNullWidget::NullWidget;
	if (!ToolBarComboButtonBlock->bSimpleComboBox)
	{
		if (Icon.IsSet())
		{
			TSharedRef<SLayeredImage> ActualIconWidget =
				SNew(SLayeredImage)
					.ColorAndOpacity(this, &SToolBarComboButtonBlock::GetIconForegroundColor)
					.Image(this, &SToolBarComboButtonBlock::GetIconBrush);

			ActualIconWidget->AddLayer(TAttribute<const FSlateBrush*>(this, &SToolBarComboButtonBlock::GetOverlayIconBrush));

			if (MultiBox->GetType() == EMultiBoxType::SlimHorizontalToolBar
				|| MultiBox->GetType() == EMultiBoxType::SlimHorizontalUniformToolBar
				|| MultiBox->GetType() == EMultiBoxType::SlimWrappingToolBar)
			{
				const FVector2f IconSize = ToolBarStyle.IconSize;

				IconWidget = SNew(SBox).WidthOverride(IconSize.X).HeightOverride(IconSize.Y)[ActualIconWidget];
			}
			else
			{
				IconWidget = ActualIconWidget;
			}
		}

		if (ToolBarComboButtonBlock->ToolbarLabelOverride.IsSet())
		{
			Label = ToolBarComboButtonBlock->ToolbarLabelOverride;
		}
		else
		{
			Label = ToolBarComboButtonBlock->Label;
		}
	}

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), Label.Get(), MultiBlock->GetSearchable());

	// Setup the string for the metatag
	FName TagName;
	if (ToolBarComboButtonBlock->GetTutorialHighlightName() == NAME_None)
	{
		TagName = *FString::Printf(TEXT("ToolbarComboButton,%s,0"), *Label.Get().ToString());
	}
	else
	{
		TagName = ToolBarComboButtonBlock->GetTutorialHighlightName();
	}
	
	// When an execute action is present, the combo button is split into two parts:
	// - The (checkbox) button that handles the action
	// - The ComboButton that opens the menu
	const bool bIsSplitButton = HasAction();

	// A custom widget overrides all of the default content, including the icon and label, so we don't need to create those if we have a custom widget
	const bool bHasCustomWidget = ToolBarComboButtonBlock->CustomWidget.IsValid();

	// Some of the same layout considerations happen if either are true
	const bool bIsSplitButtonOrHasCustomWidget = bIsSplitButton || bHasCustomWidget;

	// Create the content for our button
	TSharedRef<SWidget> ButtonContent = SNullWidget::NullWidget;
	if (!bIsSplitButton && bHasCustomWidget)
	{
		// The CustomWidget is the entire content of the button when it's not a split button
		ButtonContent = ToolBarComboButtonBlock->CustomWidget.ToSharedRef();
	}
	else if (MultiBox->GetType() == EMultiBoxType::SlimHorizontalToolBar
		|| MultiBox->GetType() == EMultiBoxType::SlimHorizontalUniformToolBar
		|| MultiBox->GetType() == EMultiBoxType::SlimWrappingToolBar)
	{
		if (bIsSplitButton && Label.Get().IsEmpty())
		{
			// A split button with an empty label just needs to show the combo carat
			ButtonContent = SNullWidget::NullWidget;
		}
		else
		{
			// clang-format off
			ButtonContent =
				SNew(SHorizontalBox)
				// Icon image
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					// A split button or custom widget will have the icon or content handled in its own widget
					bIsSplitButtonOrHasCustomWidget ? SNullWidget::NullWidget : IconWidget
				]
				// Label text
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.MinWidth(ToolBarStyle.ComboContentMinWidth)
				.MaxWidth(ToolBarStyle.ComboContentMaxWidth)
				.Padding(TAttribute<FMargin>::CreateLambda(
					[bIsSplitButtonOrHasCustomWidget, Label, ToolBarStyle]() -> FMargin
					{
						return bIsSplitButtonOrHasCustomWidget || Label.Get().IsEmpty() ? FMargin(0) : ToolBarStyle.LabelPadding;
					}
				))
				.HAlign(ToolBarStyle.ComboContentHorizontalAlignment)
				.VAlign(VAlign_Center)	// Center the label text horizontally
				[
					SNew(STextBlock)
					.Visibility(ToolBarComboButtonBlock->bSimpleComboBox ? EVisibility::Collapsed : LabelVisibility)
					.Text(Label)
					// Smaller font for tool tip labels
					.TextStyle(&ToolBarStyle.LabelStyle)
				];
			// clang-format on
		}
	}
	else
	{
		// clang-format off
		ButtonContent =
			SNew(SVerticalBox)
			// Icon image
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
			[
				IconWidget
			]
			// Label text
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(TAttribute<FMargin>::CreateLambda(
				[Label, ToolBarStyle]() -> FMargin
				{
					return Label.Get().IsEmpty() ? FMargin(0) : ToolBarStyle.LabelPadding;
				}
			))
			.HAlign(HAlign_Center)	// Center the label text horizontally
			[
				SNew(STextBlock)
				.Visibility(LabelVisibility)
				.Text(Label)
				.TextStyle(&ToolBarStyle.LabelStyle)
			];
		// clang-format on
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EMultiBlockLocation::Type BlockLocation = GetMultiBlockLocation();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FName BlockStyle = EMultiBlockLocation::ToName(ISlateStyle::Join(StyleName, ".Button"), BlockLocation);
	const FButtonStyle* ButtonStyle = BlockLocation == EMultiBlockLocation::None ? &ToolBarStyle.ButtonStyle : &StyleSet->GetWidgetStyle<FButtonStyle>(BlockStyle);
	const FComboButtonStyle* ComboStyle = &ToolBarStyle.ComboButtonStyle;
	if (ToolBarComboButtonBlock->bSimpleComboBox)
	{
		ComboStyle = &ToolBarStyle.SettingsComboButton;
		ButtonStyle = &ComboStyle->ButtonStyle;
	}

	OpenForegroundColor = ButtonStyle->HoveredForeground;
	BlockHovered = &ToolBarStyle.BlockHovered;

	TAttribute<FText> ActualToolTip;
	if (ToolBarComboButtonBlock->ToolTip.IsSet())
	{
		ActualToolTip = ToolBarComboButtonBlock->ToolTip;
	}
	else
	{
		ActualToolTip = UICommand.IsValid() ? UICommand->GetDescription() : FText::GetEmpty();
	}

	EUserInterfaceActionType UserInterfaceType = ToolBarComboButtonBlock->UserInterfaceActionType;
	if (UICommand.IsValid())
	{
		// If we have a UICommand, then this is specified in the command.
		UserInterfaceType = UICommand->GetUserInterfaceType();
	}
	
	const FCheckBoxStyle* CheckStyle = GetCheckBoxStyle(StyleSet, StyleName, bIsSplitButton);
	
	LeftHandSideWidget = SNullWidget::NullWidget;

	if (bIsSplitButton)
	{
		TSharedRef<SWidget> SplitButtonContent = SNullWidget::NullWidget;
		if (bHasCustomWidget)
		{
			// If there's a custom widget, in the case of a split button, use it in place of the icon
			SplitButtonContent = ToolBarComboButtonBlock->CustomWidget.ToSharedRef();
		}
		else
		{
			SplitButtonContent = 
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					IconWidget
				];
		}
		
		if (UserInterfaceType == EUserInterfaceActionType::Button)
		{
			// When a button is specified, the combo menu is implied to be
			// the "settings of the button"
			ComboStyle = &ToolBarStyle.SettingsComboButton;
			ButtonStyle = &ComboStyle->ButtonStyle;
		}
		else
		{
			// Allow for optional style customization of a split combo button 
			ComboStyle = &StyleSet->GetWidgetStyle<FComboButtonStyle>(StyleName, ".SplitComboButton", ComboStyle);
			ButtonStyle = &ComboStyle->ButtonStyle;
		}
	
		// clang-format off
		SAssignNew(LeftHandSideWidget, SCheckBox)
			.Style(CheckStyle)
			.CheckBoxContentUsesAutoWidth(false)
			.OnCheckStateChanged(this, &SToolBarComboButtonBlock::OnCheckStateChanged)
			.OnGetMenuContent(this, &SToolBarComboButtonBlock::OnGetMenuContent)
			.IsChecked(this, &SToolBarComboButtonBlock::GetCheckState)
			.IsEnabled(this, &SToolBarComboButtonBlock::IsEnabled)
			[
				SplitButtonContent
			];
		// clang-format on
	}
	else if (HasCheckedState())
	{
		// Only cache the checkbox style if we need to perform coloration
		// on icons in non-checkbox widgets
		CheckBoxStyle = CheckStyle;
	}
	
	// clang-format off
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage( this, &SToolBarComboButtonBlock::GetBorderImage )
		.Padding(0.f)
		[
			SNew(SHorizontalBox)
			.ToolTip(FMultiBoxSettings::ToolTipConstructor.Execute(
				ActualToolTip, nullptr, UICommand, /*ShowActionShortcut=*/true
			))
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			[
				LeftHandSideWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			[
				SAssignNew(ComboButtonWidget, SComboButton)
				.AddMetaData<FTagMetaData>(FTagMetaData(TagName))
				.ContentPadding(0.f)
				.ComboButtonStyle(ComboStyle)
				.ButtonStyle(ButtonStyle)
				.ToolTipText(ToolBarComboButtonBlock->ToolTip)
				.MenuPlacement(ToolBarComboButtonBlock->PlacementOverride)
				.ForegroundColor(this, &SToolBarComboButtonBlock::OnGetForegroundColor)
				// Route the content generator event
				.OnGetMenuContent(this, &SToolBarComboButtonBlock::OnGetMenuContent)
				.ButtonContent()
				[
					ButtonContent
				]
			]
		]
	];
	// clang-format on

	FMargin Padding = ToolBarStyle.ComboButtonPadding;
	if (ToolBarComboButtonBlock->bSimpleComboBox)
	{
		Padding = FMargin(0);
	}

	ChildSlot.Padding(Padding);

	// Bind our widget's enabled state to whether or not our action can execute
	SetEnabled(TAttribute<bool>( this, &SToolBarComboButtonBlock::IsEnabled));

	// Bind our widget's visible state to whether or not the button should be visible
	SetVisibility( TAttribute<EVisibility>(this, &SToolBarComboButtonBlock::GetVisibility) );
}

TSharedRef<SWidget> SToolBarComboButtonBlock::OnGetMenuContent()
{
	TSharedRef< const FToolBarComboButtonBlock > ToolBarButtonComboBlock = StaticCastSharedRef< const FToolBarComboButtonBlock >( MultiBlock.ToSharedRef() );
	return ToolBarButtonComboBlock->MenuContentGenerator.Execute();
}

FReply SToolBarComboButtonBlock::OnClicked()
{
	// Button was clicked, so trigger the action!
	TSharedPtr<const FUICommandList> ActionList = MultiBlock->GetActionList();
	TSharedPtr<const FUICommandInfo> Action = MultiBlock->GetAction();

	if (ActionList.IsValid() && Action.IsValid())
	{
		ActionList->ExecuteAction(Action.ToSharedRef());
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		MultiBlock->GetDirectActions().Execute();
	}

	TSharedRef<const FMultiBox> MultiBox(OwnerMultiBoxWidget.Pin()->GetMultiBox());

	// If this is a context menu, then we'll also dismiss the window after the user clicked on the item
	const bool bClosingMenu = MultiBox->ShouldCloseWindowAfterMenuSelection();
	if (bClosingMenu)
	{
		FSlateApplication::Get().DismissMenuByWidget(AsShared());
	}

	return FReply::Handled();
}

void SToolBarComboButtonBlock::OnCheckStateChanged(const ECheckBoxState NewCheckedState)
{
	OnClicked();
}

const FCheckBoxStyle* SToolBarComboButtonBlock::GetCheckBoxStyle(const ISlateStyle* StyleSet, const FName& StyleName, bool bIsSplitButton) const
{
	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);
	EMultiBlockLocation::Type BlockLocation = GetMultiBlockLocation();

	const FCheckBoxStyle* CheckStyle;

	if (OptionsBlockWidget.IsValid())
	{
		CheckStyle = &ToolBarStyle.SettingsToggleButton;
	}	
	else if (!Icon.IsSet())
	{
		CheckStyle = &FCoreStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Checkbox");
	}
	else if (BlockLocation == EMultiBlockLocation::None)
	{
		if (bIsSplitButton)
		{
			CheckStyle = &StyleSet->GetWidgetStyle<FCheckBoxStyle>(StyleName, ".SplitToggleButton", &ToolBarStyle.ToggleButton);
		}
		else
		{
			CheckStyle = &ToolBarStyle.ToggleButton;	
		}
	}
	else
	{
		CheckStyle = &StyleSet->GetWidgetStyle<FCheckBoxStyle>(EMultiBlockLocation::ToName(ISlateStyle::Join(StyleName, ".ToggleButton"), BlockLocation));
	}

	return CheckStyle;
}

ECheckBoxState SToolBarComboButtonBlock::GetCheckState() const
{
	TSharedPtr<const FUICommandList> ActionList = MultiBlock->GetActionList();
	TSharedPtr<const FUICommandInfo> Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if (ActionList.IsValid() && Action.IsValid())
	{
		CheckState = ActionList->GetCheckState(Action.ToSharedRef());
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		CheckState = DirectActions.GetCheckState();
	}

	return CheckState;
}

bool SToolBarComboButtonBlock::IsEnabled() const
{
	const FUIAction& UIAction = MultiBlock->GetDirectActions();
	if( UIAction.CanExecuteAction.IsBound() )
	{
		return UIAction.CanExecuteAction.Execute();
	}

	return true;
}

bool SToolBarComboButtonBlock::HasAction() const
{
	return (MultiBlock->GetActionList().IsValid() && MultiBlock->GetAction().IsValid()) ||
		MultiBlock->GetDirectActions().IsBound();
}

bool SToolBarComboButtonBlock::HasCheckedState() const
{
	if (MultiBlock->GetActionList() && MultiBlock->GetAction())
	{
		if (const FUIAction* Action = MultiBlock->GetActionList()->GetActionForCommand(MultiBlock->GetAction()))
		{
			return Action->GetActionCheckState.IsBound();
		}
	}
	return MultiBlock->GetDirectActions().GetActionCheckState.IsBound();
}

EVisibility SToolBarComboButtonBlock::GetVisibility() const
{
	// Let the visibility override take prescedence here.
	// However, if it returns Visible, let the other methods have a chance to change that.
	if (MultiBlock->GetVisibilityOverride().IsSet())
	{
		const EVisibility OverrideVisibility = MultiBlock->GetVisibilityOverride().Get();
		if (OverrideVisibility != EVisibility::Visible)
		{
			return OverrideVisibility;
		}
	}

	const FUIAction& UIAction = MultiBlock->GetDirectActions();
	if (UIAction.IsActionVisibleDelegate.IsBound())
	{
		return UIAction.IsActionVisibleDelegate.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

bool SToolBarComboButtonBlock::HasDynamicIcon() const
{
	return Icon.IsBound();
}

const FSlateBrush* SToolBarComboButtonBlock::GetIconBrush() const
{
	return bForceSmallIcons || FMultiBoxSettings::UseSmallToolBarIcons.Get() ? GetSmallIconBrush() : GetNormalIconBrush();
}

const FSlateBrush* SToolBarComboButtonBlock::GetNormalIconBrush() const
{
	const FSlateIcon& ActualIcon = Icon.Get();
	return ActualIcon.GetIcon();
}

const FSlateBrush* SToolBarComboButtonBlock::GetSmallIconBrush() const
{
	const FSlateIcon& ActualIcon = Icon.Get();
	return ActualIcon.GetSmallIcon();
}

EVisibility SToolBarComboButtonBlock::GetIconVisibility(bool bIsASmallIcon) const
{
	return ((bForceSmallIcons || FMultiBoxSettings::UseSmallToolBarIcons.Get()) ^ bIsASmallIcon) ? EVisibility::Collapsed : EVisibility::Visible;
}

FSlateColor SToolBarComboButtonBlock::GetIconForegroundColor() const
{
	// If any brush has a tint, don't assume it should be subdued
	const FSlateBrush* Brush = GetIconBrush();
	if (Brush && Brush->TintColor != FLinearColor::White)
	{
		return FLinearColor::White;
	}
	
	if (CheckBoxStyle)
	{
		const ECheckBoxState CheckState = GetCheckState();
		const bool bIsHovered = LeftHandSideWidget != SNullWidget::NullWidget ? LeftHandSideWidget->IsHovered() : ComboButtonWidget->IsHovered();
		
		if (bIsHovered)
		{
			switch (CheckState)
			{
			case ECheckBoxState::Unchecked:
				return CheckBoxStyle->HoveredForeground;
			case ECheckBoxState::Checked:
				return CheckBoxStyle->CheckedHoveredForeground;
			case ECheckBoxState::Undetermined:
				return CheckBoxStyle->HoveredForeground;
			}
		}
		else
		{
			switch (CheckState)
			{
			case ECheckBoxState::Unchecked:
				return CheckBoxStyle->ForegroundColor;
			case ECheckBoxState::Checked:
				return CheckBoxStyle->CheckedForeground;
			case ECheckBoxState::Undetermined:
				return CheckBoxStyle->UndeterminedForeground;
			}
		}
	}
	
	return FSlateColor::UseForeground();
}

const FSlateBrush* SToolBarComboButtonBlock::GetOverlayIconBrush() const
{
	const FSlateIcon& ActualIcon = Icon.Get();

	if (ActualIcon.IsSet())
	{
		return ActualIcon.GetOverlayIcon();
	}

	return nullptr;
}

FSlateColor SToolBarComboButtonBlock::OnGetForegroundColor() const
{
	if (ComboButtonWidget->IsOpen())
	{
		return OpenForegroundColor;
	}
	else
	{
		return FSlateColor::UseStyle();
	}
}

const FSlateBrush* SToolBarComboButtonBlock::GetBorderImage() const
{
	if ((ComboButtonWidget && ComboButtonWidget->IsHovered()) ||
		(LeftHandSideWidget && LeftHandSideWidget->IsHovered()))
	{
		return BlockHovered;
	}
	else
	{
		return FAppStyle::GetBrush("NoBorder");
	}
}
