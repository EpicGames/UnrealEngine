// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SToolBarSeparatorBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/ToolBarStyle.h"


/**
 * Constructor
 */
FToolBarSeparatorBlock::FToolBarSeparatorBlock(const FName& InExtensionHook)
	: FMultiBlock( nullptr, nullptr, InExtensionHook, EMultiBlockType::Separator )
{
}



void FToolBarSeparatorBlock::CreateMenuEntry(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddSeparator();
}



/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FToolBarSeparatorBlock::ConstructWidget() const
{
	return SNew( SToolBarSeparatorBlock );
}



/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SToolBarSeparatorBlock::Construct( const FArguments& InArgs )
{
}


/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SToolBarSeparatorBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

	const TSharedRef<const FMultiBox> MultiBox = OwnerMultiBoxWidget.Pin()->GetMultiBox();
	if (MultiBox->GetType() == EMultiBoxType::VerticalToolBar)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(ToolBarStyle.SeparatorPadding)
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
				.Thickness(ToolBarStyle.SeparatorThickness)
				.SeparatorImage(&ToolBarStyle.SeparatorBrush)
			]
		];
	}
	else
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(ToolBarStyle.SeparatorPadding)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Thickness(ToolBarStyle.SeparatorThickness)
				.SeparatorImage(&ToolBarStyle.SeparatorBrush)
			]
		];
	}

	// Add this widget to the search list of the multibox and hide it
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), FText::GetEmpty(), MultiBlock->GetSearchable());

	SetVisibility(MultiBlock->GetVisibilityOverride());
}
