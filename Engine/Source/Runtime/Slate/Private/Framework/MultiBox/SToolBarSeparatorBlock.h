// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

/**
 * Toolbar separator MultiBlock
 */
class FToolBarSeparatorBlock
	: public FMultiBlock
{

public:

	/**
	 * Constructor
	 */
	FToolBarSeparatorBlock(const FName& InExtensionHook);

	/** FMultiBlock interface */
	virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const override;


private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;


private:

	// Friend our corresponding widget class
	friend class SToolBarSeparatorBlock;

};



template <>
struct TWidgetTypeTraits<class SToolBarSeparatorBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Toolbar separator MultiBlock widget
 */
class SToolBarSeparatorBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SToolBarSeparatorBlock ){}

	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );


private:

};
