// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SToolBarStackButtonBlock.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/ToolBarStyle.h"

#include <type_traits>

/////////////////////////////////////////////////////
// SInternalStack

/** CheckBox where right & long-press clicks spawn menu. */
class SCheckBoxStack : public SCheckBox
{

public: 

	using Super = SCheckBox;

	//~ Begin SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	//~ End SWidget interface

protected:

	/** Starts countdown for spawning of menu via hold press */
	void BeginHoldPress();

	/** Cancels any existing hold presses */
	void EndHoldPress();

	/** Callback for after hold press duration has ended */
	bool OnHandleHoldPress(float DeltaTime);

protected:

	/** Ticker to handle hold actions. */
	float HoldPressDuration = 0.25f;

	/** Ticker to handle hold actions. */
	FTSTicker::FDelegateHandle HoldPressHandle;
};

template <>
struct TWidgetTypeTraits<SCheckBoxStack>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

FReply SCheckBoxStack::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = true;

		EButtonClickMethod::Type InputClickMethod = GetClickMethodFromInputType(MouseEvent);

		BeginHoldPress();

		if(InputClickMethod == EButtonClickMethod::MouseDown )
		{
			ToggleCheckedState();
			const ECheckBoxState State = GetCheckedState();
			if(State == ECheckBoxState::Checked)
			{
				PlayCheckedSound();
			}
			else if(State == ECheckBoxState::Unchecked)
			{
				PlayUncheckedSound();
			}

			// Clear our pressed state immediately so that we appear checked
			bIsPressed = false;

			// Set focus to this button, but don't capture the mouse
			return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Mouse);
		}
		else
		{
			// Capture the mouse, and also set focus to this button
			return FReply::Handled().CaptureMouse(AsShared()).SetUserFocus(AsShared(), EFocusCause::Mouse);
		}
	}
	else if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnGetMenuContent.IsBound() )
	{
		EndHoldPress(); 
		OnHandleHoldPress(0.0f);

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SCheckBoxStack::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	EndHoldPress();

	return Super::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SCheckBoxStack::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	EndHoldPress();

	Super::OnMouseCaptureLost(CaptureLostEvent);
}

void SCheckBoxStack::BeginHoldPress()
{
	HoldPressHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &SCheckBoxStack::OnHandleHoldPress), HoldPressDuration);
}

void SCheckBoxStack::EndHoldPress()
{
	if (HoldPressHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldPressHandle);
		HoldPressHandle.Reset();
	}
}

bool SCheckBoxStack::OnHandleHoldPress(float DeltaTime)
{
	// For now, always spawns menu top right
	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		OnGetMenuContent.Execute(),
		GetTickSpaceGeometry().GetRenderBoundingRect().GetTopRight(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);

	// Do not repeat ticker
	return false;
}

/////////////////////////////////////////////////////
// FToolBarStackButtonBlock

FToolBarStackButtonBlock::FToolBarStackButtonBlock(const TSharedRef< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, bool bInSimpleStackBox)
	: FMultiBlock(InCommand, InCommandList)
	, bSimpleComboBox(bInSimpleStackBox)
{
}

void FToolBarStackButtonBlock::CreateMenuEntry(class FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddWrapperSubMenu(FText::GetEmpty(), FText::GetEmpty(), GetStackContent()->OnGetContent, FSlateIcon(), GetDirectActions());
}

TSharedRef< class IMultiBlockBaseWidget > FToolBarStackButtonBlock::ConstructWidget() const
{
	return SNew( SToolBarStackButtonBlock )
		.LabelVisibility( LabelVisibility.IsSet() ? LabelVisibility.GetValue() : TOptional< EVisibility >() )
		.ForceSmallIcons( bForceSmallIcons )
		.Cursor( EMouseCursor::Default );
}

const TSharedPtr<FUIIdentifierContext> FToolBarStackButtonBlock::GetStackIdentifier() const
{
	if (CachedStackIdentifier.IsValid())
	{
		return CachedStackIdentifier.Pin();
	}

	if (const FUIActionContext* ActionContext = GetActionList()->GetContextForCommand(GetAction()))
	{
		if (TSharedPtr<FUIIdentifierContext> IdentifierContext = StaticCastSharedPtr<FUIIdentifierContext>(ActionContext->FindContext(FUIIdentifierContext::ContextName)))
		{
			CachedStackIdentifier = IdentifierContext;
			return IdentifierContext;
		}
	}

	return nullptr;
}

const TSharedPtr<FUIContentContext> FToolBarStackButtonBlock::GetStackContent() const
{
	if (CachedStackContent.IsValid())
	{
		return CachedStackContent.Pin();
	}

	if (const FUIActionContext* ActionContext = GetActionList()->GetContextForCommand(GetAction()))
	{
		if (TSharedPtr<FUIContentContext> IdentifierContext = StaticCastSharedPtr<FUIContentContext>(ActionContext->FindContext(FUIContentContext::ContextName)))
		{
			CachedStackContent = IdentifierContext;
			return IdentifierContext;
		}
	}

	return nullptr;
}

/////////////////////////////////////////////////////
// SToolBarStackButtonBlock

void SToolBarStackButtonBlock::Construct( const FArguments& InArgs )
{
	LabelVisibilityOverride = InArgs._LabelVisibility;
	bForceSmallIcons = InArgs._ForceSmallIcons;
}

void SToolBarStackButtonBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	TSharedRef< const FMultiBox > MultiBox( OwnerMultiBoxWidget.Pin()->GetMultiBox() );
	
	TSharedRef< const FToolBarStackButtonBlock > ToolBarStackButtonBlock = StaticCastSharedRef< const FToolBarStackButtonBlock >( MultiBlock.ToSharedRef() );

	TAttribute<FText> Label;

	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

	// If override is set use that
	if (LabelVisibilityOverride.IsSet())
	{
		LabelVisibility = LabelVisibilityOverride.GetValue();
	}
	else if (!ToolBarStyle.bShowLabels)
	{
		// Otherwise check the style
		LabelVisibility = EVisibility::Collapsed;
	}
	else
	{
		LabelVisibility = TAttribute< EVisibility >::Create(TAttribute< EVisibility >::FGetter::CreateSP(SharedThis(this), &SToolBarStackButtonBlock::GetIconVisibility, false));
	}

	TSharedRef<SLayeredImage> IconWidget =
		SNew(SLayeredImage)
		.ColorAndOpacity(this, &SToolBarStackButtonBlock::GetIconForegroundColor)
		.Visibility(EVisibility::HitTestInvisible)
		.Image(this, &SToolBarStackButtonBlock::GetIconBrush);

	IconWidget->AddLayer(TAttribute<const FSlateBrush*>(this, &SToolBarStackButtonBlock::GetNormalIconBrush));

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidget.Pin()->AddElement(this->AsWidget(), Label.Get(), MultiBlock->GetSearchable());

	// Setup the string for the metatag
	FName TagName;
	if (ToolBarStackButtonBlock->GetTutorialHighlightName().IsNone())
	{
		TagName = *FString::Printf(TEXT("Stack Button,%s,0"), *Label.Get().ToString());
	}
	else
	{
		TagName = ToolBarStackButtonBlock->GetTutorialHighlightName();
	}

	// Create the content for our button
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EMultiBlockLocation::Type BlockLocation = GetMultiBlockLocation();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FName BlockStyle = EMultiBlockLocation::ToName(ISlateStyle::Join(StyleName, ".Button"), BlockLocation);
	const FComboButtonStyle* ComboButtonStyle = &FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton");
	const FButtonStyle* ButtonStyle = BlockLocation == EMultiBlockLocation::None ? &ToolBarStyle.ButtonStyle : &StyleSet->GetWidgetStyle<FButtonStyle>(BlockStyle);
	const FCheckBoxStyle* CheckStyle = BlockLocation == EMultiBlockLocation::None ? &ToolBarStyle.ToggleButton : &StyleSet->GetWidgetStyle<FCheckBoxStyle>(BlockStyle);

	if (ToolBarStackButtonBlock->bSimpleComboBox)
	{
		ComboButtonStyle = &ToolBarStyle.SettingsComboButton;
		ButtonStyle = &ComboButtonStyle->ButtonStyle;
	}

	const bool bHasDownArrowShadow = !ComboButtonStyle->ShadowOffset.IsZero();

	FMargin ExtraDropDownPadding = FMargin(0, 0, 4, 0);
	FMargin ExtraLabelPadding = FMargin(4, 0, 0, 0);

	TSharedRef<SWidget> ButtonContent = SNullWidget::NullWidget;
	if (MultiBox->GetType() == EMultiBoxType::SlimHorizontalToolBar
		|| MultiBox->GetType() == EMultiBoxType::SlimHorizontalUniformToolBar
		|| MultiBox->GetType() == EMultiBoxType::SlimWrappingToolBar)
	{
		const FVector2f IconSize = ToolBarStyle.IconSize;

		IconWidget->SetDesiredSizeOverride(FVector2D(IconSize));
		ButtonContent =
			SNew(SHorizontalBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TagName))
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Padding(ExtraLabelPadding)
			[
				IconWidget
			]
			// Label text
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(ToolBarStyle.LabelPadding + ExtraLabelPadding)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(LabelVisibility)
				.Text(this, &SToolBarStackButtonBlock::GetLabel)
				.TextStyle(&ToolBarStyle.LabelStyle)	// Smaller font for tool tip labels
			]

			// Drop down arrow
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign( HAlign_Right )
			.VAlign(ComboButtonStyle->DownArrowAlign)
			.Padding(ComboButtonStyle->DownArrowPadding)
			[
				SNew(SOverlay)
				// drop shadow
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				.Padding(FMargin(ComboButtonStyle->ShadowOffset.X, ComboButtonStyle->ShadowOffset.Y, 0, 0))
				[
					SNew(SImage)
					.Visibility( bHasDownArrowShadow ? EVisibility::Visible : EVisibility::Collapsed )
					.Image( &ComboButtonStyle->DownArrowImage )
					.ColorAndOpacity( ComboButtonStyle->ShadowColorAndOpacity )
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Visibility( EVisibility::Visible  )
					.Image( &ComboButtonStyle->DownArrowImage )
					// Inherit tinting from parent
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			];
	}
	else
	{
		ButtonContent =
			SNew(SHorizontalBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TagName))
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			.Padding(ExtraLabelPadding)
			[
				SNew(SVerticalBox)
				// Icon image
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
				[
					IconWidget
				]
				// Label text
				+ SVerticalBox::Slot().AutoHeight()
				.Padding(ToolBarStyle.LabelPadding)
				.HAlign(HAlign_Center)	// Center the label text horizontally
				[
					SNew(STextBlock)
					.Visibility(LabelVisibility)
					.Text(this, &SToolBarStackButtonBlock::GetLabel)
					.TextStyle(&ToolBarStyle.LabelStyle)	// Smaller font for tool tip labels
				]
			]
		
			// Drop down arrow
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign( HAlign_Right )
			.VAlign(ComboButtonStyle->DownArrowAlign)
			.Padding(ComboButtonStyle->DownArrowPadding)
			[
				SNew(SOverlay)
				// drop shadow
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				.Padding(FMargin(ComboButtonStyle->ShadowOffset.X, ComboButtonStyle->ShadowOffset.Y, 0, 0))
				[
					SNew(SImage)
					.Visibility( bHasDownArrowShadow ? EVisibility::Visible : EVisibility::Collapsed )
					.Image( &ComboButtonStyle->DownArrowImage )
					.ColorAndOpacity( ComboButtonStyle->ShadowColorAndOpacity )
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				.Padding(ExtraDropDownPadding)
				[
					SNew(SImage)
					.Visibility( EVisibility::Visible  )
					.Image( &ComboButtonStyle->DownArrowImage )
					// Inherit tinting from parent
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			];
	}

	OpenForegroundColor = ButtonStyle->HoveredForeground;

	ChildSlot
	[
		SAssignNew(StackButtonWidget, SCheckBoxStack)
		.Style(CheckStyle)
		.ClickMethod(EButtonClickMethod::MouseDown)
		.ToolTipText(this, &SToolBarStackButtonBlock::GetDescription)
		.OnCheckStateChanged(this, &SToolBarStackButtonBlock::OnCheckStateChanged)
		.IsChecked(this, &SToolBarStackButtonBlock::GetCheckState)
		.IsEnabled(this, &SToolBarStackButtonBlock::IsEnabled)
		.OnGetMenuContent(this, &SToolBarStackButtonBlock::OnGetMenuContent)
		[
			ButtonContent
		]
	];


	FMargin Padding = ToolBarStyle.ComboButtonPadding;
	if (ToolBarStackButtonBlock->bSimpleComboBox)
	{
		Padding = FMargin(0);
	}

	ChildSlot.Padding(Padding);

	// Bind our widget's visible state to whether or not the button should be visible
	SetVisibility( TAttribute<EVisibility>(this, &SToolBarStackButtonBlock::GetVisibility) );
}

void SToolBarStackButtonBlock::OnCheckStateChanged( const ECheckBoxState NewCheckedState )
{
	TSharedPtr< const FUICommandList > ActionList = MultiBlock->GetActionList();
	TSharedPtr< const FUICommandInfo > Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	if (ActionList.IsValid() && Action.IsValid())
	{
		ActionList->ExecuteAction(Action.ToSharedRef());
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		MultiBlock->GetDirectActions().Execute();
	}
}

ECheckBoxState SToolBarStackButtonBlock::GetCheckState() const
{
	TSharedPtr<const FUICommandList> ActionList = MultiBlock->GetActionList();
	TSharedPtr<const FUICommandInfo> Action = MultiBlock->GetAction();
	const FUIAction& DirectActions = MultiBlock->GetDirectActions();

	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if( ActionList.IsValid() && Action.IsValid() )
	{
		CheckState = ActionList->GetCheckState( Action.ToSharedRef() );
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		CheckState = DirectActions.GetCheckState();
	}

	return CheckState;
}

TSharedRef<SWidget> SToolBarStackButtonBlock::OnGetMenuContent()
{
	TSharedRef< const FToolBarStackButtonBlock > ToolBarButtonComboBlock = StaticCastSharedRef< const FToolBarStackButtonBlock >( MultiBlock.ToSharedRef() );
	
	if (const TSharedPtr<FUIContentContext> StackContent = ToolBarButtonComboBlock->GetStackContent())
	{
		return StackContent->OnGetContent.Execute();
	}

	return SNullWidget::NullWidget;
}

bool SToolBarStackButtonBlock::IsEnabled() const
{
	const FUIAction& UIAction = MultiBlock->GetDirectActions();
	if( UIAction.CanExecuteAction.IsBound() )
	{
		return UIAction.CanExecuteAction.Execute();
	}

	return true;
}

EVisibility SToolBarStackButtonBlock::GetVisibility() const
{
	// Let the visibility override take prescedence here.
	// However, if it returns Visible, let the other methods have a chance to change that.
	if (MultiBlock->GetVisibilityOverride().IsSet())
	{
		const EVisibility OverrideVisibility = MultiBlock->GetVisibilityOverride().Get();
		if (OverrideVisibility != EVisibility::Visible)
		{
			return OverrideVisibility;
		}
	}

	const FUIAction& UIAction = MultiBlock->GetDirectActions();
	if (UIAction.IsActionVisibleDelegate.IsBound())
	{
		return UIAction.IsActionVisibleDelegate.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

const FSlateBrush* SToolBarStackButtonBlock::GetIconBrush() const
{
	return bForceSmallIcons || FMultiBoxSettings::UseSmallToolBarIcons.Get() ? GetSmallIconBrush() : GetNormalIconBrush();
}

FText SToolBarStackButtonBlock::GetLabel() const
{
	TSharedRef< const FToolBarStackButtonBlock > ToolBarStackButtonBlock = StaticCastSharedRef< const FToolBarStackButtonBlock >(MultiBlock.ToSharedRef());

	return ToolBarStackButtonBlock->GetStackIdentifier()
		? ToolBarStackButtonBlock->GetStackIdentifier()->OnGetContextabel.Execute()
		: ToolBarStackButtonBlock->GetAction()->GetLabel();
}

FText SToolBarStackButtonBlock::GetDescription() const
{
	TSharedRef< const FToolBarStackButtonBlock > ToolBarStackButtonBlock = StaticCastSharedRef< const FToolBarStackButtonBlock >(MultiBlock.ToSharedRef());

	return ToolBarStackButtonBlock->GetStackIdentifier()
		? ToolBarStackButtonBlock->GetStackIdentifier()->OnGetContextDescription.Execute()
		: ToolBarStackButtonBlock->GetAction()->GetDescription();
}

const FSlateBrush* SToolBarStackButtonBlock::GetNormalIconBrush() const
{
	TSharedRef< const FToolBarStackButtonBlock > ToolBarStackButtonBlock = StaticCastSharedRef< const FToolBarStackButtonBlock >(MultiBlock.ToSharedRef());
	
	const FSlateIcon ActionIcon = ToolBarStackButtonBlock->GetStackIdentifier() ? ToolBarStackButtonBlock->GetStackIdentifier()->OnGetContextIcon.Execute() : FSlateIcon();
	
	if( ActionIcon.IsSet() )
	{
		return ActionIcon.GetSmallIcon();
	}
	else
	{
		check( OwnerMultiBoxWidget.IsValid() );

		TSharedPtr<SMultiBoxWidget> MultiBoxWidget = OwnerMultiBoxWidget.Pin();
		const ISlateStyle* const StyleSet = MultiBoxWidget->GetStyleSet();

		static const FName IconName("MultiBox.GenericToolBarIcon.Small" );
		return StyleSet->GetBrush(IconName);
	}
}

const FSlateBrush* SToolBarStackButtonBlock::GetSmallIconBrush() const
{
	TSharedRef< const FToolBarStackButtonBlock > ToolBarStackButtonBlock = StaticCastSharedRef< const FToolBarStackButtonBlock >(MultiBlock.ToSharedRef());

	const FSlateIcon ActionIcon = ToolBarStackButtonBlock->GetStackIdentifier() ? ToolBarStackButtonBlock->GetStackIdentifier()->OnGetContextIcon.Execute() : FSlateIcon();
	
	if( ActionIcon.IsSet() )
	{
		return ActionIcon.GetSmallIcon();
	}
	else
	{
		check( OwnerMultiBoxWidget.IsValid() );

		TSharedPtr<SMultiBoxWidget> MultiBoxWidget = OwnerMultiBoxWidget.Pin();
		const ISlateStyle* const StyleSet = MultiBoxWidget->GetStyleSet();

		static const FName IconName("MultiBox.GenericToolBarIcon.Small" );
		return StyleSet->GetBrush(IconName);
	}
}

EVisibility SToolBarStackButtonBlock::GetIconVisibility(bool bIsASmallIcon) const
{
	return ((bForceSmallIcons || FMultiBoxSettings::UseSmallToolBarIcons.Get()) ^ bIsASmallIcon) ? EVisibility::Collapsed : EVisibility::Visible;
}

FSlateColor SToolBarStackButtonBlock::GetIconForegroundColor() const
{
	// If any brush has a tint, don't assume it should be subdued
	const FSlateBrush* Brush = GetIconBrush();
	if (Brush && Brush->TintColor != FLinearColor::White)
	{
		return FLinearColor::White;
	}

	return FSlateColor::UseForeground();
}

const FSlateBrush* SToolBarStackButtonBlock::GetOverlayIconBrush() const
{
	const FSlateIcon& ActualIcon = Icon.Get();

	if (ActualIcon.IsSet())
	{
		return ActualIcon.GetOverlayIcon();
	}

	return nullptr;
}

