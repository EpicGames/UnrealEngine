// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SUniformToolbarButtonBlock.h"

FUniformToolbarButtonBlock::FUniformToolbarButtonBlock(FButtonArgs ButtonArgs) :
	FToolBarButtonBlock(ButtonArgs)
{
}

TSharedRef< class IMultiBlockBaseWidget > FUniformToolbarButtonBlock::ConstructWidget() const
{
	return SNew( SUniformToolbarButtonBlock )
		.LabelVisibility(EVisibility::Visible)
		.IsFocusable(GetIsFocusable())
		.ForceSmallIcons(true)
		.TutorialHighlightName(GetTutorialHighlightName())
		.Cursor(EMouseCursor::Default);
}