// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SUniformToolbarPanel.h"
#include "Layout/LayoutUtils.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Styling/ToolBarStyle.h"

SUniformToolbarPanel::SUniformToolbarPanel()
: Children(this)
{
}

void SUniformToolbarPanel::Construct( const FArguments& InArgs )
{
	SlotPadding = InArgs._SlotPadding;

	MinDesiredSlotSize = InArgs._MinDesiredSlotSize;
	MaxUniformSize = InArgs._MaxUniformSize;
	MinUniformSize = InArgs._MinUniformSize;

	StyleSet = InArgs._StyleSet;
	StyleName = InArgs._StyleName;

	Orientation = InArgs._Orientation;

	ClippedIndex = INDEX_NONE;

	Children.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));

	const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);
	
	// Add the optional dropdown arrow as a child.
	Children.AddSlot(FSlot::FSlotArguments(MakeUnique<FSlot>(
		SAssignNew(Dropdown, SComboButton)
		.HasDownArrow(false)
		.ButtonStyle(&ToolBarStyle.ButtonStyle)
		.ContentPadding(0.f)
		.ToolTipText(NSLOCTEXT("Slate", "ExpandToolbar", "Click to expand toolbar"))
		.OnGetMenuContent(InArgs._OnDropdownOpened)
		.Cursor(EMouseCursor::Default)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(&ToolBarStyle.WrapButtonStyle.ExpandBrush)
		]
		)));
}

void SUniformToolbarPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	if ( Children.Num() > 0 )
	{
		ClippedIndex = INDEX_NONE;

		const float MaxUniformSizeVal = MaxUniformSize.Get();
		const float MinUniformSizeVal = MinUniformSize.Get();

		const FVector2D UniformCellSize = Orientation == EOrientation::Orient_Horizontal
			? FVector2D(MajorAxisUniformDesiredSize, AllottedGeometry.GetLocalSize().Y)
			: FVector2D(AllottedGeometry.GetLocalSize().X, MajorAxisUniformDesiredSize);

		const FVector2D SlotPaddingDesiredSize = SlotPadding.Get().GetDesiredSize();

		const FMargin& CurrentSlotPadding(SlotPadding.Get());
		FVector2D CurrentOffset = FVector2D::ZeroVector;

		const int32 ClippedChildrenDropDownIndex = Children.Num() - 1;

		const FVector2D ClippedChildrenDropDownDesiredSize = Children[ClippedChildrenDropDownIndex].GetWidget()->GetDesiredSize();

		for (int32 ChildIndex=0; ChildIndex < ClippedChildrenDropDownIndex; ++ChildIndex)
		{
			const FSlot& Child = Children[ChildIndex];
			const EVisibility ChildVisibility = Child.GetWidget()->GetVisibility();
			if ( ArrangedChildren.Accepts(ChildVisibility) )
			{
				AlignmentArrangeResult XAxisResult(0,0); 
				AlignmentArrangeResult YAxisResult(0,0);

				const FVector2D ChildDesiredSize = Child.GetWidget()->GetDesiredSize() + SlotPaddingDesiredSize;
				const float MajorAxisDesiredSize = Orientation == EOrientation::Orient_Horizontal ? ChildDesiredSize.X : ChildDesiredSize.Y;

				if ((MaxUniformSizeVal > 0.0f && MajorAxisDesiredSize > MaxUniformSizeVal)
					|| (MinUniformSizeVal > 0.0f && MajorAxisDesiredSize < MinUniformSizeVal))
				{
					// Do the standard arrangement of elements within a slot
					// Takes care of alignment and padding.
					XAxisResult = AlignChild<Orient_Horizontal>(ChildDesiredSize.X, Child, CurrentSlotPadding);
					YAxisResult = AlignChild<Orient_Vertical>(UniformCellSize.Y, Child, CurrentSlotPadding);
				}
				else
				{
					// Do the standard arrangement of elements within a slot
					// Takes care of alignment and padding.
					XAxisResult = AlignChild<Orient_Horizontal>(UniformCellSize.X, Child, CurrentSlotPadding);
					YAxisResult = AlignChild<Orient_Vertical>(UniformCellSize.Y, Child, CurrentSlotPadding);
				}


				int32 WidgetExtents = 0;
				int32 AllottedGeometryExtents = 0;

				FArrangedWidget ArrangedChild = FArrangedWidget::GetNullWidget();

				if(Orientation == EOrientation::Orient_Horizontal)
				{
					ArrangedChild =
						AllottedGeometry.MakeChild(Child.GetWidget(),
							FVector2D(CurrentOffset.X + XAxisResult.Offset, YAxisResult.Offset),
							FVector2D(XAxisResult.Size, YAxisResult.Size)
						);

					WidgetExtents = FMath::TruncToInt(ArrangedChild.Geometry.AbsolutePosition.X + ArrangedChild.Geometry.GetLocalSize().X * ArrangedChild.Geometry.Scale);
					AllottedGeometryExtents = FMath::TruncToInt((AllottedGeometry.AbsolutePosition.X + AllottedGeometry.GetLocalSize().X * AllottedGeometry.Scale) - ClippedChildrenDropDownDesiredSize.X);

				}
				else
				{
					ArrangedChild =
						AllottedGeometry.MakeChild(Child.GetWidget(),
							FVector2D(XAxisResult.Offset, CurrentOffset.Y + YAxisResult.Offset),
							FVector2D(XAxisResult.Size, YAxisResult.Size)
						);

					WidgetExtents = FMath::TruncToInt(ArrangedChild.Geometry.AbsolutePosition.Y + ArrangedChild.Geometry.GetLocalSize().Y * ArrangedChild.Geometry.Scale);
					AllottedGeometryExtents = FMath::TruncToInt((AllottedGeometry.AbsolutePosition.Y + AllottedGeometry.GetLocalSize().Y * AllottedGeometry.Scale) - ClippedChildrenDropDownDesiredSize.Y);

				}

				if (WidgetExtents > AllottedGeometryExtents)
				{
					ClippedIndex = ChildIndex;

					// Arrange dropdown 
					TSharedRef<SWidget> Widget = Children[ClippedChildrenDropDownIndex].GetWidget();

					FVector2D DropdownDesiredSize = Widget->GetDesiredSize();
					ArrangedChildren.AddWidget(
						AllottedGeometry.MakeChild(Widget,
							FVector2D(CurrentOffset.X, (AllottedGeometry.GetLocalSize().Y - DropdownDesiredSize.Y) / 2.0f),
							DropdownDesiredSize)
					);

					break;
				}
				else
				{
					ArrangedChildren.AddWidget(ChildVisibility, ArrangedChild);
				}


				CurrentOffset += FVector2D(XAxisResult.Size, YAxisResult.Size);

			}

		}
	}
}

FVector2D SUniformToolbarPanel::ComputeDesiredSize( float ) const
{
	FVector2D MaxUniformChildDesiredSize = FVector2D::ZeroVector;
	FVector2D NonUniformChildDesiredSize = FVector2D::ZeroVector;
	const FVector2D SlotPaddingDesiredSize = SlotPadding.Get().GetDesiredSize();
	
	const FVector2D CachedMinDesiredSlotSize = MinDesiredSlotSize.Get();

	const float MaxUniformSizeVal = MaxUniformSize.Get();
	const float MinUniformSizeVal = MinUniformSize.Get();

	int32 NumUniformCells = 0;

	float MinorAxisUniformDesiredSize = 0;

	const int32 ClippedChildrenDropDownIndex = Children.Num() - 1;

	const int32 NumChildren = Children.Num();
	for ( int32 ChildIndex=0; ChildIndex < ClippedChildrenDropDownIndex; ++ChildIndex )
	{
		const FSlot& Child = Children[ ChildIndex ];

		if (Child.GetWidget()->GetVisibility() != EVisibility::Collapsed)
		{	
			FVector2D ChildDesiredSize = Child.GetWidget()->GetDesiredSize() + SlotPaddingDesiredSize;

			ChildDesiredSize.X = FMath::Max(ChildDesiredSize.X, CachedMinDesiredSlotSize.X);
			ChildDesiredSize.Y = FMath::Max(ChildDesiredSize.Y, CachedMinDesiredSlotSize.Y);

			const float MajorAxisDesiredSize = Orientation == EOrientation::Orient_Horizontal ? ChildDesiredSize.X : ChildDesiredSize.Y;
			const float MinorAxisDesiredSize = Orientation == EOrientation::Orient_Horizontal ? ChildDesiredSize.Y : ChildDesiredSize.X;

			if ((MaxUniformSizeVal > 0.0f && MajorAxisDesiredSize > MaxUniformSizeVal)
				|| (MinUniformSizeVal > 0.0f && MajorAxisDesiredSize < MinUniformSizeVal))
			{
				NonUniformChildDesiredSize += ChildDesiredSize;
			}
			else
			{
				++NumUniformCells;
				MajorAxisUniformDesiredSize = FMath::Max(MajorAxisUniformDesiredSize, MajorAxisDesiredSize);
			}

			MinorAxisUniformDesiredSize = FMath::Max(MinorAxisUniformDesiredSize, MinorAxisDesiredSize);

		}
	}

	FVector2D DropdownDesiredSize = FVector2D::ZeroVector;
	if (ClippedIndex != INDEX_NONE)
	{
		// factor in the desired size of the drop down since it will be shown when children are clipped
		DropdownDesiredSize = Dropdown->GetDesiredSize();
	}

	return (Orientation == Orient_Horizontal
		? FVector2D(NumUniformCells*MajorAxisUniformDesiredSize + NonUniformChildDesiredSize.X, NumChildren*MinorAxisUniformDesiredSize)
		: FVector2D(NumChildren*MinorAxisUniformDesiredSize, NumUniformCells*MajorAxisUniformDesiredSize + NonUniformChildDesiredSize.Y)) + DropdownDesiredSize;
}


FChildren* SUniformToolbarPanel::GetChildren()
{
	return &Children;
}

void SUniformToolbarPanel::SetSlotPadding(TAttribute<FMargin> InSlotPadding)
{
	SlotPadding = InSlotPadding;
	Invalidate(EInvalidateWidgetReason::Layout);
}

SUniformToolbarPanel::FSlot::FSlotArguments SUniformToolbarPanel::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

SUniformToolbarPanel::FScopedWidgetSlotArguments SUniformToolbarPanel::AddSlot()
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Children, INDEX_NONE };
}

bool SUniformToolbarPanel::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	for (int32 SlotIdx = 0; SlotIdx < Children.Num(); ++SlotIdx)
	{
		if ( SlotWidget == Children[SlotIdx].GetWidget() )
		{
			Children.RemoveAt(SlotIdx);
			return true;
		}
	}
	
	return false;
}
