// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Layout/Margin.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Framework/SlateDelegates.h"

class FArrangedChildren;
class SComboButton;
class ISlateStyle;

/**
 * This panel evenly divides up the available space between all of its children but allows for a min and max threshold to be supplied so that
 * very large children do not force siblings to grow unnecessarily large and very small children are not forced to be unnecessarily large to maintain uniformity
 */
class SUniformToolbarPanel : public SPanel
{
public:
	/** Stores the per-child info for this panel type */
	struct FSlot : public TSlotBase<FSlot>, public TAlignmentWidgetSlotMixin<FSlot>
	{
		FSlot()
			: TSlotBase<FSlot>()
			, TAlignmentWidgetSlotMixin<FSlot>(HAlign_Fill, VAlign_Fill)
		{
		}
		FSlot(const TSharedRef<SWidget>& InWidget)
			: TSlotBase<FSlot>(InWidget)
			, TAlignmentWidgetSlotMixin<FSlot>(HAlign_Fill, VAlign_Fill)
		{
		}

		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TSlotBase<FSlot>, TAlignmentWidgetSlotMixin<FSlot>)
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			TAlignmentWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
		}
	};

	SLATE_API SUniformToolbarPanel();

	SLATE_BEGIN_ARGS(SUniformToolbarPanel)
		: _Orientation(EOrientation::Orient_Horizontal)
		, _SlotPadding(FMargin(0.0f))
		, _MinDesiredSlotSize(FVector2D(0.0f,0.0f))
		, _MaxUniformSize(0.0f)
		, _MinUniformSize(0.0f)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** Slot type supported by this panel */
		SLATE_SLOT_ARGUMENT(FSlot, Slots)

		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		SLATE_ARGUMENT(FName, StyleName)

		/** Orientation (i.e major axis for this panel) */
		SLATE_ARGUMENT(EOrientation, Orientation)

		/** Padding given to each slot */
		SLATE_ATTRIBUTE(FMargin, SlotPadding)

		/** The minimum desired width and height of the slots */
		SLATE_ATTRIBUTE(FVector2D, MinDesiredSlotSize)

		/** 
		 * The maximum size a child can be to be uniformly arranged.  Widgets over this size will not be uniformly sized and will not contribute to the uniform size of siblings. 
		 * If this value is 0, all widgets will be uniformly sized
		 */
		SLATE_ATTRIBUTE(float, MaxUniformSize)

		/**
		 * The minimum size a child must be to be uniformly arranged.  Widgets under this size will not be uniformly sized. 
		 * If this value is 0 all widgets will be uniformly sized
		 */
		SLATE_ATTRIBUTE(float, MinUniformSize)

		/**
		 * Called when the dropdown button is clicked and the menu opened
		 */
		SLATE_EVENT(FOnGetContent, OnDropdownOpened)

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	//~ Begin SPanel Interface	
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	//~ End SPanel Interface

	/** See SlotPadding attribute */
	SLATE_API void SetSlotPadding(TAttribute<FMargin> InSlotPadding);

	/**
	 * Used by declarative syntax to create a Slot in the specified Column, Row.
	 */
	static SLATE_API FSlot::FSlotArguments Slot();

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Dynamically add a new slot to the Panel
	 *
	 * @return A reference to the newly-added slot
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot();
	
	/**
	 * Removes a slot from this panel which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The true if the slot was removed and false if no slot was found matching the widget
	 */
	SLATE_API bool RemoveSlot( const TSharedRef<SWidget>& SlotWidget );
	
	/**
	 * Return the index to the first widget cli
	 */
	int32 GetClippedIndex() const { return ClippedIndex; }

protected:
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
private:
	/** The button that is displayed when the toolbar is clipped */
	TSharedPtr<SComboButton> Dropdown;

	TPanelChildren<FSlot> Children;
	TAttribute<FMargin> SlotPadding;
	
	mutable float MajorAxisUniformDesiredSize = 0;

	/** Index of the first child that is cliped. Children at this index or beyond will be in a dropdown menu */
	mutable int32 ClippedIndex;

	EOrientation Orientation;
	TAttribute<FVector2D> MinDesiredSlotSize;
	TAttribute<float> MaxUniformSize;
	TAttribute<float> MinUniformSize;

	/** The style to use */
	const ISlateStyle* StyleSet;
	FName StyleName;
};
