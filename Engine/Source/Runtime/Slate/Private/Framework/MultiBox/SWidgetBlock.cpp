// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/SWidgetBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SImage.h"

FWidgetBlock::FWidgetBlock(TSharedRef<SWidget> InContent, const FText& InLabel, const TAttribute<FText>& InToolTipText, const FMenuEntryStyleParams& InStyleParams, const TAttribute<FSlateIcon>& InIcon)
	: FMultiBlock(nullptr, nullptr, NAME_None, EMultiBlockType::Widget)
    , ContentWidget(InContent)
	, Label(InLabel)
	, ToolTipText(InToolTipText)
	, Icon(InIcon)
	, StyleParams(InStyleParams)
{
}

void FWidgetBlock::SetCustomMenuDelegate( FNewMenuDelegate& InCustomMenuDelegate )
{
	CustomMenuDelegate = InCustomMenuDelegate;	
}

void FWidgetBlock::CreateMenuEntry(FMenuBuilder& MenuBuilder) const
{
	if ( CustomMenuDelegate.IsBound() )
	{
		CustomMenuDelegate.Execute(MenuBuilder);
	}
	else
	{
		FText EntryLabel = (!Label.IsEmpty()) ? Label : NSLOCTEXT("WidgetBlock", "CustomControl", "Custom Control");

		FMenuEntryStyleParams MenuStyleParams;
		MenuStyleParams.bNoIndent = true;

		MenuBuilder.AddWidget(ContentWidget, FText::GetEmpty(), MenuStyleParams);
	}
}

/**
 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
 *
 * @return  MultiBlock widget object
 */
TSharedRef< class IMultiBlockBaseWidget > FWidgetBlock::ConstructWidget() const
{
	return SNew( SWidgetBlock )
			.Cursor(EMouseCursor::Default);
}

bool FWidgetBlock::GetAlignmentOverrides(FMenuEntryStyleParams& OutAlignmentParameters) const
{
	OutAlignmentParameters.HorizontalAlignment = StyleParams.HorizontalAlignment;
	OutAlignmentParameters.VerticalAlignment = StyleParams.VerticalAlignment.Get(VAlign_Fill);

	if (StyleParams.SizeRule.IsSet())
	{
		OutAlignmentParameters.SizeRule = StyleParams.SizeRule;
		OutAlignmentParameters.FillSize = StyleParams.FillSize;
		OutAlignmentParameters.FillSizeMin = StyleParams.FillSizeMin;
	}
	else if (StyleParams.HorizontalAlignment == HAlign_Fill)
	{
		OutAlignmentParameters.SizeRule = FSizeParam::ESizeRule::SizeRule_Auto;
	}

	OutAlignmentParameters.MinimumSize = StyleParams.MinimumSize;
	OutAlignmentParameters.MaximumSize = StyleParams.MaximumSize;

	return true;
}

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SWidgetBlock::Construct( const FArguments& InArgs )
{
}

/**
 * Builds this MultiBlock widget up from the MultiBlock associated with it
 */
void SWidgetBlock::BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName)
{
	const TSharedPtr<SMultiBoxWidget> OwnerMultiBoxWidgetPinned = OwnerMultiBoxWidget.Pin();

	TSharedPtr< const FMultiBox > MultiBox = OwnerMultiBoxWidgetPinned->GetMultiBox();
	const TSharedRef<const FWidgetBlock> WidgetBlock = StaticCastSharedRef<const FWidgetBlock>(MultiBlock.ToSharedRef());

	const bool bHasLabel = !WidgetBlock->Label.IsEmpty();

	// Initially use default behavior
	EVerticalAlignment ContentVerticalAlignment = bHasLabel ? VAlign_Center : VAlign_Fill;

	// Support menus which do not have a defined widget style yet
	FMargin Padding;
	const FTextBlockStyle* LabelStyle = nullptr;
	if(StyleSet->HasWidgetStyle<FToolBarStyle>(StyleName))
	{
		const FToolBarStyle& ToolBarStyle = StyleSet->GetWidgetStyle<FToolBarStyle>(StyleName);

		Padding = WidgetBlock->StyleParams.bNoIndent ? ToolBarStyle.BlockPadding : ToolBarStyle.IndentedBlockPadding;
		LabelStyle = &ToolBarStyle.LabelStyle;

		if (ToolBarStyle.VerticalAlignmentOverride.IsSet())
		{
			ContentVerticalAlignment = ToolBarStyle.VerticalAlignmentOverride.GetValue();
		}
	}
	else
	{
		Padding = WidgetBlock->StyleParams.bNoIndent ? StyleSet->GetMargin(StyleName, ".Block.Padding") : StyleSet->GetMargin(StyleName, ".Block.IndentedPadding");
		LabelStyle = &StyleSet->GetWidgetStyle<FTextBlockStyle>(ISlateStyle::Join(StyleName, ".Label"));
	}

	if (WidgetBlock->StyleParams.VerticalAlignment.IsSet())
	{
		ContentVerticalAlignment = WidgetBlock->StyleParams.VerticalAlignment.GetValue();
	}

	if(OwnerMultiBoxWidgetPinned->GetMultiBox()->GetType() == EMultiBoxType::Menu)
	{
		// Account for checkmark used in other menu blocks but not used in for widget rows
		Padding = Padding + FMargin(14, 0, 8, 0);

		// If there is no label, allow the custom menu widget to consume the entire space
		if (!bHasLabel)
		{
			// We use 1 pixel of padding to ensure the menu border shows up
			Padding = FMargin(1);
		}
	}

	// For searchable menus with a custom widgets without a label, find a TextBlock to connect with for search
	// This is similar to how SMenuEntryBlock works
	FText SearchLabel = WidgetBlock->Label;
	TAttribute<FText> SearchHighlightText;

	if( OwnerMultiBoxWidgetPinned->GetMultiBox()->GetType() == EMultiBoxType::Menu && OwnerMultiBoxWidgetPinned->GetSearchable())
	{
		SearchHighlightText.Bind(OwnerMultiBoxWidgetPinned.Get(), &SMultiBoxWidget::GetSearchText);

		if (!bHasLabel)
		{
			TSharedRef<SWidget> TextBlock = FindTextBlockWidget(WidgetBlock->ContentWidget);
			if (TextBlock != SNullWidget::NullWidget )
			{
				// Bind the search text to the widgets text to highlight
				TSharedRef<STextBlock> TheTextBlock = StaticCastSharedRef<STextBlock>(TextBlock);
				TheTextBlock->SetHighlightText(SearchHighlightText);

				SearchLabel = TheTextBlock.Get().GetText();
			}
		}
	}

	const TSharedRef<SWidget> ThisWidget = this->AsWidget();

	// Add this widget to the search list of the multibox
	OwnerMultiBoxWidgetPinned->AddElement(ThisWidget, SearchLabel, MultiBlock->GetSearchable());

	// This widget holds the search text, set it as the search block widget
	if (OwnerMultiBoxWidgetPinned->GetSearchTextWidget() && OwnerMultiBoxWidgetPinned->GetSearchTextWidget()->GetParentWidget() == WidgetBlock->ContentWidget)
	{
		OwnerMultiBoxWidgetPinned->SetSearchBlockWidget(ThisWidget);
	}

	static constexpr float IconRightPadding = 4.f;

	// If we were supplied an image than go ahead and use that, otherwise we use a null widget
	TSharedRef<SWidget> IconWidget = SNullWidget::NullWidget;
	if (WidgetBlock->Icon.IsSet())
	{
		const FSlateIcon& ActualIcon = WidgetBlock->Icon.Get();
		const FSlateBrush* IconBrush = ActualIcon.GetIcon();
		if (IconBrush->GetResourceName() != NAME_None)
		{
			IconWidget = SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(IconBrush);
		}
	}

	const float MenuIconSize = StyleSet->GetFloat(StyleName, ".MenuIconSize", 16.f);

	// clang-format off
	ChildSlot
	.Padding( Padding )	// Large left margin mimics the indent of normal menu items when bNoIndent is false
	[
		SNew(SHorizontalBox)
		.ToolTip(FMultiBoxSettings::ToolTipConstructor.Execute(WidgetBlock->ToolTipText, nullptr, nullptr, /*ShowActionShortcut=*/ false ))
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2, 0, 6, 0))
		[
			SNew(SBox)
			.Visibility(IconWidget != SNullWidget::NullWidget ? EVisibility::Visible : EVisibility::Collapsed)
			.WidthOverride(MenuIconSize + 2)
			.HeightOverride(MenuIconSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(MenuIconSize)
				.HeightOverride(MenuIconSize)
				[
					IconWidget
				]
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(2, 0, 6, 0))
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			.Visibility(bHasLabel ? EVisibility::Visible : EVisibility::Collapsed)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(bHasLabel ? FMargin(0.f, 0.f, IconRightPadding, 0.f) : FMargin(0))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(LabelStyle)
				.Text(WidgetBlock->Label)
				.HighlightText(SearchHighlightText)
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.ForegroundHover"))
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(ContentVerticalAlignment)
		.FillWidth(1.f)
		[
			WidgetBlock->ContentWidget
		]
	];
	// clang-format on

	SetVisibility(MultiBlock->GetVisibilityOverride());
}

TSharedRef<SWidget> SWidgetBlock::FindTextBlockWidget( TSharedRef<SWidget> Content )
{
	if (Content->GetType() == FName("STextBlock"))
	{
		return Content;
	}

	FChildren* Children = Content->GetChildren();
	int32 NumChildren = Children->Num();

	for( int32 Index = 0; Index < NumChildren; ++Index )
	{
		TSharedRef<SWidget> Found = FindTextBlockWidget( Children->GetChildAt( Index ));
		if (Found != SNullWidget::NullWidget)
		{
			return Found;
		}
	}
	return SNullWidget::NullWidget;
}

void SWidgetBlock::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// If this widget is in a Menu, attempt to close any other open submenus within that menu
	TSharedPtr<SMultiBoxWidget> OwnerMultiBoxWidgetPinned = OwnerMultiBoxWidget.Pin();
	if(OwnerMultiBoxWidgetPinned->GetMultiBox()->GetType() == EMultiBoxType::Menu)
	{
		const TSharedPtr< const SMenuAnchor > OpenedMenuAnchor( OwnerMultiBoxWidgetPinned->GetOpenMenu() );
		if ( OpenedMenuAnchor.IsValid() && OpenedMenuAnchor->IsOpen() )
		{
			OwnerMultiBoxWidgetPinned->CloseSummonedMenus();
		}
	}

	SMultiBlockBaseWidget::OnMouseEnter(MyGeometry, MouseEvent);
}
