// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

/**
 * Arbitrary Widget MultiBlock
 */
class FWidgetBlock
	: public FMultiBlock
{

public:
	/**
	 * Constructor
	 *
	 * @param	InContent					The widget to place in the block
	 * @param	InLabel						Optional label text to be added to the left of the content
	 * @param	InToolTipText				Optional tooltip text to be added to the widget and label
	 * @param	InStyleParams				Optional additional style parameters. @see FMenuEntryStyleParams
	 * @param	InIcon						Optional icon to be shown to the left of the label/content
	 */
	FWidgetBlock(TSharedRef<SWidget> InContent, const FText& InLabel, const TAttribute<FText>& InToolTipText = FText(), const FMenuEntryStyleParams& InStyleParams = FMenuEntryStyleParams(), const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>());

	/** FMultiBlock interface */
	virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const override;

	/** Set optional delegate to customize when a menu appears instead of the widget, such as in toolbars */
	void SetCustomMenuDelegate( FNewMenuDelegate& InOnFillMenuDelegate);

private:

	/** FMultiBlock private interface */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;
	virtual bool GetAlignmentOverrides(FMenuEntryStyleParams& OutAlignmentParameters) const override;

private:

	// Friend our corresponding widget class
	friend class SWidgetBlock;

	/** Content widget */
	TSharedRef<SWidget> ContentWidget;

	/** Optional label text */
	FText Label;

	/** Optional ToolTip text */
	TAttribute<FText> ToolTipText;

	/** Optional Icon */
	TAttribute<FSlateIcon> Icon;

	/** Style parameters */
	FMenuEntryStyleParams StyleParams;

	/** Optional delegate to customize when a menu appears instead of the widget, such as in toolbars */
	FNewMenuDelegate CustomMenuDelegate;
};




template <>
struct TWidgetTypeTraits<class SWidgetBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Arbitrary Widget MultiBlock widget
 */
class SWidgetBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SWidgetBlock ){}
	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

protected:

	/**
	* Finds the STextBlock that gets displayed in the UI
	*
	* @param Content	Widget to check for an STextBlock
	* @return	The STextBlock widget found
	*/
	SLATE_API TSharedRef<SWidget> FindTextBlockWidget(TSharedRef<SWidget> Content);
};
