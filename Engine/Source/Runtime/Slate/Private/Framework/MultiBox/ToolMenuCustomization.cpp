// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/ToolMenuBase.h"

namespace UE::ToolMenuCustomization::Private
{
	static void HandleToolMenuProfile(FToolMenuProfile& Result, const FToolMenuProfile* Current)
	{
		if (!Current)
		{
			return;
		}

		if (Current->IsSuppressExtenders() && !Result.IsSuppressExtenders())
		{
			Result.SuppressExtenders = Current->SuppressExtenders;
		}

		for (const auto& EntryIterator : Current->Entries)
		{
			if (FCustomizedToolMenuEntry* ExistingEntry = Result.FindEntry(EntryIterator.Key))
			{
				if (EntryIterator.Value.Visibility != ECustomizedToolMenuVisibility::None)
				{
					ExistingEntry->Visibility = EntryIterator.Value.Visibility;
				}
			}
			else
			{
				Result.Entries.Add(EntryIterator.Key, EntryIterator.Value);
			}
		}

		for (const auto& SectionIterator : Current->Sections)
		{
			if (FCustomizedToolMenuSection* ExistingSection = Result.FindSection(SectionIterator.Key))
			{
				if (SectionIterator.Value.Visibility != ECustomizedToolMenuVisibility::None)
				{
					ExistingSection->Visibility = SectionIterator.Value.Visibility;
				}
			}
			else
			{
				Result.Sections.Add(SectionIterator.Key, SectionIterator.Value);
			}
		}

		Result.MenuPermissions.Append(Current->MenuPermissions);

	}
	
	static void HandleToolMenuCustomization(FCustomizedToolMenu& Result, const FCustomizedToolMenu* Current)
	{
		if (Current->SectionOrder.Num() > 0)
		{
			Result.SectionOrder = Current->SectionOrder;
		}

		for (const auto& SectionEntryOrderIterator : Current->EntryOrder)
		{
			Result.EntryOrder.Add(SectionEntryOrderIterator.Key, SectionEntryOrderIterator.Value);
		}
	}
	
}

FCustomizedToolMenuEntry* FToolMenuProfile::FindEntry(const FName InEntryName)
{
	return Entries.Find(InEntryName);
	//return Entries.FindByPredicate([=](const FCustomizedToolMenuEntry& Entry) { return Entry.Name == InEntryName; });
}

const FCustomizedToolMenuEntry* FToolMenuProfile::FindEntry(const FName InEntryName) const
{
	return Entries.Find(InEntryName);
	//return Entries.FindByPredicate([=](const FCustomizedToolMenuEntry& Entry) { return Entry.Name == InEntryName; });
}

FCustomizedToolMenuEntry* FToolMenuProfile::AddEntry(const FName InEntryName)
{
	return &Entries.FindOrAdd(InEntryName);
}

ECustomizedToolMenuVisibility FToolMenuProfile::GetEntryVisiblity(const FName InEntryName) const
{
	if (const FCustomizedToolMenuEntry* Found = FindEntry(InEntryName))
	{
		return Found->Visibility;
	}

	return ECustomizedToolMenuVisibility::None;
}

bool FToolMenuProfile::IsEntryHidden(const FName InEntryName) const
{
	return GetEntryVisiblity(InEntryName) == ECustomizedToolMenuVisibility::Hidden;
}

FCustomizedToolMenuSection* FToolMenuProfile::FindSection(const FName InSectionName)
{
	return Sections.Find(InSectionName);
	//return Sections.FindByPredicate([=](const FCustomizedToolMenuSection& Section) { return Section.Name == InSectionName; });
}

const FCustomizedToolMenuSection* FToolMenuProfile::FindSection(const FName InSectionName) const
{
	return Sections.Find(InSectionName);
	//return Sections.FindByPredicate([=](const FCustomizedToolMenuSection& Section) { return Section.Name == InSectionName; });
}

FCustomizedToolMenuSection* FToolMenuProfile::AddSection(const FName InSectionName)
{
	return &Sections.FindOrAdd(InSectionName);
}

ECustomizedToolMenuVisibility FToolMenuProfile::GetSectionVisiblity(const FName InSectionName) const
{
	if (const FCustomizedToolMenuSection* Found = FindSection(InSectionName))
	{
		return Found->Visibility;
	}

	return ECustomizedToolMenuVisibility::None;
}

bool FToolMenuProfile::IsSectionHidden(const FName InEntryName) const
{
	return GetSectionVisiblity(InEntryName) == ECustomizedToolMenuVisibility::Hidden;
}

void FToolMenuProfile::SetSuppressExtenders(const FName InOwnerName, const bool bInSuppress)
{
	if (bInSuppress)
	{
		SuppressExtenders.AddUnique(InOwnerName);
	}
	else
	{
		SuppressExtenders.Remove(InOwnerName);
	}
}

bool FToolMenuProfile::IsSuppressExtenders() const
{
	return !SuppressExtenders.IsEmpty();
}

FName FCustomizedToolMenu::GetEntrySectionName(const FName InEntryName) const
{
	for (auto& It : EntryOrder)
	{
		if (It.Value.Names.Contains(InEntryName))
		{
			return It.Key;
		}
	}

	return NAME_None;
}

FToolMenuProfile FToolMenuProfileHierarchy::GenerateFlattenedMenuProfile() const
{
	// Process parents first then children
	// Each customization has chance to override what has already been customized before it
	FToolMenuProfile Destination;

	for (const FToolMenuProfile* Current : ProfileHierarchy)
	{
		UE::ToolMenuCustomization::Private::HandleToolMenuProfile(Destination, Current);
	}

	for (const FToolMenuProfile* Current : RuntimeProfileHierarchy)
	{
		UE::ToolMenuCustomization::Private::HandleToolMenuProfile(Destination, Current);
	}

	return Destination;
}

FName FCustomizedToolMenuHierarchy::GetEntrySectionName(const FName InEntryName) const
{
	for (int32 i = Hierarchy.Num() - 1; i >= 0; --i)
	{
		if (Hierarchy[i])
		{
			FName SectionName = Hierarchy[i]->GetEntrySectionName(InEntryName);
			if (SectionName != NAME_None)
			{
				return SectionName;
			}
		}
	}

	return NAME_None;
}

bool FCustomizedToolMenuHierarchy::IsEntryHidden(const FName InEntryName) const
{
	for (int32 i = Hierarchy.Num() - 1; i >= 0; --i)
	{
		if (Hierarchy[i])
		{
			if (const FCustomizedToolMenuEntry* Found = Hierarchy[i]->FindEntry(InEntryName))
			{
				if (Found->Visibility == ECustomizedToolMenuVisibility::Hidden)
				{
					return true;
				}
				else if (Found->Visibility == ECustomizedToolMenuVisibility::Visible)
				{
					return false;
				}
			}
		}
	}

	return false;
}

bool FCustomizedToolMenuHierarchy::IsSectionHidden(const FName InSectionName) const
{
	for (int32 i = Hierarchy.Num() - 1; i >= 0; --i)
	{
		if (Hierarchy[i])
		{
			if (const FCustomizedToolMenuSection* Found = Hierarchy[i]->FindSection(InSectionName))
			{
				if (Found->Visibility == ECustomizedToolMenuVisibility::Hidden)
				{
					return true;
				}
				else if (Found->Visibility == ECustomizedToolMenuVisibility::Visible)
				{
					return false;
				}
			}
		}
	}

	return false;
}

bool FCustomizedToolMenuHierarchy::IsSuppressExtenders() const
{
	for (int32 i = Hierarchy.Num() - 1; i >= 0; --i)
	{
		if (Hierarchy[i])
		{
			if (Hierarchy[i]->IsSuppressExtenders())
			{
				return true;
			}
		}
	}

	return false;
}

FCustomizedToolMenu FCustomizedToolMenuHierarchy::GenerateFlattened() const
{
	static auto HandleCustomizedToolMenu = [](FCustomizedToolMenu& Result, const FCustomizedToolMenu* Current)
	{
		if (!Current)
		{
			return;
		}

		if (Current->IsSuppressExtenders() && !Result.IsSuppressExtenders())
		{
			Result.SuppressExtenders = Current->SuppressExtenders;
		}

		if (Current->SectionOrder.Num() > 0)
		{
			Result.SectionOrder = Current->SectionOrder;
		}

		for (const auto& SectionEntryOrderIterator : Current->EntryOrder)
		{
			Result.EntryOrder.Add(SectionEntryOrderIterator.Key, SectionEntryOrderIterator.Value);
		}

		for (const auto& EntryIterator : Current->Entries)
		{
			if (FCustomizedToolMenuEntry* ExistingEntry = Result.FindEntry(EntryIterator.Key))
			{
				if (EntryIterator.Value.Visibility != ECustomizedToolMenuVisibility::None)
				{
					ExistingEntry->Visibility = EntryIterator.Value.Visibility;
				}
			}
			else
			{
				Result.Entries.Add(EntryIterator.Key, EntryIterator.Value);
			}
		}

		for (const auto& SectionIterator : Current->Sections)
		{
			if (FCustomizedToolMenuSection* ExistingSection = Result.FindSection(SectionIterator.Key))
			{
				if (SectionIterator.Value.Visibility != ECustomizedToolMenuVisibility::None)
				{
					ExistingSection->Visibility = SectionIterator.Value.Visibility;
				}
			}
			else
			{
				Result.Sections.Add(SectionIterator.Key, SectionIterator.Value);
			}
		}

		Result.MenuPermissions.Append(Current->MenuPermissions);
	};

	// Process parents first then children
	// Each customization has chance to override what has already been customized before it
	FCustomizedToolMenu Destination;

	for (const FCustomizedToolMenu* Current : Hierarchy)
	{
		UE::ToolMenuCustomization::Private::HandleToolMenuProfile(Destination, Current);
		UE::ToolMenuCustomization::Private::HandleToolMenuCustomization(Destination, Current);
	}

	for (const FCustomizedToolMenu* Current : RuntimeHierarchy)
	{
		UE::ToolMenuCustomization::Private::HandleToolMenuProfile(Destination, Current);
		UE::ToolMenuCustomization::Private::HandleToolMenuCustomization(Destination, Current);
	}

	return Destination;
}

