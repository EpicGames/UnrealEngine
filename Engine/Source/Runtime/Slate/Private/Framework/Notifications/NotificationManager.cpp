// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Notifications/NotificationManager.h"
#include "Misc/ScopeLock.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SlateNotifications"

namespace NotificationManagerConstants
{
	// Offsets from the bottom-right corner of the anchor area (work area, or parent window when parented)
	const FVector2D NotificationOffset( 15.0f, 20.0f );
}

int32 FSlateNotificationManager::ProgressHandleCounter = 0;


FSlateNotificationManager::FRegionalNotificationList::FRegionalNotificationList(const FSlateRect& InRectangle)
	: Region(InRectangle)
{

}

void FSlateNotificationManager::FRegionalNotificationList::RemoveDeadNotifications()
{
	// Iterate backwards and remove anything that has finished
	for (int32 ListIndex = Notifications.Num() - 1; ListIndex >= 0; --ListIndex)
	{
		if (Notifications[ListIndex]->bDone)
		{
			TSharedPtr<SWindow> PinnedWindow = Notifications[ListIndex]->ParentWindowPtr.Pin();
			if( PinnedWindow.IsValid() )
			{
				PinnedWindow->RequestDestroyWindow();
			}

			Notifications.RemoveAt(ListIndex);
		}
	}
}

void FSlateNotificationManager::FRegionalNotificationList::Arrange()
{
	// If the app has a status bar push up notifications so that they don't overlap the status bar
	static const float StausBarHeight = FAppStyle::Get().GetFloat("StatusBar.Height", nullptr, 0.0f);

	const FVector2D ScaledOffset = NotificationManagerConstants::NotificationOffset;

	// When notifications must be parented (pure Wayland), anchor to the parent window's client rect
	// instead of the monitor work area. Other platforms keep the work-area anchor.
	TSharedPtr<SWindow> PinnedAnchorWindow = AnchorWindow.Pin();
	const bool bAnchorToParent = FPlatformApplicationMisc::RequiresParentedNotifications() && PinnedAnchorWindow.IsValid();
	const FSlateRect AnchorRect = bAnchorToParent ? PinnedAnchorWindow->GetClientRectInScreen() : Region;

	FVector2D AnchorPoint(
		AnchorRect.Right - ScaledOffset.X,
		AnchorRect.Bottom - (ScaledOffset.Y + StausBarHeight));

	for (int32 ListIndex = Notifications.Num() - 1; ListIndex >= 0; --ListIndex)
	{
		TSharedPtr<SWindow> PinnedWindow = Notifications[ListIndex]->ParentWindowPtr.Pin();
		if( PinnedWindow.IsValid() )
		{
			const float StackOffset = ScaledOffset.Y * ((Notifications.Num() - 1) - ListIndex);

#if PLATFORM_MAC
			// Ensure the DesiredSize is up to date to avoid the os detecting the window as moving to right screen
			// On Windows this worsened the animation of the scaling because the content doesn't scale fast enough
			// On Linux the animation doesn't work at all for unknown reason likely related to SDL
			PinnedWindow->SlatePrepass(FSlateApplicationBase::Get().GetApplicationScale() * PinnedWindow->GetDPIScaleFactor());
#endif // PLATFORM_MAC
			const FVector2D DesiredSize = PinnedWindow->GetDesiredSize();

			const FVector2D NewPosition(AnchorPoint.X - DesiredSize.X, AnchorPoint.Y - DesiredSize.Y - StackOffset);

			// Check if the point is inside the anchor area to prevent triggering change screen event
			if (AnchorRect.ContainsPoint(NewPosition))
			{
				if( NewPosition != PinnedWindow->GetPositionInScreen() && DesiredSize != PinnedWindow->GetSizeInScreen())
				{
					PinnedWindow->ReshapeWindow( NewPosition, DesiredSize );
				}
				else if (NewPosition != PinnedWindow->GetPositionInScreen())
				{
					PinnedWindow->MoveWindowTo(NewPosition);
				}
				
				// Only display the notification when it's ready and arranged
				if (!PinnedWindow->IsVisible())
				{
					PinnedWindow->ShowWindow();
				}
			}
			else if (PinnedWindow->IsVisible())
			{
				PinnedWindow->HideWindow();
			}
			
			AnchorPoint.Y -= DesiredSize.Y;
		}
	}
}


FSlateNotificationManager& FSlateNotificationManager::Get()
{
	static FSlateNotificationManager Instance;
	return Instance;
}

FSlateNotificationManager::FSlateNotificationManager() :
	CVarAllowNotifications(TEXT("Slate.bAllowNotifications"),
						   bAllowNotifications,
						   *LOCTEXT("AllowDescription", "If false, then notifications will not be displayed to the user.").ToString())
{
	FCoreDelegates::OnPreExit.AddRaw(this, &FSlateNotificationManager::ShutdownOnPreExit);
}

void FSlateNotificationManager::ShutdownOnPreExit()
{
	FCoreDelegates::OnPreExit.RemoveAll(this);

	RegionalLists.Empty();

	FScopeLock Lock(&StagedNotificationCS);
	StagedNotifications.Empty();

}

void FSlateNotificationManager::SetRootWindow( const TSharedRef<SWindow> InRootWindow )
{
	RootWindowPtr = InRootWindow;
}

TSharedRef<SNotificationList> FSlateNotificationManager::CreateStackForArea(const FSlateRect& InRectangle, TSharedPtr<SWindow> Window)
{
	// Doesn't work, per-pixel transparency on windows is required which is broken
#define ONE_WINDOW_FOR_NOTIFICATIONS 0
#if ONE_WINDOW_FOR_NOTIFICATIONS 
	for (FRegionalNotificationList& List : RegionalLists)
	{
		if (FSlateRect::IsRectangleContained(List.Region, InRectangle))
		{
			return List.Notifications[0];
		}
	}

	TSharedRef<SNotificationList> NewNotificationList = SNew(SNotificationList);
	TSharedRef<SWindow> NotificationWindow = SWindow::MakeNotificationWindow();
	NotificationWindow->SetContent(NewNotificationList);
	NewNotificationList->ParentWindowPtr = NotificationWindow;

	if (Window.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(NotificationWindow, Window.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(NotificationWindow);
	}

	if (!FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		if (NotificationWindow->IsActive() || NotificationWindow->HasActiveParent())
		{
			NotificationWindow->BringToFront();
		}
	}


	FRegionalNotificationList NewList(FSlateApplication::Get().GetWorkArea(InRectangle));
	NewList.Notifications.Add(NewNotificationList);
	RegionalLists.Add(NewList);

	return NewNotificationList;
#else
	TSharedRef<SNotificationList> NotificationList = SNew(SNotificationList);
	TSharedRef<SWindow> NotificationWindow = SWindow::MakeNotificationWindow();
	NotificationWindow->SetContent(NotificationList);
	NotificationList->ParentWindowPtr = NotificationWindow;

	// Prevent tying a notification to a window that is minimized because a minimized window isn't painted
	// and therefore the notification wouldn't be either.

	// Also guard against a parent window whose native OS handle has been destroyed (e.g. during a
	// project switch). AddWindowAsNativeChild skips MakeWindow when GetOSWindowHandle() is null,
	// leaving the notification SWindow without a NativeWindow and causing an ensure in ShowWindow().
	const bool bParentedToWindow = Window.IsValid() && Window->IsVisible() && !Window->IsWindowMinimized()
		&& Window->GetNativeWindow().IsValid() && Window->GetNativeWindow()->GetOSWindowHandle();
	if (bParentedToWindow)
	{
		FSlateApplication::Get().AddWindowAsNativeChild(NotificationWindow, Window.ToSharedRef(), bShowNotificationImmediately);
	}
	else
	{
		FSlateApplication::Get().AddWindow(NotificationWindow, bShowNotificationImmediately);
	}

	if (!FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		if (NotificationWindow->IsActive() || NotificationWindow->HasActiveParent())
		{
			NotificationWindow->BringToFront();
		}
	}

	// When notifications must be parented (pure Wayland), group lists by anchor window rather than
	// work-area rectangle, so toasts sharing a parent stack together. Only when actually parented --
	// a minimized/hidden parent falls back to a top-level window and must anchor to the work area.
	const bool bGroupByAnchorWindow = FPlatformApplicationMisc::RequiresParentedNotifications() && bParentedToWindow;

	bool bFound = false;
	for (FRegionalNotificationList& List : RegionalLists)
	{
		const bool bMatches = bGroupByAnchorWindow
			? (List.AnchorWindow.Pin() == Window)
			: FSlateRect::IsRectangleContained(List.Region, InRectangle);
		if (bMatches)
		{
			List.Notifications.Add(NotificationList);
			bFound = true;
		}
	}

	if (!bFound)
	{
		FRegionalNotificationList NewList(FSlateApplication::Get().GetWorkArea(InRectangle));
		NewList.Notifications.Add(NotificationList);
		if (bGroupByAnchorWindow)
		{
			NewList.AnchorWindow = Window;
		}
		RegionalLists.Add(NewList);
	}

	return NotificationList;
#endif // ONE_WINDOW_FOR_NOTIFICATIONS

	
}

TSharedPtr<SNotificationItem> FSlateNotificationManager::AddNotification(const FNotificationInfo& Info)
{
	checkf(IsInGameThread(), TEXT("FSlateNotificationManager::AddNotification must be called on game thread. Use QueueNotification if necessary."));

	// Early calls of this function can happen before Slate is initialized.
	if( FSlateApplication::IsInitialized() )
	{
		// Parent to ForWindow, else the root window. When notifications must be parented (pure Wayland)
		// with neither set, fall back to the active top-level window; if still none, ParentWindow stays
		// invalid and the notification is created top-level as before.
		TSharedPtr<SWindow> ParentWindow;
		if (Info.ForWindow.IsValid())
		{
			ParentWindow = Info.ForWindow;
		}
		else if (RootWindowPtr.IsValid())
		{
			ParentWindow = RootWindowPtr.Pin();
		}
		else if (FPlatformApplicationMisc::RequiresParentedNotifications())
		{
			ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow();
		}

		FSlateRect PreferredWorkArea;
		if (ParentWindow.IsValid())
		{
			// Retrieve the parent window to display the notification in, otherwise use the preferred work area
			PreferredWorkArea = FSlateApplication::Get().GetWorkArea(ParentWindow->GetRectInScreen());
		}
		else
		{
			PreferredWorkArea = FSlateApplication::Get().GetPreferredWorkArea();
		}

		TSharedRef<SNotificationList> List = CreateStackForArea(PreferredWorkArea, ParentWindow);

		return List->AddNotification(Info);
	}

	return nullptr;
}

void FSlateNotificationManager::QueueNotification(FNotificationInfo* Info)
{
	PendingNotifications.Push(Info);
}

FProgressNotificationHandle FSlateNotificationManager::StartProgressNotification(FText DisplayText, int32 TotalWorkToDo)
{
	if (ProgressNotificationHandler)
	{
		FProgressNotificationHandle NewHandle(++ProgressHandleCounter);
		ProgressNotificationHandler->StartProgressNotification(NewHandle, DisplayText, TotalWorkToDo);

		return NewHandle;
	}

	return FProgressNotificationHandle();
}

void FSlateNotificationManager::UpdateProgressNotification(FProgressNotificationHandle InHandle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo, FText UpdatedDisplayText)
{
	if (ProgressNotificationHandler)
	{
		ProgressNotificationHandler->UpdateProgressNotification(InHandle, TotalWorkDone, UpdatedTotalWorkToDo, UpdatedDisplayText);
	}
}

void FSlateNotificationManager::CancelProgressNotification(FProgressNotificationHandle InHandle)
{
	if (ProgressNotificationHandler)
	{
		ProgressNotificationHandler->CancelProgressNotification(InHandle);
	}
}

void FSlateNotificationManager::SetProgressNotificationHandler(IProgressNotificationHandler* NewHandler)
{
	ProgressNotificationHandler = NewHandler;
}


void FSlateNotificationManager::GetWindows(TArray< TSharedRef<SWindow> >& OutWindows) const
{
	for (const FRegionalNotificationList& RegionList : RegionalLists)
	{
		for (const auto& NotificationList : RegionList.Notifications)
		{
			TSharedPtr<SWindow> PinnedWindow = NotificationList->ParentWindowPtr.Pin();
			if( PinnedWindow.IsValid() )
			{
				OutWindows.Add(PinnedWindow.ToSharedRef());
			}
		}
	}
}

void FSlateNotificationManager::Tick()
{
	// Ensure that the region rectangles still match the screen work areas.
	// This is necessary if the desktop configuration has changed
	for (FRegionalNotificationList& RegionList : RegionalLists)
	{
		RegionList.Region = FSlateApplication::Get().GetWorkArea(RegionList.Region);
	}

	for (;;)
	{
		FNotificationInfo* Notification = PendingNotifications.Pop();
		if (!Notification)
		{
			break;
		}

		AddNotification(*Notification);
		delete Notification;
	}

	// Check notifications to see if any have timed out & need to be removed.
	// We need to do this here as we cant remove their windows in the normal
	// window-tick callstack (as the SlateWindows array gets corrupted)
	
	// We don't need to worry about duplicates here as there is always a unique list per-region
	for (int32 RegionIndex = RegionalLists.Num() - 1; RegionIndex >= 0; --RegionIndex)
	{
		RegionalLists[RegionIndex].RemoveDeadNotifications();

		if (RegionalLists[RegionIndex].Notifications.Num() == 0)
		{
			// It's empty, so remove it
			RegionalLists.RemoveAt(RegionIndex);
		}
		else
		{
			// Arrange the notifications in the list
			RegionalLists[RegionIndex].Arrange();
		}
	}
}

void FSlateNotificationManager::ForceNotificationsInFront( const TSharedRef<SWindow> InWindow )
{
	// check to see if this is a re-entrant call from one of our windows
	for (const auto& RegionList : RegionalLists)
	{
		for (auto& Notification : RegionList.Notifications)
		{
			TSharedPtr<SWindow> PinnedWindow = Notification->ParentWindowPtr.Pin();
			if( PinnedWindow.IsValid() && InWindow == PinnedWindow )
			{
				return;
			}
		}
	}

	const bool IsActiveModelWindowNotValid = !FSlateApplication::Get().GetActiveModalWindow().IsValid();
	// now bring all of our windows back to the front
	for (const auto& RegionList : RegionalLists)
	{
		for (auto& Notification : RegionList.Notifications)
		{
			TSharedPtr<SWindow> PinnedWindow = Notification->ParentWindowPtr.Pin();
			if( PinnedWindow.IsValid() && IsActiveModelWindowNotValid )
			{
				PinnedWindow->BringToFront();
			}
		}
	}
}

void FSlateNotificationManager::RegisterStagedNotification(TSharedPtr<IAsyncTaskNotificationImpl> InNotification)
{
	FScopeLock Lock(&StagedNotificationCS);

	StagedNotifications.Add(InNotification);
}

void FSlateNotificationManager::UnregisterStagedNotification(TSharedPtr<IAsyncTaskNotificationImpl> InNotification)
{
	FScopeLock Lock(&StagedNotificationCS);

	StagedNotifications.Remove(InNotification);
}


#undef LOCTEXT_NAMESPACE
