// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Notifications/SlateAsyncTaskNotificationImpl.h"

#include "Widgets/Notifications/INotificationWidget.h"
#include "Framework/Notifications/NotificationManager.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"

#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "Templates/Atomic.h"

#define LOCTEXT_NAMESPACE "SlateAsyncTaskNotification"

/*
 * FSlateAsyncTaskNotificationImpl
 */

FSlateAsyncTaskNotificationImpl::FSlateAsyncTaskNotificationImpl()
	: PromptAction(EAsyncTaskNotificationPromptAction::None)
{	
}

FSlateAsyncTaskNotificationImpl::~FSlateAsyncTaskNotificationImpl()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}
}

void FSlateAsyncTaskNotificationImpl::Initialize(const FAsyncTaskNotificationConfig& InConfig)
{
	NotificationConfig = InConfig;
	
	// Note: FCoreAsyncTaskNotificationImpl guarantees this is being called from the game thread

	// Initialize the UI if the Notification is not headless
	if (!NotificationConfig.bIsHeadless)
	{
		// Register this as a Staged Notification (to keep 'this' alive until UnregisterStagedNotification is called)
		FSlateNotificationManager::Get().RegisterStagedNotification(AsShared());

		PromptAction = FApp::IsUnattended() ? EAsyncTaskNotificationPromptAction::Unattended : EAsyncTaskNotificationPromptAction::None;
		bCanCancelAttr = InConfig.bCanCancel;
		bKeepOpenOnSuccessAttr = InConfig.bKeepOpenOnSuccess;
		bKeepOpenOnFailureAttr = InConfig.bKeepOpenOnFailure;

		// Set the initial pending state prior to calling Tick to initialize the UI to that state
		CurrentNotificationState = EAsyncTaskNotificationState::None;
		SetPendingNotificationState(State);

		// Create the notification UI
		CreateNotification();

		// Run a Tick to initialize the UI to the initial state
		const bool bContinueTicking = TickNotification(0.0f);
		if (bContinueTicking)
		{
			// Register the ticker to update the notification every frame
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::TickNotification));
		}
	}
	
	// This calls UpdateNotification to update the UI initialized above, 
	// which will happen immediately since bInitializedNotification is false
	FCoreAsyncTaskNotificationImpl::Initialize(InConfig);
	bInitializedNotification = true;
}

void FSlateAsyncTaskNotificationImpl::DestroyNotification()
{
	if (OwningNotification)
	{
		// Perform the normal automatic fadeout
		OwningNotification->ExpireAndFadeout();

		// Release our reference to our owner so that everything can be destroyed
		OwningNotification.Reset();
	}
}

void FSlateAsyncTaskNotificationImpl::CreateNotification()
{
	check(!NotificationConfig.bIsHeadless);

	if (OwningNotification)
	{
		return;
	}

	FNotificationInfo NotificationInfo(FText::GetEmpty());
	NotificationInfo.FadeOutDuration = NotificationConfig.FadeOutDuration;
	NotificationInfo.ExpireDuration = NotificationConfig.ExpireDuration;
	NotificationInfo.FadeInDuration = NotificationConfig.FadeInDuration;
	NotificationInfo.bFireAndForget = false;
	NotificationInfo.bUseThrobber = true;
	NotificationInfo.bUseSuccessFailIcons = true;
	NotificationInfo.Image = NotificationConfig.Icon;

	{
		FNotificationButtonInfo PromptButtonInfo(
			TAttribute<FText>::CreateSP(this, &FSlateAsyncTaskNotificationImpl::GetPromptButtonText),
			FText::GetEmpty(),
			FSimpleDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::OnPromptButtonClicked),
			FNotificationButtonInfo::FVisibilityDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::GetPromptButtonVisibility),
			FNotificationButtonInfo::FIsEnabledDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::IsPromptButtonEnabled)
		);
		NotificationInfo.ButtonDetails.Add(PromptButtonInfo);
	}

	{
		FNotificationButtonInfo CancelButtonInfo(
			LOCTEXT("CancelButton", "Cancel"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::OnCancelButtonClicked),
			FNotificationButtonInfo::FVisibilityDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::GetCancelButtonVisibility),
			FNotificationButtonInfo::FIsEnabledDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::IsCancelButtonEnabled)
		);
		NotificationInfo.ButtonDetails.Add(CancelButtonInfo);
	}

	{
		FNotificationButtonInfo CloseButtonInfo(
			LOCTEXT("CloseButton", "Close"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::OnCloseButtonClicked),
			FNotificationButtonInfo::FVisibilityDelegate::CreateSP(this, &FSlateAsyncTaskNotificationImpl::GetCloseButtonVisibility)
		);
		NotificationInfo.ButtonDetails.Add(CloseButtonInfo);
	}

	OwningNotification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	check(OwningNotification);
}

void FSlateAsyncTaskNotificationImpl::SyncAttributes()
{
	FScopeLock Lock(&AttributesCS);

	bCanCancel = bCanCancelAttr.Get(false);
	bKeepOpenOnSuccess = bKeepOpenOnSuccessAttr.Get(false);
	bKeepOpenOnFailure = bKeepOpenOnFailureAttr.Get(false);
}

void FSlateAsyncTaskNotificationImpl::SetPendingNotificationState(const EAsyncTaskNotificationState InPendingNotificationState)
{
	FScopeLock Lock(&CompletionCS);

	// Set the completion state
	PendingNotificationState = InPendingNotificationState;
}

void FSlateAsyncTaskNotificationImpl::SetCanCancel(const TAttribute<bool>& InCanCancel)
{
	if (!NotificationConfig.bIsHeadless)
	{
		FScopeLock Lock(&AttributesCS);

		bCanCancelAttr = InCanCancel;
	}
}

void FSlateAsyncTaskNotificationImpl::SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess)
{
	if (!NotificationConfig.bIsHeadless)
	{
		FScopeLock Lock(&AttributesCS);

		bKeepOpenOnSuccessAttr = InKeepOpenOnSuccess;
	}
}

void FSlateAsyncTaskNotificationImpl::SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure)
{
	if (!NotificationConfig.bIsHeadless)
	{
		FScopeLock Lock(&AttributesCS);

		bKeepOpenOnFailureAttr = InKeepOpenOnFailure;
	}
}

bool FSlateAsyncTaskNotificationImpl::IsCancelButtonEnabled(SNotificationItem::ECompletionState InState) const
{
	return bCanCancel && PromptAction == EAsyncTaskNotificationPromptAction::None;
}

EVisibility FSlateAsyncTaskNotificationImpl::GetCancelButtonVisibility(SNotificationItem::ECompletionState InState) const
{
	return (bCanCancel && (CurrentNotificationState == EAsyncTaskNotificationState::Pending || CurrentNotificationState == EAsyncTaskNotificationState::Prompt))
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void FSlateAsyncTaskNotificationImpl::OnCancelButtonClicked()
{
	PromptAction = EAsyncTaskNotificationPromptAction::Cancel;
}

bool FSlateAsyncTaskNotificationImpl::IsPromptButtonEnabled(SNotificationItem::ECompletionState InState) const
{
	return PromptAction == EAsyncTaskNotificationPromptAction::None;
}

EVisibility FSlateAsyncTaskNotificationImpl::GetPromptButtonVisibility(SNotificationItem::ECompletionState InState) const
{
	return (!FApp::IsUnattended() && CurrentNotificationState == EAsyncTaskNotificationState::Prompt)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void FSlateAsyncTaskNotificationImpl::OnPromptButtonClicked()
{
	PromptAction = EAsyncTaskNotificationPromptAction::Continue;
}

FText FSlateAsyncTaskNotificationImpl::GetPromptButtonText() const
{
	return PromptButtonText;
}

EVisibility FSlateAsyncTaskNotificationImpl::GetCloseButtonVisibility(SNotificationItem::ECompletionState InState) const
{
	return (!FApp::IsUnattended() && ((bKeepOpenOnSuccess && CurrentNotificationState == EAsyncTaskNotificationState::Success) || (bKeepOpenOnFailure && CurrentNotificationState == EAsyncTaskNotificationState::Failure)))
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void FSlateAsyncTaskNotificationImpl::OnCloseButtonClicked()
{
	if (OwningNotification)
	{
		// Expire the notification immediately and ensure it fades quickly so that clicking the buttons feels responsive
		OwningNotification->SetExpireDuration(0.0f);
		OwningNotification->SetFadeOutDuration(0.5f);
		OwningNotification->ExpireAndFadeout();

		// Release our reference to our owner so that everything can be destroyed
		OwningNotification.Reset();

		// Unregister our ticker now that we're closing
		if (TickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickerHandle);
			TickerHandle.Reset();
		}

		// Unregister the Staged Notification to complete the cleanup
		FSlateNotificationManager::Get().UnregisterStagedNotification(AsShared());
	}
}

void FSlateAsyncTaskNotificationImpl::UpdateNotification()
{
	FCoreAsyncTaskNotificationImpl::UpdateNotification();
	
	if (!NotificationConfig.bIsHeadless)
	{
		// Update the notification UI
		if (OwningNotification)
		{
			if (bInitializedNotification)
			{
				// Slate requires the notification to be updated from the game thread, so we add a one frame ticker for it using the values captured from whichever thread is calling UpdateNotification
				// Note: We also capture OwningNotification as transitioning to a success/fail state can reset this->OwningNotification before UpdateNotificationDeferred runs, which would cause the deferred update to fail if using this->OwningNotification
				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&FSlateAsyncTaskNotificationImpl::UpdateNotificationDeferred, AsWeak(), OwningNotification, TitleText, ProgressText, PromptText, Hyperlink, HyperlinkText));
			}
			else
			{
				// This is the UpdateNotification call made during Initialize
				// We're on the game thread here so can push the initial state directly into the notification
				FSlateAsyncTaskNotificationImpl::UpdateNotificationDeferred(0.0f, AsWeak(), OwningNotification, TitleText, ProgressText, PromptText, Hyperlink, HyperlinkText);
			}
		}

		// Set the pending state in case the notification has to change
		SetPendingNotificationState(State);
	}
}

bool FSlateAsyncTaskNotificationImpl::UpdateNotificationDeferred(float InDeltaTime, TWeakPtr<FSlateAsyncTaskNotificationImpl> WeakThis, TSharedPtr<SNotificationItem> OwningNotification, FText TitleText, FText ProgressText, FText PromptText, FSimpleDelegate Hyperlink, FText HyperlinkText)
{
	OwningNotification->SetText(TitleText);
	OwningNotification->SetSubText(ProgressText);
	OwningNotification->SetHyperlink(Hyperlink, HyperlinkText);

	if (TSharedPtr<FSlateAsyncTaskNotificationImpl> This = WeakThis.Pin())
	{
		This->PromptButtonText = PromptText;
	}

	// We only want this function to tick once
	return false;
}

EAsyncTaskNotificationPromptAction FSlateAsyncTaskNotificationImpl::GetPromptAction() const
{
	if(NotificationConfig.bIsHeadless)
	{
		return EAsyncTaskNotificationPromptAction::Unattended;
	}
	return PromptAction;
}

bool FSlateAsyncTaskNotificationImpl::TickNotification(float InDeltaTime)
{
	SyncAttributes();
	
	TOptional<EAsyncTaskNotificationState> NextNotificationState;
	{
		FScopeLock Lock(&CompletionCS);

		NextNotificationState = MoveTemp(PendingNotificationState);
		PendingNotificationState.Reset();
	}

	// Update the notification UI state if the task state changed
	if (NextNotificationState.IsSet() && CurrentNotificationState != NextNotificationState.GetValue())
	{
		CurrentNotificationState = NextNotificationState.GetValue();

		if (OwningNotification)
		{
			SNotificationItem::ECompletionState OwningNotificationState = SNotificationItem::CS_None;
			switch (CurrentNotificationState)
			{
			case EAsyncTaskNotificationState::Pending:
				OwningNotificationState = SNotificationItem::CS_Pending;
				break;
			case EAsyncTaskNotificationState::Failure:
				OwningNotificationState = SNotificationItem::CS_Fail;
				break;
			case EAsyncTaskNotificationState::Success:
				OwningNotificationState = SNotificationItem::CS_Success;
				break;
			case EAsyncTaskNotificationState::Prompt:
				OwningNotification->Pulse(FLinearColor(0.f, 0.f, 1.f));
				break;
			}
			OwningNotification->SetCompletionState(OwningNotificationState);
		}
		
		// Reset the `PromptAction` state when changing notification state
		PromptAction = FApp::IsUnattended() ? EAsyncTaskNotificationPromptAction::Unattended : EAsyncTaskNotificationPromptAction::None;
	}

	// If we completed and we aren't keeping the notification open (which will show the Close button), then expire the notification immediately
	{
		const SNotificationItem::ECompletionState OwningNotificationState = OwningNotification ? OwningNotification->GetCompletionState() : SNotificationItem::CS_None;
		if ((CurrentNotificationState == EAsyncTaskNotificationState::Success || CurrentNotificationState == EAsyncTaskNotificationState::Failure) && GetCloseButtonVisibility(OwningNotificationState) == EVisibility::Collapsed)
		{
			DestroyNotification();
			TickerHandle.Reset(); // Reset this before potentially destroying 'this' when calling UnregisterStagedNotification

			FSlateNotificationManager::Get().UnregisterStagedNotification(AsShared());
			return false; // No longer need to Tick
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
