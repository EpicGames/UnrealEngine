// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreAsyncTaskNotificationImpl.h"
#include "Containers/Ticker.h"
#include "Layout/Visibility.h"
#include "Widgets/Notifications/SNotificationList.h"

class SSlateAsyncTaskNotificationWidget;

/**
 * Slate asynchronous task notification that uses a notification item.
 */
class FSlateAsyncTaskNotificationImpl : public TSharedFromThis<FSlateAsyncTaskNotificationImpl, ESPMode::ThreadSafe>,  public FCoreAsyncTaskNotificationImpl
{
public:
	//~ IAsyncTaskNotificationImpl
	virtual void Initialize(const FAsyncTaskNotificationConfig& InConfig) override;
	virtual void SetCanCancel(const TAttribute<bool>& InCanCancel) override;
	virtual void SetKeepOpenOnSuccess(const TAttribute<bool>& InKeepOpenOnSuccess) override;
	virtual void SetKeepOpenOnFailure(const TAttribute<bool>& InKeepOpenOnFailure) override;
	virtual EAsyncTaskNotificationPromptAction GetPromptAction() const override;

	FSlateAsyncTaskNotificationImpl();
	virtual ~FSlateAsyncTaskNotificationImpl() override;
	
private:
	//~ FCoreAsyncTaskNotificationImpl
	virtual void UpdateNotification() override;

	/** Run every frame on the game thread to update the notification */
	bool TickNotification(float InDeltaTime);

	/** Sync attribute bindings with the cached values (once per-frame from the game thread) */
	void SyncAttributes();

	/** Set the pending state of the notification UI (applied during the next Tick) */
	void SetPendingNotificationState(const EAsyncTaskNotificationState InPendingCompletionState);

	/** Cancel button */
	bool IsCancelButtonEnabled(SNotificationItem::ECompletionState InState) const;
	EVisibility GetCancelButtonVisibility(SNotificationItem::ECompletionState InState) const;
	void OnCancelButtonClicked();

	/** Prompt button */
	bool IsPromptButtonEnabled(SNotificationItem::ECompletionState InState) const;
	EVisibility GetPromptButtonVisibility(SNotificationItem::ECompletionState InState) const;
	void OnPromptButtonClicked();
	FText GetPromptButtonText() const;

	/** Close button */
	EVisibility GetCloseButtonVisibility(SNotificationItem::ECompletionState InState) const;
	void OnCloseButtonClicked();

	/* Create the notification, if not already created */
	void CreateNotification();

	/* Cleanly destroy the current notification, if any */
	void DestroyNotification();

	/* Static function to update the notification from the main thread */
	static bool UpdateNotificationDeferred(float InDeltaTime, TWeakPtr<FSlateAsyncTaskNotificationImpl> WeakThis, TSharedPtr<SNotificationItem> OwningNotification, FText TitleText, FText ProgressText, FText PromptText, FSimpleDelegate Hyperlink, FText HyperlinkText);

private:

	/** The Config used for all notifications */
	FAsyncTaskNotificationConfig NotificationConfig;

	/** Handle for TickNotification() */
	FTSTicker::FDelegateHandle TickerHandle;

	/** Action taken for the task, resets to none on notification state change. */
	TAtomic<EAsyncTaskNotificationPromptAction> PromptAction;

	/** The text used by the notification prompt button (if any; UI should query this rather than PromptText) */
	FText PromptButtonText;

	/** Can this task be canceled? Will show a cancel button for in-progress tasks */
	TAttribute<bool> bCanCancelAttr = false;
	bool bCanCancel = false;

	/** Keep this notification open on success? Will show a close button */
	TAttribute<bool> bKeepOpenOnSuccessAttr = false;
	bool bKeepOpenOnSuccess = false;

	/** Keep this notification open on failure? Will show an close button */
	TAttribute<bool> bKeepOpenOnFailureAttr = false;
	bool bKeepOpenOnFailure = false;

	/** Have we finished initializing the notification? */
	bool bInitializedNotification = false;

	/** The current state of the notification (UI should query this rather than State) */
	EAsyncTaskNotificationState CurrentNotificationState = EAsyncTaskNotificationState::None;

	/** The pending state of the notification (if any, applied during the next Tick) */
	TOptional<EAsyncTaskNotificationState> PendingNotificationState;

	/** Pointer to the notification item that owns this widget (this is a deliberate reference cycle as we need this object alive until we choose to expire it, at which point we release our reference to allow everything to be destroyed) */
	TSharedPtr<SNotificationItem> OwningNotification;

	/** Critical section preventing concurrent access to the attributes */
	FCriticalSection AttributesCS;

	/** Critical section preventing the game thread from completing this widget while another thread is in the progress of setting the completion state and cleaning up its UI references */
	FCriticalSection CompletionCS;
};
