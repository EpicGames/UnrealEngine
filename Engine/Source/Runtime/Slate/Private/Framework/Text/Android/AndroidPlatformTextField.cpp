// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/Android/AndroidPlatformTextField.h"

#if USE_ANDROID_JNI
#include "HAL/IConsoleManager.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"

// Java InputType class
#define TYPE_CLASS_TEXT						0x00000001
#define TYPE_CLASS_NUMBER					0x00000002

// Java InputType number flags
#define TYPE_NUMBER_FLAG_SIGNED				0x00001000
#define TYPE_NUMBER_FLAG_DECIMAL			0x00002000

// Java InputType text variation flags
#define TYPE_TEXT_VARIATION_EMAIL_ADDRESS	0x00000020
#define TYPE_TEXT_VARIATION_NORMAL			0x00000000
#define TYPE_TEXT_VARIATION_PASSWORD		0x00000080
#define TYPE_TEXT_VARIATION_URI				0x00000010

// Java InputType text flags
#define TYPE_TEXT_FLAG_NO_SUGGESTIONS		0x00080001
#define TYPE_TEXT_FLAG_MULTI_LINE			0x00020000
#define TYPE_TEXT_FLAG_AUTO_CORRECT			0x00008000

int32 GAndroidNewKeyboard = 0;
static FAutoConsoleVariableRef CVarAndroidNewKeyboard(
	TEXT("Android.NewKeyboard"),
	GAndroidNewKeyboard,
	TEXT("Controls usage of experimental new keyboard input. 0 uses the checkbox setting, 1 forces new keyboard, 2 forces dialog. (Default: 0)"),
	ECVF_Default );

void FAndroidPlatformTextField::ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget)
{
	// Set the EditBox inputType based on keyboard type
	int32 InputType;
	
	if (bShow)
	{
		switch (TextEntryWidget->GetVirtualKeyboardType())
		{
			case EKeyboardType::Keyboard_Number:
				InputType = TYPE_CLASS_NUMBER | TYPE_NUMBER_FLAG_SIGNED | TYPE_NUMBER_FLAG_DECIMAL;
				break;
			case EKeyboardType::Keyboard_Web:
				InputType = TYPE_CLASS_TEXT | TYPE_TEXT_VARIATION_URI;
				break;
			case EKeyboardType::Keyboard_Email:
				InputType = TYPE_CLASS_TEXT | TYPE_TEXT_VARIATION_EMAIL_ADDRESS;
				break;
			case EKeyboardType::Keyboard_Password:
				InputType = TYPE_CLASS_TEXT | TYPE_TEXT_VARIATION_PASSWORD;
				break;
			case EKeyboardType::Keyboard_AlphaNumeric:
			case EKeyboardType::Keyboard_Default:
			default:
				InputType = TYPE_CLASS_TEXT | TYPE_TEXT_VARIATION_NORMAL;
				break;
		}
		
		// Do not make suggestions as user types
		InputType |= TYPE_TEXT_FLAG_NO_SUGGESTIONS;

		// Ensure that the autocorrect flag is turned on or off based on project settings
		if (ShouldUseAutocorrect(TextEntryWidget))
		{
			InputType |= TYPE_TEXT_FLAG_AUTO_CORRECT;
		}
		else
		{
			InputType &= ~TYPE_TEXT_FLAG_AUTO_CORRECT;
		}
	}

	bool bIsUsingIntegratedKeyboard = EnableNewKeyboardConfig();
	switch (GAndroidNewKeyboard)
	{
	case 1:
		bIsUsingIntegratedKeyboard = true;
		break;

	case 2:
		bIsUsingIntegratedKeyboard = false;
		break;

	default:
		break;
	}

	if (bIsUsingIntegratedKeyboard)
	{
		if (bShow)
		{
			// Show alert for input
			if (TextEntryWidget->IsMultilineEntry())
				InputType |= TYPE_TEXT_FLAG_MULTI_LINE;
			extern void AndroidThunkCpp_ShowVirtualKeyboardInput(TSharedPtr<IVirtualKeyboardEntry>, int32, const FString&, const FString&);
			AndroidThunkCpp_ShowVirtualKeyboardInput(TextEntryWidget, InputType, TextEntryWidget->GetHintText().ToString(), TextEntryWidget->GetText().ToString());
		}
		else
		{
			extern void AndroidThunkCpp_HideVirtualKeyboardInput();
			AndroidThunkCpp_HideVirtualKeyboardInput();
		}
	}
	else
	{
		if (bShow)
		{
			// Show alert for input
			extern void AndroidThunkCpp_ShowVirtualKeyboardInputDialog(TSharedPtr<IVirtualKeyboardEntry>, int32, const FString&, const FString&);
			AndroidThunkCpp_ShowVirtualKeyboardInputDialog(TextEntryWidget, InputType, TextEntryWidget->GetHintText().ToString(), TextEntryWidget->GetText().ToString());
		}
		else
		{
			extern void AndroidThunkCpp_HideVirtualKeyboardInputDialog();
			AndroidThunkCpp_HideVirtualKeyboardInputDialog();
		}
	}
}

bool FAndroidPlatformTextField::AllowMoveCursor()
{
	return !EnableNewKeyboardConfig();
}

bool FAndroidPlatformTextField::ShouldUseAutocorrect(TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) const
{
	bool bUseAutocorrect = IPlatformTextField::ShouldUseVirtualKeyboardAutocorrect(TextEntryWidget);

	if (bUseAutocorrect)
	{
		static FString DeviceModel = FAndroidMisc::GetDeviceModel();

		TArray<FString> ExcludedDeviceModels;
		GConfig->GetArray(TEXT("/Script/Engine.InputSettings"), TEXT("ExcludedAutocorrectDeviceModels"), ExcludedDeviceModels, GInputIni);

		for (const FString& ExcludedModel : ExcludedDeviceModels)
		{
			if (DeviceModel.StartsWith(ExcludedModel))
			{
				bUseAutocorrect = false;
				break;
			}
		}
	}

	return bUseAutocorrect;
}

bool FAndroidPlatformTextField::EnableNewKeyboardConfig() const
{
	// read the value from the config file
	static bool bEnableNewKeyboardConfig = false;
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bEnableNewKeyboard"), bEnableNewKeyboardConfig, GEngineIni);

	// use integrated keyboard if the runtime setting is set or the console variable is set to 1
	return bEnableNewKeyboardConfig;
}
#endif