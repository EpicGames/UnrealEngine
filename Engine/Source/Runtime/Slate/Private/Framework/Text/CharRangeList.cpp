// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/CharRangeList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CharRangeList)

namespace CharRangeConstants
{
	static const FString EscapedCharacters = "-\\";
	static const TCHAR EscapeCharacter = TEXT('\\');
	static const TCHAR RangeSeparator = TEXT('-');
}

bool FCharRange::InitializeFromString(const FString& InDefinitionString, int32 InFromIndex, int32* OutLastIndex)
{
	int32 Length = InDefinitionString.Len();
	int32 Index = InFromIndex;
	TCHAR Char;

	// Parse the first character (and last if this is a single character range).
	Char = InDefinitionString[Index++];
	if (Char != CharRangeConstants::EscapeCharacter)
	{
		First = Char;
	}
	else
	{
		if (Index < Length)
		{
			First = InDefinitionString[Index++];
		}
		else
		{
			// Error, stray escape character found at the end.
			*OutLastIndex = Index;
			return false;
		}
	}

	Last = First;

	// Parse the range's last character, if present.
	if ((Index < Length) && (InDefinitionString[Index] == CharRangeConstants::RangeSeparator))
	{
		Index++;

		Char = InDefinitionString[Index++];
		if (Char != CharRangeConstants::EscapeCharacter)
		{
			Last = Char;
		}
		else
		{
			if (Index < Length)
			{
				Last = InDefinitionString[Index++];
			}
			else
			{
				// Error, stray escape character found at the end.
				*OutLastIndex = Index;
				return false;
			}
		}
	}

	*OutLastIndex = Index;
	return true;
}

FString FCharRange::ToString() const
{
	FString Result;

	TCHAR FirstChar = (TCHAR)First;
	TCHAR LastChar = (TCHAR)Last;
	int32 Unused = 0;

	// First character.
	if (CharRangeConstants::EscapedCharacters.FindChar(FirstChar, Unused))
	{
		Result.AppendChar(CharRangeConstants::EscapeCharacter);
	}
	Result.AppendChar(FirstChar);

	// Is this a multi-character range?
	if (Last > First)
	{
		Result.AppendChar(CharRangeConstants::RangeSeparator);

		// Last character.
		if (CharRangeConstants::EscapedCharacters.FindChar(LastChar, Unused))
		{
			Result.AppendChar(CharRangeConstants::EscapeCharacter);
		}
		Result.AppendChar(LastChar);
	}

	return Result;
}

bool FCharRangeList::InitializeFromString(const FString& InDefinitionString)
{
	Ranges.Empty();

	int32 Length = InDefinitionString.Len();
	int32 Index = 0;
	int32 NextIndex;
	FCharRange NewRange;
	while (Index < Length)
	{
		if (!NewRange.InitializeFromString(InDefinitionString, Index, &NextIndex))
		{
			return false;
		}

		Ranges.Add(NewRange);
		Index = NextIndex;
	}

	return true;
}

FString FCharRangeList::ToString() const
{
	FString Result;
	for (const FCharRange& Range : Ranges)
	{
		Result.Append(Range.ToString());
	}

	return Result;
}

void FCharRangeList::Empty()
{
	Ranges.Empty();
}

bool FCharRangeList::IsEmpty() const
{
	return Ranges.IsEmpty();
}

bool FCharRangeList::IsCharIncluded(TCHAR InChar) const
{
	for (const FCharRange& Range : Ranges)
	{
		if ((InChar >= Range.First) && (InChar <= Range.Last))
		{
			return true;
		}
	}

	return false;
}

bool FCharRangeList::AreAllCharsIncluded(const FString& InString) const
{
	for (TCHAR Char : InString)
	{
		if (!IsCharIncluded(Char))
		{
			return false;
		}
	}

	return true;
}

TSet<TCHAR> FCharRangeList::FindCharsNotIncluded(const FString& InString) const
{
	TSet<TCHAR> InvalidChars;
	for (TCHAR Char : InString)
	{
		if (!IsCharIncluded(Char))
		{
			InvalidChars.Add(Char);
		}
	}

	return InvalidChars;
}
