// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/IOS/IOSPlatformTextField.h"
#include "Apple/AppleStringUtils.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSAsyncTask.h"
#include "IOS/IOSView.h"
#include "Misc/ConfigCacheIni.h"
#include <atomic>
#include "Widgets/Input/IVirtualKeyboardEntry.h"

static int32 GIOSTextFieldImplementation = 0;
FAutoConsoleVariableRef CVarIOSTextFieldImplementation(
		TEXT("ios.TextFieldImplementation"),
		GIOSTextFieldImplementation,
		TEXT("iOS text field implementation. 0 = UIAlertController popup (default), 1 = floating text field above keyboard, 2 = force integrated keyboard."),
		ECVF_Default);

extern std::atomic<float> GIOSVirtualKeyboardInputFieldHeight;

namespace
{
	void GetKeyboardConfig(TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget, FKeyboardConfig& KeyboardConfig)
	{
		bool bUseAutocorrect = IPlatformTextField::ShouldUseVirtualKeyboardAutocorrect(TextEntryWidget);

		KeyboardConfig.KeyboardType = UIKeyboardTypeDefault;
		KeyboardConfig.bSecureTextEntry = NO;
		KeyboardConfig.AutocorrectionType = bUseAutocorrect ? UITextAutocorrectionTypeYes : UITextAutocorrectionTypeNo;

		EKeyboardType TargetKeyboardType = TextEntryWidget.IsValid() ? TextEntryWidget->GetVirtualKeyboardType() : Keyboard_Default;
		
		switch (TargetKeyboardType)
		{
		case EKeyboardType::Keyboard_Email:
			KeyboardConfig.KeyboardType = UIKeyboardTypeEmailAddress;
			break;
		case EKeyboardType::Keyboard_Number:
			KeyboardConfig.KeyboardType = UIKeyboardTypeDecimalPad;
			break;
		case EKeyboardType::Keyboard_Web:
			KeyboardConfig.KeyboardType = UIKeyboardTypeURL;
			break;
		case EKeyboardType::Keyboard_AlphaNumeric:
			KeyboardConfig.KeyboardType = UIKeyboardTypeASCIICapable;
			break;
		case EKeyboardType::Keyboard_Password:
			KeyboardConfig.bSecureTextEntry = YES;
		case EKeyboardType::Keyboard_Default:
		default:
			KeyboardConfig.KeyboardType = UIKeyboardTypeDefault;
			break;
		}
	}
}

FIOSPlatformTextField::FIOSPlatformTextField()
	: TextField( nullptr )
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	, TextField2( nullptr )
#endif
{
}

FIOSPlatformTextField::~FIOSPlatformTextField()
{
	if(TextField != nullptr)
	{
        UE_LOGF(LogIOS, Log, "Deleting text field: %p", TextField);
        SlateTextField* LocalTextField = TextField;
        TextField = nullptr;
		dispatch_async(dispatch_get_main_queue(), ^{
            NSLog(@"Finally releasing text field %@", LocalTextField);
			if (LocalTextField != nullptr && [LocalTextField respondsToSelector:@selector(hide:)])
			{
            	[LocalTextField hide];
			}
		});
	}
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	if (TextField2 != nullptr)
	{
        UE_LOGF(LogIOS, Log, "Deleting text field2: %p", TextField2);
        SlateTextField2* LocalTextField2 = TextField2;
        TextField2 = nullptr;
		dispatch_async(dispatch_get_main_queue(), ^{
            NSLog(@"Finally releasing text field2 %@", LocalTextField2);
			if (LocalTextField2 != nullptr && [LocalTextField2 respondsToSelector:@selector(hide)])
			{
            	[LocalTextField2 hide];
				[LocalTextField2 release];
			}
		});
	}
#endif
}

bool FIOSPlatformTextField::AllowMoveCursor()
{
	// OS field owns the cursor since changes in slate would not be replicated in the text field
	return GIOSTextFieldImplementation != 1;
}

void FIOSPlatformTextField::ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget)
{
	FIOSView* View = [IOSAppDelegate GetDelegate].IOSView;

	bool bAllowWidgetEnablingIntegratedKeyboard = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bAllowWidgetEnablingIntegratedKeyboard"), bAllowWidgetEnablingIntegratedKeyboard, GEngineIni);

	bool bShouldUseIntegratedKeyboardGlobally = [View ShouldUseIntegratedKeyboard];

	// Prevent any changes to View->bIsUsingIntegratedKeyboard in the event bShouldUseIntegratedKeyboardGlobally is enabled as it is already intialized during the KeyboardInitialization and only needs to be done once throughout the app lifetime.
	// bShouldUseIntegratedKeyboardGlobally should always have priority over individually toggled asset (bAllowWidgetEnablingIntegratedKeyboard).
	if (bShow && !bShouldUseIntegratedKeyboardGlobally && bAllowWidgetEnablingIntegratedKeyboard)
	{
		View->bIsUsingIntegratedKeyboard = TextEntryWidget.IsValid() ? TextEntryWidget->IsIntegratedKeyboardEnabled() : false;
	}
	 
	if (View->bIsUsingIntegratedKeyboard || (GIOSTextFieldImplementation == 2))
	{
		if (bShow)
		{
			FKeyboardConfig KeyboardConfig;
			GetKeyboardConfig(TextEntryWidget, KeyboardConfig);
			
			[View ActivateKeyboard:false keyboardConfig:KeyboardConfig];
		}
		else
		{
			[View DeactivateKeyboard];
		}
	}
	else
	{
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
		const bool bUseNewImpl = (GIOSTextFieldImplementation == 1);
#else
		const bool bUseNewImpl = false;
#endif

		if (bShow)
		{
			// capture some gamethread strings before we toss over to main thread
			NSString* TextContents = [NSString stringWithFString : TextEntryWidget->GetText().ToString()];
			NSString* PlaceholderContents = [NSString stringWithFString : TextEntryWidget->GetHintText().ToString()];
			FKeyboardConfig KeyboardConfig;
			GetKeyboardConfig(TextEntryWidget, KeyboardConfig);

			// these functions must be run on the main thread
			if (bUseNewImpl)
			{
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
				dispatch_async(dispatch_get_main_queue(), ^{
					if (TextField2 == nullptr)
					{
						TextField2 = [[SlateTextField2 alloc] init];
					}
					[TextField2 show:TextEntryWidget text:TextContents placeholder:PlaceholderContents keyboardConfig:KeyboardConfig];
				});
#endif
			}
			else
			{
				dispatch_async(dispatch_get_main_queue(), ^{
					if (TextField == nullptr)
					{
						TextField = [[[SlateTextField alloc] init] retain];
					}
					[TextField show:TextEntryWidget text:TextContents placeholder:PlaceholderContents keyboardConfig:KeyboardConfig];
				});
			}
		}
        else
        {
#if !PLATFORM_TVOS
			if (bUseNewImpl)
			{
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
				if (TextField2 != nullptr && [TextField2 hasTextWidget])
				{
	                UE_LOGF(LogIOS, Log, "Hiding field2: %p", TextField2);
	                SlateTextField2* LocalTextField2 = TextField2;
	                dispatch_async(dispatch_get_main_queue(), ^{
	                    NSLog(@"Finally hiding text field2 %@", LocalTextField2);
	                    if (LocalTextField2 != nullptr)
	                    {
	                        [LocalTextField2 hide];
	                    }
	                });
				}
#endif
			}
			else
			{
				if (TextField != nullptr && [TextField hasTextWidget])
				{
	                UE_LOGF(LogIOS, Log, "Hiding field: %p", TextField);
	                SlateTextField* LocalTextField = TextField;
	                dispatch_async(dispatch_get_main_queue(), ^{
	                    NSLog(@"Finally releasing text field %@", LocalTextField);
	                    if (LocalTextField != nullptr)
	                    {
	                        [LocalTextField hide];
	                    }
	                });
				}
			}
#endif
        }
	}
}

@implementation SlateTextField

-(id)init
{
	self = [super init];
	
	if (self)
	{
		self->AlertController = nil;
        self->bTransitioning = false;
        self->bWantsToShow = false;
        self->CachedTextContents = nil;
        self->CachedPlaceholderContents = nil;
	}
	
	return self;
}

-(void)hide
{
	bWantsToShow = false;
	if(CachedTextContents != nil)
	{
		[CachedTextContents release];
		CachedTextContents = nil;
	}
	if(CachedPlaceholderContents != nil)
	{
		[CachedPlaceholderContents release];
		CachedPlaceholderContents = nil;
	}
	
	if(!bTransitioning)
	{
		if(AlertController != nil)
		{
			if ([AlertController respondsToSelector:@selector(dismissViewControllerAnimated: completion:)])
			{
				bTransitioning = true;
				[AlertController dismissViewControllerAnimated: YES completion : ^(){
					bTransitioning = false;
					[self updateToDesiredState];
				}];
				AlertController = nil;
			}
			else
			{
				UE_LOGF(LogTemp, Log, "AlertController didn't support needed selector");
			}
		}

		TextWidget = nullptr;
	}
}

-(bool)hasTextWidget
{
	return TextWidget.IsValid();
}

-(void)show:(TSharedPtr<IVirtualKeyboardEntry>)InTextWidget text:(NSString*)TextContents placeholder:(NSString*)PlaceholderContents keyboardConfig:(FKeyboardConfig)KeyboardConfig
{
	TextWidget = InTextWidget;
	TextEntry = FText::FromString(TEXT(""));
	if(CachedTextContents != nil && CachedTextContents != TextContents)
	{
        [CachedTextContents release];
	}
	if(CachedPlaceholderContents != nil && CachedPlaceholderContents != PlaceholderContents)
	{
		[CachedPlaceholderContents release];
	}

	if(CachedTextContents != TextContents)
	{
		CachedTextContents = [[NSString alloc] initWithString:TextContents];
	}
	if(CachedPlaceholderContents != PlaceholderContents)
	{
		CachedPlaceholderContents = [[NSString alloc] initWithString:PlaceholderContents];
	}

	CachedKeyboardConfig = KeyboardConfig;
	bWantsToShow = true;

	if(AlertController == nil && !bTransitioning)
	{
		AlertController = [UIAlertController alertControllerWithTitle : @"" message:@"" preferredStyle:UIAlertControllerStyleAlert];
		UIAlertAction* okAction = [UIAlertAction
										actionWithTitle:NSLocalizedString(@"OK", nil)
										style:UIAlertActionStyleDefault
										handler:^(UIAlertAction* action)
										{
											if ([AlertController respondsToSelector:@selector(dismissViewControllerAnimated: completion:)])
											{
												bTransitioning = true;
												[AlertController dismissViewControllerAnimated : YES completion : ^(){
													bTransitioning = false;
													[self updateToDesiredState];
												}];

												UITextField* AlertTextField = AlertController.textFields.firstObject;
												TextEntry = FText::FromString(AlertTextField.text);
												AlertController = nil;
											
												FAppleAsyncTask* AsyncTask = [[FAppleAsyncTask alloc] init];
												AsyncTask.GameThreadCallback = ^ bool(void)
												{
													if(TextWidget.IsValid())
													{
														TSharedPtr<IVirtualKeyboardEntry> TextEntryWidgetPin = TextWidget.Pin();
														TextEntryWidgetPin->SetTextFromVirtualKeyboard(TextEntry, ETextEntryType::TextEntryAccepted);
													}

													// clear the TextWidget
													TextWidget = nullptr;
													return true;
												};
												[AsyncTask FinishedTask];
											}
											else
											{
												TextWidget = nullptr;
												UE_LOGF(LogTemp, Log, "AlertController didn't support needed selector");
											}
										}
		];
		UIAlertAction* cancelAction = [UIAlertAction
										actionWithTitle: NSLocalizedString(@"Cancel", nil)
										style:UIAlertActionStyleDefault
										handler:^(UIAlertAction* action)
										{
											if ([AlertController respondsToSelector:@selector(dismissViewControllerAnimated: completion:)])
											{
												bTransitioning = true;
												[AlertController dismissViewControllerAnimated : YES completion : ^(){
													bTransitioning = false;
													[self updateToDesiredState];
												}];
												AlertController = nil;
											
												FAppleAsyncTask* AsyncTask = [[FAppleAsyncTask alloc] init];
												AsyncTask.GameThreadCallback = ^ bool(void)
												{
													// clear the TextWidget
													TextWidget = nullptr;
													return true;
												};
												[AsyncTask FinishedTask];
											}
											else
											{
												TextWidget = nullptr;
												UE_LOGF(LogTemp, Log, "AlertController didn't support needed selector");
											}
										}
		];

		[AlertController addAction: okAction];
		[AlertController addAction: cancelAction];
		[AlertController
						addTextFieldWithConfigurationHandler:^(UITextField* AlertTextField)
						{
							AlertTextField.clearsOnBeginEditing = NO;
							AlertTextField.clearsOnInsertion = NO;
							if (TextWidget.IsValid())
							{
								AlertTextField.text = TextContents;
								AlertTextField.placeholder = PlaceholderContents;
								AlertTextField.keyboardType = KeyboardConfig.KeyboardType;
								AlertTextField.autocorrectionType = KeyboardConfig.AutocorrectionType;
								AlertTextField.autocapitalizationType = KeyboardConfig.AutocapitalizationType;
								AlertTextField.secureTextEntry = KeyboardConfig.bSecureTextEntry;
							}
						}
		];
		
		bTransitioning = true;
		[[IOSAppDelegate GetDelegate].IOSController presentViewController : AlertController animated : YES completion : ^(){
			bTransitioning = false;
			[self updateToDesiredState];
		}];
	}
}

-(void)updateToDesiredState
{
	if(bWantsToShow)
	{
		if(TextWidget.IsValid())
		{
			[self show:TextWidget.Pin() text:CachedTextContents placeholder:CachedPlaceholderContents keyboardConfig:CachedKeyboardConfig];
		}
	}
	else
	{
		[self hide];
	}
}

@end

#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
@implementation SlateTextField2

-(id)init
{
	self = [super init];

	if (self)
	{
		CGSize ScreenSize = [UIScreen mainScreen].bounds.size;
		const CGFloat TextFieldHeight = 46.0f;

		InputTextField = [[UITextField alloc] initWithFrame:CGRectMake(0, ScreenSize.height, ScreenSize.width, TextFieldHeight)];
		InputTextField.hidden = YES;
		InputTextField.borderStyle = UITextBorderStyleNone;
		InputTextField.backgroundColor = [[UIColor secondarySystemBackgroundColor] colorWithAlphaComponent:0.98];
		InputTextField.layer.cornerRadius = 19.0f;
		InputTextField.layer.masksToBounds = YES;
		InputTextField.layer.borderWidth = 1.0f;
		InputTextField.layer.borderColor = [UIColor systemGray4Color].CGColor;
		InputTextField.returnKeyType = UIReturnKeyDone;
		InputTextField.clearButtonMode = UITextFieldViewModeWhileEditing;
		InputTextField.delegate = self;
		[InputTextField addTarget:self action:@selector(onTextChanged:) forControlEvents:UIControlEventEditingChanged];

		UIView* LeftSpacer = [[UIView alloc] initWithFrame:CGRectMake(0, 0, 12, 1)];
		InputTextField.leftView = LeftSpacer;
		InputTextField.leftViewMode = UITextFieldViewModeAlways;
		[LeftSpacer release];

		// Register keyboard frame notification to position the text field above the keyboard
		[[NSNotificationCenter defaultCenter]
			addObserver:self
			selector:@selector(keyboardWillShow:)
			name:UIKeyboardWillShowNotification
			object:nil];
	}

	return self;
}

-(void)dealloc
{
	[[NSNotificationCenter defaultCenter] removeObserver:self name:UIKeyboardWillShowNotification object:nil];

	[InputTextField removeFromSuperview];
	[InputTextField release];
	InputTextField = nil;

	[super dealloc];
}

-(void)keyboardWillShow:(NSNotification*)Notification
{
	if (!TextWidget.IsValid())
	{
		return;
	}

	NSDictionary* Info = [Notification userInfo];
	CGRect KBFrame = [[Info objectForKey:UIKeyboardFrameEndUserInfoKey] CGRectValue];

	CGFloat FieldHeight = InputTextField.frame.size.height;
	CGFloat ScreenWidth = [UIScreen mainScreen].bounds.size.width;

	const CGFloat InnerInset = 8.0f;
	const CGFloat BottomInset = 4.0f;
	UIEdgeInsets SafeArea = [IOSAppDelegate GetDelegate].IOSController.view.safeAreaInsets;
	CGFloat LeftInset = SafeArea.left + InnerInset;
	CGFloat RightInset = SafeArea.right + InnerInset;

	InputTextField.frame = CGRectMake(LeftInset, KBFrame.origin.y - FieldHeight - BottomInset, ScreenWidth - LeftInset - RightInset, FieldHeight);
	InputTextField.hidden = NO;

	// Just the sum of the text field and inset, keyboard height is not included
	GIOSVirtualKeyboardInputFieldHeight.store(FieldHeight + BottomInset, std::memory_order_relaxed);
}

-(void)onTextChanged:(UITextField*)InTextField
{
	if (!TextWidget.IsValid())
	{
		return;
	}

	FText CurrentText = FText::FromString(InTextField.text);
	TWeakPtr<IVirtualKeyboardEntry> LocalWidget = TextWidget;

	FAppleAsyncTask* AsyncTask = [[FAppleAsyncTask alloc] init];
	AsyncTask.GameThreadCallback = ^ bool(void)
	{
		if (LocalWidget.IsValid())
		{
			LocalWidget.Pin()->SetTextFromVirtualKeyboard(CurrentText, ETextEntryType::TextEntryUpdated);
		}
		return true;
	};
	[AsyncTask FinishedTask];
}

-(void)textFieldDidChangeSelection:(UITextField*)InTextField
{
	if (!TextWidget.IsValid())
	{
		return;
	}

	UITextRange* SelectedRange = InTextField.selectedTextRange;
	if (SelectedRange == nil)
	{
		return;
	}

	int SelStart = (int)[InTextField offsetFromPosition:InTextField.beginningOfDocument toPosition:SelectedRange.start];
	int SelEnd = (int)[InTextField offsetFromPosition:InTextField.beginningOfDocument toPosition:SelectedRange.end];
	TWeakPtr<IVirtualKeyboardEntry> LocalWidget = TextWidget;

	FAppleAsyncTask* AsyncTask = [[FAppleAsyncTask alloc] init];
	AsyncTask.GameThreadCallback = ^ bool(void)
	{
		if (LocalWidget.IsValid())
		{
			LocalWidget.Pin()->SetSelectionFromVirtualKeyboard(SelStart, SelEnd);
		}
		return true;
	};
	[AsyncTask FinishedTask];
}

-(BOOL)textFieldShouldReturn:(UITextField*)InTextField
{
	FText LocalEntry = FText::FromString(InTextField.text);
	TWeakPtr<IVirtualKeyboardEntry> LocalWidget = TextWidget;

	[self hide];

	FAppleAsyncTask* AsyncTask = [[FAppleAsyncTask alloc] init];
	AsyncTask.GameThreadCallback = ^ bool(void)
	{
		if (LocalWidget.IsValid())
		{
			LocalWidget.Pin()->SetTextFromVirtualKeyboard(LocalEntry, ETextEntryType::TextEntryAccepted);
		}
		return true;
	};
	[AsyncTask FinishedTask];

	return YES;
}

-(void)show:(TSharedPtr<IVirtualKeyboardEntry>)InTextWidget text:(NSString*)TextContents placeholder:(NSString*)PlaceholderContents keyboardConfig:(FKeyboardConfig)KeyboardConfig
{
	TextWidget = InTextWidget;

	InputTextField.text = TextContents;
	InputTextField.attributedPlaceholder = [[NSAttributedString alloc] 
		initWithString:PlaceholderContents 
		attributes:@{NSForegroundColorAttributeName: [UIColor secondaryLabelColor]}];
	InputTextField.keyboardType = KeyboardConfig.KeyboardType;
	InputTextField.autocorrectionType = KeyboardConfig.AutocorrectionType;
	InputTextField.autocapitalizationType = KeyboardConfig.AutocapitalizationType;
	InputTextField.secureTextEntry = KeyboardConfig.bSecureTextEntry;

	// Attach the text field to the window on first use.
	if (InputTextField.superview == nil)
	{
		UIWindow* AppWindow = [IOSAppDelegate GetDelegate].IOSController.view.window;
		[AppWindow addSubview:InputTextField];
	}

	InputTextField.hidden = NO;

	[InputTextField becomeFirstResponder];
}

-(void)hide
{
	TextWidget = nullptr;

	GIOSVirtualKeyboardInputFieldHeight.store(0.0f, std::memory_order_relaxed);

	InputTextField.hidden = YES;

	[InputTextField resignFirstResponder];
}

-(bool)hasTextWidget
{
	return TextWidget.IsValid();
}

@end
#endif
