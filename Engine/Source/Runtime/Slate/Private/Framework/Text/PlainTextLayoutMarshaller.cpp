// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateTextLayout.h"
#include "Framework/Text/SlatePasswordRun.h"
#include "Framework/Text/SlateTextUnderlineLineHighlighter.h"

TSharedRef< FPlainTextLayoutMarshaller > FPlainTextLayoutMarshaller::Create()
{
	return MakeShareable(new FPlainTextLayoutMarshaller());
}

FPlainTextLayoutMarshaller::~FPlainTextLayoutMarshaller()
{
}

void FPlainTextLayoutMarshaller::SetIsPassword(const TAttribute<bool>& InIsPassword)
{
	bIsPassword = InIsPassword;
}

void FPlainTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
{
	const FTextBlockStyle& DefaultTextStyle = static_cast<FSlateTextLayout&>(TargetTextLayout).GetDefaultTextStyle();

	TArray<FTextRange> LineRanges;
	FTextRange::CalculateLineRangesFromString(SourceString, LineRanges);

	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(LineRanges.Num());

	TArray<FTextLineHighlight> LineHighlightsToAdd;

	TSharedPtr<FSlateTextUnderlineLineHighlighter> UnderlineLineHighlighter;
	if (!DefaultTextStyle.UnderlineBrush.GetResourceName().IsNone())
	{
		UnderlineLineHighlighter = FSlateTextUnderlineLineHighlighter::Create(DefaultTextStyle.UnderlineBrush, DefaultTextStyle.Font, DefaultTextStyle.ColorAndOpacity, DefaultTextStyle.ShadowOffset, DefaultTextStyle.ShadowColorAndOpacity);
	}

	TSharedPtr<FSlateTextStrikeLineHighlighter> StrikeLineHighlighter;
	if (!DefaultTextStyle.StrikeBrush.GetResourceName().IsNone())
	{
		StrikeLineHighlighter = FSlateTextStrikeLineHighlighter::Create(DefaultTextStyle.StrikeBrush, DefaultTextStyle.Font, DefaultTextStyle.ColorAndOpacity, DefaultTextStyle.ShadowOffset, DefaultTextStyle.ShadowColorAndOpacity);
	}

	const bool bUsePasswordRun = bIsPassword.Get(false);
	for (int32 LineIndex = 0; LineIndex < LineRanges.Num(); ++LineIndex)
	{
		const FTextRange& LineRange = LineRanges[LineIndex];
		TSharedRef<FString> LineText = MakeShareable(new FString(SourceString.Mid(LineRange.BeginIndex, LineRange.Len())));

		TArray<TSharedRef<IRun>> Runs;
		if (bUsePasswordRun)
		{
			Runs.Add(FSlatePasswordRun::Create(FRunInfo(), LineText, DefaultTextStyle));
		}
		else
		{
			Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, DefaultTextStyle));
		}

		if (UnderlineLineHighlighter.IsValid())
		{
			LineHighlightsToAdd.Add(FTextLineHighlight(LineIndex, FTextRange(0, LineRange.Len()), FSlateTextUnderlineLineHighlighter::DefaultZIndex, UnderlineLineHighlighter.ToSharedRef()));
		}

		if (StrikeLineHighlighter.IsValid())
		{
			LineHighlightsToAdd.Add(FTextLineHighlight(LineIndex, FTextRange(0, LineRange.Len()), FSlateTextStrikeLineHighlighter::DefaultZIndex, StrikeLineHighlighter.ToSharedRef()));
		}

		LinesToAdd.Emplace(MoveTemp(LineText), MoveTemp(Runs));
	}

	TargetTextLayout.AddLines(LinesToAdd);
	TargetTextLayout.SetLineHighlights(LineHighlightsToAdd);
}

void FPlainTextLayoutMarshaller::GetText(FString& TargetString, const FTextLayout& SourceTextLayout)
{
	SourceTextLayout.GetAsText(TargetString);
}

FPlainTextLayoutMarshaller::FPlainTextLayoutMarshaller()
{
	bIsPassword = false;
}
