// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/IRichTextMarkupWriter.h"
#include "Framework/Text/RichTextMarkupProcessing.h"
#include "Framework/Text/SlateTextLayout.h"
#include "Framework/Text/SlateTextUnderlineLineHighlighter.h"

#if WITH_FANCY_TEXT

TSharedRef< FRichTextLayoutMarshaller > FRichTextLayoutMarshaller::Create(TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet)
{
	return MakeShareable(new FRichTextLayoutMarshaller(MoveTemp(InDecorators), InDecoratorStyleSet));
}

TSharedRef< FRichTextLayoutMarshaller > FRichTextLayoutMarshaller::Create(TSharedPtr< IRichTextMarkupParser > InParser, TSharedPtr< IRichTextMarkupWriter > InWriter, TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet)
{
	return MakeShareable(new FRichTextLayoutMarshaller(MoveTemp(InParser), MoveTemp(InWriter), MoveTemp(InDecorators), InDecoratorStyleSet));
}

FRichTextLayoutMarshaller::~FRichTextLayoutMarshaller()
{
}

void FRichTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
{
	const FTextBlockStyle& DefaultTextStyle = static_cast<FSlateTextLayout&>(TargetTextLayout).GetDefaultTextStyle();

	TArray<FTextLineParseResults> LineParseResultsArray;
	FString ProcessedString;
	Parser->Process(LineParseResultsArray, SourceString, ProcessedString);

	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(LineParseResultsArray.Num());

	TArray<FTextLineHighlight> LineHighlightsToAdd;
	TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextUnderlineLineHighlighter>> CachedUnderlineHighlighters;
	TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextStrikeLineHighlighter>> CachedStrikeLineHighlighters;

	// Iterate through parsed line results and create processed lines with runs.
	for (int32 LineIndex = 0; LineIndex < LineParseResultsArray.Num(); ++LineIndex)
	{
		const FTextLineParseResults& LineParseResults = LineParseResultsArray[LineIndex];

		TSharedRef<FString> ModelString = MakeShareable(new FString());
		TArray< TSharedRef< IRun > > Runs;

		for (const FTextRunParseResults& RunParseResult : LineParseResults.Runs)
		{
			AppendRunsForText(LineIndex, RunParseResult, ProcessedString, DefaultTextStyle, ModelString, TargetTextLayout, Runs, LineHighlightsToAdd, CachedUnderlineHighlighters, CachedStrikeLineHighlighters);
		}

		LinesToAdd.Emplace(MoveTemp(ModelString), MoveTemp(Runs));
	}

	TargetTextLayout.AddLines(LinesToAdd);
	TargetTextLayout.SetLineHighlights(LineHighlightsToAdd);
}

void FRichTextLayoutMarshaller::GetText(FString& TargetString, const FTextLayout& SourceTextLayout)
{
	const auto& SourceLineModels = SourceTextLayout.GetLineModels();

	TArray<IRichTextMarkupWriter::FRichTextLine> WriterLines;
	WriterLines.Reserve(SourceLineModels.Num());

	for (const FTextLayout::FLineModel& LineModel : SourceLineModels)
	{
		IRichTextMarkupWriter::FRichTextLine WriterLine;
		WriterLine.Runs.Reserve(LineModel.Runs.Num());

		for (const FTextLayout::FRunModel& RunModel : LineModel.Runs)
		{
			FString RunText;
			RunModel.AppendTextTo(RunText);
			WriterLine.Runs.Emplace(IRichTextMarkupWriter::FRichTextRun(RunModel.GetRun()->GetRunInfo(), RunText));
		}

		WriterLines.Add(WriterLine);
	}

	Writer->Write(WriterLines, TargetString);
}

void FRichTextLayoutMarshaller::SetFontSizeMultiplier(const float NewFontSizeMultiplier)
{
	if (ensure(NewFontSizeMultiplier != 0.0f))
	{
		FontSizeMultiplier = NewFontSizeMultiplier;
	}
}

FRichTextLayoutMarshaller::FRichTextLayoutMarshaller(TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet)
	: Parser(FDefaultRichTextMarkupParser::GetStaticInstance())
	, Writer(FDefaultRichTextMarkupWriter::Create())
	, Decorators(MoveTemp(InDecorators))
	, InlineDecorators()
	, DecoratorStyleSet(InDecoratorStyleSet)
{
}

FRichTextLayoutMarshaller::FRichTextLayoutMarshaller(TSharedPtr< IRichTextMarkupParser > InParser, TSharedPtr< IRichTextMarkupWriter > InWriter, TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet)
	: Parser(MoveTemp(InParser))
	, Writer(MoveTemp(InWriter))
	, Decorators(MoveTemp(InDecorators))
	, InlineDecorators()
	, DecoratorStyleSet(InDecoratorStyleSet)
{
}

TSharedPtr< ITextDecorator > FRichTextLayoutMarshaller::TryGetDecorator(const FString& Line, const FTextRunParseResults& TextRun) const
{
	for (auto DecoratorIter = InlineDecorators.CreateConstIterator(); DecoratorIter; ++DecoratorIter)
	{
		if ((*DecoratorIter)->Supports(TextRun, Line))
		{
			return *DecoratorIter;
		}
	}

	for (auto DecoratorIter = Decorators.CreateConstIterator(); DecoratorIter; ++DecoratorIter)
	{
		if ((*DecoratorIter)->Supports(TextRun, Line))
		{
			return *DecoratorIter;
		}
	}

	return nullptr;
}

void FRichTextLayoutMarshaller::AppendRunsForText(
	const int32 LineIndex,
	const FTextRunParseResults& TextRun,
	const FString& ProcessedString,
	const FTextBlockStyle& DefaultTextStyle,
	const TSharedRef<FString>& InOutModelText,
	FTextLayout& TargetTextLayout,
	TArray<TSharedRef<IRun>>& Runs,
	TArray<FTextLineHighlight>& LineHighlights,
	TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextUnderlineLineHighlighter>>& CachedUnderlineHighlighters,
	TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextStrikeLineHighlighter>>& CachedStrikeLineHighlighters
	)
{
	TSharedPtr< ISlateRun > Run;
	TSharedPtr< ITextDecorator > Decorator = TryGetDecorator(ProcessedString, TextRun);

	if (Decorator.IsValid())
	{
		// Create run and update model string.
		Run = Decorator->Create(TargetTextLayout.AsShared(), TextRun, ProcessedString, InOutModelText, DecoratorStyleSet);
	}
	else
	{
		FRunInfo RunInfo(TextRun.Name);
		for (const TPair<FString, FTextRange>& Pair : TextRun.MetaData)
		{
			int32 Length = FMath::Max(0, Pair.Value.EndIndex - Pair.Value.BeginIndex);
			RunInfo.MetaData.Add(Pair.Key, ProcessedString.Mid(Pair.Value.BeginIndex, Length));
		}

		const FTextBlockStyle* TextBlockStyle;
		FTextRange ModelRange;
		ModelRange.BeginIndex = InOutModelText->Len();
		if (!(TextRun.Name.IsEmpty()) && DecoratorStyleSet->HasWidgetStyle< FTextBlockStyle >(FName(*TextRun.Name)))
		{
			*InOutModelText += ProcessedString.Mid(TextRun.ContentRange.BeginIndex, TextRun.ContentRange.EndIndex - TextRun.ContentRange.BeginIndex);
			TextBlockStyle = &(DecoratorStyleSet->GetWidgetStyle< FTextBlockStyle >(FName(*TextRun.Name)));
		}
		else
		{
			*InOutModelText += ProcessedString.Mid(TextRun.OriginalRange.BeginIndex, TextRun.OriginalRange.EndIndex - TextRun.OriginalRange.BeginIndex);
			TextBlockStyle = &DefaultTextStyle;
		}
		ModelRange.EndIndex = InOutModelText->Len();

		// Create run.
		TSharedPtr< FSlateTextRun > SlateTextRun = FSlateTextRun::Create(RunInfo, InOutModelText, *TextBlockStyle, ModelRange);

		if (SlateTextRun)
		{
			// Apply the FontSizeMultiplier at the style use by the IRun
			SlateTextRun->ApplyFontSizeMultiplierOnTextStyle(FontSizeMultiplier);
		}
		Run = SlateTextRun;

		if (!TextBlockStyle->UnderlineBrush.GetResourceName().IsNone())
		{
			TSharedPtr<FSlateTextUnderlineLineHighlighter> UnderlineLineHighlighter = CachedUnderlineHighlighters.FindRef(TextBlockStyle);
			if (!UnderlineLineHighlighter.IsValid())
			{
				UnderlineLineHighlighter = FSlateTextUnderlineLineHighlighter::Create(TextBlockStyle->UnderlineBrush, TextBlockStyle->Font, TextBlockStyle->ColorAndOpacity, TextBlockStyle->ShadowOffset, TextBlockStyle->ShadowColorAndOpacity);
				CachedUnderlineHighlighters.Add(TextBlockStyle, UnderlineLineHighlighter);
			}

			LineHighlights.Add(FTextLineHighlight(LineIndex, ModelRange, FSlateTextUnderlineLineHighlighter::DefaultZIndex, UnderlineLineHighlighter.ToSharedRef()));
		}

		if (!TextBlockStyle->StrikeBrush.GetResourceName().IsNone())
		{
			TSharedPtr<FSlateTextStrikeLineHighlighter> StrikeLineHighlighter = CachedStrikeLineHighlighters.FindRef(TextBlockStyle);
			if (!StrikeLineHighlighter.IsValid())
			{
				StrikeLineHighlighter = FSlateTextStrikeLineHighlighter::Create(TextBlockStyle->StrikeBrush, TextBlockStyle->Font, TextBlockStyle->ColorAndOpacity, TextBlockStyle->ShadowOffset, TextBlockStyle->ShadowColorAndOpacity);
				CachedStrikeLineHighlighters.Add(TextBlockStyle, StrikeLineHighlighter);
			}

			LineHighlights.Add(FTextLineHighlight(LineIndex, ModelRange, FSlateTextStrikeLineHighlighter::DefaultZIndex, StrikeLineHighlighter.ToSharedRef()));
		}
	}

	Runs.Add(Run.ToSharedRef());
}

#endif //WITH_FANCY_TEXT
