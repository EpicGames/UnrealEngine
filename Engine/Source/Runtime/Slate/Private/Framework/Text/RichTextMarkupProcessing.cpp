// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/RichTextMarkupProcessing.h"

#if WITH_FANCY_TEXT


struct FUnescapeHelper
{
	TArray<FString> EscapeSequences;
	TArray<FString> UnescapedCharacters;

	FUnescapeHelper()
	{
		EscapeSequences.Add(TEXT("quot"));	UnescapedCharacters.Add(TEXT("\""));
		EscapeSequences.Add(TEXT("lt"));		UnescapedCharacters.Add(TEXT("<"));
		EscapeSequences.Add(TEXT("gt"));		UnescapedCharacters.Add(TEXT(">"));
		EscapeSequences.Add(TEXT("amp"));		UnescapedCharacters.Add(TEXT("&"));
	}

} static const UnescapeHelper;

struct FEscapeSequenceRegexPatternString
{
	// Generate a regular expression pattern string that matches each of the escape sequences as alternatives, each in its own capture group.
	static FString Get(const TArray<FString>& EscapeSequences)
	{
		FString EscapeSequenceRegexPatternString;

		for (const FString& EscapeSequence : EscapeSequences)
		{
			// Add alternation operator to regex.
			if (!(EscapeSequenceRegexPatternString.IsEmpty()))
			{
				EscapeSequenceRegexPatternString += TEXT("|");
			}

			// Add capture group for escape sequence.
			EscapeSequenceRegexPatternString += TEXT("(");
			EscapeSequenceRegexPatternString += TEXT("&");
			EscapeSequenceRegexPatternString += EscapeSequence;
			EscapeSequenceRegexPatternString += TEXT(";");
			EscapeSequenceRegexPatternString += TEXT(")");
		}

		return EscapeSequenceRegexPatternString;
	}
};

TSharedRef< FDefaultRichTextMarkupParser > FDefaultRichTextMarkupParser::Create()
{
	return MakeShareable(new FDefaultRichTextMarkupParser());
}

TSharedRef< FDefaultRichTextMarkupParser > FDefaultRichTextMarkupParser::GetStaticInstance()
{
	static TSharedRef< FDefaultRichTextMarkupParser > Parser = MakeShareable( new FDefaultRichTextMarkupParser() );
	return Parser;
}

FDefaultRichTextMarkupParser::FDefaultRichTextMarkupParser()
	: EscapeSequenceRegexPattern(FRegexPattern(FEscapeSequenceRegexPatternString::Get(UnescapeHelper.EscapeSequences)))
	, ElementRegexPattern( FRegexPattern(TEXT("<([\\w\\d\\.-]+)((?: (?:[\\w\\d\\.-]+=(?>\".*?\")))+)?(?:(?:/>)|(?:>(.*?)</>))") ))
	, AttributeRegexPattern( FRegexPattern(TEXT("([\\w\\d\\.]+)=(?>\"(.*?)\")")) )
{

}

void FDefaultRichTextMarkupParser::Process(TArray<FTextLineParseResults>& Results, const FString& Input, FString& Output)
{
#if UE_ENABLE_ICU
	TArray<FTextRange> LineRanges;
	FTextRange::CalculateLineRangesFromString(Input, LineRanges);
	ParseLineRanges(Input, LineRanges, Results);
	HandleEscapeSequences(Input, Results, Output);
#else
	Output = TEXT("WARNING: Rich text formatting is disabled.");
	FTextLineParseResults FakeLineParseResults(FTextRange(0, Output.Len()));
	FTextRunParseResults FakeRunParseResults(TEXT(""), FTextRange(0, Output.Len()));
	FakeLineParseResults.Runs.Add(FakeRunParseResults);
	Results.Add(FakeLineParseResults);
#endif
}

void FDefaultRichTextMarkupParser::ParseLineRanges(const FString& Input, const TArray<FTextRange>& LineRanges, TArray<FTextLineParseResults>& LineParseResultsArray) const
{
	if (LineRanges.IsEmpty())
	{
		return;
	}

	// Special regular expression pattern for matching rich text markup elements. IE: <ElementName AttributeName="AttributeValue">Content</>
	FRegexMatcher ElementRegexMatcher(ElementRegexPattern, Input);

	// Lazily constructed attribute parser
	TOptional<FRegexMatcher> LazyAttributeRegexMatcher;

	// Parse line ranges, creating line parse results and run parse results.
	for(int32 i = 0; i < LineRanges.Num(); ++i)
	{
		FTextLineParseResults LineParseResults;
		LineParseResults.Range = LineRanges[i];

		// Limit the element regex matcher to the current line.
		ElementRegexMatcher.SetLimits(LineParseResults.Range.BeginIndex, LineParseResults.Range.EndIndex);

		// Iterate through the line, each time trying to find a match for the element regex, adding it as a run and any intervening text as another run.
		int32 LastRunEnd = LineParseResults.Range.BeginIndex;
		while(ElementRegexMatcher.FindNext()) 
		{
			int32 ElementBegin = ElementRegexMatcher.GetMatchBeginning();
			int32 ElementEnd = ElementRegexMatcher.GetMatchEnding();

			FTextRange OriginalRange(ElementBegin, ElementEnd);

			// Capture Group 1 is the element name.
			int32 ElementNameBegin = ElementRegexMatcher.GetCaptureGroupBeginning(1);
			int32 ElementNameEnd = ElementRegexMatcher.GetCaptureGroupEnding(1);

			// Name
			FString ElementName = Input.Mid(ElementNameBegin, ElementNameEnd - ElementNameBegin);

			// Capture Group 2 is the attribute list.
			int32 AttributeListBegin = ElementRegexMatcher.GetCaptureGroupBeginning(2);
			int32 AttributeListEnd = ElementRegexMatcher.GetCaptureGroupEnding(2);

			// Capture Group 3 is the content.
			int32 ElementContentBegin = ElementRegexMatcher.GetCaptureGroupBeginning(3);
			int32 ElementContentEnd = ElementRegexMatcher.GetCaptureGroupEnding(3);

			FTextRange ContentRange(ElementContentBegin, ElementContentEnd);

			TMap<FString, FTextRange> Attributes;

			if(AttributeListBegin != INDEX_NONE && AttributeListEnd != INDEX_NONE)
			{
				if (!LazyAttributeRegexMatcher.IsSet())
				{
					LazyAttributeRegexMatcher.Emplace(AttributeRegexPattern, Input);
				}
				FRegexMatcher& AttributeRegexMatcher = *LazyAttributeRegexMatcher;
				AttributeRegexMatcher.SetLimits(AttributeListBegin, AttributeListEnd);

				// Iterate through the attribute list, each time trying to find a match for the attribute regex.
				while(AttributeRegexMatcher.FindNext())
				{
					// Capture Group 1 is the attribute key.
					int32 AttributeKeyBegin = AttributeRegexMatcher.GetCaptureGroupBeginning(1);
					int32 AttributeKeyEnd = AttributeRegexMatcher.GetCaptureGroupEnding(1);

					// Capture Group 2 is the attribute value.
					int32 AttributeValueBegin = AttributeRegexMatcher.GetCaptureGroupBeginning(2);
					int32 AttributeValueEnd = AttributeRegexMatcher.GetCaptureGroupEnding(2);

					// Attribute
					Attributes.Add(Input.Mid(AttributeKeyBegin, AttributeKeyEnd - AttributeKeyBegin), FTextRange(AttributeValueBegin, AttributeValueEnd));
				}
			}

			// Add intervening run to line.
			FTextRange InterveningRunRange(LastRunEnd, ElementBegin);
			if(!InterveningRunRange.IsEmpty())
			{
				FTextRunParseResults InterveningRunParseResults(FString(), InterveningRunRange);
				LastRunEnd = ElementBegin;
				LineParseResults.Runs.Add(InterveningRunParseResults);
			}

			// Add element run to line.
			FTextRunParseResults RunParseResults(ElementName, OriginalRange, ContentRange);
			RunParseResults.MetaData = Attributes;
			LineParseResults.Runs.Add(RunParseResults);
			LastRunEnd = ElementEnd;
		}

		// Add dangling run to line.
		FTextRange InterveningRunRange(LastRunEnd, LineParseResults.Range.EndIndex);
		if(!InterveningRunRange.IsEmpty())
		{
			FTextRunParseResults InterveningRunParseResults(FString(), InterveningRunRange);
			LastRunEnd = LineParseResults.Range.EndIndex;
			LineParseResults.Runs.Add(InterveningRunParseResults);
		}

		// Add blank, empty run if none are present.
		if(LineParseResults.Runs.Num() == 0)
		{
			FTextRunParseResults EmptyRunParseResults(FString(), LineParseResults.Range);
			LastRunEnd = EmptyRunParseResults.OriginalRange.EndIndex;
			LineParseResults.Runs.Add(EmptyRunParseResults);
		}

		LineParseResultsArray.Add(LineParseResults);
	}
}

void FDefaultRichTextMarkupParser::HandleEscapeSequences(const FString& Input, TArray<FTextLineParseResults>& LineParseResultsArray, FString& ConcatenatedUnescapedLines) const
{
	if (LineParseResultsArray.IsEmpty())
	{
		return;
	}

	FRegexMatcher EscapeSequenceRegexMatcher(EscapeSequenceRegexPattern, Input);

	// Modify original string to handle escape sequences that need to be replaced while updating run ranges.
	for(int32 i = 0; i < LineParseResultsArray.Num(); ++i)
	{
		FTextLineParseResults& LineParseResults = LineParseResultsArray[i];

		// Adjust begin indices for previous substitutions.
		LineParseResults.Range.BeginIndex = ConcatenatedUnescapedLines.Len();

		for(int32 j = 0; j < LineParseResults.Runs.Num(); ++j)
		{
			FTextRunParseResults& RunParseResults = LineParseResults.Runs[j];

			TArray<int32*> IndicesToUpdate;
			IndicesToUpdate.Add(&RunParseResults.OriginalRange.BeginIndex);
			for(TPair<FString, FTextRange>& Pair : RunParseResults.MetaData)
			{
				IndicesToUpdate.Add(&Pair.Value.BeginIndex);
				IndicesToUpdate.Add(&Pair.Value.EndIndex);
			}
			if(RunParseResults.ContentRange.BeginIndex != INDEX_NONE && RunParseResults.ContentRange.EndIndex != INDEX_NONE)
			{
				IndicesToUpdate.Add(&RunParseResults.ContentRange.BeginIndex);
				IndicesToUpdate.Add(&RunParseResults.ContentRange.EndIndex);
			}
			IndicesToUpdate.Add(&RunParseResults.OriginalRange.EndIndex);

			{
				const auto GetUnescapedString = [&]()
				{
					int32 LastCopiedIndex = EscapeSequenceRegexMatcher.GetBeginLimit();
					while(EscapeSequenceRegexMatcher.FindNext()) 
					{
						// Copy intervening characters between the end of the last copy and the beginning of this match.
						ConcatenatedUnescapedLines += Input.Mid(LastCopiedIndex, EscapeSequenceRegexMatcher.GetMatchBeginning() - LastCopiedIndex);
						LastCopiedIndex = EscapeSequenceRegexMatcher.GetMatchBeginning();

						// Identify which escape sequence was captured based on which capture group has a valid range.
						for(int32 k = 0; k < UnescapeHelper.EscapeSequences.Num(); ++k)
						{
							const int32 GroupOrdinal = 1 + k; // Groups are accessed by ordinal, not index.
							const int32 EscapeSequenceBeginIndex = EscapeSequenceRegexMatcher.GetCaptureGroupBeginning(GroupOrdinal);
							const int32 EscapeSequenceEndIndex = EscapeSequenceRegexMatcher.GetCaptureGroupEnding(GroupOrdinal);

							// Lookup and copy unescaped character in place of the escape sequence.
							if(EscapeSequenceBeginIndex != INDEX_NONE && EscapeSequenceEndIndex != INDEX_NONE)
							{
								ConcatenatedUnescapedLines += UnescapeHelper.UnescapedCharacters[k];
								break;
							}
						}

						LastCopiedIndex = EscapeSequenceRegexMatcher.GetMatchEnding();
					}

					// Copy intervening characters between the end of the last copy and the end of the run.
					ConcatenatedUnescapedLines += Input.Mid(LastCopiedIndex, EscapeSequenceRegexMatcher.GetEndLimit() - LastCopiedIndex);
				};

				int32 k;
				for(k = 0; k + 1 < IndicesToUpdate.Num(); ++k)
				{
					EscapeSequenceRegexMatcher.SetLimits(*(IndicesToUpdate[k]), *(IndicesToUpdate[k + 1]));
					*(IndicesToUpdate[k]) = ConcatenatedUnescapedLines.Len();
					GetUnescapedString();
				}
				*(IndicesToUpdate[k]) = ConcatenatedUnescapedLines.Len();
			}
		}

		// Adjust end indices for previous substitutions.
		LineParseResults.Range.EndIndex = ConcatenatedUnescapedLines.Len();
	}
}


TSharedRef< FDefaultRichTextMarkupWriter > FDefaultRichTextMarkupWriter::Create()
{
	return MakeShareable( new FDefaultRichTextMarkupWriter() );
}

TSharedRef< FDefaultRichTextMarkupWriter > FDefaultRichTextMarkupWriter::GetStaticInstance()
{
	static TSharedRef< FDefaultRichTextMarkupWriter > Writer = MakeShareable(new FDefaultRichTextMarkupWriter());
	return Writer;
}

void FDefaultRichTextMarkupWriter::Write(const TArray<FRichTextLine>& InLines, FString& Output)
{
	for (int32 LineIndex = 0; LineIndex < InLines.Num(); ++LineIndex)
	{
		const FRichTextLine& Line = InLines[LineIndex];

		// Append line terminator to the end of the previous line
		if(LineIndex > 0)
		{
			Output += LINE_TERMINATOR;
		}

		for (const FRichTextRun& Run : Line.Runs)
		{
			// Our rich-text format takes the form of <Name metakey1="metavalue1" metakey2="metavalue2">The Text</>
			const bool bHasTag = !Run.Info.Name.IsEmpty();
			if (bHasTag)
			{
				Output.AppendChar('<');

				Output.Append(Run.Info.Name);

				for(const TPair<FString, FString>& MetaDataEntry : Run.Info.MetaData)
				{
					Output.AppendChar(' ');
					Output.Append(MetaDataEntry.Key);
					Output.AppendChar('=');
					Output.AppendChar('"');
					Output.Append(MetaDataEntry.Value);
					Output.AppendChar('"');
				}

				Output.AppendChar('>');
			}

			FString RunText = Run.Text;
			EscapeText(RunText);
			Output.Append(RunText);

			if (bHasTag)
			{
				Output.Append(TEXT("</>"));
			}
		}
	}
}

void FDefaultRichTextMarkupWriter::EscapeText(FString& TextToEscape)
{
	// List of characters that we have to escape to avoid accidental rich-text formatting
	static const TPair<TCHAR, FString> EscapeCharacters[] = {
		TPair<TCHAR, FString>('&', "&amp;"),
		TPair<TCHAR, FString>('"', "&quot;"),
		TPair<TCHAR, FString>('<', "&lt;"),
		TPair<TCHAR, FString>('>', "&gt;"),
	};

	// First count the extra space needed
	int32 EscapedStringLen = TextToEscape.Len();
	for (int32 TextIndex = 0; TextIndex < TextToEscape.Len(); ++TextIndex)
	{
		const TCHAR CharToEscape = TextToEscape[TextIndex];
		for (const auto& EscapeCharacter : EscapeCharacters)
		{
			if (CharToEscape == EscapeCharacter.Key)
			{
				EscapedStringLen += (EscapeCharacter.Value.Len() - 1);
				break;
			}
		}
	}

	if (EscapedStringLen == TextToEscape.Len())
	{
		return;
	}

	TextToEscape.Reserve(EscapedStringLen);

	// Then perform the actual escape (backwards to make iteration simpler)
	for (int32 TextIndex = TextToEscape.Len() - 1; TextIndex >= 0; --TextIndex)
	{
		const TCHAR CharToEscape = TextToEscape[TextIndex];
		for (const auto& EscapeCharacter : EscapeCharacters)
		{
			if (CharToEscape == EscapeCharacter.Key)
			{
				TextToEscape.RemoveAt(TextIndex, EAllowShrinking::No);
				TextToEscape.InsertAt(TextIndex, EscapeCharacter.Value);
				break;
			}
		}
	}
}

#endif //WITH_FANCY_TEXT
