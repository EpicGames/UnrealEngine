// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/RunUtils.h"
#include "Framework/Text/TextHitPoint.h"

ETextHitPoint RunUtils::CalculateTextHitPoint(const int32 InIndex, const FTextRange& InTextRange, const TextBiDi::ETextDirection InTextDirection)
{
	//if (InTextDirection == TextBiDi::ETextDirection::LeftToRight)
	{
		return (InIndex == InTextRange.EndIndex) ? ETextHitPoint::RightGutter : ETextHitPoint::WithinText;
	}
	/*
	else
	{
		return (InIndex == InTextRange.BeginIndex) ? ETextHitPoint::RightGutter : ETextHitPoint::WithinText;
	}
	*/
}

FTextRange RunUtils::CalculateOffsetMeasureRange(const int32 InOffset, const FTextRange& InTextRange, const TextBiDi::ETextDirection InTextDirection)
{
	if (InTextDirection == TextBiDi::ETextDirection::LeftToRight)
	{
		return FTextRange(InTextRange.BeginIndex, InTextRange.BeginIndex + InOffset);
	}
	else
	{
		return FTextRange(InTextRange.BeginIndex + InOffset, InTextRange.EndIndex);
	}
}
