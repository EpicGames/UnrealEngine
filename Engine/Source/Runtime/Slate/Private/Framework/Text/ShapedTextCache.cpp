// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/ShapedTextCache.h"
#include "Fonts/FontCache.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"
#include "Trace/SlateMemoryTags.h"

FShapedGlyphSequencePtr FShapedTextCache::FindShapedText(const FCachedShapedTextKey& InKey) const
{
	FShapedGlyphSequencePtr ShapedText = CachedShapedText.FindRef(InKey);

	if (ShapedText.IsValid() && !ShapedText->IsDirty())
	{
		return ShapedText;
	}
	
	return nullptr;
}

FShapedGlyphSequenceRef FShapedTextCache::AddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	const TSharedPtr<FSlateFontCache> FontCache = FontCachePtr.Pin();

	if (!FontCache.IsValid())
	{
		return MakeShareable(new FShapedGlyphSequence());
	}

	FShapedGlyphSequenceRef ShapedText = FontCache->ShapeBidirectionalText(
		InText, 
		InKey.TextRange.BeginIndex, 
		InKey.TextRange.Len(), 
		InKey.FontInfo, 
		InKey.Scale, 
		InKey.TextContext.BaseDirection, 
		InKey.TextContext.TextShapingMethod,
		OutLoadingFontFaces
		);

	return AddShapedText(InKey, ShapedText);
}

FShapedGlyphSequenceRef FShapedTextCache::AddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, const TextBiDi::ETextDirection InTextDirection, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	const TSharedPtr<FSlateFontCache> FontCache = FontCachePtr.Pin();

	if (!FontCache.IsValid())
	{
		return MakeShareable(new FShapedGlyphSequence());
	}

	FShapedGlyphSequenceRef ShapedText = FontCache->ShapeUnidirectionalText(
		InText, 
		InKey.TextRange.BeginIndex, 
		InKey.TextRange.Len(), 
		InKey.FontInfo, 
		InKey.Scale, 
		InTextDirection, 
		InKey.TextContext.TextShapingMethod,
		OutLoadingFontFaces
		);

	return AddShapedText(InKey, ShapedText);
}

FShapedGlyphSequenceRef FShapedTextCache::AddShapedText(const FCachedShapedTextKey& InKey, FShapedGlyphSequenceRef InShapedText)
{
	CachedShapedText.Add(InKey, InShapedText);
	return InShapedText;
}

FShapedGlyphSequenceRef FShapedTextCache::FindOrAddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	FShapedGlyphSequencePtr ShapedText = FindShapedText(InKey);

	if (!ShapedText.IsValid())
	{
		ShapedText = AddShapedText(InKey, InText, OutLoadingFontFaces);
	}

	return ShapedText.ToSharedRef();
}

FShapedGlyphSequenceRef FShapedTextCache::FindOrAddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, const TextBiDi::ETextDirection InTextDirection, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	FShapedGlyphSequencePtr ShapedText = FindShapedText(InKey);

	if (!ShapedText.IsValid())
	{
		ShapedText = AddShapedText(InKey, InText, InTextDirection, OutLoadingFontFaces);
	}

	return ShapedText.ToSharedRef();
}

FShapedGlyphSequenceRef FShapedTextCache::FindOrAddOverflowEllipsisText(const float InScale, const FShapedTextContext& InTextContext, const FSlateFontInfo& InFontInfo)
{
	FCachedShapedTextKey Key = FCachedShapedTextKey(FTextRange(), InScale, InTextContext, InFontInfo);
	FShapedGlyphSequencePtr ShapedText = FindShapedText(Key);
	if (ShapedText)
	{
		return ShapedText.ToSharedRef();
	}

	const TSharedPtr<FSlateFontCache> FontCache = FontCachePtr.Pin();
	if (FontCache.IsValid())
	{
		return FontCache->ShapeOverflowEllipsisText(InFontInfo, InScale);
	}
	return MakeShareable(new FShapedGlyphSequence());
}

void FShapedTextCache::Clear()
{
	CachedShapedText.Reset();
}

FVector2D ShapedTextCacheUtil::MeasureShapedText(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const FTextRange& InMeasureRange, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	// Get the shaped text for the entire run and try and take a sub-measurement from it - this can help minimize the amount of text shaping that needs to be done when measuring text
	FShapedGlyphSequenceRef ShapedText = InShapedTextCache->FindOrAddShapedText(InRunKey, InText, OutLoadingFontFaces);

	TOptional<int32> MeasuredWidth = ShapedText->GetMeasuredWidth(InMeasureRange.BeginIndex, InMeasureRange.EndIndex);
	if (!MeasuredWidth.IsSet())
	{
		FCachedShapedTextKey MeasureKey = InRunKey;
		MeasureKey.TextRange = InMeasureRange;

		// Couldn't measure the sub-range, try and measure from a shape of the specified range
		ShapedText = InShapedTextCache->FindOrAddShapedText(MeasureKey, InText, OutLoadingFontFaces);
		MeasuredWidth = ShapedText->GetMeasuredWidth();
	}

	check(MeasuredWidth.IsSet());
	return FVector2D(MeasuredWidth.GetValue(), ShapedText->GetMaxTextHeight());
}

int32 ShapedTextCacheUtil::FindCharacterIndexAtOffset(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const FTextRange& InTextRange, const TCHAR* InText, const int32 InHorizontalOffset)
{
	TSharedPtr<FSlateFontCache> FontCache = InShapedTextCache->GetFontCache();

	if (!FontCache.IsValid())
	{
		return INDEX_NONE;
	}

	// Get the shaped text for the entire run and try and take a sub-measurement from it - this can help minimize the amount of text shaping that needs to be done when measuring text
	FShapedGlyphSequenceRef ShapedText = InShapedTextCache->FindOrAddShapedText(InRunKey, InText);

	TOptional<FShapedGlyphSequence::FGlyphOffsetResult> GlyphOffsetResult = ShapedText->GetGlyphAtOffset(*FontCache, InTextRange.BeginIndex, InTextRange.EndIndex, InHorizontalOffset);
	if (!GlyphOffsetResult.IsSet())
	{
		FCachedShapedTextKey IndexAtOffsetKey = InRunKey;
		IndexAtOffsetKey.TextRange = InTextRange;

		// Couldn't search the sub-range, try and search from a shape of the specified range
		ShapedText = InShapedTextCache->FindOrAddShapedText(IndexAtOffsetKey, InText);
		GlyphOffsetResult = ShapedText->GetGlyphAtOffset(*FontCache, InHorizontalOffset);
	}

	check(GlyphOffsetResult.IsSet());

	// We need to handle the fact that the found glyph may have been a ligature, and if so, we need to measure each part of the ligature to find the best character index match
	const FShapedGlyphSequence::FGlyphOffsetResult& GlyphOffsetResultValue = GlyphOffsetResult.GetValue();
	if (GlyphOffsetResultValue.Glyph && GlyphOffsetResultValue.Glyph->NumGraphemeClustersInGlyph > 1)
	{
		// Process each grapheme cluster within the ligature
		const FString LigatureString = FString::ConstructFromPtrSize(InText + GlyphOffsetResultValue.Glyph->SourceIndex, GlyphOffsetResultValue.Glyph->NumCharactersInGlyph);
		TSharedRef<IBreakIterator> GraphemeBreakIterator = FBreakIterator::CreateCharacterBoundaryIterator();
		GraphemeBreakIterator->SetString(LigatureString);

		FCachedShapedTextKey LigatureKey = InRunKey;
		LigatureKey.TextRange = FTextRange(0, LigatureString.Len());

		int32 CurrentOffset = GlyphOffsetResultValue.GlyphOffset;
		if (GlyphOffsetResultValue.Glyph->TextDirection == TextBiDi::ETextDirection::LeftToRight)
		{
			int32 PrevCharIndex = GraphemeBreakIterator->ResetToBeginning();
			for (int32 CurrentCharIndex = GraphemeBreakIterator->MoveToNext(); CurrentCharIndex != INDEX_NONE; CurrentCharIndex = GraphemeBreakIterator->MoveToNext())
			{
				FShapedGlyphSequenceRef GraphemeShapedText = GetShapedTextSubSequence(InShapedTextCache, LigatureKey, FTextRange(PrevCharIndex, CurrentCharIndex), *LigatureString, GlyphOffsetResultValue.Glyph->TextDirection);
				
				const FShapedGlyphSequence::FGlyphOffsetResult GraphemeOffsetResult = GraphemeShapedText->GetGlyphAtOffset(*FontCache, InHorizontalOffset, CurrentOffset);
				if (GraphemeOffsetResult.Glyph)
				{
					return GlyphOffsetResultValue.Glyph->SourceIndex + GraphemeOffsetResult.CharacterIndex;
				}

				PrevCharIndex = CurrentCharIndex;
				CurrentOffset += GraphemeShapedText->GetMeasuredWidth();
			}

			return GlyphOffsetResultValue.Glyph->SourceIndex + GlyphOffsetResultValue.Glyph->NumCharactersInGlyph;
		}
		else
		{
			int32 PrevCharIndex = GraphemeBreakIterator->ResetToEnd();
			for (int32 CurrentCharIndex = GraphemeBreakIterator->MoveToPrevious(); CurrentCharIndex != INDEX_NONE; CurrentCharIndex = GraphemeBreakIterator->MoveToPrevious())
			{
				FShapedGlyphSequenceRef GraphemeShapedText = GetShapedTextSubSequence(InShapedTextCache, LigatureKey, FTextRange(CurrentCharIndex, PrevCharIndex), *LigatureString, GlyphOffsetResultValue.Glyph->TextDirection);

				const FShapedGlyphSequence::FGlyphOffsetResult GraphemeOffsetResult = GraphemeShapedText->GetGlyphAtOffset(*FontCache, InHorizontalOffset, CurrentOffset);
				if (GraphemeOffsetResult.Glyph)
				{
					return GlyphOffsetResultValue.Glyph->SourceIndex + ((PrevCharIndex != INDEX_NONE) ? PrevCharIndex : GraphemeOffsetResult.CharacterIndex);
				}

				PrevCharIndex = CurrentCharIndex;
				CurrentOffset += GraphemeShapedText->GetMeasuredWidth();
			}

			return GlyphOffsetResultValue.Glyph->SourceIndex;
		}
	}

	return GlyphOffsetResultValue.CharacterIndex;
}

int8 ShapedTextCacheUtil::GetShapedGlyphKerning(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const int32 InGlyphIndex, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	// Get the shaped text for the entire run and try and get the kerning from it - this can help minimize the amount of text shaping that needs to be done when calculating kerning
	FShapedGlyphSequenceRef ShapedText = InShapedTextCache->FindOrAddShapedText(InRunKey, InText, OutLoadingFontFaces);

	TOptional<int8> Kerning = ShapedText->GetKerning(InGlyphIndex);
	if (!Kerning.IsSet())
	{
		FCachedShapedTextKey KerningKey = InRunKey;
		KerningKey.TextRange = FTextRange(InGlyphIndex, InGlyphIndex + 2);

		// Couldn't get the kerning from the main run data, try and get the kerning from a shape of the specified range
		ShapedText = InShapedTextCache->FindOrAddShapedText(KerningKey, InText, OutLoadingFontFaces);
		Kerning = ShapedText->GetKerning(InGlyphIndex);
	}

	return Kerning.Get(0);
}

FShapedGlyphSequenceRef ShapedTextCacheUtil::GetShapedTextSubSequence(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const FTextRange& InTextRange, const TCHAR* InText, const TextBiDi::ETextDirection InTextDirection, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	LLM_SCOPE_BYTAG(UI_Text);

	// Get the shaped text for the entire run and try and make a sub-sequence from it - this can help minimize the amount of text shaping that needs to be done when drawing text
	FShapedGlyphSequencePtr ShapedText = InShapedTextCache->FindOrAddShapedText(InRunKey, InText, OutLoadingFontFaces);

	if (InRunKey.TextRange != InTextRange)
	{
		FCachedShapedTextKey SubSequenceKey = InRunKey;
		SubSequenceKey.TextRange = InTextRange;

		// Do we already have a cached entry for this? We don't use FindOrAdd here as if it's missing, we first want to try and extract it from our run of shaped text
		FShapedGlyphSequencePtr FoundShapedText = InShapedTextCache->FindShapedText(SubSequenceKey);

		if (FoundShapedText.IsValid())
		{
			ShapedText = FoundShapedText;
		}
		else
		{
			// Didn't find it in the cache, so first try and extract a sub-sequence from the run of shaped text
			ShapedText = ShapedText->GetSubSequence(InTextRange.BeginIndex, InTextRange.EndIndex);

			if (ShapedText.IsValid())
			{
				// Add this new sub-sequence to the cache
				InShapedTextCache->AddShapedText(SubSequenceKey, ShapedText.ToSharedRef());
			}
			else
			{
				// Couldn't get the sub-sequence, try and make a new shape for it instead
				ShapedText = InShapedTextCache->FindOrAddShapedText(SubSequenceKey, InText, InTextDirection, OutLoadingFontFaces);
			}
		}
	}

	check(ShapedText.IsValid());
	return ShapedText.ToSharedRef();
}

