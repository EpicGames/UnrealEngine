// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateHyperlinkRun.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/WidgetLayoutBlock.h"

#if WITH_FANCY_TEXT

#include "Widgets/Input/SRichTextHyperlink.h"
#include "Framework/Text/ShapedTextCache.h"
#include "Framework/Text/RunUtils.h"

TSharedRef< FSlateHyperlinkRun > FSlateHyperlinkRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate )
{
	return MakeShareable( new FSlateHyperlinkRun( InRunInfo, InText, InStyle, NavigateDelegate, InTooltipDelegate, InTooltipTextDelegate ) );
}

TSharedRef< FSlateHyperlinkRun > FSlateHyperlinkRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange )
{
	return MakeShareable( new FSlateHyperlinkRun( InRunInfo, InText, InStyle, NavigateDelegate, InTooltipDelegate, InTooltipTextDelegate, InRange ) );
}

FTextRange FSlateHyperlinkRun::GetTextRange() const 
{
	return Range;
}

void FSlateHyperlinkRun::SetTextRange( const FTextRange& Value )
{
	Range = Value;
}

int16 FSlateHyperlinkRun::GetBaseLine( float Scale ) const 
{
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	return FontMeasure->GetBaseline( Style.TextStyle.Font, Scale ) - FMath::Min(0.0f, Style.TextStyle.ShadowOffset.Y * Scale);
}

int16 FSlateHyperlinkRun::GetMaxHeight( float Scale ) const 
{
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	return FontMeasure->GetMaxCharacterHeight( Style.TextStyle.Font, Scale ) + FMath::Abs(Style.TextStyle.ShadowOffset.Y * Scale);
}

FVector2D FSlateHyperlinkRun::Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	const FVector2d OutlineSize = GetOutlineSize(StartIndex, EndIndex, Scale);
	const FVector2d ShadowSize = GetShadowSize(StartIndex, EndIndex, Scale);
	
	if (EndIndex - StartIndex == 0)
	{
		return FVector2D(0, GetMaxHeight(Scale)) + OutlineSize + ShadowSize;
	}
	
	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	return ShapedTextCacheUtil::MeasureShapedText(TextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, TextContext, Style.TextStyle.Font), FTextRange(StartIndex, EndIndex), **Text) + OutlineSize + ShadowSize;
}

int8 FSlateHyperlinkRun::GetKerning( int32 CurrentIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	return 0;
}

FVector2d FSlateHyperlinkRun::GetOutlineSize(int32 StartIndex, int32 EndIndex, float Scale) const
{
	// Offset the measured shaped text by the outline since the outline was not factored into the size of the text
	// Need to add the outline offsetting to the beginning and the end because it surrounds both sides.
	const float ScaledOutlineSize = Style.TextStyle.Font.OutlineSettings.OutlineSize * Scale;

	return FVector2D((StartIndex == Range.BeginIndex ? ScaledOutlineSize : 0) + (EndIndex == Range.EndIndex ? ScaledOutlineSize : 0), ScaledOutlineSize);
}

FVector2d FSlateHyperlinkRun::GetShadowSize(int32 StartIndex, int32 EndIndex, float Scale) const
{
	return FVector2d((EndIndex == Range.EndIndex) ? FMath::Abs(Style.TextStyle.ShadowOffset.X * Scale) : 0.0f, FMath::Abs(Style.TextStyle.ShadowOffset.Y * Scale));
}

TSharedRef< ILayoutBlock > FSlateHyperlinkRun::CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer )
{
	FText ToolTipText;
	TSharedPtr<IToolTip> ToolTip;
	
	if(TooltipDelegate.IsBound())
	{
		ToolTip = TooltipDelegate.Execute(RunInfo.MetaData);
	}
	else
	{
		const FString* Url = RunInfo.MetaData.Find(TEXT("href"));
		if(TooltipTextDelegate.IsBound())
		{
			ToolTipText = TooltipTextDelegate.Execute(RunInfo.MetaData);
		}
		else if(Url != nullptr)
		{
			ToolTipText = FText::FromString(*Url);
		}
	}

	TSharedRef< SWidget > Widget = SNew( SRichTextHyperlink, ViewModel )
		.Style( &Style )
		.Text( FText::FromString( FString::ConstructFromPtrSize( **Text + StartIndex, EndIndex - StartIndex ) ) )
		.ToolTip( ToolTip )
		.ToolTipText( ToolTipText )
		.OnNavigate( this, &FSlateHyperlinkRun::OnNavigate )
		.TextShapingMethod( TextContext.TextShapingMethod );
	
	// We need to do a prepass here as CreateBlock can be called after the main Slate prepass has been run, 
	// which can result in the hyperlink widget not being correctly setup before it is painted
	Widget->SlatePrepass();

	Children.Add( Widget );

	return FWidgetLayoutBlock::Create( SharedThis( this ), Widget, FTextRange( StartIndex, EndIndex ), Size, TextContext, Renderer );
}

void FSlateHyperlinkRun::OnNavigate()
{
	NavigateDelegate.Execute( RunInfo.MetaData );
}

int32 FSlateHyperlinkRun::OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const TSharedRef< FWidgetLayoutBlock > WidgetBlock = StaticCastSharedRef< FWidgetLayoutBlock >(TextArgs.Block);

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	const FGeometry WidgetGeometry = AllottedGeometry.MakeChild(TransformVector(InverseScale, TextArgs.Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, TextArgs.Block->GetLocationOffset())));
	return WidgetBlock->GetWidget()->Paint(PaintArgs, WidgetGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

const TArray< TSharedRef<SWidget> >& FSlateHyperlinkRun::GetChildren()
{
	return Children;
}

void FSlateHyperlinkRun::ArrangeChildren( const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const 
{
	const TSharedRef< FWidgetLayoutBlock > WidgetBlock = StaticCastSharedRef< FWidgetLayoutBlock >( Block );
	
	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	ArrangedChildren.AddWidget(
		AllottedGeometry.MakeChild(WidgetBlock->GetWidget(), TransformVector(InverseScale, Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, Block->GetLocationOffset())))
		);
}

int32 FSlateHyperlinkRun::GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint ) const
{
	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FVector2D& BlockSize = Block->GetSize();

	const float Left = BlockOffset.X;
	const float Top = BlockOffset.Y;
	const float Right = BlockOffset.X + BlockSize.X;
	const float Bottom = BlockOffset.Y + BlockSize.Y;

	const bool ContainsPoint = Location.X >= Left && Location.X < Right && Location.Y >= Top && Location.Y < Bottom;

	if ( !ContainsPoint )
	{
		return INDEX_NONE;
	}

	const FTextRange BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	const int32 Index = ShapedTextCacheUtil::FindCharacterIndexAtOffset(BlockTextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, BlockTextContext, Style.TextStyle.Font), BlockRange, **Text, Location.X - BlockOffset.X);
	if (OutHitPoint)
	{
		*OutHitPoint = RunUtils::CalculateTextHitPoint(Index, BlockRange, BlockTextContext.TextDirection);
	}

	return Index;
}

FVector2D FSlateHyperlinkRun::GetLocationAt( const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale ) const
{
	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FTextRange& BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	const FTextRange RangeToMeasure = RunUtils::CalculateOffsetMeasureRange(Offset, BlockRange, BlockTextContext.TextDirection);
	const FVector2D OffsetLocation = ShapedTextCacheUtil::MeasureShapedText(BlockTextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, BlockTextContext, Style.TextStyle.Font), RangeToMeasure, **Text);

	return BlockOffset + OffsetLocation;
}

void FSlateHyperlinkRun::Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange)
{
	Text = NewText;
	Range = NewRange;
}

TSharedRef<IRun> FSlateHyperlinkRun::Clone() const
{
	return FSlateHyperlinkRun::Create(RunInfo, Text, Style, NavigateDelegate, TooltipDelegate, TooltipTextDelegate, Range);
}

void FSlateHyperlinkRun::AppendTextTo(FString& AppendToText) const
{
	AppendToText.Append(**Text + Range.BeginIndex, Range.Len());
}

void FSlateHyperlinkRun::AppendTextTo(FString& AppendToText, const FTextRange& PartialRange) const
{
	check(Range.BeginIndex <= PartialRange.BeginIndex);
	check(Range.EndIndex >= PartialRange.EndIndex);

	AppendToText.Append(**Text + PartialRange.BeginIndex, PartialRange.Len());
}

const FRunInfo& FSlateHyperlinkRun::GetRunInfo() const
{
	return RunInfo;
}

ERunAttributes FSlateHyperlinkRun::GetRunAttributes() const
{
	return ERunAttributes::SupportsText;
}

FSlateHyperlinkRun::FSlateHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate ) 
	: RunInfo( InRunInfo )
	, Text( InText )
	, Range( 0, Text->Len() )
	, Style( InStyle )
	, NavigateDelegate( InNavigateDelegate )
	, TooltipDelegate( InTooltipDelegate )
	, TooltipTextDelegate( InTooltipTextDelegate )
	, ViewModel( MakeShareable( new FSlateHyperlinkRun::FWidgetViewModel() ) )
	, Children()
{

}

FSlateHyperlinkRun::FSlateHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange ) 
	: RunInfo( InRunInfo )
	, Text( InText )
	, Range( InRange )
	, Style( InStyle )
	, NavigateDelegate( InNavigateDelegate )
	, TooltipDelegate( InTooltipDelegate )
	, TooltipTextDelegate( InTooltipTextDelegate )
	, ViewModel( MakeShareable( new FSlateHyperlinkRun::FWidgetViewModel() ) )
	, Children()
{

}

FSlateHyperlinkRun::FSlateHyperlinkRun( const FSlateHyperlinkRun& Run ) 
	: RunInfo( Run.RunInfo )
	, Text( Run.Text )
	, Range( Run.Range )
	, Style( Run.Style )
	, NavigateDelegate( Run.NavigateDelegate )
	, TooltipDelegate( Run.TooltipDelegate )
	, TooltipTextDelegate( Run.TooltipTextDelegate )
	, ViewModel( MakeShareable( new FSlateHyperlinkRun::FWidgetViewModel() ) )
	, Children()
{

}



#endif //WITH_FANCY_TEXT
