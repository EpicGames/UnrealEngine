// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateImageRun.h"
#include "Styling/StyleDefaults.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/DefaultLayoutBlock.h"

#if WITH_FANCY_TEXT

#include "Framework/Text/RunUtils.h"

TSharedRef< FSlateImageRun > FSlateImageRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FSlateBrush* InImage, const TAttribute<int16>& InBaseline )
{
	if ( InImage == nullptr)
	{
		InImage = FStyleDefaults::GetNoBrush();
	}

	return MakeShareable( new FSlateImageRun( InRunInfo, InText, InImage, InBaseline ) );
}

TSharedRef< FSlateImageRun > FSlateImageRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FSlateBrush* InImage, const TAttribute<int16>& InBaseline, const FTextRange& InRange )
{
	if ( InImage == nullptr)
	{
		InImage = FStyleDefaults::GetNoBrush();
	}

	return MakeShareable( new FSlateImageRun( InRunInfo, InText, InImage, InBaseline, InRange ) );
}

TSharedRef< FSlateImageRun > FSlateImageRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, FName InDynamicBrushName, const TAttribute<int16>& InBaseline )
{
	return MakeShareable( new FSlateImageRun( InRunInfo, InText, InDynamicBrushName, InBaseline ) );
}

TSharedRef< FSlateImageRun > FSlateImageRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, FName InDynamicBrushName, const TAttribute<int16>& InBaseline, const FTextRange& InRange )
{
	return MakeShareable( new FSlateImageRun( InRunInfo, InText, InDynamicBrushName, InBaseline, InRange ) );
}

FSlateImageRun::FSlateImageRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FSlateBrush* InImage, const TAttribute<int16>& InBaseline, const FTextRange& InRange )
	: RunInfo( InRunInfo )
	, Text( InText )
	, Range( InRange )
	, Image( InImage )
	, Baseline( InBaseline )
{
	check( Image != nullptr);
}

FSlateImageRun::FSlateImageRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FSlateBrush* InImage, const TAttribute<int16>& InBaseline )
	: RunInfo( InRunInfo )
	, Text( InText )
	, Range( 0, Text->Len() )
	, Image( InImage )
	, Baseline( InBaseline )
{
	check( Image != nullptr);
}

FSlateImageRun::FSlateImageRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, FName InDynamicBrushName, const TAttribute<int16>& InBaseline )
	: RunInfo( InRunInfo )
	, Text( InText )
	, Range( 0, Text->Len() )
	, Image( nullptr )
	, Baseline( InBaseline )
{
	FIntPoint Size = FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(InDynamicBrushName);
	DynamicBrush = MakeShareable(new FSlateDynamicImageBrush( InDynamicBrushName, FVector2f(Size.X, Size.Y) ) );
	Image = DynamicBrush.Get();
}

FSlateImageRun::FSlateImageRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, FName InDynamicBrushName, const TAttribute<int16>& InBaseline, const FTextRange& InRange )
	: RunInfo( InRunInfo )
	, Text( InText )
	, Range( InRange )
	, Image( nullptr )
	, Baseline( InBaseline )
{
	FIntPoint Size = FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(InDynamicBrushName);
	DynamicBrush = MakeShareable(new FSlateDynamicImageBrush( InDynamicBrushName, FVector2f(Size.X, Size.Y) ) );
	Image = DynamicBrush.Get();
}

FSlateImageRun::~FSlateImageRun()
{
	if (DynamicBrush.IsValid())
	{
		DynamicBrush->ReleaseResource();
	}
}

const TArray< TSharedRef<SWidget> >& FSlateImageRun::GetChildren() 
{
	static TArray< TSharedRef<SWidget> > NoChildren;
	return NoChildren;
}

void FSlateImageRun::ArrangeChildren( const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const 
{
	// no widgets
}

int32 FSlateImageRun::GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint ) const
{
	// An image should always contain a single character (a breaking space)
	check(Range.Len() == 1);

	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FVector2D& BlockSize = Block->GetSize();

	const float Left = BlockOffset.X;
	const float Top = BlockOffset.Y;
	const float Right = BlockOffset.X + BlockSize.X;
	const float Bottom = BlockOffset.Y + BlockSize.Y;

	const bool ContainsPoint = Location.X >= Left && Location.X < Right && Location.Y >= Top && Location.Y < Bottom;

	if ( !ContainsPoint )
	{
		return INDEX_NONE;
	}

	const FVector2f ScaledImageSize = Image->ImageSize * Scale;
	const int32 Index = (Location.X <= (Left + (ScaledImageSize.X * 0.5f))) ? Range.BeginIndex : Range.EndIndex;
	
	if (OutHitPoint)
	{
		const FTextRange BlockRange = Block->GetTextRange();
		const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

		// The block for an image will always detect a LTR reading direction, so use the base direction (of the line) for the image hit-point detection
		*OutHitPoint = RunUtils::CalculateTextHitPoint(Index, BlockRange, BlockTextContext.BaseDirection);
	}

	return Index;
}

FVector2D FSlateImageRun::GetLocationAt( const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale ) const
{
	return Block->GetLocationOffset();
}

int32 FSlateImageRun::OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	if (Image->DrawAs != ESlateBrushDrawType::NoDrawType)
	{
		const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * Image->GetTint(InWidgetStyle));
		const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, TextArgs.Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, TextArgs.Block->GetLocationOffset()))),
			Image,
			DrawEffects,
			FinalColorAndOpacity
		);
	}

	return LayerId;
}

TSharedRef< ILayoutBlock > FSlateImageRun::CreateBlock( int32 BeginIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer )
{
	return FDefaultLayoutBlock::Create( SharedThis( this ), FTextRange( BeginIndex, EndIndex ), Size, TextContext, Renderer );
}

int8 FSlateImageRun::GetKerning( int32 CurrentIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	return 0;
}

FVector2D FSlateImageRun::Measure( int32 BeginIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	if ( EndIndex - BeginIndex == 0 )
	{
		return FVector2D( 0, GetMaxHeight( Scale ) );
	}

	return FVector2D(Image->ImageSize * Scale);
}

int16 FSlateImageRun::GetMaxHeight( float Scale ) const 
{
	return Image->ImageSize.Y * Scale;
}

int16 FSlateImageRun::GetBaseLine( float Scale ) const 
{
	return Baseline.Get(0) * Scale;
}

FTextRange FSlateImageRun::GetTextRange() const 
{
	return Range;
}

void FSlateImageRun::SetTextRange( const FTextRange& Value )
{
	Range = Value;
}

void FSlateImageRun::Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange)
{
	Text = NewText;
	Range = NewRange;
}

TSharedRef<IRun> FSlateImageRun::Clone() const
{
	TSharedRef<FSlateImageRun> NewRun = FSlateImageRun::Create(RunInfo, Text, Image, Baseline, Range);

	// make sure we copy the dynamic brush so it doesnt get released
	NewRun->DynamicBrush = DynamicBrush;

	return NewRun;
}

void FSlateImageRun::AppendTextTo(FString& AppendToText) const
{
	AppendToText.Append(**Text + Range.BeginIndex, Range.Len());
}

void FSlateImageRun::AppendTextTo(FString& AppendToText, const FTextRange& PartialRange) const
{
	check(Range.BeginIndex <= PartialRange.BeginIndex);
	check(Range.EndIndex >= PartialRange.EndIndex);

	AppendToText.Append(**Text + PartialRange.BeginIndex, PartialRange.Len());
}

const FRunInfo& FSlateImageRun::GetRunInfo() const
{
	return RunInfo;
}

ERunAttributes FSlateImageRun::GetRunAttributes() const
{
	return ERunAttributes::None;
}

#endif //WITH_FANCY_TEXT
