// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlatePasswordRun.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/RunUtils.h"

TSharedRef< FSlatePasswordRun > FSlatePasswordRun::Create(const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style)
{
	return MakeShareable(new FSlatePasswordRun(InRunInfo, InText, Style));
}

TSharedRef< FSlatePasswordRun > FSlatePasswordRun::Create(const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style, const FTextRange& InRange)
{
	return MakeShareable(new FSlatePasswordRun(InRunInfo, InText, Style, InRange));
}

FVector2D FSlatePasswordRun::Measure(int32 BeginIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext) const
{
	const FVector2D ShadowOffsetToApply((EndIndex == Range.EndIndex) ? FMath::Abs(Style.ShadowOffset.X * Scale) : 0.0f, FMath::Abs(Style.ShadowOffset.Y * Scale));

	if ( EndIndex - BeginIndex == 0 )
	{
		return FVector2D( ShadowOffsetToApply.X * Scale, GetMaxHeight( Scale ) );
	}

	// We draw the password text, so that's what we need to measure
	const FString PasswordString = BuildPasswordString(EndIndex - BeginIndex);
	return FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(PasswordString, Style.Font, Scale) + ShadowOffsetToApply;
}

int8 FSlatePasswordRun::GetKerning(int32 CurrentIndex, float Scale, const FRunTextContext& TextContext) const
{
	const int32 PreviousIndex = CurrentIndex - 1;
	if ( PreviousIndex < 0 || CurrentIndex == Text->Len() )
	{
		return 0;
	}

	return FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->GetKerning(Style.Font, Scale, GetPasswordChar(), GetPasswordChar());
}

int32 FSlatePasswordRun::OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const bool ShouldDropShadow = Style.ShadowColorAndOpacity.A > 0.f && Style.ShadowOffset.SizeSquared() > 0.f;
	const FVector2D BlockLocationOffset = TextArgs.Block->GetLocationOffset();
	const FTextRange BlockRange = TextArgs.Block->GetTextRange();

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	// A negative shadow offset should be applied as a positive offset to the text to avoid clipping issues
	const FVector2D DrawShadowOffset(
		(Style.ShadowOffset.X > 0.0f) ? Style.ShadowOffset.X * AllottedGeometry.Scale : 0.0f,
		(Style.ShadowOffset.Y > 0.0f) ? Style.ShadowOffset.Y * AllottedGeometry.Scale : 0.0f
	);
	const FVector2D DrawTextOffset(
		(Style.ShadowOffset.X < 0.0f) ? -Style.ShadowOffset.X * AllottedGeometry.Scale : 0.0f,
		(Style.ShadowOffset.Y < 0.0f) ? -Style.ShadowOffset.Y * AllottedGeometry.Scale : 0.0f
	);

	const FString PasswordString = BuildPasswordString(BlockRange.Len());

	// Draw the optional shadow
	if (ShouldDropShadow)
	{
		FSlateDrawElement::MakeText(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, TextArgs.Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, TextArgs.Block->GetLocationOffset() + DrawShadowOffset))),
			PasswordString,
			0,
			PasswordString.Len(),
			TextArgs.DefaultStyle.Font,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * Style.ShadowColorAndOpacity
		);
	}

	// Draw the text itself
	FSlateDrawElement::MakeText(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, TextArgs.Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, TextArgs.Block->GetLocationOffset() + DrawTextOffset))),
		PasswordString,
		0,
		PasswordString.Len(),
		TextArgs.DefaultStyle.Font,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * Style.ColorAndOpacity.GetColor(InWidgetStyle)
	);

	return LayerId;
}

int32 FSlatePasswordRun::GetTextIndexAt(const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint) const
{
	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FVector2D& BlockSize = Block->GetSize();

	const float Left = BlockOffset.X;
	const float Top = BlockOffset.Y;
	const float Right = BlockOffset.X + BlockSize.X;
	const float Bottom = BlockOffset.Y + BlockSize.Y;

	const bool ContainsPoint = Location.X >= Left && Location.X < Right && Location.Y >= Top && Location.Y < Bottom;

	if ( !ContainsPoint )
	{
		return INDEX_NONE;
	}

	const FTextRange BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	const FString PasswordString = BuildPasswordString(BlockRange.Len());

	int32 Index = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->FindCharacterIndexAtOffset(PasswordString, Style.Font, Location.X - BlockOffset.X, Scale);
	if (BlockTextContext.TextDirection == TextBiDi::ETextDirection::RightToLeft)
	{
		Index = PasswordString.Len() - Index;
	}
	Index += BlockRange.BeginIndex;

	if (OutHitPoint)
	{
		*OutHitPoint = RunUtils::CalculateTextHitPoint(Index, BlockRange, BlockTextContext.TextDirection);
	}

	return Index;
}

FVector2D FSlatePasswordRun::GetLocationAt(const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale) const
{
	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FTextRange& BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	const FTextRange RangeToMeasure = RunUtils::CalculateOffsetMeasureRange(Offset, BlockRange, BlockTextContext.TextDirection);

	// We draw the password text, so that's what we need to measure
	const FString PasswordString = BuildPasswordString(RangeToMeasure.Len());
	const FVector2D OffsetLocation = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(PasswordString, Style.Font, Scale);

	return BlockOffset + OffsetLocation;
}

TSharedRef<IRun> FSlatePasswordRun::Clone() const
{
	return FSlatePasswordRun::Create(RunInfo, Text, Style, Range);
}

FSlatePasswordRun::FSlatePasswordRun(const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle)
	: FSlateTextRun(InRunInfo, InText, InStyle)
{

}

FSlatePasswordRun::FSlatePasswordRun(const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle, const FTextRange& InRange)
	: FSlateTextRun(InRunInfo, InText, InStyle, InRange)
{

}

FSlatePasswordRun::FSlatePasswordRun(const FSlatePasswordRun& Run)
	: FSlateTextRun(Run)
{

}

TCHAR FSlatePasswordRun::GetPasswordChar()
{
	return TEXT('\u2022');
}

FString FSlatePasswordRun::BuildPasswordString(const int32 InLength)
{
	const TCHAR PasswordChar = GetPasswordChar();
	FString PasswordString;
	
	if (InLength > 0)
	{
		PasswordString.Reserve(InLength);

		for (int32 Index = 0; Index < InLength; ++Index)
		{
			PasswordString += PasswordChar;
		}
	}

	return PasswordString;
}
