// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateTextHighlightRunRenderer.h"
#include "Rendering/DrawElements.h"
#include "Styling/SlateTypes.h"

FSlateTextHighlightRunRenderer::FSlateTextHighlightRunRenderer()
{

}

int32 FSlateTextHighlightRunRenderer::OnPaint( const FPaintArgs& Args, const FTextLayout::FLineView& Line, const TSharedRef< ISlateRun >& Run, const TSharedRef< ILayoutBlock >& Block, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const 
{
	FVector2D Location( Block->GetLocationOffset() );
	Location.Y = Line.Offset.Y;

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	// Draw the actual highlight rectangle
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, FVector2D(Block->GetSize().X, Line.Size.Y)), FSlateLayoutTransform(TransformPoint(InverseScale, Location))),
		&DefaultStyle.HighlightShape,
		bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		InWidgetStyle.GetColorAndOpacityTint() * DefaultStyle.HighlightShape.TintColor.GetColor(InWidgetStyle)
		);

	FLinearColor InvertedHighlightColor = DefaultStyle.HighlightColor.GetColor(InWidgetStyle);
	InvertedHighlightColor.A = InWidgetStyle.GetForegroundColor().A;

	FWidgetStyle WidgetStyle( InWidgetStyle );
	WidgetStyle.SetForegroundColor( InvertedHighlightColor );

	// When highlighting text we want the actual text so do not replace any of it with ellipsis
	const ETextOverflowPolicy OverflowPolicy = ETextOverflowPolicy::Clip;

	return Run->OnPaint( Args, FTextArgs(Line, Block, DefaultStyle, OverflowPolicy, ETextOverflowDirection::NoOverflow), AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, WidgetStyle, bParentEnabled );
}

TSharedRef< FSlateTextHighlightRunRenderer > FSlateTextHighlightRunRenderer::Create()
{
	return MakeShareable( new FSlateTextHighlightRunRenderer() );
}
