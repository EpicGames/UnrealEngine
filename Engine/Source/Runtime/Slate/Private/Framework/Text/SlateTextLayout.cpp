// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateTextLayout.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ISlateRunRenderer.h"
#include "Framework/Text/ISlateLineHighlighter.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateTextUtils.h"
#include "Framework/Text/SlatePasswordRun.h"
#include "Trace/SlateMemoryTags.h"

TSharedRef< FSlateTextLayout > FSlateTextLayout::Create(SWidget* InOwner, FTextBlockStyle InDefaultTextStyle)
{
	LLM_SCOPE_BYTAG(UI_Text);
	TSharedRef< FSlateTextLayout > Layout = MakeShareable( new FSlateTextLayout(InOwner, MoveTemp(InDefaultTextStyle)) );
	Layout->AggregateChildren();

	return Layout;
}

FSlateTextLayout::FSlateTextLayout(SWidget* InOwner, FTextBlockStyle InDefaultTextStyle)
	: DefaultTextStyle(MoveTemp(InDefaultTextStyle))
	, Children(InOwner, false)
	, bIsPassword(false)
	, FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
{

}

FChildren* FSlateTextLayout::GetChildren()
{
	return &Children;
}

void FSlateTextLayout::ArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	for (int32 LineIndex = 0; LineIndex < LineViews.Num(); LineIndex++)
	{
		const FTextLayout::FLineView& LineView = LineViews[ LineIndex ];

		for (int32 BlockIndex = 0; BlockIndex < LineView.Blocks.Num(); BlockIndex++)
		{
			const TSharedRef< ILayoutBlock > Block = LineView.Blocks[ BlockIndex ];
			const TSharedRef< ISlateRun > Run = StaticCastSharedRef< ISlateRun >( Block->GetRun() );
			Run->ArrangeChildren( Block, AllottedGeometry, ArrangedChildren );
		}
	}
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

int32 GShowTextDebugging = 0;
static FAutoConsoleVariableRef CVarSlateShowTextDebugging(TEXT("Slate.ShowTextDebugging"), GShowTextDebugging, TEXT("Show debugging painting for text rendering."), ECVF_Default);

#endif

int32 FSlateTextLayout::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// The layout pass would have already started shaping all of the text in this text layout and caching the shaped text as part of measurement 
	// We should know by this point whether any of the text runs are waiting on a font to load
	if (AreAnyFontFacesLoading())
	{
		if (GetFontFacesLoadingPaintPolicy() == EFontFacesLoadingPaintPolicy::DoNotPaint)
		{
			return LayerId;
		}
	}

	const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FLinearColor BlockDebugHue( 0, 1.0f, 1.0f, 0.5 );
#endif

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	int32 HighestLayerId = LayerId;

	auto IsAtLeastPartiallyVisible = [&AllottedGeometry, &InverseScale, &MyCullingRect](FVector2D Offset, FVector2D Size)
	{
		// Is this line visible?  This checks if the culling rect, which represents the AABB around the last clipping rect, intersects the 
		// line of text, this requires that we get the text line into render space.
		// TODO perhaps save off this line view rect during text layout?
		const FVector2D LocalLineOffset = Offset * InverseScale;
		FSlateRect Rect(AllottedGeometry.GetRenderBoundingRect(FSlateRect(LocalLineOffset, LocalLineOffset + (Size * InverseScale))));

		return FSlateRect::DoRectanglesIntersect(Rect, MyCullingRect);
	};

	auto IsFullyVisibleOnVerticalAxis = [&AllottedGeometry, &InverseScale, &MyCullingRect](FVector2D Offset, FVector2D Size)
	{
		const FVector2D LocalLineOffset = Offset * InverseScale;
		FSlateRect Rect(AllottedGeometry.GetRenderBoundingRect(FSlateRect(LocalLineOffset, LocalLineOffset + (Size * InverseScale))));

		//Clamp on the horizontal axis because we care only about vertical one.
		Rect.Left = MyCullingRect.Left;
		Rect.Right = MyCullingRect.Right;
		return FSlateRect::IsRectangleContained(MyCullingRect, Rect);
	};


	const ETextOverflowPolicy OverflowPolicy = TextOverflowPolicyOverride.Get(DefaultTextStyle.OverflowPolicy);
	const bool bIsMultiline = LineViews.Num() > 1;

	int32 LineToDisplayNum = LineViews.Num();
	int32 LastLineIndexToDisplay = 0;
	if (bIsMultiline)
	{
		if (OverflowPolicy == ETextOverflowPolicy::MultilineEllipsis)
		{
			for (int32 LineIndex = 0; LineIndex < LineViews.Num(); ++LineIndex)
			{	//Depending on their length, some lines can be invisible, but following ones could be visible, so we have to parse all of them to get the real last one.
				if (IsFullyVisibleOnVerticalAxis(LineViews[LineIndex].Offset, LineViews[LineIndex].Size))
				{
					LastLineIndexToDisplay = LineIndex;
				}
			}
			LineToDisplayNum = LastLineIndexToDisplay + 1;
		}
		else if (OverflowPolicy == ETextOverflowPolicy::Ellipsis)
		{
			for (int32 LineIndex = 0; LineIndex < LineViews.Num(); ++LineIndex)
			{	//Depending on their length, some lines can be invisible, but following ones could be visible, so we have to parse all of them to get the real last one.
				if (IsAtLeastPartiallyVisible(LineViews[LineIndex].Offset, LineViews[LineIndex].Size))
				{
					LastLineIndexToDisplay = LineIndex;
				}
			}
			LineToDisplayNum = LastLineIndexToDisplay + 1;
		}
		else
		{
			LastLineIndexToDisplay = LineToDisplayNum - 1;
		}
	}

	for (int32 LineIndex = 0; LineIndex < LineToDisplayNum; ++LineIndex)
	{
		const FTextLayout::FLineView& LineView = LineViews[LineIndex];
		ETextOverflowPolicy LineOverflowPolicy = OverflowPolicy;
		ETextOverflowDirection LineOverflowDirection = ETextOverflowDirection::NoOverflow;

		if (!IsAtLeastPartiallyVisible(LineView.Offset, LineView.Size))
		{
			continue;
		}

		// Render any underlays for this line
		const int32 HighestUnderlayLayerId = OnPaintHighlights( Args, LineView, LineView.UnderlayHighlights, DefaultTextStyle, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

		const int32 BlockDebugLayer = HighestUnderlayLayerId;
		const int32 TextLayer = BlockDebugLayer + 1;
		int32 HighestBlockLayerId = TextLayer;

		bool bIsLastVisibleLine = false;
		bool bIsNextLineClipped = false;
		if (SlateTextUtils::IsEllipsisPolicy(LineOverflowPolicy))
		{
			// Middle Ellipsis currently do not support multiline
			if (bIsMultiline)
			{
				if (LineOverflowPolicy != ETextOverflowPolicy::MiddleEllipsis)
				{
					bIsLastVisibleLine = LineIndex == LastLineIndexToDisplay;
					bIsNextLineClipped = LineViews.IsValidIndex(LineIndex + 1) ? bIsLastVisibleLine : false;

					//When wrapping/multiline text, we have to use the reading direction of the text, not the justification.
					LineOverflowDirection = LineView.TextBaseDirection == TextBiDi::ETextDirection::LeftToRight ? ETextOverflowDirection::LeftToRight : ETextOverflowDirection::RightToLeft;
				}
			}
			else
			{
				bIsLastVisibleLine = true;
				const ETextJustify::Type VisualJustification = CalculateLineViewVisualJustification(LineView);
				LineOverflowDirection = VisualJustification == ETextJustify::Left ? ETextOverflowDirection::LeftToRight : (VisualJustification == ETextJustify::Right ? ETextOverflowDirection::RightToLeft : ETextOverflowDirection::NoOverflow);
			}
		}

		int32 LastVisibleBlockIndex = -1;
		int32 NextBlockOffset = 0;
		if (bIsLastVisibleLine)
		{
			int32 StartValue = 0;
			int32 EndValue = 0;
			if (LineOverflowDirection == ETextOverflowDirection::LeftToRight)
			{
				EndValue = LineView.Blocks.Num();
				NextBlockOffset = 1;
			}
			else if (LineOverflowDirection == ETextOverflowDirection::RightToLeft)
			{
				StartValue = LineView.Blocks.Num() - 1;
				EndValue = -1;
				NextBlockOffset = -1;
			}

			for (int32 BlockIndex = StartValue; BlockIndex != EndValue; BlockIndex += NextBlockOffset)
			{	//Depending on their position, some blocks can be invisible, but a following one could be visible, so we have to parse all of them.
				const TSharedRef< ILayoutBlock >& Block = LineView.Blocks[BlockIndex];
				if (IsAtLeastPartiallyVisible(Block->GetLocationOffset(), Block->GetSize()))
				{
					LastVisibleBlockIndex = BlockIndex;
				}
			}
		}
		else
		{
			LineOverflowDirection = ETextOverflowDirection::NoOverflow;
			LineOverflowPolicy = ETextOverflowPolicy::Clip;
		}

		// Render every block for this line
		for (int32 BlockIndex = 0; BlockIndex < LineView.Blocks.Num(); ++BlockIndex)
		{
			const TSharedRef< ILayoutBlock >& Block = LineView.Blocks[BlockIndex];
			if (!IsAtLeastPartiallyVisible(Block->GetLocationOffset(), Block->GetSize()))
			{
				continue;
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if ( GShowTextDebugging)
			{
				BlockDebugHue.R += 50.0f;

				FSlateDrawElement::MakeBox(
					OutDrawElements, 
					BlockDebugLayer,
					AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, Block->GetLocationOffset()))),
					&DefaultTextStyle.HighlightShape,
					DrawEffects,
					InWidgetStyle.GetColorAndOpacityTint() * BlockDebugHue.HSVToLinearRGB()
					);
			}
#endif

			FTextArgs TextArgs(LineView, Block, DefaultTextStyle, LineOverflowPolicy, LineOverflowDirection);
			TextArgs.bIsLastVisibleBlock = bIsLastVisibleLine && BlockIndex == LastVisibleBlockIndex;
			TextArgs.bIsNextBlockClipped = bIsNextLineClipped && (LineView.Blocks.IsValidIndex(BlockIndex + NextBlockOffset) ? TextArgs.bIsLastVisibleBlock : true);

			const TSharedRef< ISlateRun > Run = StaticCastSharedRef< ISlateRun >( Block->GetRun() );

			int32 HighestRunLayerId = TextLayer;
			const TSharedPtr< ISlateRunRenderer > RunRenderer = StaticCastSharedPtr< ISlateRunRenderer >( Block->GetRenderer() );
			if ( RunRenderer.IsValid() )
			{
				HighestRunLayerId = RunRenderer->OnPaint( Args, LineView, Run, Block, DefaultTextStyle, AllottedGeometry, MyCullingRect, OutDrawElements, TextLayer, InWidgetStyle, bParentEnabled );
			}
			else
			{
				HighestRunLayerId = Run->OnPaint( Args, TextArgs, AllottedGeometry, MyCullingRect, OutDrawElements, TextLayer, InWidgetStyle, bParentEnabled );
			}

			HighestBlockLayerId = FMath::Max( HighestBlockLayerId, HighestRunLayerId );
		}

		// Render any overlays for this line
		const int32 HighestOverlayLayerId = OnPaintHighlights(Args, LineView, LineView.OverlayHighlights, DefaultTextStyle, AllottedGeometry, MyCullingRect, OutDrawElements, HighestBlockLayerId, InWidgetStyle, bParentEnabled);
		HighestLayerId = FMath::Max(HighestLayerId, HighestOverlayLayerId);
	}

	return HighestLayerId;
}

int32 FSlateTextLayout::OnPaintHighlights( const FPaintArgs& Args, const FTextLayout::FLineView& LineView, const TArray<FLineViewHighlight>& Highlights, const FTextBlockStyle& InDefaultTextStyle, const FGeometry& AllottedGeometry, const FSlateRect& CullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 CurrentLayerId = LayerId;

	for (const FLineViewHighlight& Highlight : Highlights)
	{
		const TSharedPtr< ISlateLineHighlighter > LineHighlighter = StaticCastSharedPtr< ISlateLineHighlighter >( Highlight.Highlighter );
		if (LineHighlighter.IsValid())
		{
			CurrentLayerId = LineHighlighter->OnPaint( Args, LineView, Highlight.Offset, Highlight.Width, InDefaultTextStyle, AllottedGeometry, CullingRect, OutDrawElements, CurrentLayerId, InWidgetStyle, bParentEnabled );
		}
	}

	return CurrentLayerId;
}

void FSlateTextLayout::EndLayout()
{
	FTextLayout::EndLayout();
	AggregateChildren();
}

void FSlateTextLayout::SetDefaultTextStyle(FTextBlockStyle InDefaultTextStyle)
{
	DefaultTextStyle = MoveTemp(InDefaultTextStyle);
}

const FTextBlockStyle& FSlateTextLayout::GetDefaultTextStyle() const
{
	return DefaultTextStyle;
}

void FSlateTextLayout::SetIsPassword(const TAttribute<bool>& InIsPassword)
{
	bIsPassword = InIsPassword;
}

void FSlateTextLayout::AggregateChildren()
{
	Children.Empty();
	const TArray< FLineModel >& LayoutLineModels = GetLineModels();
	for (int32 LineModelIndex = 0; LineModelIndex < LayoutLineModels.Num(); LineModelIndex++)
	{
		const FLineModel& LineModel = LayoutLineModels[ LineModelIndex ];
		for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
		{
			const FRunModel& LineRun = LineModel.Runs[ RunIndex ];
			const TSharedRef< ISlateRun > SlateRun = StaticCastSharedRef< ISlateRun >( LineRun.GetRun() );

			const TArray< TSharedRef<SWidget> >& RunChildren = SlateRun->GetChildren();
			for (int32 ChildIndex = 0; ChildIndex < RunChildren.Num(); ChildIndex++)
			{
				const TSharedRef< SWidget >& Child = RunChildren[ ChildIndex ];
				Children.Add( Child );
			}
		}
	}
}

TSharedRef<IRun> FSlateTextLayout::CreateDefaultTextRun(const TSharedRef<FString>& NewText, const FTextRange& NewRange) const
{
	if (bIsPassword.Get(false))
	{
		return FSlatePasswordRun::Create(FRunInfo(), NewText, DefaultTextStyle, NewRange);
	}
	return FSlateTextRun::Create(FRunInfo(), NewText, DefaultTextStyle, NewRange);
}

void FSlateTextLayout::SetOnAllFontFacesFinishLoading(const FSimpleDelegate& InAllFontFacesFinishLoading)
{
	SWidget& OwnerWidget = Children.GetOwner();
	TSharedPtr< FSlateTextLayoutPendingFontFacesLoadingMetaData> MetaData = OwnerWidget.GetMetaData<FSlateTextLayoutPendingFontFacesLoadingMetaData>();
	if (MetaData)
	{
		MetaData->OnAllFontFacesFinishLoading = InAllFontFacesFinishLoading;
	}
	// No point creating a new metadata and adding it to the widget if the passed in delegate isn't bound 
	else if (InAllFontFacesFinishLoading.IsBound())
	{
		TSharedRef<FSlateTextLayoutPendingFontFacesLoadingMetaData> NewMetaData = MakeShared<FSlateTextLayoutPendingFontFacesLoadingMetaData>();
		NewMetaData->OnAllFontFacesFinishLoading = InAllFontFacesFinishLoading;
		OwnerWidget.AddMetadata(NewMetaData);
	}
}

void FSlateTextLayout::HandleAllFontFacesFinishLoading()
{
	TSharedPtr<FSlateTextLayoutPendingFontFacesLoadingMetaData> MetaData = Children.GetOwner().GetMetaData<FSlateTextLayoutPendingFontFacesLoadingMetaData>();
	if (MetaData)
	{
		MetaData->OnAllFontFacesFinishLoading.ExecuteIfBound();
	}
}

EFontFacesLoadingPaintPolicy FSlateTextLayout::GetFontFacesLoadingPaintPolicy() const
{
	return FontFacesLoadingPaintPolicy;
}

void FSlateTextLayout::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	FontFacesLoadingPaintPolicy = InFontFacesLoadingPaintPolicy;
}
