// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateTextRun.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/DefaultLayoutBlock.h"
#include "Framework/Text/ShapedTextCache.h"
#include "Framework/Text/SlateTextUtils.h"
#include "Framework/Text/RunUtils.h"
#include "Fonts/ShapedTextFwd.h"

TSharedRef< FSlateTextRun > FSlateTextRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style )
{
	return MakeShareable( new FSlateTextRun( InRunInfo, InText, Style ) );
}

TSharedRef< FSlateTextRun > FSlateTextRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style, const FTextRange& InRange )
{
	return MakeShareable( new FSlateTextRun( InRunInfo, InText, Style, InRange ) );
}

FTextRange FSlateTextRun::GetTextRange() const 
{
	return Range;
}

void FSlateTextRun::SetTextRange( const FTextRange& Value )
{
	Range = Value;
}

int16 FSlateTextRun::GetBaseLine( float Scale ) const 
{
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	return FontMeasure->GetBaseline( Style.Font, Scale ) - ( FMath::Min(0.0f, Style.ShadowOffset.Y) + Style.Font.OutlineSettings.OutlineSize ) * Scale;
}

int16 FSlateTextRun::GetMaxHeight( float Scale ) const 
{
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	return FontMeasure->GetMaxCharacterHeight( Style.Font, Scale ) + ( FMath::Abs(Style.ShadowOffset.Y) + Style.Font.OutlineSettings.OutlineSize ) * Scale;
}

FVector2D FSlateTextRun::Measure( int32 BeginIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	const FVector2d OutlineSize = GetOutlineSize(BeginIndex, EndIndex, Scale);
	const FVector2d ShadowSize = GetShadowSize(BeginIndex, EndIndex, Scale);
	
	if (EndIndex - BeginIndex == 0)
	{
		return FVector2D(0, GetMaxHeight(Scale) + OutlineSize.Y + ShadowSize.Y) + OutlineSize + ShadowSize;
	}
	
	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	return ShapedTextCacheUtil::MeasureShapedText(TextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, TextContext, Style.Font), FTextRange(BeginIndex, EndIndex), **Text, TextContext.LoadingFontFaces) + OutlineSize + ShadowSize;
}

int8 FSlateTextRun::GetKerning(int32 CurrentIndex, float Scale, const FRunTextContext& TextContext) const
{
	const int32 PreviousIndex = CurrentIndex - 1;
	if ( PreviousIndex < 0 || CurrentIndex == Text->Len() )
	{
		return 0;
	}

	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	return ShapedTextCacheUtil::GetShapedGlyphKerning(TextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, TextContext, Style.Font), PreviousIndex, **Text, TextContext.LoadingFontFaces);
}

FVector2d FSlateTextRun::GetOutlineSize(int32 StartIndex, int32 EndIndex, float Scale) const
{
	// Offset the measured shaped text by the outline since the outline was not factored into the size of the text
	// Need to add the outline offsetting to the beginning and the end because it surrounds both sides.
	const float ScaledOutlineSize = Style.Font.OutlineSettings.OutlineSize * Scale;

	return FVector2D((StartIndex == Range.BeginIndex ? ScaledOutlineSize : 0) + (EndIndex == Range.EndIndex ? ScaledOutlineSize : 0), ScaledOutlineSize);
}

FVector2d FSlateTextRun::GetShadowSize(int32 StartIndex, int32 EndIndex, float Scale) const
{
	// For positive values of ShadowOffset, add the shadow's width if we're measuring the end of the text run
	// For negative offsets, add if we're measuring the beginning
	if ((Style.ShadowOffset.X > 0 && (EndIndex == Range.EndIndex)) || (Style.ShadowOffset.X < 0 && (StartIndex == Range.BeginIndex)))
	{
		return FVector2d(FMath::Abs(Style.ShadowOffset.X * Scale), FMath::Abs(Style.ShadowOffset.Y * Scale));
	}

	return FVector2d(0.0f, FMath::Abs(Style.ShadowOffset.Y * Scale));
}

TSharedRef< ILayoutBlock > FSlateTextRun::CreateBlock( int32 BeginIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer )
{
	return FDefaultLayoutBlock::Create( SharedThis( this ), FTextRange( BeginIndex, EndIndex ), Size, TextContext, Renderer );
}
int32 FSlateTextRun::OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (Text->IsEmpty())
	{
		return LayerId;
	}
#if !UE_BUILD_SHIPPING
	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.LogPaintedText"));
	if (CVar->GetBool())
	{
		UE_LOGF(LogSlate, Log, "FSlateTextRun: '%ls'.", **Text);
	}
#endif
	const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const TSharedRef<ILayoutBlock>& Block = TextArgs.Block;
	const FTextBlockStyle& DefaultStyle = TextArgs.DefaultStyle;
	const FTextLayout::FLineView& Line = TextArgs.Line;

	const bool ShouldDropShadow = Style.ShadowColorAndOpacity.A > 0.f && Style.ShadowOffset.SizeSquared() > 0.f;
	const FVector2D BlockLocationOffset = Block->GetLocationOffset();
	const FTextRange BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	// A negative shadow offset should be applied as a positive offset to the text to avoid clipping issues
	const FVector2D DrawShadowOffset(
		(Style.ShadowOffset.X > 0.0f) ? Style.ShadowOffset.X * AllottedGeometry.Scale : 0.0f,
		(Style.ShadowOffset.Y > 0.0f) ? Style.ShadowOffset.Y * AllottedGeometry.Scale : 0.0f
	);
	const FVector2D DrawTextOffset(
		(Style.ShadowOffset.X < 0.0f) ? -Style.ShadowOffset.X * AllottedGeometry.Scale : 0.0f,
		(Style.ShadowOffset.Y < 0.0f) ? -Style.ShadowOffset.Y * AllottedGeometry.Scale : 0.0f
	);

	// Make sure we have up-to-date shaped text to work with
	// We use the full line view range (rather than the run range) so that text that spans runs will still be shaped correctly
	FShapedGlyphSequenceRef ShapedText = ShapedTextCacheUtil::GetShapedTextSubSequence(
		BlockTextContext.ShapedTextCache,
		FCachedShapedTextKey(Line.Range, AllottedGeometry.GetAccumulatedLayoutTransform().GetScale(), BlockTextContext, Style.Font),
		BlockRange,
		**Text,
		BlockTextContext.TextDirection
	);

	FTextOverflowArgs OverflowArgs;
	if (SlateTextUtils::IsEllipsisPolicy(TextArgs.OverflowPolicy) && TextArgs.OverflowDirection != ETextOverflowDirection::NoOverflow)
	{
		OverflowArgs.OverflowTextPtr = BlockTextContext.ShapedTextCache->FindOrAddOverflowEllipsisText(AllottedGeometry.GetAccumulatedLayoutTransform().GetScale(), BlockTextContext, Style.Font);
		OverflowArgs.OverflowDirection = TextArgs.OverflowDirection;
		OverflowArgs.OverflowPolicy = TextArgs.OverflowPolicy;
		OverflowArgs.bIsLastVisibleBlock = TextArgs.bIsLastVisibleBlock;
		OverflowArgs.bIsNextBlockClipped = TextArgs.bIsNextBlockClipped;
	}

	// Draw the optional shadow
	if (ShouldDropShadow)
	{
		FShapedGlyphSequenceRef ShadowShapedText = ShapedText;
		if (Style.ShadowColorAndOpacity != Style.Font.OutlineSettings.OutlineColor)
		{
			// Copy font info for shadow to replace the outline color
			FSlateFontInfo ShadowFontInfo = Style.Font;
			ShadowFontInfo.OutlineSettings.OutlineColor = Style.ShadowColorAndOpacity;
			ShadowFontInfo.OutlineSettings.OutlineMaterial = nullptr;
			if (!ShadowFontInfo.OutlineSettings.bApplyOutlineToDropShadows)
			{
				ShadowFontInfo.OutlineSettings.OutlineSize = 0;
			}

			// Create new shaped text for drop shadow
			ShadowShapedText = ShapedTextCacheUtil::GetShapedTextSubSequence(
				BlockTextContext.ShapedTextCache,
				FCachedShapedTextKey(Line.Range, AllottedGeometry.GetAccumulatedLayoutTransform().GetScale(), BlockTextContext, ShadowFontInfo),
				BlockRange,
				**Text,
				BlockTextContext.TextDirection
				);
		}

		FSlateDrawElement::MakeShapedText(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, Block->GetLocationOffset() + DrawShadowOffset))),
			ShadowShapedText,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * Style.ShadowColorAndOpacity,
			InWidgetStyle.GetColorAndOpacityTint() * Style.Font.OutlineSettings.OutlineColor,
			OverflowArgs
		);
	}

	// Draw the text itself
	FSlateDrawElement::MakeShapedText(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, Block->GetLocationOffset() + DrawTextOffset))),
		ShapedText,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * Style.ColorAndOpacity.GetColor(InWidgetStyle),
		InWidgetStyle.GetColorAndOpacityTint() * Style.Font.OutlineSettings.OutlineColor,
		OverflowArgs
	);

	return LayerId;
}

const TArray< TSharedRef<SWidget> >& FSlateTextRun::GetChildren()
{
	static TArray< TSharedRef<SWidget> > NoChildren;
	return NoChildren;
}

void FSlateTextRun::ArrangeChildren( const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const 
{
	// no widgets
}

int32 FSlateTextRun::GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint ) const
{
	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FVector2D& BlockSize = Block->GetSize();

	const double Left = BlockOffset.X;
	const double Top = BlockOffset.Y;
	const double Right = BlockOffset.X + BlockSize.X;
	const double Bottom = BlockOffset.Y + BlockSize.Y;

	const bool ContainsPoint = Location.X >= Left && Location.X < Right && Location.Y >= Top && Location.Y < Bottom;

	if ( !ContainsPoint )
	{
		return INDEX_NONE;
	}

	const FTextRange BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	const int32 Index = ShapedTextCacheUtil::FindCharacterIndexAtOffset(BlockTextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, BlockTextContext, Style.Font), BlockRange, **Text, Location.X - BlockOffset.X);
	if (OutHitPoint)
	{
		*OutHitPoint = RunUtils::CalculateTextHitPoint(Index, BlockRange, BlockTextContext.TextDirection);
	}

	return Index;
}

FVector2D FSlateTextRun::GetLocationAt( const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale ) const
{
	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FTextRange& BlockRange = Block->GetTextRange();
	const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

	// Use the full text range (rather than the run range) so that text that spans runs will still be shaped correctly
	const FTextRange RangeToMeasure = RunUtils::CalculateOffsetMeasureRange(Offset, BlockRange, BlockTextContext.TextDirection);
	const FVector2D OffsetLocation = ShapedTextCacheUtil::MeasureShapedText(BlockTextContext.ShapedTextCache, FCachedShapedTextKey(FTextRange(0, Text->Len()), Scale, BlockTextContext, Style.Font), RangeToMeasure, **Text);

	return BlockOffset + OffsetLocation;
}

void FSlateTextRun::Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange)
{
	Text = NewText;
	Range = NewRange;

#if TEXT_LAYOUT_DEBUG
	DebugSlice = FString( InRange.EndIndex - InRange.BeginIndex, (**Text) + InRange.BeginIndex );
#endif
}

TSharedRef<IRun> FSlateTextRun::Clone() const
{
	return FSlateTextRun::Create(RunInfo, Text, Style, Range);
}

void FSlateTextRun::AppendTextTo(FString& AppendToText) const
{
	AppendToText.Append(**Text + Range.BeginIndex, Range.Len());
}

void FSlateTextRun::AppendTextTo(FString& AppendToText, const FTextRange& PartialRange) const
{
	check(Range.BeginIndex <= PartialRange.BeginIndex);
	check(Range.EndIndex >= PartialRange.EndIndex);

	AppendToText.Append(**Text + PartialRange.BeginIndex, PartialRange.Len());
}

const FRunInfo& FSlateTextRun::GetRunInfo() const
{
	return RunInfo;
}

ERunAttributes FSlateTextRun::GetRunAttributes() const
{
	return ERunAttributes::SupportsText;
}

FSlateTextRun::FSlateTextRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle ) 
	: RunInfo( InRunInfo )
	, Text( InText )
	, Style( InStyle )
	, Range( 0, Text->Len() )
#if TEXT_LAYOUT_DEBUG
	, DebugSlice( FString( Text->Len(), **Text ) )
#endif
{

}

FSlateTextRun::FSlateTextRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle, const FTextRange& InRange ) 
	: RunInfo( InRunInfo )
	, Text( InText )
	, Style( InStyle )
	, Range( InRange )
#if TEXT_LAYOUT_DEBUG
	, DebugSlice( FString( InRange.EndIndex - InRange.BeginIndex, (**Text) + InRange.BeginIndex ) )
#endif
{

}

FSlateTextRun::FSlateTextRun( const FSlateTextRun& Run ) 
	: RunInfo( Run.RunInfo )
	, Text( Run.Text )
	, Style( Run.Style )
	, Range( Run.Range )
#if TEXT_LAYOUT_DEBUG
	, DebugSlice( Run.DebugSlice )
#endif
{

}

void FSlateTextRun::ApplyFontSizeMultiplierOnTextStyle(float FontSizeMultiplier)
{
	if (FontSizeMultiplier != 0.0f)
	{
		Style.SetFontSize(Style.Font.Size * FontSizeMultiplier);
	}
}
