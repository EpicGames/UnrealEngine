// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateTextUnderlineLineHighlighter.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"

ISlateTextLineHighlighter::ISlateTextLineHighlighter(const FSlateBrush& InLineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity)
	: LineBrush(InLineBrush)
	, FontInfo(InFontInfo)
	, ColorAndOpacity(InColorAndOpacity)
	, ShadowOffset(InShadowOffset)
	, ShadowColorAndOpacity(InShadowColorAndOpacity)
{
}

int32 ISlateTextLineHighlighter::OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();

	const uint16 MaxHeight = FontCache->GetMaxCharacterHeight(FontInfo, AllottedGeometry.Scale);
	const int16 Baseline = FontCache->GetBaseline(FontInfo, AllottedGeometry.Scale);

	int16 LinePos, LineThickness;
	GetLineMetrics(AllottedGeometry.Scale, LinePos, LineThickness);

	const FVector2f Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y + MaxHeight + Baseline - (LinePos * 0.5f));
	const FVector2f Size(Width, FMath::Max<int16>(1, LineThickness));

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	if (Size.X)
	{
		const FLinearColor LineColorAndOpacity = ColorAndOpacity.GetColor(InWidgetStyle);

		const bool ShouldDropShadow = ShadowColorAndOpacity.A > 0.f && ShadowOffset.SizeSquared() > 0.f;

		// A negative shadow offset should be applied as a positive offset to the underline to avoid clipping issues
		const FVector2f DrawShadowOffset(
			(ShadowOffset.X > 0.0f) ? ShadowOffset.X * AllottedGeometry.Scale : 0.0f,
			(ShadowOffset.Y > 0.0f) ? ShadowOffset.Y * AllottedGeometry.Scale : 0.0f
			);
		const FVector2f DrawUnderlineOffset(
			(ShadowOffset.X < 0.0f) ? -ShadowOffset.X * AllottedGeometry.Scale : 0.0f,
			(ShadowOffset.Y < 0.0f) ? -ShadowOffset.Y * AllottedGeometry.Scale : 0.0f
			);

		// Draw the optional shadow
		if (ShouldDropShadow)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, Size), FSlateLayoutTransform(TransformPoint(InverseScale, Location + DrawShadowOffset))),
				&LineBrush,
				bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				ShadowColorAndOpacity * InWidgetStyle.GetColorAndOpacityTint()
				);
		}

		// Draw underline/strike
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, Size), FSlateLayoutTransform(TransformPoint(InverseScale, Location + DrawUnderlineOffset))),
			&LineBrush,
			bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			LineColorAndOpacity * InWidgetStyle.GetColorAndOpacityTint()
			);
	}

	return LayerId;
}

FSlateTextUnderlineLineHighlighter::FSlateTextUnderlineLineHighlighter(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity)
	: ISlateTextLineHighlighter(InUnderlineBrush, InFontInfo, InColorAndOpacity, InShadowOffset, InShadowColorAndOpacity)
{
}

TSharedRef<FSlateTextUnderlineLineHighlighter> FSlateTextUnderlineLineHighlighter::Create(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity)
{
	return MakeShareable(new FSlateTextUnderlineLineHighlighter(InUnderlineBrush, InFontInfo, InColorAndOpacity, InShadowOffset, InShadowColorAndOpacity));
}

void FSlateTextUnderlineLineHighlighter::GetLineMetrics(const float InFontScale, int16& OutLinePos, int16& OutLineThickness) const
{
	TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();
	FontCache->GetUnderlineMetrics(FontInfo, InFontScale, OutLinePos, OutLineThickness);
}

FSlateTextStrikeLineHighlighter::FSlateTextStrikeLineHighlighter(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity)
	: ISlateTextLineHighlighter(InUnderlineBrush, InFontInfo, InColorAndOpacity, InShadowOffset, InShadowColorAndOpacity)
{
}

TSharedRef<FSlateTextStrikeLineHighlighter> FSlateTextStrikeLineHighlighter::Create(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity)
{
	return MakeShareable(new FSlateTextStrikeLineHighlighter(InUnderlineBrush, InFontInfo, InColorAndOpacity, InShadowOffset, InShadowColorAndOpacity));
}

void FSlateTextStrikeLineHighlighter::GetLineMetrics(const float InFontScale, int16& OutLinePos, int16& OutLineThickness) const
{
	TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();
	FontCache->GetStrikeMetrics(FontInfo, InFontScale, OutLinePos, OutLineThickness);
}
