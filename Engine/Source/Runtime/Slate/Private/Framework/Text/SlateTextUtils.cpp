// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateTextUtils.h"
#include "Styling/SlateTypes.h"

bool SlateTextUtils::IsEllipsisPolicy(const ETextOverflowPolicy& InOverflowPolicy)
{
	return	InOverflowPolicy == ETextOverflowPolicy::Ellipsis ||
			InOverflowPolicy == ETextOverflowPolicy::MultilineEllipsis ||
			InOverflowPolicy == ETextOverflowPolicy::MiddleEllipsis;
}
