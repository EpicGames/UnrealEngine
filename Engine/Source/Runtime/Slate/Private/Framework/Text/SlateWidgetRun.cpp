// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SlateWidgetRun.h"
#include "Layout/ArrangedChildren.h"
#include "Framework/Text/DefaultLayoutBlock.h"

#if WITH_FANCY_TEXT

#include "Framework/Text/RunUtils.h"

TSharedRef< FSlateWidgetRun > FSlateWidgetRun::Create(const TSharedRef<class FTextLayout>& TextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo)
{
	return MakeShareable(new FSlateWidgetRun(TextLayout, InRunInfo, InText, InWidgetInfo));
}

TSharedRef< FSlateWidgetRun > FSlateWidgetRun::Create(const TSharedRef<class FTextLayout>& TextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo, const FTextRange& InRange)
{
	return MakeShareable(new FSlateWidgetRun(TextLayout, InRunInfo, InText, InWidgetInfo, InRange));
}

FTextRange FSlateWidgetRun::GetTextRange() const 
{
	return Range;
}

void FSlateWidgetRun::SetTextRange( const FTextRange& Value )
{
	Range = Value;
}

int16 FSlateWidgetRun::GetBaseLine( float Scale ) const 
{
	return Info.Baseline.Get(0) * Scale;
}

int16 FSlateWidgetRun::GetMaxHeight( float Scale ) const 
{
	return Info.Size.Get( Info.Widget->GetDesiredSize() ).Y * Scale;
}

FVector2D FSlateWidgetRun::Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	if ( EndIndex - StartIndex == 0 )
	{
		return FVector2D( 0, GetMaxHeight( Scale ) );
	}

	return Info.Size.Get( Info.Widget->GetDesiredSize() ) * Scale;
}

int8 FSlateWidgetRun::GetKerning( int32 CurrentIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	return 0;
}

TSharedRef< ILayoutBlock > FSlateWidgetRun::CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer )
{
	return FDefaultLayoutBlock::Create( SharedThis( this ), FTextRange( StartIndex, EndIndex ), Size, TextContext, Renderer );
}

int32 FSlateWidgetRun::OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	const FVector2D DesiredWidgetSize = Info.Widget->GetDesiredSize();
	if (DesiredWidgetSize != WidgetSize)
	{
		WidgetSize = DesiredWidgetSize;

		const TSharedPtr<FTextLayout> TextLayoutPtr = TextLayout.Pin();
		if (TextLayoutPtr.IsValid())
		{
			TextLayoutPtr->DirtyRunLayout(SharedThis(this));
		}
	}

	const FGeometry WidgetGeometry = AllottedGeometry.MakeChild(TransformVector(InverseScale, TextArgs.Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, TextArgs.Block->GetLocationOffset())));
	return Info.Widget->Paint(PaintArgs, WidgetGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

const TArray< TSharedRef<SWidget> >& FSlateWidgetRun::GetChildren()
{
	return Children;
}

void FSlateWidgetRun::ArrangeChildren( const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	ArrangedChildren.AddWidget(
		AllottedGeometry.MakeChild(Info.Widget, TransformVector(InverseScale, Block->GetSize()), FSlateLayoutTransform(TransformPoint(InverseScale, Block->GetLocationOffset())))
		);
}

int32 FSlateWidgetRun::GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint ) const
{
	// A widget should always contain a single character (a breaking space)
	check(Range.Len() == 1);

	const FVector2D& BlockOffset = Block->GetLocationOffset();
	const FVector2D& BlockSize = Block->GetSize();

	const float Left = BlockOffset.X;
	const float Top = BlockOffset.Y;
	const float Right = BlockOffset.X + BlockSize.X;
	const float Bottom = BlockOffset.Y + BlockSize.Y;

	const bool ContainsPoint = Location.X >= Left && Location.X < Right && Location.Y >= Top && Location.Y < Bottom;

	if ( !ContainsPoint )
	{
		return INDEX_NONE;
	}

	const FVector2D ScaledWidgetSize = Info.Widget->GetDesiredSize() * Scale;
	const int32 Index = (Location.X <= (Left + (ScaledWidgetSize.X * 0.5f))) ? Range.BeginIndex : Range.EndIndex;
	
	if (OutHitPoint)
	{
		const FTextRange BlockRange = Block->GetTextRange();
		const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();

		// The block for a widget will always detect a LTR reading direction, so use the base direction (of the line) for the image hit-point detection
		*OutHitPoint = RunUtils::CalculateTextHitPoint(Index, BlockRange, BlockTextContext.BaseDirection);
	}

	return Index;
}

FVector2D FSlateWidgetRun::GetLocationAt( const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale ) const
{
	return Block->GetLocationOffset();
}

void FSlateWidgetRun::Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange)
{
	Text = NewText;
	Range = NewRange;
}

TSharedRef<IRun> FSlateWidgetRun::Clone() const
{
	return FSlateWidgetRun::Create(TextLayout.Pin().ToSharedRef(), RunInfo, Text, Info, Range);
}

void FSlateWidgetRun::AppendTextTo(FString& AppendToText) const
{
	AppendToText.Append(**Text + Range.BeginIndex, Range.Len());
}

void FSlateWidgetRun::AppendTextTo(FString& AppendToText, const FTextRange& PartialRange) const
{
	check(Range.BeginIndex <= PartialRange.BeginIndex);
	check(Range.EndIndex >= PartialRange.EndIndex);

	AppendToText.Append(**Text + PartialRange.BeginIndex, PartialRange.Len());
}

const FRunInfo& FSlateWidgetRun::GetRunInfo() const
{
	return RunInfo;
}

ERunAttributes FSlateWidgetRun::GetRunAttributes() const
{
	return ERunAttributes::None;
}

FSlateWidgetRun::FSlateWidgetRun(const TSharedRef<class FTextLayout>& InTextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo)
	: FSlateWidgetRun(InTextLayout, InRunInfo, InText, InWidgetInfo, FTextRange(0, InText->Len()))
{
}

FSlateWidgetRun::FSlateWidgetRun(const TSharedRef<class FTextLayout>& InTextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo, const FTextRange& InRange)
	: TextLayout(InTextLayout)
	, RunInfo(InRunInfo)
	, Text( InText )
	, Range( InRange )
	, Info( InWidgetInfo )
	, Children()
	, WidgetSize()
{
	Info.Widget->SlatePrepass(InTextLayout.Get().GetScale());
	WidgetSize = Info.Widget->GetDesiredSize();
	Children.Add( Info.Widget );
}

FSlateWidgetRun::FSlateWidgetRun( const FSlateWidgetRun& Run ) 
	: TextLayout(Run.TextLayout)
	, RunInfo(Run.RunInfo)
	, Text( Run.Text )
	, Range( Run.Range )
	, Info( Run.Info )
	, Children()
	, WidgetSize(Run.WidgetSize)
{

}


#endif //WITH_FANCY_TEXT
