// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SyntaxHighlighterTextLayoutMarshaller.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/SlateTextRun.h"
#include "Misc/ExpressionParserTypes.h"
#include "Framework/Text/SlateTextUnderlineLineHighlighter.h"

#if WITH_FANCY_TEXT

FSyntaxHighlighterTextLayoutMarshaller::~FSyntaxHighlighterTextLayoutMarshaller()
{
}

void FSyntaxHighlighterTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
{
	if(bSyntaxHighlightingEnabled)
	{
		TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines;
		Tokenizer->Process(TokenizedLines, SourceString);
		ParseTokens(SourceString, TargetTextLayout, TokenizedLines);
	}
	else
	{
		FPlainTextLayoutMarshaller::SetText(SourceString, TargetTextLayout);
	}
}

bool FSyntaxHighlighterTextLayoutMarshaller::RequiresLiveUpdate() const
{
	return bSyntaxHighlightingEnabled;
}

void FSyntaxHighlighterTextLayoutMarshaller::EnableSyntaxHighlighting(const bool bEnable)
{
	bSyntaxHighlightingEnabled = bEnable;
	MakeDirty();
}

bool FSyntaxHighlighterTextLayoutMarshaller::IsSyntaxHighlightingEnabled() const
{
	return bSyntaxHighlightingEnabled;
}

FSyntaxHighlighterTextLayoutMarshaller::FSyntaxHighlighterTextLayoutMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer)
	: Tokenizer(MoveTemp(InTokenizer))
	, bSyntaxHighlightingEnabled(true)
{
}


TSharedRef< FRichTextSyntaxHighlighterTextLayoutMarshaller > FRichTextSyntaxHighlighterTextLayoutMarshaller::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	TArray<FSyntaxTokenizer::FRule> TokenizerRules;
	TokenizerRules.Emplace(FSyntaxTokenizer::FRule(TEXT("</>")));
	TokenizerRules.Emplace(FSyntaxTokenizer::FRule(TEXT("<")));
	TokenizerRules.Emplace(FSyntaxTokenizer::FRule(TEXT(">")));
	TokenizerRules.Emplace(FSyntaxTokenizer::FRule(TEXT("=")));
	TokenizerRules.Emplace(FSyntaxTokenizer::FRule(TEXT("\"")));

	return MakeShareable(new FRichTextSyntaxHighlighterTextLayoutMarshaller(FSyntaxTokenizer::Create(TokenizerRules), InSyntaxTextStyle));
}

FRichTextSyntaxHighlighterTextLayoutMarshaller::~FRichTextSyntaxHighlighterTextLayoutMarshaller()
{
}

void FRichTextSyntaxHighlighterTextLayoutMarshaller::ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines)
{
	enum class EParseState : uint8
	{
		LookingForNode,
		LookingForNodeName,
		LookingForNodeAttributeKey,
		LookingForNodeAttribueValueBegin,
		LookingForNodeAttribueValueBody,
	};

	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(TokenizedLines.Num());

	TArray<FTextLineHighlight> LineHighlightsToAdd;
	TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextUnderlineLineHighlighter>> CachedUnderlineHighlighters;
	TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextStrikeLineHighlighter>> CachedStrikeLineHighlighters;

	// Parse the tokens, generating the styled runs for each line
	EParseState ParseState = EParseState::LookingForNode;
	for (int32 LineIndex = 0; LineIndex < TokenizedLines.Num(); ++LineIndex)
	{
		const ISyntaxTokenizer::FTokenizedLine& TokenizedLine = TokenizedLines[LineIndex];

		TSharedRef<FString> ModelString = MakeShareable(new FString());
		TArray< TSharedRef< IRun > > Runs;

		for(const ISyntaxTokenizer::FToken& Token : TokenizedLine.Tokens)
		{
			const FString TokenText = SourceString.Mid(Token.Range.BeginIndex, Token.Range.Len());

			const FTextRange ModelRange(ModelString->Len(), ModelString->Len() + TokenText.Len());
			ModelString->Append(TokenText);

			FRunInfo RunInfo(TEXT("SyntaxHighlight.Normal"));
			const FTextBlockStyle* TextBlockStyle = &SyntaxTextStyle.NormalTextStyle;

			const bool bIsWhitespace = FString(TokenText).TrimEnd().IsEmpty();
			if(!bIsWhitespace)
			{
				bool bHasMatchedSyntax = false;
				if(Token.Type == ISyntaxTokenizer::ETokenType::Syntax)
				{
					if(ParseState == EParseState::LookingForNode && TokenText == TEXT("<"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.Node");
						TextBlockStyle = &SyntaxTextStyle.NodeTextStyle;
						ParseState = EParseState::LookingForNodeName;
						bHasMatchedSyntax = true;
					}
					else if(ParseState == EParseState::LookingForNodeAttributeKey && TokenText == TEXT(">"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.Node");
						TextBlockStyle = &SyntaxTextStyle.NodeTextStyle;
						ParseState = EParseState::LookingForNode;
					}
					else if(ParseState == EParseState::LookingForNode && TokenText == TEXT("</>"))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.Node");
						TextBlockStyle = &SyntaxTextStyle.NodeTextStyle;
						// No state change
						bHasMatchedSyntax = true;
					}
					else if(ParseState == EParseState::LookingForNodeAttributeKey && TokenText == TEXT("="))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.NodeAttribueAssignment");
						TextBlockStyle = &SyntaxTextStyle.NodeAttribueAssignmentTextStyle;
						ParseState = EParseState::LookingForNodeAttribueValueBegin;
						bHasMatchedSyntax = true;
					}
					else if(ParseState == EParseState::LookingForNodeAttribueValueBegin && TokenText == TEXT("\""))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.NodeAttributeValue");
						TextBlockStyle = &SyntaxTextStyle.NodeAttributeValueTextStyle;
						ParseState = EParseState::LookingForNodeAttribueValueBody;
						bHasMatchedSyntax = true;
					}
					else if(ParseState == EParseState::LookingForNodeAttribueValueBody && TokenText == TEXT("\""))
					{
						RunInfo.Name = TEXT("SyntaxHighlight.NodeAttributeValue");
						TextBlockStyle = &SyntaxTextStyle.NodeAttributeValueTextStyle;
						ParseState = EParseState::LookingForNodeAttributeKey;
						bHasMatchedSyntax = true;
					}
				}
				
				// It's possible that we fail to match a syntax token if we're in a state where it isn't parsed
				// In this case, we treat it as a literal token
				if(Token.Type == ISyntaxTokenizer::ETokenType::Literal || !bHasMatchedSyntax)
				{
					if(ParseState == EParseState::LookingForNodeName)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.Node");
						TextBlockStyle = &SyntaxTextStyle.NodeTextStyle;
						ParseState = EParseState::LookingForNodeAttributeKey;
					}
					else if(ParseState == EParseState::LookingForNodeAttributeKey)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.NodeAttributeKey");
						TextBlockStyle = &SyntaxTextStyle.NodeAttributeKeyTextStyle;
						// No state change, a key can be multiple tokens - consume until we find an equals
					}
					else if(ParseState == EParseState::LookingForNodeAttribueValueBody)
					{
						RunInfo.Name = TEXT("SyntaxHighlight.NodeAttributeValue");
						TextBlockStyle = &SyntaxTextStyle.NodeAttributeValueTextStyle;
						// No state change, a value can be multiple tokens - consume until we find a closing quote
					}
				}
			}

			TSharedRef< ISlateRun > Run = FSlateTextRun::Create(RunInfo, ModelString, *TextBlockStyle, ModelRange);
			Runs.Add(Run);

			if (!TextBlockStyle->UnderlineBrush.GetResourceName().IsNone())
			{
				TSharedPtr<FSlateTextUnderlineLineHighlighter> UnderlineLineHighlighter = CachedUnderlineHighlighters.FindRef(TextBlockStyle);
				if (!UnderlineLineHighlighter.IsValid())
				{
					UnderlineLineHighlighter = FSlateTextUnderlineLineHighlighter::Create(TextBlockStyle->UnderlineBrush, TextBlockStyle->Font, TextBlockStyle->ColorAndOpacity, TextBlockStyle->ShadowOffset, TextBlockStyle->ShadowColorAndOpacity);
					CachedUnderlineHighlighters.Add(TextBlockStyle, UnderlineLineHighlighter);
				}

				LineHighlightsToAdd.Add(FTextLineHighlight(LineIndex, ModelRange, FSlateTextUnderlineLineHighlighter::DefaultZIndex, UnderlineLineHighlighter.ToSharedRef()));
			}

			if (!TextBlockStyle->StrikeBrush.GetResourceName().IsNone())
			{
				TSharedPtr<FSlateTextStrikeLineHighlighter> StrikeLineHighlighter = CachedStrikeLineHighlighters.FindRef(TextBlockStyle);
				if (!StrikeLineHighlighter.IsValid())
				{
					StrikeLineHighlighter = FSlateTextStrikeLineHighlighter::Create(TextBlockStyle->StrikeBrush, TextBlockStyle->Font, TextBlockStyle->ColorAndOpacity, TextBlockStyle->ShadowOffset, TextBlockStyle->ShadowColorAndOpacity);
					CachedStrikeLineHighlighters.Add(TextBlockStyle, StrikeLineHighlighter);
				}

				LineHighlightsToAdd.Add(FTextLineHighlight(LineIndex, ModelRange, FSlateTextStrikeLineHighlighter::DefaultZIndex, StrikeLineHighlighter.ToSharedRef()));
			}
		}

		LinesToAdd.Emplace(MoveTemp(ModelString), MoveTemp(Runs));
	}

	TargetTextLayout.AddLines(LinesToAdd);
	TargetTextLayout.SetLineHighlights(LineHighlightsToAdd);
}

FRichTextSyntaxHighlighterTextLayoutMarshaller::FRichTextSyntaxHighlighterTextLayoutMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle)
	: FSyntaxHighlighterTextLayoutMarshaller(MoveTemp(InTokenizer))
	, SyntaxTextStyle(InSyntaxTextStyle)
{
}

#endif //WITH_FANCY_TEXT
