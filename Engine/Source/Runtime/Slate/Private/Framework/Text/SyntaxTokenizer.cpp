// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/SyntaxTokenizer.h"
#include "Internationalization/IBreakIterator.h"
#include "Internationalization/BreakIterator.h"

TSharedRef< FSyntaxTokenizer > FSyntaxTokenizer::Create(TArray<FRule> InRules)
{
	return MakeShareable(new FSyntaxTokenizer(MoveTemp(InRules)));
}

FSyntaxTokenizer::~FSyntaxTokenizer()
{
}

void FSyntaxTokenizer::Process(TArray<FTokenizedLine>& OutTokenizedLines, const FString& Input)
{
#if UE_ENABLE_ICU
	TArray<FTextRange> LineRanges;
	FTextRange::CalculateLineRangesFromString(Input, LineRanges);
	TokenizeLineRanges(Input, LineRanges, OutTokenizedLines);
#else
	FTokenizedLine FakeTokenizedLine;
	FakeTokenizedLine.Range = FTextRange(0, Input.Len());
	FakeTokenizedLine.Tokens.Emplace(FToken(ETokenType::Literal, FakeTokenizedLine.Range));
	OutTokenizedLines.Add(FakeTokenizedLine);
#endif
}

FSyntaxTokenizer::FSyntaxTokenizer(TArray<FRule> InRules)
	: Rules(MoveTemp(InRules))
{
}

void FSyntaxTokenizer::TokenizeLineRanges(const FString& Input, const TArray<FTextRange>& LineRanges, TArray<FTokenizedLine>& OutTokenizedLines)
{
	TSharedRef<IBreakIterator> WBI = FBreakIterator::CreateWordBreakIterator();
	WBI->SetString(Input);

	// Tokenize line ranges
	for(const FTextRange& LineRange : LineRanges)
	{
		FTokenizedLine TokenizedLine;
		TokenizedLine.Range = LineRange;

		if(TokenizedLine.Range.IsEmpty())
		{
			TokenizedLine.Tokens.Emplace(FToken(ETokenType::Literal, TokenizedLine.Range));
		}
		else
		{
			int32 CurrentOffset = LineRange.BeginIndex;
			while(CurrentOffset < LineRange.EndIndex)
			{
				// First check for a match against any syntax token rules
				bool bHasMatchedSyntax = false;
				for(const FRule& Rule : Rules)
				{
					if(FCString::Strncmp(&Input[CurrentOffset], *Rule.MatchText, Rule.MatchText.Len()) == 0)
					{
						const int32 SyntaxTokenEnd = CurrentOffset + Rule.MatchText.Len();
						TokenizedLine.Tokens.Emplace(FToken(ETokenType::Syntax, FTextRange(CurrentOffset, SyntaxTokenEnd)));

						check(SyntaxTokenEnd <= LineRange.EndIndex);

						bHasMatchedSyntax = true;
						CurrentOffset = SyntaxTokenEnd;
						break;
					}
				}

				if(bHasMatchedSyntax)
				{
					continue;
				}

				// If none matched, consume the character(s) as text
				const int32 NextWordBoundary = WBI->MoveToCandidateAfter(CurrentOffset);
				const int32 TextTokenEnd = (NextWordBoundary == INDEX_NONE) ? LineRange.EndIndex : FMath::Min(NextWordBoundary, LineRange.EndIndex);
				TokenizedLine.Tokens.Emplace(FToken(ETokenType::Literal, FTextRange(CurrentOffset, TextTokenEnd)));
				CurrentOffset = TextTokenEnd;
			}
		}

		OutTokenizedLines.Add(TokenizedLine);
	}
}
