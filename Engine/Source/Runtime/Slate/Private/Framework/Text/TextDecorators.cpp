// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/TextDecorators.h"
#include "Styling/CoreStyle.h"
#include "Framework/Text/SlateImageRun.h"

#if WITH_FANCY_TEXT


TSharedRef< FWidgetDecorator > FWidgetDecorator::Create( FString InRunName, const FCreateWidget& InCreateWidgetDelegate )
{
	return MakeShareable( new FWidgetDecorator( InRunName, InCreateWidgetDelegate ) );
}

FWidgetDecorator::FWidgetDecorator( FString InRunName, const FCreateWidget& InCreateWidgetDelegate )
	: RunName( InRunName )
	, CreateWidgetDelegate( InCreateWidgetDelegate )
{

}

bool FWidgetDecorator::Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const
{
	return ( RunParseResult.Name == RunName );
}

TSharedRef< ISlateRun > FWidgetDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style)
{
	FTextRange ModelRange;
	ModelRange.BeginIndex = InOutModelText->Len();
	*InOutModelText += TEXT('\u00A0'); // Zero-Width Breaking Space
	ModelRange.EndIndex = InOutModelText->Len();

	FTextRunInfo RunInfo( RunParseResult.Name, FText::FromString( OriginalText.Mid( RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex ) ) );
	for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid( Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	return FSlateWidgetRun::Create(TextLayout, RunInfo, InOutModelText, CreateWidgetDelegate.Execute(RunInfo, Style), ModelRange);
}

TSharedRef< FImageDecorator > FImageDecorator::Create( FString InRunName, const ISlateStyle* const InStyle )
{
	return MakeShareable( new FImageDecorator( InRunName, InStyle ) );
}

FImageDecorator::FImageDecorator( FString InRunName, const ISlateStyle* const InOverrideStyle )
	: RunName( InRunName )
	, OverrideStyle( InOverrideStyle )
{

}

bool FImageDecorator::Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const
{
	return ( RunParseResult.Name == RunName );
}

TSharedRef< ISlateRun > FImageDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style)
{
	const FTextRange* const BrushNameRange = RunParseResult.MetaData.Find( TEXT("src") );

	FTextRange ModelRange;
	ModelRange.BeginIndex = InOutModelText->Len();
	*InOutModelText += TEXT('\x200B'); // Zero-Width Breaking Space
	ModelRange.EndIndex = InOutModelText->Len();

	FRunInfo RunInfo( RunParseResult.Name );
	for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid( Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	if ( BrushNameRange != NULL )
	{
		const FString BrushNameString = OriginalText.Mid(BrushNameRange->BeginIndex, BrushNameRange->EndIndex - BrushNameRange->BeginIndex);
		if ( OverrideStyle != NULL )
		{
			Style = OverrideStyle;
		}

		FName BrushName( *BrushNameString );
		if ( Style->HasWidgetStyle< FInlineTextImageStyle >( BrushName ) )
		{
			const FInlineTextImageStyle& ImageStyle = Style->GetWidgetStyle< FInlineTextImageStyle >( BrushName );
			return FSlateImageRun::Create( RunInfo, InOutModelText, &ImageStyle.Image, ImageStyle.Baseline, ModelRange );
		}

		return FSlateImageRun::Create( RunInfo, InOutModelText, Style->GetBrush( BrushName ), 0, ModelRange );
	}

	const FInlineTextImageStyle& ImageStyle = FInlineTextImageStyle::GetDefault();
	return FSlateImageRun::Create( RunInfo, InOutModelText, &ImageStyle.Image, ImageStyle.Baseline, ModelRange );
}

TSharedRef< FHyperlinkDecorator > FHyperlinkDecorator::Create( FString Id, const FSlateHyperlinkRun::FOnClick& NavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate )
{
	return MakeShareable( new FHyperlinkDecorator( Id, NavigateDelegate, InToolTipTextDelegate, InToolTipDelegate ) );
}

FHyperlinkDecorator::FHyperlinkDecorator( FString InId, const FSlateHyperlinkRun::FOnClick& InNavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate )
	: NavigateDelegate( InNavigateDelegate )
	, Id(InId)
	, ToolTipTextDelegate( InToolTipTextDelegate )
	, ToolTipDelegate( InToolTipDelegate )
{

}

bool FHyperlinkDecorator::Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const
{
	const FTextRange* const MetaDataIdRange = RunParseResult.MetaData.Find( TEXT("id") );
	FString MetaDataId;
	if(MetaDataIdRange)
	{
		MetaDataId = Text.Mid(MetaDataIdRange->BeginIndex, MetaDataIdRange->EndIndex - MetaDataIdRange->BeginIndex);
	}

	return (RunParseResult.Name == TEXT("a") && MetaDataId == Id);
}

TSharedRef< ISlateRun > FHyperlinkDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style)
{
	FString StyleName = "Hyperlink";
	FString TextStyleName;

	const FTextRange* const MetaDataStyleNameRange = RunParseResult.MetaData.Find( TEXT("style") );
	if ( MetaDataStyleNameRange != NULL )
	{
		const FString MetaDataStyleName = OriginalText.Mid(MetaDataStyleNameRange->BeginIndex, MetaDataStyleNameRange->EndIndex - MetaDataStyleNameRange->BeginIndex);
		StyleName = *MetaDataStyleName;
	}

	const FTextRange* const MetaDataTextStyleNameRange = RunParseResult.MetaData.Find(TEXT("textstyle"));
	if (MetaDataTextStyleNameRange != NULL)
	{
		const FString MetaDataTextStyleName = OriginalText.Mid(MetaDataTextStyleNameRange->BeginIndex, MetaDataTextStyleNameRange->EndIndex - MetaDataTextStyleNameRange->BeginIndex);
		TextStyleName = *MetaDataTextStyleName;
	}

	FTextRange ModelRange;
	ModelRange.BeginIndex = InOutModelText->Len();
	*InOutModelText += OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex);
	ModelRange.EndIndex = InOutModelText->Len();

	if ( !Style->HasWidgetStyle<FHyperlinkStyle>( FName( *StyleName ) ) )
	{
		Style = &FCoreStyle::Get();
	}

	FRunInfo RunInfo( RunParseResult.Name );
	for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid( Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	FHyperlinkStyle HyperlinkStyle = Style->GetWidgetStyle<FHyperlinkStyle>(FName(*StyleName));

	if (!TextStyleName.IsEmpty() && Style->HasWidgetStyle<FTextBlockStyle>(FName(*TextStyleName)))
	{
		HyperlinkStyle.SetTextStyle(Style->GetWidgetStyle<FTextBlockStyle>(FName(*TextStyleName)));
	}

	return FSlateHyperlinkRun::Create(RunInfo, InOutModelText, HyperlinkStyle, NavigateDelegate, ToolTipDelegate, ToolTipTextDelegate, ModelRange);
}
#endif //WITH_FANCY_TEXT
