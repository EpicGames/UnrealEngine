// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/TextEditHelper.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "FTextEditHelper"

float FTextEditHelper::GetFontHeight(const FSlateFontInfo& FontInfo)
{
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	return FontMeasure->GetMaxCharacterHeight(FontInfo);
}

float FTextEditHelper::CalculateCaretWidth(const float FontMaxCharHeight)
{
	// We adjust the width of the caret to avoid it becoming too wide on smaller or larger fonts and overlapping the characters it's next to.
	// We clamp the lower limit to 1 to avoid it being invisible, and the upper limit to 2 to avoid tall fonts having very wide carets.
	return FMath::Clamp(EditableTextDefs::CaretWidthPercent * FontMaxCharHeight, 1.0f, 2.0f);
}

bool FTextEditHelper::VerifyTextLength(const FText& InText, FText& OutErrorMessage, int32 InMaximumLength)
{
	const int TextLength = InText.ToString().Len();
	if (InMaximumLength > 0 && TextLength > InMaximumLength)
	{
		OutErrorMessage = FText::Format(LOCTEXT("TextTooLong", "This text is too long. It uses {0} characters of {1} allowed."), TextLength, InMaximumLength);
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
