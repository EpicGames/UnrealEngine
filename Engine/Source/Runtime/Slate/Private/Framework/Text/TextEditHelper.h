// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSlateFontInfo;

/**
 * Design constraints
 */
namespace EditableTextDefs
{
	/** Maximum number of undo levels to store */
	static const int32 MaxUndoLevels = 99;

	/** Width of the caret, as a scalar percentage of the font's maximum character height */
	static const float CaretWidthPercent = 0.08f;

	/** How long after the user last interacted with the keyboard should we keep the caret at full opacity? */
	static const float CaretBlinkPauseTime = 0.1f;

	/** How many times should the caret blink per second (full on/off cycles) */
	static const float BlinksPerSecond = 1.0f;
}

/**
 * Static helper functions
 */
class FTextEditHelper
{
public:
	/**
	 * Gets the height of the largest character in the font
	 *
	 * @return  The fonts height
	 */
	static float GetFontHeight(const FSlateFontInfo& FontInfo);

	/** 
	 * Calculate the width of the caret 
	 * @param FontMaxCharHeight The height of the font to calculate the caret width for
	 * @return The width of the caret (might be clamped for very small fonts)
	 */
	static float CalculateCaretWidth(const float FontMaxCharHeight);

	static bool VerifyTextLength(const FText& InText, FText& OutErrorMessage, int32 LengthToCheck);
};
