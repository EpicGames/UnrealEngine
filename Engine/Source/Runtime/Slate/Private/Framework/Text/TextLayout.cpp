// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Text/TextLayout.h"
#include "Fonts/FontCache.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextHitPoint.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextTransformer.h"
#include "Framework/Text/ShapedTextCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextLayout)

DECLARE_CYCLE_STAT(TEXT("Text Layout"), STAT_SlateTextLayout, STATGROUP_Slate);

static TAutoConsoleVariable<int32> CVarDefaultTextFlowDirection(
	TEXT("Slate.DefaultTextFlowDirection"),
	static_cast<int32>(ETextFlowDirection::Auto),
	TEXT("0: Auto (default), 1: LeftToRight, 2: RightToLeft."),
	ECVF_Default
	);

ETextFlowDirection GetDefaultTextFlowDirection()
{
	const int32 DefaultTextFlowDirectionAsInt = CVarDefaultTextFlowDirection.AsVariable()->GetInt();
	if (DefaultTextFlowDirectionAsInt >= static_cast<int32>(ETextFlowDirection::Auto) && DefaultTextFlowDirectionAsInt <= static_cast<int32>(ETextFlowDirection::RightToLeft))
	{
		return static_cast<ETextFlowDirection>(DefaultTextFlowDirectionAsInt);
	}
	return ETextFlowDirection::Auto;
}


FTextLayout::FBreakCandidate FTextLayout::CreateBreakCandidate( int32& OutRunIndex, FLineModel& Line, int32 PreviousBreak, int32 CurrentBreak )
{
	const FRunTextContext RunTextContext(TextShapingMethod, Line.TextBaseDirection, Line.ShapedTextCache, &LoadingFontFaces);

	bool SuccessfullyMeasuredSlice = false;
	int16 MaxAboveBaseline = 0;
	int16 MaxBelowBaseline = 0;
	FVector2D BreakSize( ForceInitToZero );
	FVector2D BreakSizeWithoutTrailingWhitespace( ForceInitToZero );
	float FirstTrailingWhitespaceCharWidth = 0.0f;
	int32 WhitespaceStopIndex = CurrentBreak;
	int8 Kerning = 0;

	if ( Line.Runs.IsValidIndex( OutRunIndex ) )
	{
		FRunModel& Run = Line.Runs[ OutRunIndex ];
		const FTextRange Range = Run.GetTextRange();
		int32 BeginIndex = FMath::Max( PreviousBreak, Range.BeginIndex );

		if ( BeginIndex > 0 )
		{
			Kerning = Run.GetKerning( BeginIndex, Scale, RunTextContext );
		}
	}

	// We need to consider the Runs when detecting and measuring the text lengths of Lines because
	// the font style used makes a difference.
	const int32 FirstRunIndexChecked = OutRunIndex;
	for (; OutRunIndex < Line.Runs.Num(); OutRunIndex++)
	{
		FRunModel& Run = Line.Runs[ OutRunIndex ];
		const FTextRange Range = Run.GetTextRange();

		FVector2D SliceSize;
		FVector2D SliceSizeWithoutTrailingWhitespace;
		int32 StopIndex = PreviousBreak;

		WhitespaceStopIndex = StopIndex = FMath::Min( Range.EndIndex, CurrentBreak );
		int32 BeginIndex = FMath::Max( PreviousBreak, Range.BeginIndex );

		while( WhitespaceStopIndex > BeginIndex && FTextChar::IsWhitespace( (*Line.Text)[ WhitespaceStopIndex - 1 ] ) )
		{
			--WhitespaceStopIndex;
		}

		if ( BeginIndex == StopIndex )
		{
			// This slice is empty, no need to adjust anything
			SliceSize = SliceSizeWithoutTrailingWhitespace = FVector2D::ZeroVector;
		}
		else if ( BeginIndex == WhitespaceStopIndex )
		{
			// This slice contains only whitespace, no need to adjust SliceSizeWithoutTrailingWhitespace
			SliceSize = Run.Measure( BeginIndex, StopIndex, Scale, RunTextContext );
			SliceSizeWithoutTrailingWhitespace = FVector2D::ZeroVector;
		}
		else if ( WhitespaceStopIndex != StopIndex )
		{
			// This slice contains trailing whitespace, measure the text size, then add on the whitespace size
			SliceSize = SliceSizeWithoutTrailingWhitespace = Run.Measure( BeginIndex, WhitespaceStopIndex, Scale, RunTextContext );
			const float WhitespaceWidth = Run.Measure( WhitespaceStopIndex, StopIndex, Scale, RunTextContext ).X;
			SliceSize.X += WhitespaceWidth;

			// We also need to measure the width of the first piece of trailing whitespace
			if ( WhitespaceStopIndex + 1 == StopIndex )
			{
				// Only have one piece of whitespace
				FirstTrailingWhitespaceCharWidth = WhitespaceWidth;
			}
			else
			{
				// Deliberately use the run version of Measure as we don't want the run model to cache this measurement since it may be out of order and break the binary search
				FirstTrailingWhitespaceCharWidth = Run.GetRun()->Measure( WhitespaceStopIndex, WhitespaceStopIndex + 1, Scale, RunTextContext ).X;
			}
		}
		else
		{
			// This slice contains no whitespace, both sizes are the same and can use the same measurement
			SliceSize = SliceSizeWithoutTrailingWhitespace = Run.Measure( BeginIndex, StopIndex, Scale, RunTextContext );
		}

		BreakSize.X += SliceSize.X; // We accumulate the slice widths
		BreakSizeWithoutTrailingWhitespace.X += SliceSizeWithoutTrailingWhitespace.X; // We accumulate the slice widths

		// Get the baseline and flip it's sign; Baselines are generally negative
		const int16 Baseline = -(Run.GetBaseLine( Scale ));

		// For the height of the slice we need to take into account the largest value below and above the baseline and add those together
		MaxAboveBaseline = FMath::Max( MaxAboveBaseline, (int16)( Run.GetMaxHeight( Scale ) - Baseline ) );
		MaxBelowBaseline = FMath::Max( MaxBelowBaseline, Baseline );

		if ( StopIndex == CurrentBreak )
		{
			SuccessfullyMeasuredSlice = true;

			if ( OutRunIndex < Line.Runs.Num() && StopIndex == Line.Runs[ OutRunIndex ].GetTextRange().EndIndex )
			{
				++OutRunIndex;
			}
			break;
		}
	}
	
#if DO_CHECK
	if (!SuccessfullyMeasuredSlice)
	{
		FString AnonymizedText;
		AnonymizedText.Reserve(Line.Text->Len());
		for (const TCHAR Char : *Line.Text)
		{
			if (Char == TCHAR('\\'))
			{
				AnonymizedText += TEXT("\\\\");
			}
			else if (FChar::IsWhitespace(Char) || FChar::IsPunct(Char))
			{
				AnonymizedText += Char;
			}
			else if (FChar::IsDigit(Char))
			{
				AnonymizedText += TEXT("0");
			}
			else if (FChar::IsLower(Char))
			{
				AnonymizedText += TEXT("a");
			}
			else
			{
				AnonymizedText += TEXT("A");
			}
		}

		FString RunDebugData;
		for (int32 RunIndex = 0; RunIndex < Line.Runs.Num(); ++RunIndex)
		{
			const FRunModel& Run = Line.Runs[RunIndex];
			const FTextRange RunRange = Run.GetTextRange();
			RunDebugData.Append(FString::Printf(TEXT("\t\t[%d] - Range: {%d, %d}\n"), RunIndex, RunRange.BeginIndex, RunRange.EndIndex));
		}

		checkf(SuccessfullyMeasuredSlice, TEXT("Failed to measure a slice of text!\n\tDebug Source: %s\n\tAnonymized Text: %s\n\tStart Index: %d\n\tEnd Index: %d\n\tStart Run Index: %d\n\tLine Runs:\n%s"), *DebugSourceInfo.Get(FString()), *AnonymizedText, PreviousBreak, CurrentBreak, FirstRunIndexChecked, *RunDebugData);
	}
#endif // DO_CHECK

	BreakSize.Y = BreakSizeWithoutTrailingWhitespace.Y = MaxAboveBaseline + MaxBelowBaseline;

	FBreakCandidate BreakCandidate;
	BreakCandidate.ActualSize = BreakSize;
	BreakCandidate.TrimmedWidth = BreakSizeWithoutTrailingWhitespace.X;
	BreakCandidate.ActualRange = FTextRange( PreviousBreak, CurrentBreak );
	BreakCandidate.TrimmedRange = FTextRange( PreviousBreak, WhitespaceStopIndex );
	BreakCandidate.FirstTrailingWhitespaceCharWidth = FirstTrailingWhitespaceCharWidth;
	BreakCandidate.MaxAboveBaseline = MaxAboveBaseline;
	BreakCandidate.MaxBelowBaseline = MaxBelowBaseline;
	BreakCandidate.Kerning = Kerning;

#if TEXT_LAYOUT_DEBUG
	BreakCandidate.DebugSlice = FString( BreakCandidate.Range.EndIndex - BreakCandidate.Range.BeginIndex, (**Line.Text) + BreakCandidate.Range.BeginIndex );
#endif

	return BreakCandidate;
}

void FTextLayout::CreateLineViewBlocks( int32 LineModelIndex, const int32 StopIndex, const float WrappedLineWidth, const TOptional<float>& JustificationWidth, int32& OutRunIndex, int32& OutRendererIndex, int32& OutPreviousBlockEnd, TArray< TSharedRef< ILayoutBlock > >& OutSoftLine )
{
	const FLineModel& LineModel = LineModels[ LineModelIndex ];

	const FRunTextContext RunTextContext(TextShapingMethod, LineModel.TextBaseDirection, LineModel.ShapedTextCache, &LoadingFontFaces);

	int16 MaxAboveBaseline = 0;
	int16 MaxBelowBaseline = 0;

	int32 CurrentLineBegin = OutPreviousBlockEnd;
	if (OutRunIndex < LineModel.Runs.Num())
	{
		CurrentLineBegin = FMath::Max(CurrentLineBegin, LineModel.Runs[OutRunIndex].GetTextRange().BeginIndex);
	}

	int32 CurrentLineEnd = StopIndex;
	if (CurrentLineEnd == INDEX_NONE)
	{
		CurrentLineEnd = (LineModel.Runs.Num() > 0) ? LineModel.Runs.Last().GetTextRange().EndIndex : 0;
	}
	
	// KerningOnly shaping implies LTR only text, so we can skip the bidirectional detection and splitting
	TextBiDi::ETextDirection LineTextDirection = TextBiDi::ETextDirection::LeftToRight;
	TArray<TextBiDi::FTextDirectionInfo> TextDirectionInfos;
	if (TextShapingMethod != ETextShapingMethod::KerningOnly)
	{
		// The bidirectional text detection tells us the correct order for the blocks of text with regard to the base direction of the current line
		LineTextDirection = TextBiDiDetection->ComputeTextDirection(**LineModel.Text, CurrentLineBegin, CurrentLineEnd - CurrentLineBegin, LineModel.TextBaseDirection, TextDirectionInfos);
	}

	// Ensure there is at least one directional block. This can happen when using KerningOnly shaping (since we skip the bidirectional detection), or for empty strings that are run through the bidirectional detection.
	if (TextDirectionInfos.Num() == 0)
	{
		TextBiDi::FTextDirectionInfo TextDirectionInfo;
		TextDirectionInfo.StartIndex = CurrentLineBegin;
		TextDirectionInfo.Length = CurrentLineEnd - CurrentLineBegin;
		TextDirectionInfo.TextDirection = TextBiDi::ETextDirection::LeftToRight;
		TextDirectionInfos.Add(MoveTemp(TextDirectionInfo));
	}

	// We always add the runs to the line in ascending index order, so re-order a copy of the text direction data so that we can iterate it forwards by ascending index
	// We'll re-sort the line into the correct visual order once we've finished generating the blocks
	int32 CurrentSortedTextDirectionInfoIndex = 0;
	TArray<TextBiDi::FTextDirectionInfo> SortedTextDirectionInfos = TextDirectionInfos;
	SortedTextDirectionInfos.Sort([](const TextBiDi::FTextDirectionInfo& InFirst, const TextBiDi::FTextDirectionInfo& InSecond) -> bool
	{
		return InFirst.StartIndex < InSecond.StartIndex;
	});

	FTextRange SoftLineRange = FTextRange(MAX_int32, MIN_int32);
	for (; OutRunIndex < LineModel.Runs.Num(); )
	{
		const FRunModel& Run = LineModel.Runs[ OutRunIndex ];
		const FTextRange RunRange = Run.GetTextRange();

		int32 BlockBeginIndex = FMath::Max( OutPreviousBlockEnd, RunRange.BeginIndex );
		int32 BlockStopIndex = RunRange.EndIndex;

		// Blocks can only contain text with the same reading direction
		TextBiDi::ETextDirection BlockTextDirection = TextBiDi::ETextDirection::LeftToRight;
		int32 CurrentTextDirectionStopIndex = 0;
		if (CurrentSortedTextDirectionInfoIndex < SortedTextDirectionInfos.Num())
		{
			const TextBiDi::FTextDirectionInfo& CurrentTextDirectionInfo = SortedTextDirectionInfos[CurrentSortedTextDirectionInfoIndex];
			CurrentTextDirectionStopIndex = CurrentTextDirectionInfo.StartIndex + CurrentTextDirectionInfo.Length;

			checkf(BlockBeginIndex >= CurrentTextDirectionInfo.StartIndex, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

			BlockStopIndex = FMath::Min(BlockStopIndex, CurrentTextDirectionStopIndex);
			BlockTextDirection = CurrentTextDirectionInfo.TextDirection;
		}

		TSharedPtr< IRunRenderer > BlockRenderer = nullptr;

		if ( OutRendererIndex != INDEX_NONE )
		{
			// Grab the currently active renderer
			const FTextRunRenderer& Renderer = LineModel.RunRenderers[OutRendererIndex];

			// Check to see if the last block was rendered with the same renderer
			if ( OutPreviousBlockEnd >= Renderer.Range.BeginIndex )
			{
				//If the renderer ends before our directional run...
				if ( Renderer.Range.EndIndex <= BlockStopIndex )
				{
					// Adjust the stopping point of the block to be the end of the renderer range,
					// since highlights need their own block segments
					BlockStopIndex = Renderer.Range.EndIndex;
					BlockRenderer = Renderer.Renderer;
				}
				else
				{
					// This whole run is encompassed by the renderer
					BlockRenderer = Renderer.Renderer;
				}
			}
			else
			{
				// Does the renderer range begin before our directional run ends?
				if ( Renderer.Range.BeginIndex <= BlockStopIndex )
				{
					// then adjust the current block stopping point to just before the renderer range begins,
					// since renderers need their own block segments
					BlockStopIndex = Renderer.Range.BeginIndex;
					BlockRenderer = nullptr;
				}
			}
		}

		if (StopIndex != INDEX_NONE)
		{
			BlockStopIndex = FMath::Min(StopIndex, BlockStopIndex);
		}

		// Have we reached the end of this bidirectional block?
		if (BlockStopIndex == CurrentTextDirectionStopIndex)
		{
			++CurrentSortedTextDirectionInfoIndex;
		}

		const bool IsLastBlock = BlockStopIndex == StopIndex;

		checkf(BlockBeginIndex <= BlockStopIndex, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

		// Add the new block
		{
			FBlockDefinition BlockDefine;
			BlockDefine.ActualRange = FTextRange(BlockBeginIndex, BlockStopIndex);
			BlockDefine.Renderer = BlockRenderer;

			OutSoftLine.Add( Run.CreateBlock( BlockDefine, Scale, FLayoutBlockTextContext(RunTextContext, BlockTextDirection) ) );
			OutPreviousBlockEnd = BlockStopIndex;

			// Update the soft line bounds based on this new block (needed within this loop due to bi-directional text, as the extents of the line array are not always the start and end of the range)
			const FTextRange& BlockRange = OutSoftLine.Last()->GetTextRange();
			SoftLineRange.BeginIndex = FMath::Min(SoftLineRange.BeginIndex, BlockRange.BeginIndex);
			SoftLineRange.EndIndex   = FMath::Max(SoftLineRange.EndIndex, BlockRange.EndIndex);
		}

		// Get the baseline and flip it's sign; Baselines are generally negative
		const int16 Baseline = -(Run.GetBaseLine( Scale ));

		// For the height of the slice we need to take into account the largest value below and above the baseline and add those together
		MaxAboveBaseline = FMath::Max( MaxAboveBaseline, (int16)( Run.GetMaxHeight( Scale ) - Baseline ) );
		MaxBelowBaseline = FMath::Max( MaxBelowBaseline, Baseline );

		if ( BlockStopIndex == RunRange.EndIndex )
		{
			++OutRunIndex;
		}

		if ( OutRendererIndex != INDEX_NONE && BlockStopIndex == LineModel.RunRenderers[ OutRendererIndex ].Range.EndIndex )
		{
			++OutRendererIndex;

			if ( OutRendererIndex >= LineModel.RunRenderers.Num() )
			{
				OutRendererIndex = INDEX_NONE;
			}
		}

		if ( IsLastBlock )
		{
			break;
		}
	}

	FVector2D LineSize( ForceInitToZero );
	float UnscaleLineHeight = 0.f;
	
	// Use a negative scroll offset since positive scrolling moves things negatively in screen space
	FVector2D CurrentOffset(-ScrollOffset.X, TextLayoutSize.Height - ScrollOffset.Y);

	if ( OutSoftLine.Num() > 0 )
	{
		// Re-order the blocks based on their visual direction
		if (OutSoftLine.Num() > 1 && LineTextDirection != TextBiDi::ETextDirection::LeftToRight)
		{
			TArray<TSharedRef<ILayoutBlock>> VisualSoftLine;
			VisualSoftLine.Reserve(OutSoftLine.Num());

			TArray<TSharedRef<ILayoutBlock>> CurrentVisualSoftLine;
			for (const TextBiDi::FTextDirectionInfo& VisualTextDirectionInfo : TextDirectionInfos)
			{
				const int32 VisualTextEndIndex = VisualTextDirectionInfo.StartIndex + VisualTextDirectionInfo.Length;

				for (int32 CurrentBlockStartIndex = VisualTextDirectionInfo.StartIndex; CurrentBlockStartIndex < VisualTextEndIndex; )
				{
					const TSharedRef<ILayoutBlock>* FoundLineBlock = OutSoftLine.FindByPredicate([&](const TSharedRef<ILayoutBlock>& InLineBlock) -> bool
					{
						return !InLineBlock->GetTextRange().IsEmpty() && InLineBlock->GetTextRange().BeginIndex == CurrentBlockStartIndex;
					});

					checkf(FoundLineBlock, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

					const TSharedRef<ILayoutBlock>& FoundLineBlockRef = *FoundLineBlock;
					if (VisualTextDirectionInfo.TextDirection == TextBiDi::ETextDirection::LeftToRight)
					{
						CurrentVisualSoftLine.Add(FoundLineBlockRef);
					}
					else
					{
						CurrentVisualSoftLine.Insert(FoundLineBlockRef, 0);
					}
					CurrentBlockStartIndex = FoundLineBlockRef->GetTextRange().EndIndex;
				}

				VisualSoftLine.Append(MoveTemp(CurrentVisualSoftLine));
				CurrentVisualSoftLine.Reset();
			}

			OutSoftLine = MoveTemp(VisualSoftLine);
		}

		float MaxLineWidth = 0.f;
		float PrevBlockOutlineWidth = 0.f;
		float CurrentHorizontalPos = 0.0f;
		for (int32 Index = 0; Index < OutSoftLine.Num(); Index++)
		{
			const TSharedRef< ILayoutBlock > Block = OutSoftLine[ Index ];
			const TSharedRef< IRun > Run = Block->GetRun();

			const FTextRange BlockTextRange = Block->GetTextRange();

			const int16 BlockBaseline = Run->GetBaseLine(Scale);
			const int16 VerticalOffset = MaxAboveBaseline - Block->GetSize().Y - BlockBaseline;
			const int8 BlockKerning = Run->GetKerning(BlockTextRange.BeginIndex, Scale, RunTextContext);

			const FVector2d BlockShadowSize = Run->GetShadowSize(BlockTextRange.BeginIndex, BlockTextRange.EndIndex, Scale);
			const FVector2d BlockOutlineSize = Run->GetOutlineSize(BlockTextRange.BeginIndex, BlockTextRange.EndIndex, Scale);
		
			if (Index > 0)
			{
				// Scooch adjacent blocks with different outline sizes as needed to keep the actual glyph spacing correct 
				const float PrevBlockOutlineAdjustment = (PrevBlockOutlineWidth - BlockOutlineSize.X) * 0.5f;
				CurrentHorizontalPos += PrevBlockOutlineAdjustment;
			}
			Block->SetLocationOffset(FVector2D(CurrentOffset.X + CurrentHorizontalPos + BlockKerning, CurrentOffset.Y + VerticalOffset));

			// Track what the final width would be as each block is added, on the off-chance that one has
			// such a dramatic outline or shadow that it extends beyond the extent of the final block
			MaxLineWidth = FMath::Max(MaxLineWidth, CurrentHorizontalPos + Block->GetSize().X);
			CurrentHorizontalPos += Block->GetSize().X - BlockShadowSize.X - BlockOutlineSize.X;

			PrevBlockOutlineWidth = BlockOutlineSize.X;
		}

		UnscaleLineHeight = MaxAboveBaseline + MaxBelowBaseline;

		LineSize.X = MaxLineWidth;
		LineSize.Y = UnscaleLineHeight * LineHeightPercentage;

		FTextLayout::FLineView LineView;
		LineView.Offset = CurrentOffset;
		LineView.Size = LineSize;
		LineView.TextHeight = UnscaleLineHeight;
		LineView.JustificationWidth = JustificationWidth.Get(LineView.Size.X);
		LineView.Range = SoftLineRange;
		LineView.TextBaseDirection = LineModel.TextBaseDirection;
		LineView.ModelIndex = LineModelIndex;
		LineView.Blocks.Append( OutSoftLine );

		LineViews.Add( LineView );

		// Does this new line view require justification?
		// Any text that visually doesn't justify left requires justification, but so does a line that visually starts with RTL text as we may 
		// need to adjust for the trailing whitespace on that soft-wrapped line (which will be visually leading whitespace for RTL text)
		if (CalculateLineViewVisualJustification(LineView) != ETextJustify::Left || LineView.Blocks[0]->GetTextContext().TextDirection == TextBiDi::ETextDirection::RightToLeft)
		{
			LineViewsToJustify.Add(LineViews.Num() - 1);
		}
	}

	TextLayoutSize.DrawWidth = FMath::Max( TextLayoutSize.DrawWidth, LineSize.X ); // DrawWidth is the size of the longest line + the Margin
	TextLayoutSize.WrappedWidth = FMath::Max( TextLayoutSize.WrappedWidth, (StopIndex == INDEX_NONE) ? LineSize.X : WrappedLineWidth ); // WrappedWidth is the size of the longest line + the Margin + any trailing whitespace width
	TextLayoutSize.Height += LineSize.Y; // Height is the total height of all lines
	OverHeight = LineSize.Y - UnscaleLineHeight;
}

void FTextLayout::JustifyLayout()
{
	if (LineViewsToJustify.Num() == 0)
	{
		return;
	}

	const float LayoutWidthNoMargin = FMath::Max(TextLayoutSize.DrawWidth, ViewSize.X * Scale) - (Margin.GetTotalSpaceAlong<Orient_Horizontal>() * Scale);

	for (const int32 LineViewIndex : LineViewsToJustify)
	{
		FLineView& LineView = LineViews[LineViewIndex];

		const ETextJustify::Type VisualJustification = CalculateLineViewVisualJustification(LineView);

		// The JustificationWidth of a line omits any trailing whitespace that might have been left over from soft-wrapping the text
		// We only want to use this (rather than the full line width) when the justification is at odds with the text flow direction at the edge being justified
		// That is, when right-justifying LTR text we will use the JustificationWidth as the trailing whitespace is visually on the right, but when 
		// right-justifying RTL text we will use the full line width as the trailing whitespace is visually on the left
		float LineJustificationWidth = LineView.JustificationWidth;
		if ((VisualJustification == ETextJustify::Right && LineView.Blocks.Last()->GetTextContext().TextDirection == TextBiDi::ETextDirection::RightToLeft) ||
			(VisualJustification == ETextJustify::Left && LineView.Blocks[0]->GetTextContext().TextDirection == TextBiDi::ETextDirection::LeftToRight))
		{
			LineJustificationWidth = LineView.Size.X;
		}

		FVector2D OffsetAdjustment = FVector2D::ZeroVector;
		switch (VisualJustification)
		{
		case ETextJustify::Left:
		case ETextJustify::InvariantLeft:
			{
				const float ExtraSpace = LineView.Size.X - LineJustificationWidth;
				OffsetAdjustment.X = -ExtraSpace;
			}
			break;

		case ETextJustify::Center:
			{
				const float ExtraSpace = LayoutWidthNoMargin - LineJustificationWidth;
				OffsetAdjustment.X = ExtraSpace * 0.5f;
			}
			break;

		case ETextJustify::Right:
		case ETextJustify::InvariantRight:
			{
				const float ExtraSpace = LayoutWidthNoMargin - LineJustificationWidth;
				OffsetAdjustment.X = ExtraSpace;
			}
			break;

		default:
			checkf(false, TEXT("Unknown ETextJustify"));
			break;
		}

		LineView.Offset += OffsetAdjustment;

		for (const TSharedRef< ILayoutBlock >& Block : LineView.Blocks)
		{
			Block->SetLocationOffset( Block->GetLocationOffset() + OffsetAdjustment );
		}
	}
}

float FTextLayout::GetWrappingDrawWidth() const
{
	checkf(WrappingWidth >= 0, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
	return FMath::Max( 0.01f, ( WrappingWidth - Margin.GetTotalSpaceAlong<Orient_Horizontal>() ) * Scale );
}

void FTextLayout::FlowLayout()
{
	const float WrappingDrawWidth = GetWrappingDrawWidth();
	TArray< TSharedRef< ILayoutBlock > > SoftLine;
	for (int32 LineModelIndex = 0; LineModelIndex < LineModels.Num(); LineModelIndex++)
	{
		FLineModel& LineModel = LineModels[ LineModelIndex ];
		CalculateLineTextDirection(LineModel);
		FlushLineTextShapingCache(LineModel);
		CreateLineWrappingCache(LineModel);

		FlowLineLayout(LineModelIndex, WrappingDrawWidth, SoftLine);
	}
}

void FTextLayout::MarginLayout()
{
	// Add on the margins to the layout size
	const float MarginWidth = Margin.GetTotalSpaceAlong<Orient_Horizontal>() * Scale;
	const float MarginHeight = Margin.GetTotalSpaceAlong<Orient_Vertical>() * Scale;
	TextLayoutSize.DrawWidth += MarginWidth;
	TextLayoutSize.WrappedWidth += MarginWidth;
	TextLayoutSize.Height += MarginHeight;

	// Adjust the lines to be offset
	FVector2D OffsetAdjustment = FVector2D(Margin.Left, Margin.Top) * Scale;
	for (FLineView& LineView : LineViews)
	{
		LineView.Offset += OffsetAdjustment;

		for (const TSharedRef< ILayoutBlock >& Block : LineView.Blocks)
		{
			Block->SetLocationOffset( Block->GetLocationOffset() + OffsetAdjustment );
		}
	}
}

void FTextLayout::FlowLineLayout(const int32 LineModelIndex, const float WrappingDrawWidth, TArray<TSharedRef<ILayoutBlock>>& SoftLine)
{
	const FLineModel& LineModel = LineModels[ LineModelIndex ];

	int32 CurrentRunIndex = 0;
	int32 PreviousBlockEnd = 0;

	int32 CurrentRendererIndex = 0;
	if ( CurrentRendererIndex >= LineModel.RunRenderers.Num() )
	{
		CurrentRendererIndex = INDEX_NONE;
	}

	const bool IsWrapping = WrappingWidth > 0.0f;

	// if the Line doesn't have any BreakCandidates, or we're not wrapping text
	if (!IsWrapping || LineModel.BreakCandidates.Num() == 0 )
	{
		//Then iterate over all of its runs
		CreateLineViewBlocks( LineModelIndex, INDEX_NONE, 0.0f, TOptional<float>(), /*OUT*/CurrentRunIndex, /*OUT*/CurrentRendererIndex, /*OUT*/PreviousBlockEnd, SoftLine );
		checkf(CurrentRunIndex == LineModel.Runs.Num(), TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
		SoftLine.Reset();
	}
	else
	{
		float CurrentWidth = 0.0f;
		for (int32 BreakIndex = 0; BreakIndex < LineModel.BreakCandidates.Num(); BreakIndex++)
		{
			const FBreakCandidate& Break = LineModel.BreakCandidates[ BreakIndex ];

			const bool IsLastBreak = BreakIndex + 1 == LineModel.BreakCandidates.Num();
			const bool IsFirstBreakOnSoftLine = CurrentWidth == 0.0f;
			const int8 Kerning = ( IsFirstBreakOnSoftLine ) ? Break.Kerning : 0;
			const bool BreakDoesFit = CurrentWidth + Break.ActualSize.X + Kerning <= WrappingDrawWidth;
			const bool BreakWithoutTrailingWhitespaceDoesFit = CurrentWidth + Break.TrimmedWidth + Kerning <= WrappingDrawWidth;

			if ( WrappingPolicy == ETextWrappingPolicy::AllowPerCharacterWrapping && !BreakWithoutTrailingWhitespaceDoesFit && IsFirstBreakOnSoftLine )
			{
				const FRunTextContext RunTextContext(TextShapingMethod, LineModel.TextBaseDirection, LineModel.ShapedTextCache, &LoadingFontFaces);

				// This is a single word that's too long to fit onto a line, so we'll fallback to wrapping this word at grapheme cluster boundaries - this may require more than a single split
				const int32 NonBreakingStringIndexOffset = PreviousBlockEnd;
				const FString NonBreakingString = FString::ConstructFromPtrSize(**LineModel.Text + NonBreakingStringIndexOffset, Break.TrimmedRange.EndIndex - NonBreakingStringIndexOffset);
				GraphemeBreakIterator->SetStringRef(&NonBreakingString);

				CurrentWidth = 0.0f;
				for (int32 PreviousBreak = 0, CurrentBreak = GraphemeBreakIterator->MoveToNext(); CurrentBreak != INDEX_NONE;)
				{
					// The break range may span multiple runs
					float BreakWidth = 0.0f;
					for (int32 BreakRunIndex = CurrentRunIndex; BreakRunIndex < LineModel.Runs.Num(); ++BreakRunIndex)
					{
						const FRunModel& RunModel = LineModel.Runs[BreakRunIndex];
						const FTextRange RunRange = RunModel.GetTextRange();

						const int32 StartIndex = FMath::Max(RunRange.BeginIndex - NonBreakingStringIndexOffset, PreviousBreak);
						const int32 StopIndex = FMath::Max(StartIndex, FMath::Min(RunRange.EndIndex - NonBreakingStringIndexOffset, CurrentBreak));

						BreakWidth += RunModel.GetRun()->Measure(StartIndex + NonBreakingStringIndexOffset, StopIndex + NonBreakingStringIndexOffset, Scale, RunTextContext).X;

						if (StopIndex == CurrentBreak)
						{
							break;
						}
					}

					bool bAdvanceIterator = true;

					const bool IsLastGraphemeBreak = CurrentBreak == NonBreakingString.Len();
					const bool IsFirstGraphemeBreakOnSoftLine = CurrentWidth == 0.0f;
					const bool GraphemeBreakDoesFit = CurrentWidth + BreakWidth <= WrappingDrawWidth;
					
					// TODO : This probably needs to be handled differently
					// if your break fits, 
					// and you're the last grapheme, 
					// and you have another break candidate available, 
					// and that break candidate fits on the line (excluding trailing whitespace), 
					// then hand over responsibility of adding the line to the next iteration of the outer loop
					if (GraphemeBreakDoesFit && IsLastGraphemeBreak && BreakIndex + 1 < LineModel.BreakCandidates.Num())
					{
						const bool NextBreakWithoutTrailingWhitespaceDoesFit = CurrentWidth + BreakWidth + LineModel.BreakCandidates[BreakIndex + 1].TrimmedWidth + Kerning <= WrappingDrawWidth;
						if (NextBreakWithoutTrailingWhitespaceDoesFit)
						{
							break;
						}
					}

					if (!GraphemeBreakDoesFit || IsLastGraphemeBreak)
					{
						bool bHasTrailingText = IsLastGraphemeBreak;

						int32 CurrentBlockEnd = INDEX_NONE;
						if (IsLastGraphemeBreak && (GraphemeBreakDoesFit || IsFirstGraphemeBreakOnSoftLine))
						{
							bHasTrailingText = false;

							// Use the actual range if we've reached the end of the string, as that includes any trailing whitespace that we want to include on this line
							CurrentBlockEnd = Break.ActualRange.EndIndex;
						}
						else if (IsFirstGraphemeBreakOnSoftLine)
						{
							// We mustn't break without adding any grapheme clusters as that would cause an infinite loop
							CurrentBlockEnd = CurrentBreak + NonBreakingStringIndexOffset;
						}
						else
						{
							bAdvanceIterator = false; // Don't advance the iterator in this case since we need to include the current grapheme cluster on the line after we split
							CurrentBlockEnd = PreviousBreak + NonBreakingStringIndexOffset;
						}
						
						CreateLineViewBlocks(LineModelIndex, CurrentBlockEnd, CurrentWidth, TOptional<float>(), /*OUT*/CurrentRunIndex, /*OUT*/CurrentRendererIndex, /*OUT*/PreviousBlockEnd, SoftLine);

						if (CurrentRunIndex < LineModel.Runs.Num() && CurrentBlockEnd == LineModel.Runs[CurrentRunIndex].GetTextRange().EndIndex)
						{
							++CurrentRunIndex;
						}

						PreviousBlockEnd = CurrentBlockEnd;

						CurrentWidth = 0.0f;
						SoftLine.Reset();

						// Add any trailing text
						if (bHasTrailingText)
						{
							CurrentBlockEnd = Break.ActualRange.EndIndex;
							CreateLineViewBlocks(LineModelIndex, CurrentBlockEnd, CurrentWidth, TOptional<float>(), /*OUT*/CurrentRunIndex, /*OUT*/CurrentRendererIndex, /*OUT*/PreviousBlockEnd, SoftLine);

							if (CurrentRunIndex < LineModel.Runs.Num() && CurrentBlockEnd == LineModel.Runs[CurrentRunIndex].GetTextRange().EndIndex)
							{
								++CurrentRunIndex;
							}

							PreviousBlockEnd = CurrentBlockEnd;

							CurrentWidth = 0.0f;
							SoftLine.Reset();

							// Always advance the iterator when we've finished processing the text so that we exit the loop
							bAdvanceIterator = true;
						}
					}
					else
					{
						CurrentWidth += BreakWidth;
					}

					if (bAdvanceIterator)
					{
						PreviousBreak = CurrentBreak;
						CurrentBreak = GraphemeBreakIterator->MoveToNext();
					}
				}

				GraphemeBreakIterator->ClearString();
			}
			else if ( !BreakDoesFit || IsLastBreak )
			{
				const bool IsFirstBreak = BreakIndex == 0;

				const FBreakCandidate& FinalBreakOnSoftLine = ( !IsFirstBreak && !IsFirstBreakOnSoftLine && !BreakWithoutTrailingWhitespaceDoesFit ) ? LineModel.BreakCandidates[ --BreakIndex ] : Break;
				
				// We want the wrapped line width to contain the first piece of trailing whitespace for a line, however we only do this if we have trailing whitespace
				// otherwise very long non-breaking words can cause the wrapped line width to expand beyond the desired wrap width
				float WrappedLineWidth = CurrentWidth;
				if ( BreakWithoutTrailingWhitespaceDoesFit )
				{
					// This break has trailing whitespace
					WrappedLineWidth += ( FinalBreakOnSoftLine.TrimmedWidth + FinalBreakOnSoftLine.FirstTrailingWhitespaceCharWidth );
				}
				else
				{
					// This break is longer than the wrapping point, so make sure and clamp the line size to the given wrapping width
					WrappedLineWidth += FinalBreakOnSoftLine.ActualSize.X;
					WrappedLineWidth = FMath::Min(WrappedLineWidth, WrappingDrawWidth);
				}

				// We want wrapped lines to ignore any trailing whitespace when justifying
				// If FinalBreakOnSoftLine isn't the current Break, then the size of FinalBreakOnSoftLine (including its trailing whitespace) will have already
				// been added to CurrentWidth, so we need to remove that again before adding the trimmed width (which is the width we should justify with)
				// We should not attempt to adjust the last break on a soft-line as that might have explicit trailing whitespace
				TOptional<float> JustifiedLineWidth;
				if ( &FinalBreakOnSoftLine != &LineModel.BreakCandidates.Last() )
				{
					JustifiedLineWidth = CurrentWidth - (&FinalBreakOnSoftLine == &Break ? 0.0f : FinalBreakOnSoftLine.ActualSize.X) + FinalBreakOnSoftLine.TrimmedWidth;
				}

				CreateLineViewBlocks( LineModelIndex, FinalBreakOnSoftLine.ActualRange.EndIndex, WrappedLineWidth, JustifiedLineWidth, /*OUT*/CurrentRunIndex, /*OUT*/CurrentRendererIndex, /*OUT*/PreviousBlockEnd, SoftLine );

				if ( CurrentRunIndex < LineModel.Runs.Num() && FinalBreakOnSoftLine.ActualRange.EndIndex == LineModel.Runs[ CurrentRunIndex ].GetTextRange().EndIndex )
				{
					++CurrentRunIndex;
				}

				PreviousBlockEnd = FinalBreakOnSoftLine.ActualRange.EndIndex;

				CurrentWidth = 0.0f;
				SoftLine.Reset();
			} 
			else
			{
				CurrentWidth += Break.ActualSize.X;
			}
		}
	}
	if (!ApplyLineHeightToBottomLine && LineModelIndex == LineModels.Num() - 1)
	{
		TextLayoutSize.Height -= OverHeight;
	}
}

void FTextLayout::FlowHighlights()
{
	// FlowLayout must have been called first
	checkf(!(DirtyFlags & ETextLayoutDirtyState::Layout), TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

	for (FLineView& LineView : LineViews)
	{
		LineView.UnderlayHighlights.Empty();
		LineView.OverlayHighlights.Empty();

		FLineModel& LineModel = LineModels[LineView.ModelIndex];
		
		const FRunTextContext RunTextContext(TextShapingMethod, LineModel.TextBaseDirection, LineModel.ShapedTextCache, &LoadingFontFaces);

		// Insert each highlighter into every line view that's within its range, either as an underlay, or as an overlay
		for (FTextLineHighlight& LineHighlight : LineModel.LineHighlights)
		{
			if (LineHighlight.LineIndex != LineView.ModelIndex)
			{
				continue;
			}

			const FTextRange LineViewHighlightRange = LineView.Range.Intersect(LineHighlight.Range);
			if (LineViewHighlightRange.IsEmpty() && LineViewHighlightRange != LineView.Range)
			{
				continue;
			}

			FLineViewHighlight LineViewHighlight;
			LineViewHighlight.Offset = FVector2D(0.0f, 0.0f);
			LineViewHighlight.Width = 0.0f;
			LineViewHighlight.Highlighter = LineHighlight.Highlighter;

			bool bHasStartedHighlighting = false;
			float RunningBlockOffset = LineViewHighlight.Offset.X;

			auto AppendLineViewHighlight = [&LineHighlight, &LineView, &LineViewHighlight, &LineViewHighlightRange]()
			{
				if (LineViewHighlightRange.IsEmpty() || LineViewHighlight.Width > 0.0f)
				{
					if (LineHighlight.ZOrder < 0)
					{
						LineView.UnderlayHighlights.Add(LineViewHighlight);
					}
					else
					{
						LineView.OverlayHighlights.Add(LineViewHighlight);
					}
				}
			};

			auto AppendLineViewHighlightAndReset = [&AppendLineViewHighlight, &LineViewHighlight, &RunningBlockOffset]()
			{
				AppendLineViewHighlight();

				LineViewHighlight.Offset.X = RunningBlockOffset;
				LineViewHighlight.Width = 0.0f;
			};

			float PrevBlockOutlineWidth = 0.f;
			for (int32 CurrentBlockIndex = 0; CurrentBlockIndex < LineView.Blocks.Num(); ++CurrentBlockIndex)
			{
				const TSharedRef<ILayoutBlock>& Block = LineView.Blocks[CurrentBlockIndex];
				const FTextRange& BlockTextRange = Block->GetTextRange();
				const TSharedRef<IRun> Run = Block->GetRun();

				const FVector2d BlockShadowSize = Run->GetShadowSize(BlockTextRange.BeginIndex, BlockTextRange.EndIndex, Scale);
				const FVector2d BlockOutlineSize = Run->GetOutlineSize(BlockTextRange.BeginIndex, BlockTextRange.EndIndex, Scale);
				
				if (CurrentBlockIndex > 0)
				{
					// Scooch adjacent blocks with different outline sizes as needed to keep the actual glyph spacing correct 
					const float PrevBlockOutlineAdjustment = (PrevBlockOutlineWidth - BlockOutlineSize.X) * 0.5f;
					RunningBlockOffset += PrevBlockOutlineAdjustment;
				}
				PrevBlockOutlineWidth = BlockOutlineSize.X;

				const float CurrentBlockRunningOffset = RunningBlockOffset;

				RunningBlockOffset += Block->GetSize().X - BlockShadowSize.X - BlockOutlineSize.X;

				const FTextRange IntersectedRange = BlockTextRange.Intersect(LineViewHighlightRange);
				const bool bBlockIsHighlighted = !IntersectedRange.IsEmpty() || IntersectedRange == LineViewHighlightRange;
				const bool bBlockIsFullyHighlighted = IntersectedRange == BlockTextRange;

				if (bBlockIsHighlighted)
				{
					// This block should be part of the highlight... but how?
					if (!bHasStartedHighlighting)
					{
						// This block is the start of the highlight
						bHasStartedHighlighting = true;

						LineViewHighlight.Offset.X = CurrentBlockRunningOffset;
						LineViewHighlight.Offset.Y = Block->GetLocationOffset().Y - LineView.Offset.Y;

						// The width always includes size of the intersecting text
						if (!IntersectedRange.IsEmpty())
						{
							LineViewHighlight.Width += Run->Measure(IntersectedRange.BeginIndex, IntersectedRange.EndIndex, Scale, RunTextContext).X;

							// In LTR text, the space before the start of the text is added as an offset
							// In RTL text, the space after the end of the text (which is visually on the left) is added as an offset
							if (Block->GetTextContext().TextDirection == TextBiDi::ETextDirection::LeftToRight)
							{
								const FVector2d ShadowSize = Run->GetShadowSize(BlockTextRange.BeginIndex, IntersectedRange.BeginIndex, Scale);
								const FVector2d OutlineSize = Run->GetOutlineSize(BlockTextRange.BeginIndex, IntersectedRange.BeginIndex, Scale);

								LineViewHighlight.Offset.X += Run->Measure(BlockTextRange.BeginIndex, IntersectedRange.BeginIndex, Scale, RunTextContext).X - ShadowSize.X - OutlineSize.X;
							}
							else
							{
								const FVector2d ShadowSize = Run->GetShadowSize(IntersectedRange.EndIndex, BlockTextRange.EndIndex, Scale);
								const FVector2d OutlineSize = Run->GetOutlineSize(IntersectedRange.EndIndex, BlockTextRange.EndIndex, Scale);

								LineViewHighlight.Offset.X += Run->Measure(IntersectedRange.EndIndex, BlockTextRange.EndIndex, Scale, RunTextContext).X - ShadowSize.X - OutlineSize.X;
							}
						}
					}
					else
					{

						// For the y offset, we'll use the smallest LocationOffset included in the highlighted area. This should align any highlights with the tallest block
						if (Block->GetLocationOffset().Y < LineViewHighlight.Offset.Y + LineView.Offset.Y)
						{
							LineViewHighlight.Offset.Y = Block->GetLocationOffset().Y - LineView.Offset.Y;
						}

						// Test to see whether the current highlight covered the RHS of the previous block
						const bool bPrevBlockHighlightedRHS = (LineViewHighlight.Offset.X + LineViewHighlight.Width) >= CurrentBlockRunningOffset;

						// This block is part of an existing highlight... but should it split the current visual highlight?
						if (bBlockIsFullyHighlighted)
						{
							if (bPrevBlockHighlightedRHS)
							{
								// Fully highlighted blocks following a block that highlighted its RHS can continue the current visual highlight, so just append its width
								LineViewHighlight.Width += Block->GetSize().X;
							}
							else
							{
								// Fully highlighted blocks following a visual gap highlighted block have to start a new visual highlight
								AppendLineViewHighlightAndReset();

								LineViewHighlight.Offset.X = CurrentBlockRunningOffset;
								LineViewHighlight.Width += Block->GetSize().X;
							}
						}
						else
						{
							// When the text flow direction doesn't match the block text flow direction, we'll need to apply an offset to compensate for the selection potentially starting mid-way through the block
							float BlockHighlightOffsetAdjustment = 0.0f;
							if (Block->GetTextContext().TextDirection != Block->GetTextContext().BaseDirection)
							{
								// In LTR text, the space before the start of the text is added as an offset
								// In RTL text, the space after the end of the text (which is visually on the left) is added as an offset
								if (Block->GetTextContext().TextDirection == TextBiDi::ETextDirection::LeftToRight)
								{
									BlockHighlightOffsetAdjustment += Run->Measure(BlockTextRange.BeginIndex, IntersectedRange.BeginIndex, Scale, RunTextContext).X;
								}
								else
								{
									BlockHighlightOffsetAdjustment += Run->Measure(IntersectedRange.EndIndex, BlockTextRange.EndIndex, Scale, RunTextContext).X;
								}
							}

							// Append the partial width to the current visual highlight before starting another
							float BlockHighlightWidth = 0.0f;
							if (!IntersectedRange.IsEmpty())
							{
								BlockHighlightWidth += Run->Measure(IntersectedRange.BeginIndex, IntersectedRange.EndIndex, Scale, RunTextContext).X;
							}

							// Test to see whether the extra highlight will cover the LHS of this block
							const bool bBlockHighlighsLHS = BlockHighlightOffsetAdjustment == 0.0f;

							// Partially highlighted blocks can continue the current visual highlight if it highlighted the RHS of the previous block, and this block highlights its LHS
							const bool bIsVisuallyContiguous = bPrevBlockHighlightedRHS && bBlockHighlighsLHS;
							
							if (bIsVisuallyContiguous)
							{
								// Append the partial width to the current visual highlight before starting another
								LineViewHighlight.Width += BlockHighlightWidth;
								AppendLineViewHighlightAndReset();
							}
							else
							{
								// There's a visual gap, so we need to end this visual highlight and start another
								AppendLineViewHighlightAndReset();
								LineViewHighlight.Offset.X = CurrentBlockRunningOffset + BlockHighlightOffsetAdjustment;
								LineViewHighlight.Width += BlockHighlightWidth;
							}
						}
					}
				}
				else
				{
					// This block is not part of the current highlight... end any current highlight
					AppendLineViewHighlightAndReset();
				}
			}

			// Append any trailing highlight
			AppendLineViewHighlight();
		}
	}
}

void FTextLayout::EndLayout()
{
	for (FLineModel& LineModel : LineModels)
	{
		EndLineLayout(LineModel);
	}

	if (LoadingFontFaces.IsEmpty())
	{
		// We just finished loading all fonts 
		if (bLoadingFontFacesOnPreviousLayout)
		{
			HandleAllFontFacesFinishLoading();
			bLoadingFontFacesOnPreviousLayout = !LoadingFontFaces.IsEmpty();
		}
	}
	else
	{
		// We just started loading necessary font faces 
		if (!bLoadingFontFacesOnPreviousLayout)
		{
			bLoadingFontFacesOnPreviousLayout = !LoadingFontFaces.IsEmpty();
		}
	}
}

void FTextLayout::EndLineLayout(FLineModel& LineModel)
{
	for (FRunModel& RunModel : LineModel.Runs)
	{
		RunModel.EndLayout();
	}
}

void FTextLayout::BeginLayout()
{
	for (FLineModel& LineModel : LineModels)
	{
		BeginLineLayout(LineModel);
	}
}

void FTextLayout::BeginLineLayout(FLineModel& LineModel)
{
	for (FRunModel& RunModel : LineModel.Runs)
	{
		RunModel.BeginLayout();
	}
}

void FTextLayout::ClearView()
{
	TextLayoutSize = FTextLayoutSize();
	OverHeight = 0.f;
	LineViews.Empty();
	LineViewsToJustify.Empty();
}

void FTextLayout::TransformLineText(FLineModel& LineModel) const
{
	auto ApplyTransformedText = [&LineModel](FString&& InNewText)
	{
		if (ensure(LineModel.Text->Len() == InNewText.Len()))
		{
			*LineModel.Text = MoveTemp(InNewText);
		}
	};

	switch (TransformPolicy)
	{
	case ETextTransformPolicy::ToLower:
		ApplyTransformedText(FTextTransformer::ToLower(*LineModel.Text));
		break;
	case ETextTransformPolicy::ToUpper:
		ApplyTransformedText(FTextTransformer::ToUpper(*LineModel.Text));
		break;
	default:
		break;
	}
}

void FTextLayout::CalculateTextDirection()
{
	for (FLineModel& LineModel : LineModels)
	{
		CalculateLineTextDirection(LineModel);
	}
}

void FTextLayout::CalculateLineTextDirection(FLineModel& LineModel) const
{
	if (!(LineModel.DirtyFlags & ELineModelDirtyState::TextBaseDirection))
	{
		return;
	}

	switch(TextFlowDirection)
	{
	case ETextFlowDirection::Auto:
		// KerningOnly shaping implies LTR only text, so we can skip the text direction detection
		LineModel.TextBaseDirection = (TextShapingMethod == ETextShapingMethod::KerningOnly) ? TextBiDi::ETextDirection::LeftToRight : TextBiDi::ComputeBaseDirection(*LineModel.Text);
		break;
	case ETextFlowDirection::LeftToRight:
		LineModel.TextBaseDirection = TextBiDi::ETextDirection::LeftToRight;
		break;
	case ETextFlowDirection::RightToLeft:
		LineModel.TextBaseDirection = TextBiDi::ETextDirection::RightToLeft;
		break;
	case ETextFlowDirection::Culture:
		LineModel.TextBaseDirection = FInternationalization::Get().GetCurrentLanguage()->IsRightToLeft() ? TextBiDi::ETextDirection::RightToLeft : TextBiDi::ETextDirection::LeftToRight;
		break;
	default:
		break;
	}
	
	LineModel.DirtyFlags &= ~ELineModelDirtyState::TextBaseDirection;
}

ETextJustify::Type FTextLayout::CalculateLineViewVisualJustification(const FLineView& LineView) const
{
	// Work out the visual justification to use for this line
	ETextJustify::Type VisualJustification = Justification;
	if (LineView.TextBaseDirection == TextBiDi::ETextDirection::RightToLeft)
	{
		if (VisualJustification == ETextJustify::Left)
		{
			VisualJustification = ETextJustify::Right;
		}
		else if (VisualJustification == ETextJustify::Right)
		{
			VisualJustification = ETextJustify::Left;
		}
	}
	return VisualJustification;
}

void FTextLayout::CreateWrappingCache()
{
	for (FLineModel& LineModel : LineModels)
	{
		CreateLineWrappingCache(LineModel);
	}
}

void FTextLayout::CreateLineWrappingCache(FLineModel& LineModel)
{
	if (!(LineModel.DirtyFlags & ELineModelDirtyState::WrappingInformation))
	{
		return;
	}

	LineModel.BreakCandidates.Empty();
	LineModel.DirtyFlags &= ~ELineModelDirtyState::WrappingInformation;

	for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
	{
		LineModel.Runs[RunIndex].ClearCache();
	}

	const bool IsWrapping = WrappingWidth > 0.0f;
	if (!IsWrapping)
	{
		return;
	}

	// If we've not yet been provided with a custom line break iterator, then just use the default one
	if (!LineBreakIterator.IsValid())
	{
		LineBreakIterator = GSlateUseSharedBreakIterator  ? FBreakIterator::GetLineBreakIterator() : FBreakIterator::CreateLineBreakIterator();
	}

	LineBreakIterator->SetStringRef(&LineModel.Text.Get());

	int32 PreviousBreak = 0;
	int32 CurrentBreak = 0;
	int32 CurrentRunIndex = 0;

	while( ( CurrentBreak = LineBreakIterator->MoveToNext() ) != INDEX_NONE )
	{
		LineModel.BreakCandidates.Add( CreateBreakCandidate(/*OUT*/CurrentRunIndex, LineModel, PreviousBreak, CurrentBreak) );
		PreviousBreak = CurrentBreak;
	}

	LineBreakIterator->ClearString();
}

void FTextLayout::FlushTextShapingCache()
{
	for (FLineModel& LineModel : LineModels)
	{
		FlushLineTextShapingCache(LineModel);
	}
}

void FTextLayout::FlushLineTextShapingCache(FLineModel& LineModel)
{
	if (!(LineModel.DirtyFlags & ELineModelDirtyState::ShapingCache))
	{
		return;
	}

	LineModel.ShapedTextCache->Clear();
	LineModel.DirtyFlags &= ~ELineModelDirtyState::ShapingCache;
}

void FTextLayout::DirtyAllLineModels(const ELineModelDirtyState::Flags InDirtyFlags)
{
	for (FLineModel& LineModel : LineModels)
	{
		LineModel.DirtyFlags |= InDirtyFlags;
	}
}

FTextLayout::FTextLayout()
	: ViewSize(ForceInitToZero)
	, ScrollOffset(ForceInitToZero)
	, LineBreakIterator() // Initialized in FTextLayout::CreateWrappingCache if no custom iterator is provided
	, GraphemeBreakIterator(GSlateUseSharedBreakIterator ? FBreakIterator::GetCharacterBoundaryIterator() : FBreakIterator::CreateCharacterBoundaryIterator())
	, WordBreakIterator(GSlateUseSharedBreakIterator ? FBreakIterator::GetWordBreakIterator() : FBreakIterator::CreateWordBreakIterator())
	, Justification(ETextJustify::Left)
	, TextBiDiDetection(TextBiDi::CreateTextBiDi())
	, Scale(1.0f)
	, WrappingWidth(0.0f)
	, LineHeightPercentage(1.0f)
	, OverHeight(0.0f)
	, DirtyFlags( ETextLayoutDirtyState::None )
	, TextShapingMethod( GetDefaultTextShapingMethod() )
	, TextFlowDirection( GetDefaultTextFlowDirection() )
	, WrappingPolicy( ETextWrappingPolicy::DefaultWrapping )
	, TransformPolicy( ETextTransformPolicy::None )
	, ApplyLineHeightToBottomLine( true )
	, bLoadingFontFacesOnPreviousLayout(false)
{
}

void FTextLayout::UpdateIfNeeded()
{
	// GSlateLayoutGeneration is incremented when font faces are finally done async lazily loaded
	// This re-measures and lays out the text accordingly
	if (CachedLayoutGeneration != GSlateLayoutGeneration)
	{
		CachedLayoutGeneration = GSlateLayoutGeneration;
		DirtyFlags |= ETextLayoutDirtyState::Layout;
		DirtyAllLineModels(ELineModelDirtyState::All);
	}

	const bool bHasChangedLayout = !!(DirtyFlags & ETextLayoutDirtyState::Layout);
	const bool bHasChangedHighlights = !!(DirtyFlags & ETextLayoutDirtyState::Highlights);

	if ( bHasChangedLayout )
	{
		// if something has changed then create a new View
		UpdateLayout();
	}

	// If the layout has changed, we always need to update the highlights
	if ( bHasChangedLayout || bHasChangedHighlights)
	{
		UpdateHighlights();
	}
}

void FTextLayout::UpdateLayout()
{
	SCOPE_CYCLE_COUNTER(STAT_SlateTextLayout);

	// We clear all loading font faces every re-layout so that it can be populated anew during shaping 
	LoadingFontFaces.Empty();
	ClearView();
	BeginLayout();

	FlowLayout();
	JustifyLayout();
	MarginLayout();

	EndLayout();

	DirtyFlags &= ~ETextLayoutDirtyState::Layout;
}

void FTextLayout::UpdateHighlights()
{
	FlowHighlights();

	DirtyFlags &= ~ETextLayoutDirtyState::Highlights;
}

void FTextLayout::DirtyRunLayout(const TSharedRef<const IRun>& Run)
{
	for (int32 LineModelIndex = 0; LineModelIndex < LineModels.Num(); LineModelIndex++)
	{
		FLineModel& LineModel = LineModels[LineModelIndex];

		if (!(LineModel.DirtyFlags & ELineModelDirtyState::WrappingInformation))
		{
			for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
			{
				if (LineModel.Runs[RunIndex].GetRun() == Run)
				{
					LineModel.Runs[RunIndex].ClearCache();
					break;
				}
			}
		}
	}

	DirtyFlags |= ETextLayoutDirtyState::Layout;
}

void FTextLayout::DirtyLayout()
{
	DirtyFlags |= ETextLayoutDirtyState::Layout;

	// Clear out the entire cache so it gets regenerated on the text call to FlowLayout
	DirtyAllLineModels(ELineModelDirtyState::All);
}

bool FTextLayout::IsLayoutDirty() const
{
	return !!(DirtyFlags & ETextLayoutDirtyState::Layout);
}

void FTextLayout::ClearRunRenderers()
{
	for (int32 Index = 0; Index < LineModels.Num(); Index++)
	{
		if (LineModels[ Index ].RunRenderers.Num() )
		{
			LineModels[ Index ].RunRenderers.Empty();
			DirtyFlags |= ETextLayoutDirtyState::Layout;
		}
	}
}

void FTextLayout::SetRunRenderers( const TArray< FTextRunRenderer >& Renderers )
{
	ClearRunRenderers();

	for (int32 Index = 0; Index < Renderers.Num(); Index++)
	{
		AddRunRenderer( Renderers[ Index ] );
	}
}

void FTextLayout::AddRunRenderer( const FTextRunRenderer& Renderer )
{
	checkf(LineModels.IsValidIndex(Renderer.LineIndex), TEXT("Renderers must be for a valid Line Index!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));

	FLineModel& LineModel = LineModels[ Renderer.LineIndex ];

	// Renderers needs to be in order and not overlap
	bool bWasInserted = false;
	for (int32 Index = 0; Index < LineModel.RunRenderers.Num() && !bWasInserted; Index++)
	{
		if ( LineModel.RunRenderers[ Index ].Range.BeginIndex > Renderer.Range.BeginIndex )
		{
			checkf(Index == 0 || LineModel.RunRenderers[Index - 1].Range.EndIndex <= Renderer.Range.BeginIndex, TEXT("Renderers cannot overlap!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));
			LineModel.RunRenderers.Insert( Renderer, FMath::Max(0, Index - 1) );
			bWasInserted = true;
		}
		else if ( LineModel.RunRenderers[ Index ].Range.EndIndex > Renderer.Range.EndIndex )
		{
			checkf(LineModel.RunRenderers[Index].Range.BeginIndex >= Renderer.Range.EndIndex, TEXT("Renderers cannot overlap!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));
			LineModel.RunRenderers.Insert( Renderer, FMath::Max(0, Index - 1) );
			bWasInserted = true;
		}
	}

	if ( !bWasInserted )
	{
		LineModel.RunRenderers.Add( Renderer );
	}

	DirtyFlags |= ETextLayoutDirtyState::Layout;
}

void FTextLayout::RemoveRunRenderer( const FTextRunRenderer& Renderer )
{
	checkf(LineModels.IsValidIndex(Renderer.LineIndex), TEXT("Renderers must be for a valid Line Index!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));

	FLineModel& LineModel = LineModels[Renderer.LineIndex];

	bool bWasRemoved = false;
	for (int32 Index = 0; Index < LineModel.RunRenderers.Num(); ++Index)
	{
		if (LineModel.RunRenderers[Index] == Renderer)
		{
			LineModel.RunRenderers.RemoveAt(Index, EAllowShrinking::No);
			bWasRemoved = true;
			break;
		}
	}

	if (bWasRemoved)
	{
		DirtyFlags |= ETextLayoutDirtyState::Layout;
	}
}

void FTextLayout::ClearLineHighlights()
{
	for (int32 Index = 0; Index < LineModels.Num(); Index++)
	{
		if (LineModels[ Index ].LineHighlights.Num())
		{
			LineModels[ Index ].LineHighlights.Empty();
			DirtyFlags |= ETextLayoutDirtyState::Highlights;
		}
	}
}

void FTextLayout::SetLineHighlights( const TArray< FTextLineHighlight >& Highlights )
{
	ClearLineHighlights();

	for (int32 Index = 0; Index < Highlights.Num(); Index++)
	{
		AddLineHighlight( Highlights[ Index ] );
	}
}

void FTextLayout::AddLineHighlight( const FTextLineHighlight& Highlight )
{
	checkf(LineModels.IsValidIndex(Highlight.LineIndex), TEXT("Highlights must be for a valid Line Index!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));
	checkf(Highlight.ZOrder, TEXT("The highlight Z-order must be <0 to create an underlay, or >0 to create an overlay!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));

	FLineModel& LineModel = LineModels[ Highlight.LineIndex ];

	// Try and maintain a stable sorted z-order - highlights with the same z-order should just render in the order they were added
	bool bWasInserted = false;
	for (int32 Index = 0; Index < LineModel.LineHighlights.Num() && !bWasInserted; Index++)
	{
		if ( LineModel.LineHighlights[ Index ].ZOrder > Highlight.ZOrder )
		{
			LineModel.LineHighlights.Insert( Highlight, FMath::Max(0, Index - 1) );
			bWasInserted = true;
		}
	}

	if ( !bWasInserted )
	{
		LineModel.LineHighlights.Add( Highlight );
	}

	DirtyFlags |= ETextLayoutDirtyState::Highlights;
}

void FTextLayout::RemoveLineHighlight( const FTextLineHighlight& Highlight )
{
	checkf(LineModels.IsValidIndex(Highlight.LineIndex), TEXT("Highlights must be for a valid Line Index!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));
	checkf(Highlight.ZOrder, TEXT("The highlight Z-order must be <0 to create an underlay, or >0 to create an overlay!\n\tDebug Source: %s"), *DebugSourceInfo.Get(FString()));

	FLineModel& LineModel = LineModels[Highlight.LineIndex];

	bool bWasRemoved = false;
	for (int32 Index = 0; Index < LineModel.LineHighlights.Num(); ++Index)
	{
		if (LineModel.LineHighlights[Index] == Highlight)
		{
			LineModel.LineHighlights.RemoveAt(Index, EAllowShrinking::No);
			bWasRemoved = true;
			break;
		}
	}

	if (bWasRemoved)
	{
		DirtyFlags |= ETextLayoutDirtyState::Highlights;
	}
}

FTextLocation FTextLayout::GetTextLocationAt( const FLineView& LineView, const FVector2D& Relative, ETextHitPoint* const OutHitPoint ) const
{
	for (int32 BlockIndex = 0; BlockIndex < LineView.Blocks.Num(); BlockIndex++)
	{
		const TSharedRef< ILayoutBlock >& Block = LineView.Blocks[ BlockIndex ];
		const int32 TextIndex = Block->GetRun()->GetTextIndexAt(Block, FVector2D(Relative.X, Block->GetLocationOffset().Y), Scale, OutHitPoint);

		if ( TextIndex == INDEX_NONE )
		{
			continue;
		}

		return FTextLocation( LineView.ModelIndex, TextIndex );
	}

	const FLineModel& LineModel = LineModels[ LineView.ModelIndex ];
	const int32 LineTextLength = LineModel.Text->Len();
	if ( LineTextLength == 0 || !LineView.Blocks.Num() )
	{
		if(OutHitPoint)
		{
			*OutHitPoint = ETextHitPoint::WithinText;
		}
		return FTextLocation( LineView.ModelIndex, 0 );
	}
	else if (Relative.X < LineView.Blocks[0]->GetLocationOffset().X)
	{
		const auto& Block = LineView.Blocks[0];
		const FTextRange BlockRange = Block->GetTextRange();
		const FLayoutBlockTextContext BlockContext = Block->GetTextContext();
		if (BlockContext.TextDirection == TextBiDi::ETextDirection::LeftToRight)
		{
			if(OutHitPoint)
			{
				*OutHitPoint = ETextHitPoint::LeftGutter;
			}
			return FTextLocation(LineView.ModelIndex, BlockRange.BeginIndex);
		}
		else
		{
			if(OutHitPoint)
			{
				*OutHitPoint = ETextHitPoint::RightGutter;
			}
			return FTextLocation(LineView.ModelIndex, BlockRange.EndIndex);
		}
	}
	else
	{
		const auto& Block = LineView.Blocks.Last();
		const FTextRange BlockRange = Block->GetTextRange();
		const FLayoutBlockTextContext BlockContext = Block->GetTextContext();
		if (BlockContext.TextDirection == TextBiDi::ETextDirection::LeftToRight)
		{
			if(OutHitPoint)
			{
				*OutHitPoint = ETextHitPoint::RightGutter;
			}
			return FTextLocation(LineView.ModelIndex, BlockRange.EndIndex);
		}
		else
		{
			if(OutHitPoint)
			{
				*OutHitPoint = ETextHitPoint::LeftGutter;
			}
			return FTextLocation(LineView.ModelIndex, BlockRange.BeginIndex);
		}
	}
}

int32 FTextLayout::GetLineViewIndexForTextLocation(const TArray< FTextLayout::FLineView >& InLineViews, const FTextLocation& Location, const bool bPerformInclusiveBoundsCheck) const
{
	const int32 LineModelIndex = Location.GetLineIndex();
	const int32 Offset = Location.GetOffset();

	if (!LineModels.IsValidIndex(LineModelIndex))
	{
		return INDEX_NONE;
	}

	const FLineModel& LineModel = LineModels[LineModelIndex];
	for(int32 Index = 0; Index < InLineViews.Num(); Index++)
	{
		const FTextLayout::FLineView& LineView = InLineViews[Index];

		if(LineView.ModelIndex == LineModelIndex)
		{
			// Simple case where we're either the start of, or are contained within, the line view
			if(Offset == 0 || LineModel.Text->IsEmpty() || LineView.Range.Contains(Offset))
			{
				return Index;
			}

			// If we're the last line, then we also need to test for the end index being part of the range
			const bool bIsLastLineForModel = Index == (InLineViews.Num() - 1) || InLineViews[Index + 1].ModelIndex != LineModelIndex;
			if((bIsLastLineForModel || bPerformInclusiveBoundsCheck) && LineView.Range.EndIndex == Offset)
			{
				return Index;
			}
		}
	}

	return INDEX_NONE;
}

FTextLocation FTextLayout::GetTextLocationAt( const FVector2D& Relative, ETextHitPoint* const OutHitPoint ) const
{
	// Early out if we have no LineViews
	if (LineViews.Num() == 0)
	{
		return FTextLocation(0,0);
	}

	// Iterate until we find a LineView that is below our expected Y location
	int32 ViewIndex;
	for (ViewIndex = 0; ViewIndex < LineViews.Num(); ViewIndex++)
	{
		const FLineView& LineView = LineViews[ ViewIndex ];

		if (LineView.Offset.Y > Relative.Y)
		{
			// Set the ViewIndex back to the previous line, but not lower than the first line
			ViewIndex = FMath::Max( 0, ViewIndex - 1 );
			break;
		}
	}

	//if none of the lines are below our expected Y location then...
	if (ViewIndex >= LineViews.Num())
	{
		// just use the very last line
		ViewIndex = LineViews.Num() - 1;
		const FLineView& LineView = LineViews[ ViewIndex ];
		return GetTextLocationAt( LineView, Relative, OutHitPoint );
	}
	else 
	{
		// if the current LineView does not encapsulate our expected Y location then jump to the next LineView
		// if we aren't already at the last LineView
		const FLineView& LineView = LineViews[ ViewIndex ];
		if ((LineView.Offset.Y + LineView.Size.Y) < Relative.Y && ViewIndex < LineViews.Num() - 1)
		{
			++ViewIndex;
		}
	}

	const FLineView& LineView = LineViews[ ViewIndex ];
	return GetTextLocationAt( LineView, FVector2D(Relative.X, LineView.Offset.Y), OutHitPoint );
}

FVector2D FTextLayout::GetLocationAt( const FTextLocation& Location, const bool bPerformInclusiveBoundsCheck ) const
{
	const int32 LineModelIndex = Location.GetLineIndex();
	const int32 Offset = Location.GetOffset();

	// Find the LineView which encapsulates the location's offset
	const int32 LineViewIndex = GetLineViewIndexForTextLocation( LineViews, Location, bPerformInclusiveBoundsCheck );

	// if we failed to find a LineView for the location, early out
	if ( !LineViews.IsValidIndex( LineViewIndex ) )
	{
		return FVector2D( ForceInitToZero );
	}

	const FTextLayout::FLineView& LineView = LineViews[ LineViewIndex ];

	//Iterate over the LineView's blocks...
	for (int32 BlockIndex = 0; BlockIndex < LineView.Blocks.Num(); BlockIndex++)
	{
		const TSharedRef< ILayoutBlock >& Block = LineView.Blocks[ BlockIndex ];
		const FTextRange& BlockRange = Block->GetTextRange();

		//if the block's range contains the specified locations offset...
		if (BlockRange.InclusiveContains(Offset))
		{
			// Ask the block for the exact screen location
			const FVector2D ScreenLocation = Block->GetRun()->GetLocationAt( Block, Offset - BlockRange.BeginIndex, Scale );

			// if the block was unable to provide a location continue iterating
			if ( ScreenLocation.IsZero() )
			{
				continue;
			}

			return ScreenLocation;
		}
	}

	// Failed to find the screen location
	return FVector2D( ForceInitToZero );
}

bool FTextLayout::InsertAt(const FTextLocation& Location, TCHAR Character)
{
	const int32 InsertLocation = Location.GetOffset();
	const int32 LineIndex = Location.GetLineIndex();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return false;
	}

	FLineModel& LineModel = LineModels[LineIndex];

	LineModel.Text->InsertAt(InsertLocation, Character);
	LineModel.DirtyFlags |= ELineModelDirtyState::All;

	bool RunIsAfterInsertLocation = false;
	for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
	{
		FRunModel& RunModel = LineModel.Runs[ RunIndex ];
		const FTextRange& RunRange = RunModel.GetTextRange();

		const bool bIsLastRun = RunIndex == LineModel.Runs.Num() - 1;
		if (RunRange.Contains(InsertLocation) || (bIsLastRun && !RunIsAfterInsertLocation))
		{
			checkf(RunIsAfterInsertLocation == false, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
			RunIsAfterInsertLocation = true;

			if ((RunModel.GetRun()->GetRunAttributes() & ERunAttributes::SupportsText) != ERunAttributes::None)
			{
				RunModel.SetTextRange( FTextRange( RunRange.BeginIndex, RunRange.EndIndex + 1 ) );
			}
			else
			{
				// Non-text runs are supposed to have a single dummy character in them
				checkf(RunRange.Len() == 1, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

				// This run doesn't support text, so we need to insert a new text run before or after the current run depending on the insertion point
				const bool bIsInsertingToTheLeft = InsertLocation == RunRange.BeginIndex;
				if (bIsInsertingToTheLeft)
				{
					// Insert the new text run to the left of the non-text run
					TSharedRef<IRun> NewTextRun = CreateDefaultTextRun( LineModel.Text, FTextRange( RunRange.BeginIndex, RunRange.BeginIndex + 1 ) );
					RunModel.SetTextRange( FTextRange( RunRange.BeginIndex + 1, RunRange.EndIndex + 1 ) );
					LineModel.Runs.Insert( NewTextRun, RunIndex++ );
				}
				else
				{
					// Insert the new text run to the right of the non-text run
					TSharedRef<IRun> NewTextRun = CreateDefaultTextRun( LineModel.Text, FTextRange( RunRange.EndIndex, RunRange.EndIndex + 1 ) );
					LineModel.Runs.Insert( NewTextRun, ++RunIndex );
				}
			}
		}
		else if (RunIsAfterInsertLocation)
		{
			FTextRange NewRange = RunRange;
			NewRange.Offset(1);
			RunModel.SetTextRange(NewRange);
		}
	}

	DirtyFlags |= ETextLayoutDirtyState::Layout;
	return true;
}

bool FTextLayout::InsertAt(const FTextLocation& Location, const FString& Text)
{
	const int32 InsertLocation = Location.GetOffset();
	const int32 LineIndex = Location.GetLineIndex();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return false;
	}

	FLineModel& LineModel = LineModels[LineIndex];

	LineModel.Text->InsertAt(InsertLocation, Text);
	LineModel.DirtyFlags |= ELineModelDirtyState::All;

	bool RunIsAfterInsertLocation = false;
	for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
	{
		FRunModel& RunModel = LineModel.Runs[ RunIndex ];
		const FTextRange& RunRange = RunModel.GetTextRange();

		const bool bIsLastRun = RunIndex == LineModel.Runs.Num() - 1;
		if (RunRange.Contains(InsertLocation) || (bIsLastRun && !RunIsAfterInsertLocation))
		{
			checkf(RunIsAfterInsertLocation == false, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
			RunIsAfterInsertLocation = true;

			if ((RunModel.GetRun()->GetRunAttributes() & ERunAttributes::SupportsText) != ERunAttributes::None)
			{
				RunModel.SetTextRange( FTextRange( RunRange.BeginIndex, RunRange.EndIndex + Text.Len() ) );
			}
			else
			{
				// Non-text runs are supposed to have a single dummy character in them
				checkf(RunRange.Len() == 1, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

				// This run doesn't support text, so we need to insert a new text run before or after the current run depending on the insertion point
				const bool bIsInsertingToTheLeft = InsertLocation == RunRange.BeginIndex;
				if (bIsInsertingToTheLeft)
				{
					// Insert the new text run to the left of the non-text run
					TSharedRef<IRun> NewTextRun = CreateDefaultTextRun( LineModel.Text, FTextRange( RunRange.BeginIndex, RunRange.BeginIndex + Text.Len() ) );
					// Move the Non-Text Run to right to free space for the new run.
					RunModel.SetTextRange( FTextRange( RunRange.BeginIndex + Text.Len(), RunRange.EndIndex + Text.Len() ) );
					LineModel.Runs.Insert( NewTextRun, RunIndex++ );
				}
				else
				{
					// Insert the new text run to the right of the non-text run
					TSharedRef<IRun> NewTextRun = CreateDefaultTextRun( LineModel.Text, FTextRange( RunRange.EndIndex, RunRange.EndIndex + Text.Len() ) );
					LineModel.Runs.Insert( NewTextRun, ++RunIndex );
				}
			}
		}
		else if (RunIsAfterInsertLocation)
		{
			FTextRange NewRange = RunRange;
			NewRange.Offset(Text.Len());
			RunModel.SetTextRange(NewRange);
		}
	}

	DirtyFlags |= ETextLayoutDirtyState::Layout;
	return true;
}

bool FTextLayout::InsertAt(const FTextLocation& Location, TSharedRef<IRun> InRun, const bool bAlwaysKeepRightRun)
{
	const int32 InsertLocation = Location.GetOffset();
	const int32 LineIndex = Location.GetLineIndex();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return false;
	}

	FLineModel& LineModel = LineModels[LineIndex];

	FString NewRunText;
	InRun->AppendTextTo(NewRunText);

	LineModel.Text->InsertAt(InsertLocation, NewRunText);
	LineModel.DirtyFlags |= ELineModelDirtyState::All;

	bool RunIsAfterInsertLocation = false;
	for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
	{
		const TSharedRef<IRun> Run = LineModel.Runs[RunIndex].GetRun();
		const FTextRange& RunRange = Run->GetTextRange();

		const bool bIsLastRun = RunIndex == LineModel.Runs.Num() - 1;
		if (RunRange.Contains(InsertLocation) || (bIsLastRun && !RunIsAfterInsertLocation))
		{
			checkf(RunIsAfterInsertLocation == false, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
			RunIsAfterInsertLocation = true;

			const int32 InsertLocationEnd = InsertLocation + NewRunText.Len();

			// This run contains the insertion point, so we need to split it
			TSharedPtr<IRun> LeftRun;
			TSharedPtr<IRun> RightRun;
			if ((Run->GetRunAttributes() & ERunAttributes::SupportsText) != ERunAttributes::None)
			{
				LeftRun = Run->Clone();
				LeftRun->SetTextRange(FTextRange(RunRange.BeginIndex, InsertLocation));

				RightRun = Run;
				RightRun->SetTextRange(FTextRange(InsertLocationEnd, RunRange.EndIndex + NewRunText.Len()));
			}
			else
			{
				// Non-text runs are supposed to have a single dummy character in them
				checkf(RunRange.Len() == 1, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

				// This run doesn't support text, so we need to insert a new text run before or after the current run depending on the insertion point
				const bool bIsInsertingToTheLeft = InsertLocation == RunRange.BeginIndex;
				if (bIsInsertingToTheLeft)
				{
					// Insert the new text run to the left of the non-text run
					LeftRun = CreateDefaultTextRun(LineModel.Text, FTextRange(RunRange.BeginIndex, InsertLocation));
				
					RightRun = Run;
					RightRun->SetTextRange(FTextRange(InsertLocationEnd, RunRange.EndIndex + NewRunText.Len()));
				}
				else
				{
					// Insert the new text run to the right of the non-text run
					LeftRun = Run;

					RightRun = CreateDefaultTextRun(LineModel.Text, FTextRange(InsertLocationEnd, RunRange.EndIndex + NewRunText.Len()));
				}
			}

			InRun->Move(LineModel.Text, FTextRange(InsertLocation, InsertLocationEnd));

			// Remove the old run (it may get re-added again as the right hand run)
			LineModel.Runs.RemoveAt(RunIndex--, EAllowShrinking::No);

			// Insert the new runs at the correct place, and then skip over these new array entries
			const bool LeftRunHasText = !LeftRun->GetTextRange().IsEmpty();
			const bool RightRunHasText = !RightRun->GetTextRange().IsEmpty();
			if (LeftRunHasText)
			{
				LineModel.Runs.Insert(LeftRun.ToSharedRef(), ++RunIndex);
			}
			LineModel.Runs.Insert(InRun, ++RunIndex);
			if (RightRunHasText || bAlwaysKeepRightRun)
			{
				LineModel.Runs.Insert(RightRun.ToSharedRef(), ++RunIndex);
			}
		}
		else if (RunIsAfterInsertLocation)
		{
			FTextRange NewRange = RunRange;
			NewRange.Offset(NewRunText.Len());
			Run->SetTextRange(NewRange);
		}
	}

	DirtyFlags |= ETextLayoutDirtyState::Layout;
	return true;
}

bool FTextLayout::JoinLineWithNextLine(int32 LineIndex)
{
	if (!LineModels.IsValidIndex(LineIndex) || !LineModels.IsValidIndex(LineIndex + 1))
	{
		return false;
	}

	//Grab both lines we are supposed to join
	FLineModel& LineModel = LineModels[LineIndex];
	FLineModel& NextLineModel = LineModels[LineIndex + 1];

	//If the next line is empty we'll just remove it
	if (NextLineModel.Text->Len() == 0)
	{
		return RemoveLine(LineIndex + 1);
	}

	const int32 LineLength = LineModel.Text->Len();

	//Append the next line to the current line
	LineModel.Text->InsertAt(LineLength, NextLineModel.Text.Get());

	//Dirty the current line
	LineModel.DirtyFlags |= ELineModelDirtyState::All;

	//Iterate through all of the next lines runs and bring them over to the current line
	for (int32 RunIndex = 0; RunIndex < NextLineModel.Runs.Num(); RunIndex++)
	{
		const TSharedRef<IRun> Run = NextLineModel.Runs[RunIndex].GetRun();
		FTextRange NewRange = Run->GetTextRange();

		if (!NewRange.IsEmpty())
		{
			NewRange.Offset(LineLength);

			Run->Move(LineModel.Text, NewRange);
			LineModel.Runs.Add(FRunModel(Run));
		}
	}

	//Remove the next line from the list of line models
	LineModels.RemoveAt(LineIndex + 1, EAllowShrinking::No);

	DirtyFlags |= ETextLayoutDirtyState::Layout;
	return true;
}

bool FTextLayout::SplitLineAt(const FTextLocation& Location)
{
	int32 BreakLocation = Location.GetOffset();
	int32 LineIndex = Location.GetLineIndex();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return false;
	}

	FLineModel& LineModel = LineModels[LineIndex];

	FLineModel LeftLineModel(MakeShareable(new FString(FString::ConstructFromPtrSize(**LineModel.Text, BreakLocation))));
	FLineModel RightLineModel(MakeShareable(new FString(FString::ConstructFromPtrSize(**LineModel.Text + BreakLocation, LineModel.Text->Len() - BreakLocation))));

	checkf(LeftLineModel.Text->Len() == BreakLocation, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

	bool RunIsToTheLeftOfTheBreakLocation = true;
	for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
	{
		const TSharedRef<IRun> Run = LineModel.Runs[RunIndex].GetRun();
		const FTextRange& RunRange = Run->GetTextRange();
		
		const bool bIsLastRun = RunIndex == LineModel.Runs.Num() - 1;
		if (RunRange.Contains(BreakLocation) || (bIsLastRun && RunIsToTheLeftOfTheBreakLocation))
		{
			checkf(RunIsToTheLeftOfTheBreakLocation == true, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
			RunIsToTheLeftOfTheBreakLocation = false;

			TSharedPtr<IRun> LeftRun;
			TSharedPtr<IRun> RightRun;
			if ((Run->GetRunAttributes() & ERunAttributes::SupportsText) != ERunAttributes::None)
			{
				LeftRun = Run->Clone();
				LeftRun->Move(LeftLineModel.Text, FTextRange(RunRange.BeginIndex, LeftLineModel.Text->Len()));
				
				RightRun = Run;
				RightRun->Move(RightLineModel.Text, FTextRange(0, RunRange.EndIndex - LeftLineModel.Text->Len()));
			}
			else
			{
				// Non-text runs are supposed to have a single dummy character in them
				checkf(RunRange.Len() == 1, TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));

				// This run doesn't support text, so we need to insert a new text run before or after the current run depending on the insertion point
				const bool bIsInsertingToTheLeft = BreakLocation == RunRange.BeginIndex;
				if (bIsInsertingToTheLeft)
				{
					// Insert the new text run to the left of the non-text run
					LeftRun = CreateDefaultTextRun(LeftLineModel.Text, FTextRange(RunRange.BeginIndex, LeftLineModel.Text->Len()));
				
					RightRun = Run;
					RightRun->Move(RightLineModel.Text, FTextRange(0, RunRange.EndIndex - LeftLineModel.Text->Len()));
				}
				else
				{
					// Insert the new text run to the right of the non-text run
					LeftRun = Run;
					LeftRun->Move(LeftLineModel.Text, FTextRange(RunRange.BeginIndex, LeftLineModel.Text->Len()));

					RightRun = CreateDefaultTextRun(RightLineModel.Text, FTextRange(0, RunRange.EndIndex - LeftLineModel.Text->Len()));
				}
			}

			LeftLineModel.Runs.Add(FRunModel(LeftRun.ToSharedRef()));
			RightLineModel.Runs.Add(FRunModel(RightRun.ToSharedRef()));
		}
		else if (RunIsToTheLeftOfTheBreakLocation)
		{
			Run->Move(LeftLineModel.Text, RunRange);
			LeftLineModel.Runs.Add(FRunModel(Run));
		}
		else
		{
			// This run is after the break, so adjust the range to match that of RHS of the split (RightLineModel)
			// We can do this by subtracting the LeftLineModel text size, since that's the LHS of the split
			FTextRange NewRange = RunRange;
			NewRange.Offset(-LeftLineModel.Text->Len());

			Run->Move(RightLineModel.Text, NewRange);
			RightLineModel.Runs.Add(FRunModel(Run));
		}
	}

	LineModels[LineIndex] = LeftLineModel;
	LineModels.Insert(RightLineModel, LineIndex + 1);

	DirtyFlags |= ETextLayoutDirtyState::Layout;
	return true;
}

bool FTextLayout::RemoveAt( const FTextLocation& Location, int32 Count )
{
	int32 RemoveLocation = Location.GetOffset();
	int32 LineIndex = Location.GetLineIndex();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return false;
	}

	FLineModel& LineModel = LineModels[LineIndex];

	//Make sure we aren't trying to remove more characters then we have
	Count = FMath::Min(Count, LineModel.Text->Len() - RemoveLocation);

	if (Count == 0)
	{
		return false;
	}

	LineModel.Text->RemoveAt(RemoveLocation, Count);
	LineModel.DirtyFlags |= ELineModelDirtyState::All;

	const FTextRange RemoveTextRange(RemoveLocation, RemoveLocation + Count);
	for (int32 RunIndex = LineModel.Runs.Num() - 1; RunIndex >= 0; --RunIndex)
	{
		FRunModel& RunModel = LineModel.Runs[RunIndex];
		const FTextRange RunRange = RunModel.GetTextRange();

		const FTextRange IntersectedRangeToRemove = RunRange.Intersect(RemoveTextRange);
		if (IntersectedRangeToRemove.IsEmpty() && RunRange.BeginIndex >= RemoveTextRange.EndIndex)
		{
			// The whole run is contained to the right of the removal range, just adjust its range by the amount of text that was removed
			FTextRange NewRange = RunRange;
			NewRange.Offset(-Count);
			RunModel.SetTextRange(NewRange);
		}
		else if (!IntersectedRangeToRemove.IsEmpty())
		{
			const int32 RunLength = RunRange.Len();
			const int32 IntersectedLength = IntersectedRangeToRemove.Len();
			if (RunLength == IntersectedLength)
			{
				// The text for this entire run has been removed, so remove this run
				LineModel.Runs.RemoveAt(RunIndex);

				// Every line needs at least one run - if we just removed the last run for this line, add a new default text run with a zero range
				if (LineModel.Runs.Num() == 0)
				{
					TSharedRef<IRun> NewTextRun = CreateDefaultTextRun(LineModel.Text, FTextRange(0, 0));
					LineModel.Runs.Add(NewTextRun);
				}
			}
			else if (RunRange.BeginIndex > RemoveTextRange.BeginIndex)
			{
				// Some of this run has been removed, and this run is the right hand part of the removal
				// So we need to adjust the range so that we start at the removal point since the text has been removed from the beginning of this run
				const FTextRange NewRange(RemoveTextRange.BeginIndex, RunRange.EndIndex - Count);
				checkf(!NewRange.IsEmpty(), TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
				RunModel.SetTextRange(NewRange);
			}
			else
			{
				// Some of this run has been removed, and this run is the left hand part of the removal
				// So we need to adjust the range by the amount of text that has been removed from the end of this run
				const FTextRange NewRange(RunRange.BeginIndex, RunRange.EndIndex - IntersectedLength);
				checkf(!NewRange.IsEmpty(), TEXT("Debug Source: %s"), *DebugSourceInfo.Get(FString()));
				RunModel.SetTextRange(NewRange);
			}

			if (RunRange.BeginIndex <= RemoveTextRange.BeginIndex)
			{
				// break since we don't need to process the runs to the left of the removal point
				break;
			}
		}
		else if(IntersectedRangeToRemove.IsEmpty() && RunRange.IsEmpty() && RemoveTextRange.Contains(RunRange.BeginIndex) && RemoveTextRange.Contains(RunRange.EndIndex))
		{
			// empty run that was inside our removed range, safe to remove
			LineModel.Runs.RemoveAt(RunIndex);
		}
	}

	DirtyFlags |= ETextLayoutDirtyState::Layout;
	return true;
}

bool FTextLayout::RemoveLine(int32 LineIndex)
{
	if (!LineModels.IsValidIndex(LineIndex))
	{
		return false;
	}

	LineModels.RemoveAt(LineIndex);

	// If our layout is clean, then we can remove this line immediately (and efficiently)
	// If our layout is dirty, then we might as well wait as the next UpdateLayout call will remove it
	if (!(DirtyFlags & ETextLayoutDirtyState::Layout))
	{
		//Lots of room for additional optimization
		float OffsetAdjustment = 0.0f;
		float HeightAdjustment = 0.0f;

		for (int32 ViewIndex = 0; ViewIndex < LineViews.Num(); ViewIndex++)
		{
			FLineView& LineView = LineViews[ViewIndex];

			if (LineView.ModelIndex == LineIndex)
			{
				HeightAdjustment += LineView.Size.Y;

				if (LineViews.IsValidIndex(ViewIndex + 1))
				{
					OffsetAdjustment += (LineViews[ViewIndex + 1].Offset.Y - LineView.Offset.Y);
				}
				else // Last line is being removed - OverHeight needs to be re-computed from the preceding line
				{
					float NewOverHeight = 0.f;
					if (ViewIndex > 0)
					{
						FLineView& PrevLineView = LineViews[ViewIndex - 1];
						NewOverHeight = PrevLineView.Size.Y - PrevLineView.TextHeight;
					}
					if (!ApplyLineHeightToBottomLine)
					{
						HeightAdjustment += NewOverHeight - OverHeight;
					}
					OverHeight = NewOverHeight;
				}

				LineViews.RemoveAt(ViewIndex);
				LineViewsToJustify.Remove(ViewIndex);
				--ViewIndex;
			}
			else if (LineView.ModelIndex > LineIndex)
			{
				//We've removed a line model so update the LineView indices
				--LineView.ModelIndex;
				LineView.Offset.Y -= OffsetAdjustment;

				for (const TSharedRef< ILayoutBlock >& Block : LineView.Blocks)
				{
					FVector2D BlockOffset = Block->GetLocationOffset();
					BlockOffset.Y -= OffsetAdjustment;
					Block->SetLocationOffset(BlockOffset);
				}
			}
		}

		TextLayoutSize.Height -= HeightAdjustment;
	}

	return true;
}

void FTextLayout::AddLine( const FNewLineData& NewLine )
{
	// Adjust the layout height to account for the last line OverHeight now that we're adding more lines to the layout
	if (!LineModels.IsEmpty() && !ApplyLineHeightToBottomLine)
	{
		TextLayoutSize.Height += OverHeight;
	}

	{
		FLineModel LineModel( NewLine.Text );
		TransformLineText(LineModel);

		for (const auto& Run : NewLine.Runs)
		{
			LineModel.Runs.Add( FRunModel( Run ) );
		}

		LineModels.Add( LineModel );
	}

	// If our layout is clean, then we can add this new line immediately (and efficiently)
	// If our layout is dirty, then we might as well wait as the next UpdateLayout call will add it
	if (!(DirtyFlags & ETextLayoutDirtyState::Layout))
	{
		const int32 LineModelIndex = LineModels.Num() - 1;
		FLineModel& LineModel = LineModels[LineModelIndex];

		CalculateLineTextDirection(LineModel);
		FlushLineTextShapingCache(LineModel);
		CreateLineWrappingCache(LineModel);

		BeginLineLayout(LineModel);

		const int32 FirstNewLineViewIndex = LineViews.Num() + 1;

		TArray<TSharedRef<ILayoutBlock>> SoftLine;
		FlowLineLayout(LineModelIndex, GetWrappingDrawWidth(), SoftLine);

		// Apply the current margin to the newly added lines
		{
			const FVector2D MarginOffsetAdjustment = FVector2D(Margin.Left, Margin.Top) * Scale;

			for (int32 LineViewIndex = FirstNewLineViewIndex; LineViewIndex < LineViews.Num(); ++LineViewIndex)
			{
				FLineView& LineView = LineViews[LineViewIndex];
				if (LineView.ModelIndex == LineModelIndex)
				{
					LineView.Offset += MarginOffsetAdjustment;
				}
			}

			for (const TSharedRef< ILayoutBlock >& Block : SoftLine)
			{
				Block->SetLocationOffset( Block->GetLocationOffset() + MarginOffsetAdjustment );
			}
		}

		// We need to re-justify all lines, as the new line view(s) added by this line model may have affected everything
		JustifyLayout();

		EndLineLayout(LineModel);
	}
}

void FTextLayout::AddLines( const TArray<FNewLineData>& NewLines )
{
	// Adjust the layout height to account for the last line OverHeight now that we're adding more lines to the layout
	if (!LineModels.IsEmpty() && !ApplyLineHeightToBottomLine)
	{
		TextLayoutSize.Height += OverHeight;
	}

	LineModels.Reserve(NewLines.Num());
	for (const auto& NewLine : NewLines)
	{
		FLineModel LineModel( NewLine.Text );
		TransformLineText(LineModel);

		LineModel.Runs.Reserve(NewLine.Runs.Num());
		for (const auto& Run : NewLine.Runs)
		{
			LineModel.Runs.Add( FRunModel( Run ) );
		}

		LineModels.Add( LineModel );
	}

	// If our layout is clean, then we can add this new line immediately (and efficiently)
	// If our layout is dirty, then we might as well wait as the next UpdateLayout call will add it
	if (!(DirtyFlags & ETextLayoutDirtyState::Layout))
	{
		const int32 FirstNewLineModelIndex = LineModels.Num() - NewLines.Num();

		for (int32 LineModelIndex = FirstNewLineModelIndex; LineModelIndex < LineModels.Num(); ++LineModelIndex)
		{
			FLineModel& LineModel = LineModels[LineModelIndex];
			BeginLineLayout(LineModel);
		}

		for (int32 LineModelIndex = FirstNewLineModelIndex; LineModelIndex < LineModels.Num(); ++LineModelIndex)
		{
			FLineModel& LineModel = LineModels[LineModelIndex];

			CalculateLineTextDirection(LineModel);
			FlushLineTextShapingCache(LineModel);
			CreateLineWrappingCache(LineModel);

			const int32 FirstNewLineViewIndex = LineViews.Num() + 1;

			TArray<TSharedRef<ILayoutBlock>> SoftLine;
			FlowLineLayout(LineModelIndex, GetWrappingDrawWidth(), SoftLine);

			// Apply the current margin to the newly added lines
			{
				const FVector2D MarginOffsetAdjustment = FVector2D(Margin.Left, Margin.Top) * Scale;

				for (int32 LineViewIndex = FirstNewLineViewIndex; LineViewIndex < LineViews.Num(); ++LineViewIndex)
				{
					FLineView& LineView = LineViews[LineViewIndex];
					if (LineView.ModelIndex == LineModelIndex)
					{
						LineView.Offset += MarginOffsetAdjustment;
					}
				}

				for (const TSharedRef< ILayoutBlock >& Block : SoftLine)
				{
					Block->SetLocationOffset( Block->GetLocationOffset() + MarginOffsetAdjustment );
				}
			}
		}

		// We need to re-justify all lines, as the new line view(s) added by this line model may have affected everything
		JustifyLayout();

		for (int32 LineModelIndex = FirstNewLineModelIndex; LineModelIndex < LineModels.Num(); ++LineModelIndex)
		{
			FLineModel& LineModel = LineModels[LineModelIndex];
			EndLineLayout(LineModel);
		}
	}
}

void FTextLayout::ClearLines()
{
	LineModels.Empty();
	ClearView();
	DirtyFlags |= ETextLayoutDirtyState::Layout;
}

bool FTextLayout::IsEmpty() const
{
	return (LineModels.Num() == 0 || (LineModels.Num() == 1 && LineModels[0].Text->Len() == 0));
}

int32 FTextLayout::GetLineCount() const
{
	return LineModels.Num();
}

void FTextLayout::GetAsText(FString& DisplayText, FTextOffsetLocations* const OutTextOffsetLocations) const
{
	GetAsTextAndOffsets(&DisplayText, OutTextOffsetLocations, nullptr);
}

void FTextLayout::GetAsText(FText& DisplayText, FTextOffsetLocations* const OutTextOffsetLocations) const
{
	FString DisplayString;
	GetAsText(DisplayString, OutTextOffsetLocations);
	DisplayText = FText::FromString(DisplayString);
}

void FTextLayout::GetTextOffsetLocations(FTextOffsetLocations& OutTextOffsetLocations) const
{
	GetAsTextAndOffsets(nullptr, &OutTextOffsetLocations, nullptr);
}

void FTextLayout::GetAsTextAndOffsets(FString* const OutDisplayText, FTextOffsetLocations* const OutTextOffsetLocations) const
{
	GetAsTextAndOffsets(OutDisplayText, OutTextOffsetLocations, nullptr);
}

void FTextLayout::GetAsTextAndOffsets(FString* const OutDisplayText, FTextOffsetLocations* const OutTextOffsetLocations, const TCHAR* LineTerminatorOverride) const
{
	int32 DisplayTextLength = 0;

	if (OutTextOffsetLocations)
	{
		OutTextOffsetLocations->OffsetData.Reserve(LineModels.Num());
	}

	// Allow callers (e.g. the TSF/IME text-input path) to override the platform line terminator so that the
	// flat-string offsets (ACP) use a single-character line break, keeping them in sync with what the IME expects.
	const TCHAR* const ActualLineTerminator = LineTerminatorOverride ? LineTerminatorOverride : LINE_TERMINATOR;
	const int32 LineTerminatorLength = FCString::Strlen(ActualLineTerminator);

	for (int32 LineModelIndex = 0; LineModelIndex < LineModels.Num(); LineModelIndex++)
	{
		const FLineModel& LineModel = LineModels[LineModelIndex];

		// Append line terminator to the end of the previous line
		if (LineModelIndex > 0)
		{
			if (OutDisplayText)
			{
				*OutDisplayText += ActualLineTerminator;
			}
			DisplayTextLength += LineTerminatorLength;
		}

		int32 LineLength = 0;
		for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
		{
			const FRunModel& Run = LineModel.Runs[RunIndex];
			if (OutDisplayText)
			{
				Run.AppendTextTo(*OutDisplayText);
			}
			LineLength += Run.GetTextRange().Len();
		}

		if (OutTextOffsetLocations)
		{
			OutTextOffsetLocations->OffsetData.Add(FTextOffsetLocations::FOffsetEntry(DisplayTextLength, LineLength));
		}

		DisplayTextLength += LineLength;
	}
}

void GetRangeAsTextFromLine(FString& DisplayText, const FTextLayout::FLineModel& LineModel, const FTextRange& Range)
{
	for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); RunIndex++)
	{
		const FTextLayout::FRunModel& Run = LineModel.Runs[RunIndex];
		const FTextRange& RunRange = Run.GetTextRange();

		const FTextRange IntersectRange = RunRange.Intersect(Range);

		if (!IntersectRange.IsEmpty())
		{
			Run.AppendTextTo(DisplayText, IntersectRange);
		}
		else if (RunRange.BeginIndex > Range.EndIndex)
		{
			//We're past the selection range so we can stop
			break;
		}
	}
}

void FTextLayout::GetSelectionAsText(FString& DisplayText, const FTextSelection& Selection) const
{
	int32 SelectionBeginningLineIndex = Selection.GetBeginning().GetLineIndex();
	int32 SelectionBeginningLineOffset = Selection.GetBeginning().GetOffset();

	int32 SelectionEndLineIndex = Selection.GetEnd().GetLineIndex();
	int32 SelectionEndLineOffset = Selection.GetEnd().GetOffset();

	if (LineModels.IsValidIndex(SelectionBeginningLineIndex) && LineModels.IsValidIndex(SelectionEndLineIndex))
	{
		if (SelectionBeginningLineIndex == SelectionEndLineIndex)
		{
			const FTextRange SelectionRange(SelectionBeginningLineOffset, SelectionEndLineOffset);
			const FLineModel& LineModel = LineModels[SelectionBeginningLineIndex];

			GetRangeAsTextFromLine(DisplayText, LineModel, SelectionRange);
		}
		else
		{
			for (int32 LineIndex = SelectionBeginningLineIndex; LineIndex <= SelectionEndLineIndex; LineIndex++)
			{
				if (LineIndex == SelectionBeginningLineIndex)
				{
					const FLineModel& LineModel = LineModels[SelectionBeginningLineIndex];
					const FTextRange SelectionRange(SelectionBeginningLineOffset, LineModel.Text->Len());

					GetRangeAsTextFromLine(DisplayText, LineModel, SelectionRange);
				}
				else if (LineIndex == SelectionEndLineIndex)
				{
					const FLineModel& LineModel = LineModels[SelectionEndLineIndex];
					const FTextRange SelectionRange(0, SelectionEndLineOffset);

					GetRangeAsTextFromLine(DisplayText, LineModel, SelectionRange);
				}
				else
				{
					const FLineModel& LineModel = LineModels[LineIndex];
					const FTextRange SelectionRange(0, LineModel.Text->Len());

					GetRangeAsTextFromLine(DisplayText, LineModel, SelectionRange);
				}

				if (LineIndex != SelectionEndLineIndex)
				{
					DisplayText += LINE_TERMINATOR;
				}
			}
		}
	}
}

FTextSelection FTextLayout::GetGraphemeAt(const FTextLocation& Location) const
{
	const int32 LineIndex = Location.GetLineIndex();
	const int32 Offset = Location.GetOffset();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return FTextSelection();
	}

	const FLineModel& LineModel = LineModels[LineIndex];

	GraphemeBreakIterator->SetStringRef(&LineModel.Text.Get());

	const int32 PreviousBreak = (Offset < LineModel.Text->Len()) ? GraphemeBreakIterator->MoveToCandidateAfter(Offset) : GraphemeBreakIterator->ResetToEnd();
	const int32 CurrentBreak = GraphemeBreakIterator->MoveToPrevious();

	GraphemeBreakIterator->ClearString();

	if (PreviousBreak == CurrentBreak || CurrentBreak == INDEX_NONE)
	{
		return FTextSelection();
	}

	return FTextSelection(FTextLocation(LineIndex, CurrentBreak), FTextLocation(LineIndex, PreviousBreak));
}

FTextSelection FTextLayout::GetWordAt(const FTextLocation& Location) const
{
	const int32 LineIndex = Location.GetLineIndex();
	const int32 Offset = Location.GetOffset();

	if (!LineModels.IsValidIndex(LineIndex))
	{
		return FTextSelection();
	}

	const FLineModel& LineModel = LineModels[LineIndex];

	WordBreakIterator->SetStringRef(&LineModel.Text.Get());

	int32 PreviousBreak = (Offset < LineModel.Text->Len()) ? WordBreakIterator->MoveToCandidateAfter(Offset) : WordBreakIterator->ResetToEnd();
	int32 CurrentBreak = 0;

	while ((CurrentBreak = WordBreakIterator->MoveToPrevious()) != INDEX_NONE)
	{
		bool HasLetter = false;
		for (int32 Index = CurrentBreak; Index < PreviousBreak; Index++)
		{
			if (!FTextChar::IsWhitespace((*LineModel.Text)[Index]))
			{
				HasLetter = true;
				break;
			}
		}

		if (HasLetter)
		{
			break;
		}

		PreviousBreak = CurrentBreak;
	}

	WordBreakIterator->ClearString();

	if (PreviousBreak == CurrentBreak || CurrentBreak == INDEX_NONE)
	{
		return FTextSelection();
	}

	return FTextSelection(FTextLocation(LineIndex, CurrentBreak), FTextLocation(LineIndex, PreviousBreak));
}

bool FTextLayout::AreAnyFontFacesLoading() const
{
	return !LoadingFontFaces.IsEmpty();
}


void FTextLayout::SetVisibleRegion( const FVector2D& InViewSize, const FVector2D& InScrollOffset )
{
	if (ViewSize != InViewSize)
	{
		ViewSize = InViewSize;
		
		if (LineViewsToJustify.Num() > 0)
		{
			// If the view size has changed, we may need to update our positions based on our justification
			DirtyFlags |= ETextLayoutDirtyState::Layout;
		}
	}
	
	if (ScrollOffset != InScrollOffset)
	{
		const FVector2D PreviousScrollOffset = ScrollOffset;
		ScrollOffset = InScrollOffset;

		// Use a negative scroll offset since positive scrolling moves things negatively in screen space
		const FVector2D OffsetAdjustment = -(ScrollOffset - PreviousScrollOffset);

		for (FLineView& LineView : LineViews)
		{
			LineView.Offset += OffsetAdjustment;

			for (const TSharedRef< ILayoutBlock >& Block : LineView.Blocks)
			{
				Block->SetLocationOffset( Block->GetLocationOffset() + OffsetAdjustment );
			}
		}
	}
}

void FTextLayout::SetLineBreakIterator( TSharedPtr<IBreakIterator> InLineBreakIterator )
{
	LineBreakIterator = InLineBreakIterator;
	DirtyFlags |= ETextLayoutDirtyState::Layout;

	// Changing the line break iterator will affect the wrapping information for *all lines*
	// Clear out the entire cache so it gets regenerated on the text call to FlowLayout
	DirtyAllLineModels(ELineModelDirtyState::WrappingInformation);
}

void FTextLayout::SetMargin( const FMargin& InMargin )
{
	if ( Margin == InMargin )
	{
		return;
	}

	Margin = InMargin;
	DirtyFlags |= ETextLayoutDirtyState::Layout;
}

void FTextLayout::SetScale( float Value )
{
	if (FMath::IsNaN(Value))
	{
		Value = 0.0f;
	}

	if (Scale != Value)
	{
		Scale = Value;
		DirtyFlags |= ETextLayoutDirtyState::Layout;

		// Changing the scale will affect the wrapping information for *all lines*
		// Clear out the entire cache so it gets regenerated on the text call to FlowLayout
		DirtyAllLineModels(ELineModelDirtyState::WrappingInformation | ELineModelDirtyState::ShapingCache);
	}
}

void FTextLayout::SetTextShapingMethod( const ETextShapingMethod InTextShapingMethod )
{
	if ( TextShapingMethod == InTextShapingMethod )
	{
		return;
	}

	TextShapingMethod = InTextShapingMethod;
	DirtyFlags |= ETextLayoutDirtyState::Layout;

	// Changing the shaping method will affect the wrapping information for *all lines*
	// Clear out the entire cache so it gets regenerated on the text call to FlowLayout
	// Also clear our the base direction for each line, as the shaping method can affect that
	DirtyAllLineModels(ELineModelDirtyState::WrappingInformation | ELineModelDirtyState::TextBaseDirection | ELineModelDirtyState::ShapingCache);
}

void FTextLayout::SetTextFlowDirection( const ETextFlowDirection InTextFlowDirection )
{
	if ( TextFlowDirection == InTextFlowDirection )
	{
		return;
	}

	TextFlowDirection = InTextFlowDirection;
	DirtyFlags |= ETextLayoutDirtyState::Layout;

	// Changing the flow direction will affect the wrapping information for *all lines*
	// Clear out the entire cache so it gets regenerated on the text call to FlowLayout
	// Also clear our the base direction for each line, as the flow direction can affect that
	DirtyAllLineModels(ELineModelDirtyState::WrappingInformation | ELineModelDirtyState::TextBaseDirection | ELineModelDirtyState::ShapingCache);
}

void FTextLayout::SetTextOverflowPolicy(const TOptional<ETextOverflowPolicy> InTextOverflowPolicy)
{
	if (TextOverflowPolicyOverride == InTextOverflowPolicy)
	{
		return;
	}

	TextOverflowPolicyOverride = InTextOverflowPolicy;
	DirtyFlags |= ETextLayoutDirtyState::Layout;
}

void FTextLayout::SetJustification( ETextJustify::Type Value )
{
	if ( Justification == Value )
	{
		return;
	}

	Justification = Value;
	DirtyFlags |= ETextLayoutDirtyState::Layout;
}

ETextJustify::Type FTextLayout::GetVisualJustification() const
{
	return LineViews.Num() > 0 ? CalculateLineViewVisualJustification(LineViews[0]) : GetJustification();
}

void FTextLayout::SetLineHeightPercentage( float Value )
{
	if ( LineHeightPercentage != Value )
	{
		LineHeightPercentage = Value; 
		DirtyFlags |= ETextLayoutDirtyState::Layout;
	}
}

void FTextLayout::SetApplyLineHeightToBottomLine( bool Value )
{
	if ( ApplyLineHeightToBottomLine != Value )
	{
		ApplyLineHeightToBottomLine = Value; 
		DirtyFlags |= ETextLayoutDirtyState::Layout;
	}
}

void FTextLayout::SetWrappingWidth( float Value )
{
	const bool WasWrapping = WrappingWidth > 0.0f;
	const bool IsWrapping = Value > 0.0f;

	if ( WrappingWidth != Value )
	{
		WrappingWidth = Value; 
		DirtyFlags |= ETextLayoutDirtyState::Layout;

		if ( WasWrapping != IsWrapping )
		{
			// Changing from wrapping/not-wrapping will affect the wrapping information for *all lines*
			// Clear out the entire cache so it gets regenerated on the text call to FlowLayout
			DirtyAllLineModels(ELineModelDirtyState::WrappingInformation);
		}
	}
}

void FTextLayout::SetWrappingPolicy(ETextWrappingPolicy Value)
{
	if (WrappingPolicy != Value)
	{
		WrappingPolicy = Value;
		DirtyFlags |= ETextLayoutDirtyState::Layout;
	}
}

void FTextLayout::SetTransformPolicy(ETextTransformPolicy Value)
{
	if (TransformPolicy != Value)
	{
		TransformPolicy = Value;
		DirtyFlags |= ETextLayoutDirtyState::Layout;
	}
}

void FTextLayout::SetDebugSourceInfo(const TAttribute<FString>& InDebugSourceInfo)
{
	DebugSourceInfo = InDebugSourceInfo;
#if !UE_BUILD_SHIPPING
	DebugSourceInfo.Get(FString());
#endif	// !UE_BUILD_SHIPPING
}

FVector2D FTextLayout::GetSize() const
{
	return TextLayoutSize.GetDrawSize() * Inverse(Scale);
}

FVector2D FTextLayout::GetViewSize() const
{
	return ViewSize;
}

FVector2D FTextLayout::GetDrawSize() const
{
	return TextLayoutSize.GetDrawSize();
}

FVector2D FTextLayout::GetWrappedSize() const
{
	return TextLayoutSize.GetWrappedSize() * Inverse(Scale);
}

FVector2D FTextLayout::GetWrappedDrawSize() const
{
	return TextLayoutSize.GetWrappedSize();
}

FTextLayout::~FTextLayout()
{

}

FTextLayout::FLineModel::FLineModel( const TSharedRef< FString >& InText ) 
	: Text( InText )
	, ShapedTextCache( FShapedTextCache::Create(FSlateApplication::Get().GetRenderer()->GetFontCache()))
	, TextBaseDirection( TextBiDi::ETextDirection::LeftToRight )
	, Runs()
	, BreakCandidates()
	, RunRenderers()
	, DirtyFlags( ELineModelDirtyState::All )
{
}

void FTextLayout::FRunModel::ClearCache()
{
	MeasuredRanges.Empty();
	MeasuredRangeSizes.Empty();
}

void FTextLayout::FRunModel::AppendTextTo(FString& Text) const
{
	Run->AppendTextTo(Text);
}

void FTextLayout::FRunModel::AppendTextTo(FString& Text, const FTextRange& Range) const
{
	Run->AppendTextTo(Text, Range);
}

TSharedRef< ILayoutBlock > FTextLayout::FRunModel::CreateBlock( const FBlockDefinition& BlockDefine, float InScale, const FLayoutBlockTextContext& InTextContext ) const
{
	const FTextRange& SizeRange = BlockDefine.ActualRange;

	if ( MeasuredRanges.Num() == 0 )
	{
		FTextRange RunRange = Run->GetTextRange();
		return Run->CreateBlock( BlockDefine.ActualRange.BeginIndex, BlockDefine.ActualRange.EndIndex, Run->Measure( SizeRange.BeginIndex, SizeRange.EndIndex, InScale, InTextContext ), InTextContext, BlockDefine.Renderer );
	}

	int32 StartRangeIndex = 0;
	int32 EndRangeIndex = 0;

	if ( MeasuredRanges.Num() > 16 )
	{
		if ( SizeRange.BeginIndex != 0 )
		{
			StartRangeIndex = BinarySearchForBeginIndex( MeasuredRanges, SizeRange.BeginIndex );
			check( StartRangeIndex != INDEX_NONE);
		}

		EndRangeIndex = StartRangeIndex;
		if ( StartRangeIndex != MeasuredRanges.Num() - 1 )
		{
			EndRangeIndex = BinarySearchForEndIndex( MeasuredRanges, StartRangeIndex, SizeRange.EndIndex );
			check( EndRangeIndex != INDEX_NONE);
		}
	}
	else
	{
		const int32 MaxValidIndex = MeasuredRanges.Num() - 1;

		if ( SizeRange.BeginIndex != 0 )
		{
			bool FoundBeginIndexMatch = false;
			for (; StartRangeIndex < MaxValidIndex; StartRangeIndex++)
			{
				FoundBeginIndexMatch = MeasuredRanges[ StartRangeIndex ].BeginIndex >= SizeRange.BeginIndex;
				if ( FoundBeginIndexMatch ) 
				{
					break;
				}
			}
			check( FoundBeginIndexMatch == true || StartRangeIndex == MaxValidIndex );
		}

		EndRangeIndex = StartRangeIndex;
		if ( StartRangeIndex != MaxValidIndex )
		{
			bool FoundEndIndexMatch = false;
			for (; EndRangeIndex < MeasuredRanges.Num(); EndRangeIndex++)
			{
				FoundEndIndexMatch = MeasuredRanges[ EndRangeIndex ].EndIndex >= SizeRange.EndIndex;
				if ( FoundEndIndexMatch ) 
				{
					break;
				}
			}
			check( FoundEndIndexMatch == true || EndRangeIndex == MaxValidIndex );
		}
	}

	FVector2D BlockSize( ForceInitToZero );
	if ( StartRangeIndex == EndRangeIndex )
	{
		if ( MeasuredRanges[ StartRangeIndex ].BeginIndex == SizeRange.BeginIndex && 
			MeasuredRanges[ StartRangeIndex ].EndIndex == SizeRange.EndIndex )
		{
			BlockSize += FVector2D(MeasuredRangeSizes[StartRangeIndex]);
		}
		else
		{
			BlockSize += Run->Measure( SizeRange.BeginIndex, SizeRange.EndIndex, InScale, InTextContext );
		}
	}
	else
	{
		if ( MeasuredRanges[ StartRangeIndex ].BeginIndex == SizeRange.BeginIndex )
		{
			BlockSize += FVector2D(MeasuredRangeSizes[StartRangeIndex]);
		}
		else
		{
			BlockSize += Run->Measure( SizeRange.BeginIndex, MeasuredRanges[ StartRangeIndex ].EndIndex, InScale, InTextContext );
		}

		for (int32 Index = StartRangeIndex + 1; Index < EndRangeIndex; Index++)
		{
			BlockSize.X += MeasuredRangeSizes[Index].X;
			BlockSize.Y = FMath::Max(MeasuredRangeSizes[Index].Y, BlockSize.Y);
		}

		if ( MeasuredRanges[ EndRangeIndex ].EndIndex == SizeRange.EndIndex )
		{
			BlockSize.X += MeasuredRangeSizes[EndRangeIndex].X;
			BlockSize.Y = FMath::Max(MeasuredRangeSizes[EndRangeIndex].Y, BlockSize.Y);
		}
		else
		{
			FVector2D Size = Run->Measure( MeasuredRanges[ EndRangeIndex ].BeginIndex, SizeRange.EndIndex, InScale, InTextContext );
			BlockSize.X += Size.X;
			BlockSize.Y = FMath::Max( Size.Y, BlockSize.Y );
		}
	}

	return Run->CreateBlock( BlockDefine.ActualRange.BeginIndex, BlockDefine.ActualRange.EndIndex, BlockSize, InTextContext, BlockDefine.Renderer );
}

int32 FTextLayout::FRunModel::BinarySearchForEndIndex( const TArray< FTextRange >& Ranges, int32 RangeBeginIndex, int32 EndIndex )
{
	int32 Min = RangeBeginIndex;
	int32 Mid = 0;
	int32 Max = Ranges.Num() - 1;
	while (Max >= Min)
	{
		Mid = Min + ((Max - Min) / 2);
		if ( Ranges[Mid].EndIndex == EndIndex )
		{
			return Mid; 
		}
		else if ( Ranges[Mid].EndIndex < EndIndex )
		{
			Min = Mid + 1;
		}
		else
		{
			Max = Mid - 1;
		}
	}

	return Mid;
}

int32 FTextLayout::FRunModel::BinarySearchForBeginIndex( const TArray< FTextRange >& Ranges, int32 BeginIndex )
{
	int32 Min = 0;
	int32 Mid = 0;
	int32 Max = Ranges.Num() - 1;
	while (Max >= Min)
	{
		Mid = Min + ((Max - Min) / 2);
		if ( Ranges[Mid].BeginIndex == BeginIndex )
		{
			return Mid; 
		}
		else if ( Ranges[Mid].BeginIndex < BeginIndex )
		{
			Min = Mid + 1;
		}
		else
		{
			Max = Mid - 1;
		}
	}

	return Mid;
}

int8 FTextLayout::FRunModel::GetKerning(int32 CurrentIndex, float InScale, const FRunTextContext& InTextContext)
{
	return Run->GetKerning(CurrentIndex, InScale, InTextContext);
}

FVector2D FTextLayout::FRunModel::Measure(int32 BeginIndex, int32 EndIndex, float InScale, const FRunTextContext& InTextContext)
{
	FVector2D Size = Run->Measure(BeginIndex, EndIndex, InScale, InTextContext);

	MeasuredRanges.Add( FTextRange( BeginIndex, EndIndex ) );
	MeasuredRangeSizes.Add(Size);

	return Size;
}

int16 FTextLayout::FRunModel::GetMaxHeight(float InScale) const
{
	return Run->GetMaxHeight(InScale);
}

int16 FTextLayout::FRunModel::GetBaseLine( float InScale ) const
{
	return Run->GetBaseLine(InScale);
}

FTextRange FTextLayout::FRunModel::GetTextRange() const
{
	return Run->GetTextRange();
}

void FTextLayout::FRunModel::SetTextRange( const FTextRange& Value )
{
	Run->SetTextRange( Value );
}

void FTextLayout::FRunModel::EndLayout()
{
	Run->EndLayout();
}

void FTextLayout::FRunModel::BeginLayout()
{
	Run->BeginLayout();
}

TSharedRef< IRun > FTextLayout::FRunModel::GetRun() const
{
	return Run;
}

FTextLayout::FRunModel::FRunModel( const TSharedRef< IRun >& InRun )
	: Run( InRun )
{
	SLATE_CROSS_THREAD_CHECK();
}

int32 FTextLayout::FTextOffsetLocations::TextLocationToOffset(const FTextLocation& InLocation) const
{
	const int32 LineIndex = InLocation.GetLineIndex();
	if(LineIndex >= 0 && LineIndex < OffsetData.Num())
	{
		const FOffsetEntry& OffsetEntry = OffsetData[LineIndex];
		return OffsetEntry.FlatStringIndex + InLocation.GetOffset();
	}
	return INDEX_NONE;
}

FTextLocation FTextLayout::FTextOffsetLocations::OffsetToTextLocation(const int32 InOffset) const
{
	for(int32 OffsetEntryIndex = 0; OffsetEntryIndex < OffsetData.Num(); ++OffsetEntryIndex)
	{
		const FOffsetEntry& OffsetEntry = OffsetData[OffsetEntryIndex];
		const FTextRange FlatLineRange(OffsetEntry.FlatStringIndex, OffsetEntry.FlatStringIndex + OffsetEntry.DocumentLineLength);
		if(FlatLineRange.InclusiveContains(InOffset))
		{
			const int32 OffsetWithinLine = InOffset - OffsetEntry.FlatStringIndex;
			return FTextLocation(OffsetEntryIndex, OffsetWithinLine);
		}
	}
	return FTextLocation();
}

int32 FTextLayout::FTextOffsetLocations::GetTextLength() const
{
	if(OffsetData.Num())
	{
		const FOffsetEntry& OffsetEntry = OffsetData.Last();
		return OffsetEntry.FlatStringIndex + OffsetEntry.DocumentLineLength;
	}
	return 0;
}

