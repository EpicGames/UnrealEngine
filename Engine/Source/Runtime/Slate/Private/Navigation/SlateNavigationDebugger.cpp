// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/SlateNavigationDebugger.h"

#if WITH_SLATE_NAVIGATION_DEBUGGING

#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"

namespace DebugCandidateWidgets
{

static TAutoConsoleVariable<bool> CVarDisplayCandidateWidgets_Right(
	TEXT("Slate.NavigationDebugging.CandidateWidgets.Right"),
	false,
	TEXT("Enable to display candidate widgets for EUINavigation::Right of the focused widget when `Slate.NavigationDebugging` is enabled"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarDisplayCandidateWidgets_Left(
	TEXT("Slate.NavigationDebugging.CandidateWidgets.Left"),
	false,
	TEXT("Enable to display candidate widgets for EUINavigation::Left of the focused widget when `Slate.NavigationDebugging` is enabled"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarDisplayCandidateWidgets_Up(
	TEXT("Slate.NavigationDebugging.CandidateWidgets.Up"),
	false,
	TEXT("Enable to display candidate widgets for EUINavigation::Up of the focused widget when `Slate.NavigationDebugging` is enabled"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarDisplayCandidateWidgets_Down(
	TEXT("Slate.NavigationDebugging.CandidateWidgets.Down"),
	false,
	TEXT("Enable to display candidate widgets for EUINavigation::Down of the focused widget when `Slate.NavigationDebugging` is enabled"),
	ECVF_Default);


static TMap<EUINavigation, TAutoConsoleVariable<bool>*>& GetDirectionToConsoleVariableMap()
{
	static TMap<EUINavigation, TAutoConsoleVariable<bool>*> DirectionToCVar = {
		{ EUINavigation::Left, &CVarDisplayCandidateWidgets_Left },
		{ EUINavigation::Right, &CVarDisplayCandidateWidgets_Right },
		{ EUINavigation::Up, &CVarDisplayCandidateWidgets_Up },
		{ EUINavigation::Down, &CVarDisplayCandidateWidgets_Down },
	};

	return DirectionToCVar;
}

FAutoConsoleCommand ConsoleCommandDisplayCandidateWidgets_All(
	TEXT("Slate.NavigationDebugging.CandidateWidgets.All"),
	TEXT("Set enabled to display candidate widgets for 4 [Up, Down, Left, Right] Navigation Directions of the focused widget when `Slate.NavigationDebugging` is enabled"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
												  {
													  if (!InArgs.IsEmpty())
													  {
														  const bool bEnableAll = InArgs[0].ToBool();
														  
														  for (TPair<EUINavigation, TAutoConsoleVariable<bool>*>& DirectionToCVarPair : GetDirectionToConsoleVariableMap())
														  {
															  DirectionToCVarPair.Value->AsVariable()->Set(bEnableAll, EConsoleVariableFlags::ECVF_SetByConsole);
														  }
													  }
												  }));

static bool ShouldDisplayCandidateWidgetsFor(const EUINavigation InDirection)
{
	const TAutoConsoleVariable<bool>* const * ConsoleObject = GetDirectionToConsoleVariableMap().Find(InDirection);
	return ConsoleObject && (*ConsoleObject)->GetValueOnAnyThread();
}

static bool ShouldDisplayAnyCandidateWidgets()
{
	for (TPair<EUINavigation, TAutoConsoleVariable<bool>*>& DirectionToCVarPair : GetDirectionToConsoleVariableMap())
	{
		if (DirectionToCVarPair.Value->GetValueOnAnyThread())
		{
			return true;
		}
	}

	return false;
}

}

namespace ArrowGeometryHelpers
{
	constexpr bool bAntiAlias = true;
	constexpr float LineThickness = 2.0f;
	constexpr float DashLineLength = 4.0f;
	constexpr float CandidateColorAlpha = 0.25f;
	constexpr float MinimumArrowGeometrySizeToPaint = 8.0f;
	static const FVector2f ArrowBrushSize = FVector2f(30.0f, 30.0f);

	static EUINavigation GetOpposingDirection(const EUINavigation InNavDirection)
	{
		if (InNavDirection == EUINavigation::Up)
		{
			return EUINavigation::Down;
		}
		else if (InNavDirection == EUINavigation::Down)
		{
			return EUINavigation::Up;
		}
		else if (InNavDirection == EUINavigation::Left)
		{
			return EUINavigation::Right;
		}
		else if (InNavDirection == EUINavigation::Right)
		{
			return EUINavigation::Left;
		}

		return InNavDirection;
	}

	static FVector2f GetArrowCoordinatesForLine(const EUINavigation InNavDirection)
	{
		FVector2f Coordinates = FVector2f::ZeroVector;

		if (InNavDirection == EUINavigation::Up)
		{
			Coordinates = FVector2f(0.5f, 0.0f);
		}
		else if (InNavDirection == EUINavigation::Down)
		{
			Coordinates = FVector2f(0.5f, 1.0f);
		}
		else if (InNavDirection == EUINavigation::Left)
		{
			Coordinates = FVector2f(0.0f, 0.5f);
		}
		else if (InNavDirection == EUINavigation::Right)
		{
			Coordinates = FVector2f(1.0f, 0.5f);
		}

		return Coordinates;
	}

	static bool IsWidgetOutOfViewport(const FSlateUserNavigationData& InNavigationData, TSharedRef<const SWidget> InWidget)
	{
		check(InNavigationData.CanPaint());

		TSharedRef<SWidget> ViewportWidget = InNavigationData.ViewportWidget.ToSharedRef();

		const FGeometry ViewportGeometry = ViewportWidget->GetPaintSpaceGeometry();
		const FSlateRect ViewportRect = FSlateRect::FromPointAndExtent(ViewportGeometry.GetAbsolutePosition(), ViewportGeometry.GetAbsoluteSize());
		
		const FGeometry WidgetGeometry = InWidget->GetPaintSpaceGeometry();
		const FSlateRect WidgetRect = FSlateRect::FromPointAndExtent(WidgetGeometry.GetAbsolutePosition(), WidgetGeometry.GetAbsoluteSize());

		return !FSlateRect::DoRectanglesIntersect(ViewportRect, WidgetRect);
	}

	static FGeometry GetArrowGeometryForWidget_OnScreen(const FSlateUserNavigationData& InNavigationData, const EUINavigation InNavDirection, const FGeometry& InWidgetGeometry)
	{
		check(InNavigationData.CanPaint());

		const FVector2f WidgetSize = InWidgetGeometry.GetAbsoluteSize();
		const FVector2f DefaultArrowTextureSize = ArrowGeometryHelpers::ArrowBrushSize;

		const bool bArrowsWiderThanWidget = WidgetSize.X - (4.0f * DefaultArrowTextureSize.X) <= 0.0f;
		const bool bArrowsTallerThanWidget = WidgetSize.Y - (4.0f * DefaultArrowTextureSize.Y) <= 0.0f;
		const bool bScaleDownArrows = bArrowsWiderThanWidget || bArrowsTallerThanWidget;
		const float ScaledDownArrowSize = FMath::Min(WidgetSize.X, WidgetSize.Y) * 0.25f;

		const FVector2f ArrowTextureSize = bScaleDownArrows ? FVector2f(ScaledDownArrowSize) : DefaultArrowTextureSize;

		FVector2f ArrowPosition;

		if (InNavDirection == EUINavigation::Up)
		{
			const FVector2f TopCenter = InWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(0.5f, 0.0f));
			ArrowPosition = TopCenter + FVector2f(ArrowTextureSize.X * -0.5f, 0.0f);
		}
		else if (InNavDirection == EUINavigation::Down)
		{
			const FVector2f BottomCenter = InWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(0.5f, 1.0f));
			ArrowPosition = BottomCenter + FVector2f(ArrowTextureSize.X * -0.5f, -1.0f * ArrowTextureSize.Y);
		}
		else if (InNavDirection == EUINavigation::Left)
		{
			const FVector2f LeftMiddle = InWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(0.0f, 0.5f));
			ArrowPosition = LeftMiddle + FVector2f(0.0f, ArrowTextureSize.Y * -0.5f);
		}
		else if (InNavDirection == EUINavigation::Right)
		{
			const FVector2f RightMiddle = InWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(1.0f, 0.5f));
			ArrowPosition = RightMiddle + FVector2f(-1.0f * ArrowTextureSize.X, ArrowTextureSize.Y * -0.5f);
		}
		else
		{
			ArrowPosition = InWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(0.5f, 0.5f));
		}

		const FGeometry ViewportWidgetGeometry = InNavigationData.ViewportWidget->GetPaintSpaceGeometry();
		const FVector2f ArrowPositionInViewport = ViewportWidgetGeometry.AbsoluteToLocal(ArrowPosition);

		return ViewportWidgetGeometry.MakeChild(ArrowTextureSize, FSlateLayoutTransform(1.0f, ArrowPositionInViewport));
	}

	static FGeometry GetArrowGeometryForWidget_OffScreen(const FSlateUserNavigationData& InNavigationData, const EUINavigation InNavDirection, const FNavigationDebuggingStyle::EArrowType InArrowType, TSharedRef<const SWidget> InWidget)
	{
		check(InNavigationData.CanPaint());

		TSharedRef<SWindow> WindowWidget = InNavigationData.WindowWidget.ToSharedRef();
		TSharedPtr<SWidget> ViewportWidget = InNavigationData.ViewportWidget.ToSharedRef();

		const FGeometry ViewportWidgetGeometry = ViewportWidget->GetPaintSpaceGeometry();
		const FGeometry WidgetGeometry = InWidget->GetPaintSpaceGeometry();
		const FVector2f WidgetPosition = WidgetGeometry.GetAbsolutePosition();

		const bool bIsFocusArrow = InArrowType == FNavigationDebuggingStyle::EArrowType::Focus;
		const EUINavigation ArrowDirection = bIsFocusArrow ? InNavDirection : GetOpposingDirection(InNavDirection);
		// Clamping On Screen Widget Geometry Positions
		const FGeometry OnScreenArrowGeometry = GetArrowGeometryForWidget_OnScreen(InNavigationData, ArrowDirection, WidgetGeometry);
		const FVector2f ArrowCenter = OnScreenArrowGeometry.GetAbsolutePositionAtCoordinates(FVector2f(0.5f, 0.5f));

		const FVector2f ViewportTopLeft = ViewportWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(0.0f, 0.0f));
		const FVector2f ViewportExtents = ViewportWidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2f(1.0f, 1.0f));
		const FVector2f ClampedWidgetPosition = FVector2f::Clamp(ArrowCenter, FVector2f(0.0f, 0.0f), ViewportExtents);

		const FVector2f ArrowTextureSize = ArrowGeometryHelpers::ArrowBrushSize;
		const FVector2f HalfArrowSize = (ArrowTextureSize * 0.5f);

		FVector2f ArrowPositionForWidget = ClampedWidgetPosition - HalfArrowSize;

		const bool bIsBelowViewport = WidgetPosition.Y > ViewportExtents.Y;
		if (bIsBelowViewport)
		{
			ArrowPositionForWidget.Y -= HalfArrowSize.Y;
		}

		const bool bIsAboveViewport = (WidgetPosition.Y + HalfArrowSize.Y) < ViewportTopLeft.Y;
		if (bIsAboveViewport)
		{
			ArrowPositionForWidget.Y += HalfArrowSize.Y;
		}

		const bool bIsRightViewport = (WidgetPosition.X + HalfArrowSize.X) > ViewportExtents.X;
		if (bIsRightViewport)
		{
			ArrowPositionForWidget.X -= HalfArrowSize.X;
		}

		const bool bIsLeftViewport = WidgetPosition.X < ViewportTopLeft.X;
		if (bIsLeftViewport)
		{
			ArrowPositionForWidget.X += HalfArrowSize.X;
		}

		return ViewportWidgetGeometry.MakeChild(ArrowTextureSize, FSlateLayoutTransform(1.0f, ViewportWidgetGeometry.AbsoluteToLocal(ArrowPositionForWidget)));
	}

	static void DrawArrow(const FGeometry& InArrowGeometry, const FSlateBrush* InArrowBrush, float ColorAlpha, FSlateWindowElementList& OutDrawElements, int32 LayerId)
	{
		const FVector2f ArrowSize = InArrowGeometry.GetAbsoluteSize();
		const bool bIsTooSmall = ArrowSize.X < MinimumArrowGeometrySizeToPaint || ArrowSize.Y < MinimumArrowGeometrySizeToPaint;

		if (!bIsTooSmall)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				InArrowGeometry.ToPaintGeometry(),
				InArrowBrush,
				ESlateDrawEffect::None,
				FLinearColor::White.CopyWithNewOpacity(ColorAlpha)
			);
		}
	}
}

FNavigationDebuggingStyle::FNavigationDebuggingStyle()
	: FSlateStyleSet(FName(TEXT("NavigationDebuggingStyle")))
{
	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate") / TEXT("Icons")/ TEXT("Navigation"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate") / TEXT("Icons") / TEXT("Navigation"));

	// Navigation Debugging Arrows
	const FVector2f ArrowTextureSize = FVector2f(64.0f, 64.0f);
	FSlateVectorImageBrush* Navigation_Focus_Left = new CORE_IMAGE_BRUSH_SVG("focus-arrow-left", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Focus_Right = new CORE_IMAGE_BRUSH_SVG("focus-arrow-right", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Focus_Up = new CORE_IMAGE_BRUSH_SVG("focus-arrow-up", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Focus_Down = new CORE_IMAGE_BRUSH_SVG("focus-arrow-down", ArrowTextureSize);

	FSlateVectorImageBrush* Navigation_Candidate_Left = new CORE_IMAGE_BRUSH_SVG("candidate-arrow-left", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Candidate_Right = new CORE_IMAGE_BRUSH_SVG("candidate-arrow-right", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Candidate_Up = new CORE_IMAGE_BRUSH_SVG("candidate-arrow-up", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Candidate_Down = new CORE_IMAGE_BRUSH_SVG("candidate-arrow-down", ArrowTextureSize);

	FSlateVectorImageBrush* Navigation_Target_Left = new CORE_IMAGE_BRUSH_SVG("target-arrow-left", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Target_Right = new CORE_IMAGE_BRUSH_SVG("target-arrow-right", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Target_Up = new CORE_IMAGE_BRUSH_SVG("target-arrow-up", ArrowTextureSize);
	FSlateVectorImageBrush* Navigation_Target_Down = new CORE_IMAGE_BRUSH_SVG("target-arrow-down", ArrowTextureSize);

	NavigationArrowBrushes = TMap<EUINavigation, TMap<EArrowType, const FSlateBrush*>>(
		{
			{ EUINavigation::Up, TMap<EArrowType, const FSlateBrush*>(
				{
					{ EArrowType::Focus, { Navigation_Focus_Up } },
					{ EArrowType::Candidate, { Navigation_Candidate_Up } },
					{ EArrowType::Target, { Navigation_Target_Up } },
				})
			},
			{ EUINavigation::Down,  TMap<EArrowType, const FSlateBrush*>(
				{
					{ EArrowType::Focus, { Navigation_Focus_Down } },
					{ EArrowType::Candidate, { Navigation_Candidate_Down } },
					{ EArrowType::Target, { Navigation_Target_Down } },
				})
			},
			{ EUINavigation::Left,  TMap<EArrowType, const FSlateBrush*>(
				{
					{ EArrowType::Focus, { Navigation_Focus_Left } },
					{ EArrowType::Candidate, { Navigation_Candidate_Left } },
					{ EArrowType::Target, { Navigation_Target_Left } },
				})
			},
			{ EUINavigation::Right,  TMap<EArrowType, const FSlateBrush*>(
				{
					{ EArrowType::Focus, { Navigation_Focus_Right } },
					{ EArrowType::Candidate, { Navigation_Candidate_Right } },
					{ EArrowType::Target, { Navigation_Target_Right } },
				})
			}
		});

	Set("Debug.Navigation.Focus.Left", Navigation_Focus_Left);
	Set("Debug.Navigation.Focus.Right", Navigation_Focus_Right);
	Set("Debug.Navigation.Focus.Up", Navigation_Focus_Up);
	Set("Debug.Navigation.Focus.Down", Navigation_Focus_Down);

	Set("Debug.Navigation.Candidate.Left", Navigation_Candidate_Left);
	Set("Debug.Navigation.Candidate.Right", Navigation_Candidate_Right);
	Set("Debug.Navigation.Candidate.Up", Navigation_Candidate_Up);
	Set("Debug.Navigation.Candidate.Down", Navigation_Candidate_Down);

	Set("Debug.Navigation.Target.Left", Navigation_Target_Left);
	Set("Debug.Navigation.Target.Right", Navigation_Target_Right);
	Set("Debug.Navigation.Target.Up", Navigation_Target_Up);
	Set("Debug.Navigation.Target.Down", Navigation_Target_Down);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FNavigationDebuggingStyle::~FNavigationDebuggingStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FLinearColor FNavigationDebuggingStyle::GetFocusedWidgetColor() const
{
	return FLinearColor::Green;
}

FLinearColor FNavigationDebuggingStyle::GetTargetWidgetColor() const
{
	return FLinearColor::Red;
}

const FSlateBrush* FNavigationDebuggingStyle::GetArrowBrush(const EUINavigation InNavigation, const EArrowType InArrow) const
{
	const TMap<EArrowType, const FSlateBrush*>* FoundKeyOne = NavigationArrowBrushes.Find(InNavigation);
	const FSlateBrush* const * FoundKeyTwo = FoundKeyOne ? FoundKeyOne->Find(InArrow) : nullptr;
	return FoundKeyTwo ? *FoundKeyTwo : nullptr;
}

/*static*/ FNavigationDebuggingStyle& FNavigationDebuggingStyle::Get()
{
	static FNavigationDebuggingStyle Instance;
	return Instance;
}

/*static*/ FSlateNavigationDebugger& FSlateNavigationDebugger::Get()
{
	static FSlateNavigationDebugger Instance;
	return Instance;
}

bool FSlateUserNavigationData::CanPaint() const
{ 
	return FocusedWidgetPath->IsValid() && ViewportWidget.IsValid() && FocusedWidget.IsValid() && WindowWidget.IsValid();
}

FSlateUserNavigationData::FSlateUserNavigationData(FSlateUser& InSlateUser)
	: FocusedWidgetPath(InSlateUser.GetFocusPath())
{
	if (FocusedWidgetPath->IsValid())
	{
		WindowWidget = FocusedWidgetPath->GetWindow();
		FocusedWidget = FocusedWidgetPath->GetLastWidget();

		TSharedPtr<ISlateViewport> ViewportInterface = WindowWidget.IsValid() ? WindowWidget->GetViewport() : nullptr;
		TWeakPtr<SWidget> WeakViewportWidget = ViewportInterface.IsValid() ? ViewportInterface->GetWidget() : nullptr;
		ViewportWidget = WeakViewportWidget.IsValid() ? WeakViewportWidget.Pin() : nullptr;
	}

	if (CanPaint())
	{
		WindowGeometry = WindowWidget->GetPaintSpaceGeometry();
	}
}

void FSlateNavigationDebugger::SetListenToHitTestGrid(const bool bBind, TArray<FHittestGrid::FDebuggingFindNextFocusableWidgetArgs>& InContainer, FDelegateHandle& InHandle) const
{
	if (bBind)
	{
		InHandle = FHittestGrid::OnFindNextFocusableWidgetExecuted.AddLambda(
			[&InContainer](const FHittestGrid* /*HittestGrid*/, const FHittestGrid::FDebuggingFindNextFocusableWidgetArgs& Info)
			{
				InContainer.Add(Info);
			});
	}
	else
	{
		FHittestGrid::OnFindNextFocusableWidgetExecuted.Remove(InHandle);
	}
}

TMap<EUINavigation, TSharedRef<SWidget>> FSlateNavigationDebugger::SimulateNavigationDirection(const FSlateUserNavigationData& InNavigationData, const int32 SlateUserIndex) const
{
	check(InNavigationData.CanPaint());

	TMap<EUINavigation, TSharedRef<SWidget>> ReturnValue;

	const TSet<EUINavigation> DirectionsToDebug = TSet<EUINavigation>
		({
			EUINavigation::Left,
			EUINavigation::Right,
			EUINavigation::Up,
			EUINavigation::Down,
		});

	for (const EUINavigation& NavigationDirection : DirectionsToDebug)
	{
		for (int32 WidgetIndex = InNavigationData.FocusedWidgetPath->Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
		{
			FArrangedWidget& SomeWidgetGettingEvent = InNavigationData.FocusedWidgetPath->Widgets[WidgetIndex];
			if (SomeWidgetGettingEvent.Widget->IsEnabled())
			{
				const FNavigationReply FocusedNavigationReply = InNavigationData.FocusedWidget->OnNavigation(SomeWidgetGettingEvent.Geometry, NavigationDirection);

				if (FocusedNavigationReply.GetBoundaryRule() != EUINavigationRule::Escape || SomeWidgetGettingEvent.Widget == InNavigationData.WindowWidget || WidgetIndex == 0)
				{
					FSlateApplication::FNavigationResult NavigationResult = FSlateApplication::Get().CalculateDestinationWidget(
						InNavigationData.FocusedWidgetPath.Get(),
						FocusedNavigationReply,
						NavigationDirection,
						SlateUserIndex,
						SomeWidgetGettingEvent);

					const bool bFoundKeyboardFocusableWidget = NavigationResult.DestinationWidget.IsValid() && NavigationResult.DestinationWidget->SupportsKeyboardFocus();
					if (bFoundKeyboardFocusableWidget)
					{
						bool bIsViewportWidgetChild = false;
						TSharedRef<SWidget> ViewportWidgetRef = InNavigationData.ViewportWidget.ToSharedRef();
						TSharedPtr<SWidget> WidgetTreeTraverse = NavigationResult.DestinationWidget.ToSharedRef();

						while (WidgetTreeTraverse.IsValid() && !bIsViewportWidgetChild)
						{
							bIsViewportWidgetChild = WidgetTreeTraverse.IsValid() && WidgetTreeTraverse.ToSharedRef() == ViewportWidgetRef;
							TSharedPtr<SWidget> Parent = WidgetTreeTraverse.IsValid() ? WidgetTreeTraverse->GetParentWidget() : nullptr;
							WidgetTreeTraverse = Parent;
						}

						if (bIsViewportWidgetChild)
						{
							ReturnValue.Emplace(NavigationDirection, NavigationResult.DestinationWidget.ToSharedRef());
						}
					}
				}
			}
		}
	}

	return ReturnValue;
}

FGeometry FSlateNavigationDebugger::PaintOnScreenWidgetArrow(const FSlateUserNavigationData& InNavigationData, const EUINavigation InNavDirection, const FGeometry& InWidgetGeometry, const FNavigationDebuggingStyle::EArrowType InArrowType, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const bool bIsFocusedWidget = InArrowType == FNavigationDebuggingStyle::EArrowType::Focus;
	const EUINavigation DirectionForArrow = bIsFocusedWidget ? InNavDirection : ArrowGeometryHelpers::GetOpposingDirection(InNavDirection);
	const FGeometry OnScreenWidgetArrowGeometry = ArrowGeometryHelpers::GetArrowGeometryForWidget_OnScreen(InNavigationData, DirectionForArrow, InWidgetGeometry);

	const FSlateBrush* BrushToUse = FNavigationDebuggingStyle::Get().GetArrowBrush(InNavDirection, InArrowType);
	if (ensure(BrushToUse))
	{
		const float ColorAlpha = InArrowType == FNavigationDebuggingStyle::EArrowType::Candidate ? ArrowGeometryHelpers::CandidateColorAlpha : 1.0f;
		ArrowGeometryHelpers::DrawArrow(OnScreenWidgetArrowGeometry, BrushToUse, ColorAlpha, OutDrawElements, LayerId);
	}

	return OnScreenWidgetArrowGeometry;
}

FGeometry FSlateNavigationDebugger::PaintOffScreenWidgetArrow(const FSlateUserNavigationData& InNavigationData, const EUINavigation InNavDirection, TSharedRef<const SWidget> InWidget, const FNavigationDebuggingStyle::EArrowType InArrowType, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const FGeometry OffscreenArrowIndicator = ArrowGeometryHelpers::GetArrowGeometryForWidget_OffScreen(InNavigationData, InNavDirection, InArrowType, InWidget);

	const FSlateBrush* BrushToUse = FNavigationDebuggingStyle::Get().GetArrowBrush(ArrowGeometryHelpers::GetOpposingDirection(InNavDirection), InArrowType);
	if (ensure(BrushToUse))
	{
		const float ColorAlpha = InArrowType == FNavigationDebuggingStyle::EArrowType::Candidate ? ArrowGeometryHelpers::CandidateColorAlpha : 1.0f;
		ArrowGeometryHelpers::DrawArrow(OffscreenArrowIndicator, BrushToUse, ColorAlpha, OutDrawElements, LayerId);
	}

	return OffscreenArrowIndicator;
}

void FSlateNavigationDebugger::PaintGeometryOutline(const FGeometry& InGeometry, const FLinearColor& InColor, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	FSlateDrawElement::MakeGeometryOutline(
		OutDrawElements,
		LayerId,
		InGeometry.ToPaintGeometry(),
		InColor,
		ESlateDrawEffect::None,
		ArrowGeometryHelpers::bAntiAlias,
		ArrowGeometryHelpers::LineThickness
	);
}

void FSlateNavigationDebugger::PaintDashedLine(const FSlateUserNavigationData& InNavigationData, TArray<FVector2f>&& PointsToConnect, const FLinearColor& InColor, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	check(InNavigationData.CanPaint());

	FSlateDrawElement::MakeDashedLines(
		OutDrawElements,
		LayerId,
		InNavigationData.WindowGeometry.ToPaintGeometry(),
		MoveTemp(PointsToConnect),
		ESlateDrawEffect::None,
		InColor,
		ArrowGeometryHelpers::LineThickness,
		ArrowGeometryHelpers::DashLineLength,
		0.0f
	);
}

int32 FSlateNavigationDebugger::PaintCandidateWidgets(const FSlateUserNavigationData& InNavigationData, const TArray<FHittestGrid::FDebuggingFindNextFocusableWidgetArgs>& InCandidateWidgets, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (DebugCandidateWidgets::ShouldDisplayAnyCandidateWidgets())
	{
		check(InNavigationData.CanPaint());

		TMap<TSharedRef<const SWidget>, TSet<EUINavigation>> HighlightedCandidateWidgets;

		for (const FHittestGrid::FDebuggingFindNextFocusableWidgetArgs& Info : InCandidateWidgets)
		{
			for (const FHittestGrid::FDebuggingFindNextFocusableWidgetArgs::FWidgetResult& Result : Info.IntermediateResults)
			{
				if (Result.Widget.IsValid())
				{
					TSharedRef<const SWidget> CandidateWidgetRef = Result.Widget.ToSharedRef();

					const bool bNotCandidate = Result.Result.EqualTo(HittestGridDebuggingText::Valid);

					const bool bDebuggingDirection = DebugCandidateWidgets::ShouldDisplayCandidateWidgetsFor(Info.Direction);
					const bool bAlreadyOutlined = HighlightedCandidateWidgets.Contains(CandidateWidgetRef);
					const bool bAlreadyPaintedArrow = bAlreadyOutlined && HighlightedCandidateWidgets[CandidateWidgetRef].Contains(Info.Direction);

					const bool bDisplayCandidateWidget = bDebuggingDirection && !bNotCandidate && !bAlreadyPaintedArrow;
					if (bDisplayCandidateWidget)
					{
						HighlightedCandidateWidgets.FindOrAdd(CandidateWidgetRef).Add(Info.Direction);

						const FGeometry CandidateWidgetGeometry = CandidateWidgetRef->GetPaintSpaceGeometry();
						if (!bAlreadyOutlined)
						{
							const FLinearColor OutlineColor = FNavigationDebuggingStyle::Get().GetTargetWidgetColor().CopyWithNewOpacity(ArrowGeometryHelpers::CandidateColorAlpha);
							PaintGeometryOutline(CandidateWidgetGeometry, OutlineColor, OutDrawElements, ++LayerId);
						}

						const bool bOnScreen = !Result.Result.EqualTo(HittestGridDebuggingText::DoesNotIntersect);
						if (bOnScreen)
						{
							PaintOnScreenWidgetArrow(InNavigationData, Info.Direction, CandidateWidgetGeometry, FNavigationDebuggingStyle::EArrowType::Candidate, OutDrawElements, ++LayerId);
						}
						else
						{
							PaintOffScreenWidgetArrow(InNavigationData, Info.Direction, CandidateWidgetRef, FNavigationDebuggingStyle::EArrowType::Candidate, OutDrawElements, ++LayerId);
						}
					}
				}
			}
		}
	}

	return LayerId;
}

int32 FSlateNavigationDebugger::PaintFocusTargetArrowsAndLines(const TPair<EUINavigation, TSharedRef<SWidget>>& FocusedToDestination, const FSlateUserNavigationData& InNavigationData, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	check(InNavigationData.CanPaint());

	// Paint Target Widget Arrows
	FVector2f TargetWidgetArrowPosition;
	{
		const bool bTargetWidgetInViewport = !ArrowGeometryHelpers::IsWidgetOutOfViewport(InNavigationData, FocusedToDestination.Value);
		if (bTargetWidgetInViewport)
		{
			const FGeometry TargetWidgetGeometry = FocusedToDestination.Value->GetPaintSpaceGeometry();
			const FGeometry TargetArrowGeometry = PaintOnScreenWidgetArrow(InNavigationData, FocusedToDestination.Key, TargetWidgetGeometry, FNavigationDebuggingStyle::EArrowType::Target, OutDrawElements, ++LayerId);
			TargetWidgetArrowPosition = TargetArrowGeometry.GetAbsolutePositionAtCoordinates(ArrowGeometryHelpers::GetArrowCoordinatesForLine(ArrowGeometryHelpers::GetOpposingDirection(FocusedToDestination.Key)));
		}
		else
		{
			const FGeometry TargetArrowGeometry = PaintOffScreenWidgetArrow(InNavigationData, FocusedToDestination.Key, FocusedToDestination.Value, FNavigationDebuggingStyle::EArrowType::Target, OutDrawElements, ++LayerId);
			TargetWidgetArrowPosition = TargetArrowGeometry.GetAbsolutePositionAtCoordinates(ArrowGeometryHelpers::GetArrowCoordinatesForLine(ArrowGeometryHelpers::GetOpposingDirection(FocusedToDestination.Key)));
		}
	}

	// Paint Focus Widget Arrows
	FVector2f FocusedWidgetArrowPosition;
	{
		const bool bFocusWidgetInViewport = !ArrowGeometryHelpers::IsWidgetOutOfViewport(InNavigationData, InNavigationData.FocusedWidget.ToSharedRef());
		if (bFocusWidgetInViewport)
		{
			const FGeometry FocusedWidgetGeometry = InNavigationData.FocusedWidget->GetPaintSpaceGeometry();
			const FGeometry FocusArrowGeometry = PaintOnScreenWidgetArrow(InNavigationData, FocusedToDestination.Key, FocusedWidgetGeometry, FNavigationDebuggingStyle::EArrowType::Focus, OutDrawElements, ++LayerId);
			FocusedWidgetArrowPosition = FocusArrowGeometry.GetAbsolutePositionAtCoordinates(ArrowGeometryHelpers::GetArrowCoordinatesForLine(FocusedToDestination.Key));
		}
		else
		{
			const FGeometry FocusArrowGeometry = PaintOffScreenWidgetArrow(InNavigationData, FocusedToDestination.Key, InNavigationData.FocusedWidget.ToSharedRef(), FNavigationDebuggingStyle::EArrowType::Focus, OutDrawElements, ++LayerId);
			FocusedWidgetArrowPosition = FocusArrowGeometry.GetAbsolutePositionAtCoordinates(ArrowGeometryHelpers::GetArrowCoordinatesForLine(FocusedToDestination.Key));
		}
	}

	// Paint Dashed Lines
	TArray<FVector2f> FocusToTargetPoints;
	FocusToTargetPoints.Add(FocusedWidgetArrowPosition);
	FocusToTargetPoints.Add(TargetWidgetArrowPosition);

	PaintDashedLine(InNavigationData, MoveTemp(FocusToTargetPoints), FNavigationDebuggingStyle::Get().GetFocusedWidgetColor(), OutDrawElements, ++LayerId);

	return LayerId;
}

int32 FSlateNavigationDebugger::PaintForUser(FSlateUser& InSlateUser, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const FSlateUserNavigationData NavigationData = FSlateUserNavigationData(InSlateUser);

	if (NavigationData.CanPaint())
	{
		FDelegateHandle HittestGridHandle;
		TArray<FHittestGrid::FDebuggingFindNextFocusableWidgetArgs> HitTestGridInfo;

		// Bind to listen to candidate widgets
		SetListenToHitTestGrid(true, HitTestGridInfo, HittestGridHandle);

		// Gather Destination Widgets
		TMap<EUINavigation, TSharedRef<SWidget>> DirectionToDestinationWidget = SimulateNavigationDirection(NavigationData, InSlateUser.GetUserIndex());

		// Paint Candidate Widgets
		LayerId = PaintCandidateWidgets(NavigationData, HitTestGridInfo, OutDrawElements, LayerId);

		// Highlight Focused Widget
		PaintGeometryOutline(NavigationData.FocusedWidget->GetPaintSpaceGeometry(), FNavigationDebuggingStyle::Get().GetFocusedWidgetColor(), OutDrawElements, ++LayerId);

		// Paint Focused Widget Arrows and Target Arrows; with Dashed Lines
		for (const TPair<EUINavigation, TSharedRef<SWidget>>& DirectionToWidget : DirectionToDestinationWidget)
		{
			// Paint Target Widget Outline
			PaintGeometryOutline(DirectionToWidget.Value->GetPaintSpaceGeometry(), FNavigationDebuggingStyle::Get().GetTargetWidgetColor(), OutDrawElements, ++LayerId);

			// Paint Arrows and Lines
			LayerId = PaintFocusTargetArrowsAndLines(DirectionToWidget, NavigationData, OutDrawElements, LayerId);
		}
		
		// Unbind from listening to candidate widgets
		SetListenToHitTestGrid(false, HitTestGridInfo, HittestGridHandle);
	}

	return LayerId;
}

int32 FSlateNavigationDebugger::Paint(FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (GSlateNavigationDebugging)
	{
		FSlateApplication::Get().ForEachUser([&OutDrawElements, &LayerId](FSlateUser& SlateUser) -> void
											 {
												LayerId = FSlateNavigationDebugger::Get().PaintForUser(SlateUser, OutDrawElements, LayerId);
											 });
	}

	return LayerId;
}


#endif // WITH_SLATE_NAVIGATION_DEBUGGING