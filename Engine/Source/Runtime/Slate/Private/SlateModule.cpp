// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Framework/Animation/AnimatedAttributeManager.h"


/**
 * Implements the Slate module.
 */
class FSlateModule
	: public IModuleInterface
{
	virtual void StartupModule() override
	{
		FAnimatedAttributeManager::Get().SetupTick();
#if PLATFORM_MAC
		FModuleManager::Get().LoadModule("MacMenu");
#endif
	}

	virtual void ShutdownModule() override
	{
		FAnimatedAttributeManager::Get().TeardownTick();
#if PLATFORM_MAC
		FModuleManager::Get().UnloadModule("MacMenu");
#endif
	}
};


IMPLEMENT_MODULE(FSlateModule, Slate);
