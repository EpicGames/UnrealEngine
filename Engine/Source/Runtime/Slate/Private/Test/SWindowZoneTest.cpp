// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_WORKER && PLATFORM_DESKTOP

#include "Widgets/SWindow.h"
#include "Widgets/SLeafWidget.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"

namespace UE::Slate::Test
{

namespace
{
	constexpr float WindowSize = 256.0f;
	constexpr float ResizeBorderSize = 5.0f;

	struct FTestPositions
	{
		// Outside window positions
		static FVector2D Outside() { return FVector2D(-10.0, -10.0); }
		static FVector2D OutsideTopRight() { return FVector2D(WindowSize + 10.0, -10.0); }
		static FVector2D OutsideBottomLeft() { return FVector2D(-10.0, WindowSize + 10.0); }
		static FVector2D OutsideBottomRight() { return FVector2D(WindowSize + 10.0, WindowSize + 10.0); }
		static FVector2D OutsideTop() { return FVector2D(WindowSize / 2.0, -10.0); }
		static FVector2D OutsideBottom() { return FVector2D(WindowSize / 2.0, WindowSize + 10.0); }

		// Border positions (within resize border region)
		static FVector2D TopLeft() { return FVector2D(2.0, 2.0); }
		static FVector2D Top() { return FVector2D(WindowSize / 2.0, 2.0); }
		static FVector2D TopRight() { return FVector2D(WindowSize - 2.0, 2.0); }
		static FVector2D Left() { return FVector2D(2.0, WindowSize / 2.0); }
		static FVector2D Right() { return FVector2D(WindowSize - 2.0, WindowSize / 2.0); }
		static FVector2D BottomLeft() { return FVector2D(2.0, WindowSize - 2.0); }
		static FVector2D Bottom() { return FVector2D(WindowSize / 2.0, WindowSize - 2.0); }
		static FVector2D BottomRight() { return FVector2D(WindowSize - 2.0, WindowSize - 2.0); }

		// Center (client area)
		static FVector2D Center() { return FVector2D(WindowSize / 2.0, WindowSize / 2.0); }
	};

	class FMockGenericWindow : public FGenericWindow
	{
	public:
		FMockGenericWindow(EWindowMode::Type InMockWindowMode)
			: MockWindowMode(InMockWindowMode)
		{
		}

		virtual void SetWindowMode(EWindowMode::Type InNewWindowMode) override
		{
			MockWindowMode = InNewWindowMode;
		}

		virtual EWindowMode::Type GetWindowMode() const override
		{
			return MockWindowMode;
		}

		virtual bool IsMaximized() const override
		{
			return false;
		}

	private:
		EWindowMode::Type MockWindowMode;
	};

	class SZoneOverrideWidget : public SLeafWidget
	{
	public:
		SLATE_BEGIN_ARGS(SZoneOverrideWidget)
			: _ZoneOverride(EWindowZone::Unspecified)
		{
		}
		SLATE_ARGUMENT(EWindowZone::Type, ZoneOverride)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			ZoneOverride = InArgs._ZoneOverride;
		}

		virtual FVector2D ComputeDesiredSize(float) const override
		{
			return FVector2D(WindowSize, WindowSize);
		}

		virtual int32 OnPaint(
			const FPaintArgs& Args,
			const FGeometry& AllottedGeometry,
			const FSlateRect& MyCullingRect,
			FSlateWindowElementList& OutDrawElements,
			int32 LayerId,
			const FWidgetStyle& InWidgetStyle,
			bool bParentEnabled) const override
		{
			return LayerId;
		}

		virtual EWindowZone::Type GetWindowZoneOverride() const override
		{
			return ZoneOverride;
		}

	private:
		EWindowZone::Type ZoneOverride;
	};

	TSharedRef<SWindow> CreateTestWindow(
		bool bUseOSWindowBorder,
		ESizingRule SizingRule,
		EWindowMode::Type WindowMode = EWindowMode::Windowed)
	{
		TSharedRef<SWindow> Result = SNew(SWindow)
			.UseOSWindowBorder(bUseOSWindowBorder)
			.ClientSize(FVector2D(WindowSize, WindowSize))
			.ScreenPosition(FVector2f(256, 256))
			.CreateTitleBar(false)
			.SizingRule(SizingRule)
			.UserResizeBorder(FMargin(ResizeBorderSize))
			.Tag("Test");
		Result->SetNativeWindow(MakeShared<FMockGenericWindow>(WindowMode));
		return Result;
	}

	TSharedRef<SWindow> CreateTestWindow_ZoneOverride(
		bool bUseOSWindowBorder,
		ESizingRule SizingRule,
		EWindowZone::Type ZoneOverride,
		TSharedPtr<SZoneOverrideWidget>& OutZoneWidget,
		EWindowMode::Type WindowMode = EWindowMode::Windowed)
	{
		TSharedRef<SZoneOverrideWidget> ZoneWidget = SNew(SZoneOverrideWidget)
			.ZoneOverride(ZoneOverride);
		OutZoneWidget = ZoneWidget;

		TSharedRef<SWindow> Result = SNew(SWindow)
			.UseOSWindowBorder(bUseOSWindowBorder)
			.ClientSize(FVector2D(WindowSize, WindowSize))
			.ScreenPosition(FVector2f(100, 100))
			.CreateTitleBar(false)
			.SizingRule(SizingRule)
			.UserResizeBorder(FMargin(ResizeBorderSize))
			.Tag("Test")
			[
				ZoneWidget
			];
		Result->SetNativeWindow(MakeShared<FMockGenericWindow>(WindowMode));
		return Result;
	}

	class FCursorModeScope
	{
	public:
		FCursorModeScope(FAutomationTestBase& Test)
		{
			TSharedPtr<ICursor> Cursor = FSlateApplicationBase::Get().GetPlatformCursor();
			OriginalType = Cursor->GetType();
			Cursor->SetType(EMouseCursor::Default);
		}

		~FCursorModeScope()
		{
			FSlateApplicationBase::Get().GetPlatformCursor()->SetType(OriginalType);
		}

		EMouseCursor::Type OriginalType;
	};

	constexpr EAutomationTestFlags SlateWindowZoneTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext |
		EAutomationTestFlags::ProgramContext | EAutomationTestFlags::LowPriority | EAutomationTestFlags::EngineFilter;
}

	/**
	 * Test GetCurrentWindowZone returns NotInWindow for positions outside the window.
	 * Tests all sizing rules: UserSized, FixedSize, Autosized (3)
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSWindowZoneNotInWindowTest, "Slate.Window.GetCurrentWindowZone.NotInWindow", SlateWindowZoneTestFlags)

	bool FSWindowZoneNotInWindowTest::RunTest(const FString& Parameters)
	{
		FCursorModeScope CursorModeScope(*this);
		for (ESizingRule SizingRule : { ESizingRule::UserSized, ESizingRule::FixedSize, ESizingRule::Autosized })
		{
			TSharedRef<SWindow> Window = CreateTestWindow(/*bUseOSWindowBorder*/ true, SizingRule);
			for (const FVector2D& OutsidePos : {
				FTestPositions::Outside(),
				FTestPositions::OutsideTopRight(),
				FTestPositions::OutsideBottomLeft(),
				FTestPositions::OutsideBottomRight(),
				FTestPositions::OutsideTop(),
				FTestPositions::OutsideBottom(),
			})
			{
				constexpr const ANSICHAR* SizingRuleNames[] = { "UserSized", "FixedSize", "Autosized" };

				const EWindowZone::Type ActualZone = Window->GetCurrentWindowZone(OutsidePos);
				TestEqual<int32>(
					FString::Printf(TEXT("%S at (%.0f,%.0f)"), SizingRuleNames[static_cast<int32>(SizingRule)], OutsidePos.X, OutsidePos.Y),
					ActualZone,
					EWindowZone::NotInWindow);
			}

			FSlateApplication::Get().DestroyWindowImmediately(Window);
		}

		return true;
	}

	/**
	 * Test GetCurrentWindowZone in WindowedFullscreen mode.
	 * WindowedFullscreen should never return edge zones.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSWindowZoneWindowedFullscreenTest, "Slate.Window.GetCurrentWindowZone.WindowedFullscreen", SlateWindowZoneTestFlags)

	bool FSWindowZoneWindowedFullscreenTest::RunTest(const FString& Parameters)
	{
		FCursorModeScope CursorModeScope(*this);
		for (bool bOsWindowBorder : { false, true })
		{
			// Test without widget override
			{
				TSharedRef<SWindow> Window = CreateTestWindow(bOsWindowBorder, ESizingRule::UserSized, EWindowMode::WindowedFullscreen);

				for (const FVector2D& Pos : {
					FTestPositions::TopLeft(),
					FTestPositions::Top(),
					FTestPositions::TopRight(),
					FTestPositions::Left(),
					FTestPositions::Center(),
					FTestPositions::Right(),
					FTestPositions::BottomLeft(),
					FTestPositions::Bottom(),
					FTestPositions::BottomRight(),
				})
				{
					const EWindowZone::Type ActualZone = Window->GetCurrentWindowZone(Pos);
					TestEqual<int32>(
						FString::Printf(TEXT("WindowedFullscreen %S at (%.0f,%.0f)"),
							bOsWindowBorder ? "with OS window border" : "without OS window border",
							Pos.X, Pos.Y),
						ActualZone,
						EWindowZone::ClientArea);
				}

				FSlateApplication::Get().DestroyWindowImmediately(Window);
			}

			// Test with widget zone override
			{
				TSharedPtr<SZoneOverrideWidget> ZoneWidget;
				TSharedRef<SWindow> Window = CreateTestWindow_ZoneOverride(
					bOsWindowBorder, ESizingRule::UserSized, EWindowZone::TitleBar, ZoneWidget, EWindowMode::WindowedFullscreen);

				FSlateApplication::Get().ForceRedrawWindow(Window);

				const EWindowZone::Type ActualZone = Window->GetCurrentWindowZone(FTestPositions::Center());
				TestEqual<int32>(
					FString::Printf(TEXT("%S WindowedFullscreen respects widget override"),
						bOsWindowBorder ? "with OS window border" : "without OS window border"),
					ActualZone,
					EWindowZone::TitleBar);

				FSlateApplication::Get().DestroyWindowImmediately(Window);
			}
		}

		return true;
	}

	/**
	 * Test GetCurrentWindowZone for FixedSize and Autosized windows.
	 * Non-resizable windows:
	 * - Should return ClientArea for edge and middle cursor locations
	 * - BUT should respect widget zone overrides
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSWindowZoneNonResizableTest, "Slate.Window.GetCurrentWindowZone.NonResizable", SlateWindowZoneTestFlags)

	bool FSWindowZoneNonResizableTest::RunTest(const FString& Parameters)
	{
		FCursorModeScope CursorModeScope(*this);
		for (ESizingRule SizingRule : { ESizingRule::FixedSize, ESizingRule::Autosized })
		{
			// Test: Edge positions should return ClientArea (no resize handles)
			{
				TSharedRef<SWindow> Window = CreateTestWindow(/*bUseOSWindowBorder*/ true, SizingRule);

				for (const FVector2D& EdgePos : {
					FTestPositions::TopLeft(),
					FTestPositions::Top(),
					FTestPositions::TopRight(),
					FTestPositions::Left(),
					FTestPositions::Right(),
					FTestPositions::BottomLeft(),
					FTestPositions::Bottom(),
					FTestPositions::BottomRight(),
				})
				{
					const EWindowZone::Type ActualZone = Window->GetCurrentWindowZone(EdgePos);
					TestEqual<int32>(
						FString::Printf(TEXT("%S edge at (%.0f,%.0f)"),
							SizingRule == ESizingRule::FixedSize ? "FixedSize" : "Autosized", EdgePos.X, EdgePos.Y),
						ActualZone,
						EWindowZone::ClientArea);
				}

				// Center should also be ClientArea
				TestEqual<int32>(
					FString::Printf(TEXT("%S center"), SizingRule == ESizingRule::FixedSize ? "FixedSize" : "Autosized"),
					Window->GetCurrentWindowZone(FTestPositions::Center()),
					EWindowZone::ClientArea);

				FSlateApplication::Get().DestroyWindowImmediately(Window);
			}

			// Test: Widget zone override should be respected
			{
				TSharedPtr<SZoneOverrideWidget> ZoneWidget;
				TSharedRef<SWindow> Window = CreateTestWindow_ZoneOverride(
					/*bUseOSWindowBorder*/ true,
					SizingRule,
					EWindowZone::TitleBar,
					ZoneWidget);

				FSlateApplication::Get().ForceRedrawWindow(Window);

				const EWindowZone::Type ActualZone = Window->GetCurrentWindowZone(FTestPositions::Center());
				TestEqual<int32>(
					FString::Printf(TEXT("%S respects TitleBar widget override"),
						SizingRule == ESizingRule::FixedSize ? "FixedSize" : "Autosized"),
					ActualZone,
					EWindowZone::TitleBar);

				FSlateApplication::Get().DestroyWindowImmediately(Window);
			}
		}

		return true;
	}

	/**
	 * Test GetCurrentWindowZone for UserSized (resizable) windows.
	 * UserSized windows should return appropriate border zones based on cursor position.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSWindowZoneUserSizedBordersTest, "Slate.Window.GetCurrentWindowZone.UserSizedBorders", SlateWindowZoneTestFlags)

	bool FSWindowZoneUserSizedBordersTest::RunTest(const FString& Parameters)
	{
		FCursorModeScope CursorModeScope(*this);
		struct FZoneTestCase
		{
			FVector2D Position;
			EWindowZone::Type ExpectedZone;
			const ANSICHAR* Name;
		} TestCases[] = {
			{ FTestPositions::TopLeft(),     EWindowZone::TopLeftBorder,     "TopLeftBorder" },
			{ FTestPositions::Top(),         EWindowZone::TopBorder,         "TopBorder" },
			{ FTestPositions::TopRight(),    EWindowZone::TopRightBorder,    "TopRightBorder" },
			{ FTestPositions::Left(),        EWindowZone::LeftBorder,        "LeftBorder" },
			{ FTestPositions::Center(),      EWindowZone::ClientArea,        "ClientArea" },
			{ FTestPositions::Right(),       EWindowZone::RightBorder,       "RightBorder" },
			{ FTestPositions::BottomLeft(),  EWindowZone::BottomLeftBorder,  "BottomLeftBorder" },
			{ FTestPositions::Bottom(),      EWindowZone::BottomBorder,      "BottomBorder" },
			{ FTestPositions::BottomRight(), EWindowZone::BottomRightBorder, "BottomRightBorder" },
		};

		TSharedRef<SWindow> Window = CreateTestWindow(/*bUseOSWindowBorder*/ true, ESizingRule::UserSized);

		for (const FZoneTestCase& TestCase : TestCases)
		{
			const EWindowZone::Type ActualZone = Window->GetCurrentWindowZone(TestCase.Position);
			TestEqual<int32>(FString::Printf(TEXT("UserSized: %S"), TestCase.Name), ActualZone, TestCase.ExpectedZone);
		}

		FSlateApplication::Get().DestroyWindowImmediately(Window);

		return true;
	}
} // namespace UE::Slate::Test

#endif // WITH_AUTOMATION_WORKER && PLATFORM_DESKTOP
