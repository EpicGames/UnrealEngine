// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_WORKER && PLATFORM_DESKTOP

#if PLATFORM_MICROSOFT
// Needed to be able to use RECT
#include "Microsoft/WindowsHWrapper.h"
#elif PLATFORM_MAC || PLATFORM_IOS
#include "Apple/AppleTagRect.h"
#elif PLATFORM_LINUX
#include "Unix/UnixSystemIncludes.h"
#endif
#include "SlateGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/SWindow.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Layout/Children.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "GenericPlatform/ICursor.h"

namespace
{
	using namespace UE::Slate;

	/**
	 * Mock window implementation that always reports itself as being in the foreground.
	 * This allows cursor lock tests to run without depending on desktop window manager behavior,
	 * which varies across platforms (e.g., macOS and Wayland may not allow forcing windows to foreground).
	 */
	class FMockForegroundWindow : public FGenericWindow
	{
	public:
		FMockForegroundWindow()
		{
			// Initialize a minimal window definition
			Definition = MakeShareable(new FGenericWindowDefinition());
			Definition->Type = EWindowType::Normal;
			Definition->XDesiredPositionOnScreen = 0.0f;
			Definition->YDesiredPositionOnScreen = 0.0f;
			Definition->WidthDesiredOnScreen = 100.0f;
			Definition->HeightDesiredOnScreen = 100.0f;
			Definition->TransparencySupport = EWindowTransparency::None;
			Definition->HasOSWindowBorder = false;
			Definition->AppearsInTaskbar = false;
			Definition->IsTopmostWindow = false;
			Definition->AcceptsInput = true;
			Definition->ActivationPolicy = EWindowActivationPolicy::Always;
			Definition->FocusWhenFirstShown = true;
			Definition->HasCloseButton = false;
			Definition->SupportsMinimize = false;
			Definition->SupportsMaximize = false;
			Definition->IsModalWindow = false;
			Definition->IsRegularWindow = true;
			Definition->HasSizingFrame = false;
			Definition->SizeWillChangeOften = false;
			Definition->ShouldPreserveAspectRatio = false;
			Definition->ExpectedMaxWidth = 0;
			Definition->ExpectedMaxHeight = 0;
			Definition->Opacity = 1.0f;
			Definition->CornerRadius = 0;
		}

		virtual bool IsForegroundWindow() const override
		{
			return true; // Always report as foreground for testing
		}

		virtual float GetDPIScaleFactor() const override
		{
			return 1.0f;
		}

		virtual bool IsDefinitionValid() const override
		{
			return Definition.IsValid();
		}

		virtual const FGenericWindowDefinition& GetDefinition() const override
		{
			return *Definition;
		}
	};

	/**
	 * Mock cursor implementation that captures the RECT passed to Lock() for verification.
	 */
	class FMockCursorForLockTest : public ICursor
	{
	public:
		FMockCursorForLockTest() = default;

		// ICursor interface
		virtual FVector2D GetPosition() const override
		{
			return CursorPosition;
		}
		virtual void SetPosition(const int32 X, const int32 Y) override
		{
			CursorPosition = FVector2D(X, Y);
		}
		virtual void SetType(const EMouseCursor::Type InNewCursor) override
		{
			CurrentType = InNewCursor;
		}
		virtual EMouseCursor::Type GetType() const override
		{
			return CurrentType;
		}
		virtual void GetSize(int32& Width, int32& Height) const override
		{
			Width = 16; Height = 16;
		}
		virtual void Show(bool bShow) override
		{
			bIsVisible = bShow;
		}

		virtual void Lock(const RECT* const Bounds) override
		{
			bLockWasCalled = true;
			if (Bounds)
			{
				bHasCapturedBounds = true;
				CapturedBounds = *Bounds;
			}
			else
			{
				bHasCapturedBounds = false;
			}
		}

		virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override
		{
		}
		// end ICursor interface

		void Reset()
		{
			bLockWasCalled = false;
			bHasCapturedBounds = false;
			CapturedBounds = {};
		}

		bool bLockWasCalled = false;
		bool bHasCapturedBounds = false;
		RECT CapturedBounds = {};
	private:
		FVector2D CursorPosition = FVector2D::ZeroVector;
		EMouseCursor::Type CurrentType = EMouseCursor::Default;
		bool bIsVisible = true;
	};

	/**
	 * Custom leaf widget for testing that reports a specific desired size.
	 */
	class STestWidgetWithKnownSize : public SLeafWidget
	{
	public:
		SLATE_BEGIN_ARGS(STestWidgetWithKnownSize)
			: _DesiredWidth(100.0f)
			, _DesiredHeight(100.0f)
		{}
		SLATE_ARGUMENT(float, DesiredWidth)
		SLATE_ARGUMENT(float, DesiredHeight)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			DesiredWidth = InArgs._DesiredWidth;
			DesiredHeight = InArgs._DesiredHeight;
		}

		virtual FVector2D ComputeDesiredSize(float) const override
		{
			return FVector2D(DesiredWidth, DesiredHeight);
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
			const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
			int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			return LayerId;
		}

	private:
		float DesiredWidth = 100.0f;
		float DesiredHeight = 100.0f;
	};

	/**
	 * Container widget that positions its child at a specific fractional offset.
	 * Used to achieve fractional widget geometry bounds for testing cursor lock rounding.
	 */
	class SFractionalOffsetBox : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SFractionalOffsetBox)
			: _OffsetX(0.0f)
			, _OffsetY(0.0f)
		{}
		SLATE_ARGUMENT(float, OffsetX)
		SLATE_ARGUMENT(float, OffsetY)
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_END_ARGS()

		SFractionalOffsetBox()
			: OffsetX(0.0f)
			, OffsetY(0.0f)
			, ChildSlot(this)
		{
		}

		void Construct(const FArguments& InArgs)
		{
			OffsetX = InArgs._OffsetX;
			OffsetY = InArgs._OffsetY;

			ChildSlot
			[
				InArgs._Content.Widget
			];
		}

		virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
		{
			if (ChildSlot.GetWidget()->GetVisibility() != EVisibility::Collapsed)
			{
				const FVector2D ChildDesiredSize = ChildSlot.GetWidget()->GetDesiredSize();
				ArrangedChildren.AddWidget(
					AllottedGeometry.MakeChild(
						ChildSlot.GetWidget(),
						FVector2D(OffsetX, OffsetY),
						ChildDesiredSize
					)
				);
			}
		}

		virtual FChildren* GetChildren() override
		{
			return &ChildSlot;
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
		{
			const FVector2D ChildDesiredSize = ChildSlot.GetWidget()->GetDesiredSize();
			return FVector2D(OffsetX + ChildDesiredSize.X, OffsetY + ChildDesiredSize.Y);
		}

	private:
		float OffsetX;
		float OffsetY;
		FSingleWidgetChildrenWithBasicLayoutSlot ChildSlot;
	};

	/**
	 * Test case parameters for cursor lock rounding tests.
	 * Uses fractional offset within window to achieve desired geometry bounds.
	 */
	struct FCursorLockTestCase
	{
		// Fractional offset of widget within window
		float OffsetX;
		float OffsetY;
		// Widget dimensions (can be fractional to achieve fractional right/bottom bounds)
		float WidgetWidth;
		float WidgetHeight;
		// Expected clip rect after rounding (offset by window origin)
		int32 ExpectedLeft;
		int32 ExpectedTop;
		int32 ExpectedRight;
		int32 ExpectedBottom;
		const ANSICHAR* Description;
	};

	// Fixed window position for all tests (integral coordinates)
	constexpr float WindowOriginX = 100.0f;
	constexpr float WindowOriginY = 100.0f;

	/**
	 * Function to test cursor lock behavior for a specific geometry and window configuration.
	 * Returns true if the test passed, false otherwise.
	 */
	bool TestCursorLockRounding(
		FAutomationTestBase* Test,
		bool bUseOSWindowBorder,
		const FCursorLockTestCase& TestCase,
		const ANSICHAR* WindowDescription)
	{
		if (!FSlateApplication::IsInitialized())
		{
			Test->AddError(TEXT("FSlateApplication is not initialized"));
			return false;
		}

		// Window client size needs to accommodate the offset plus widget size
		const float ClientWidth = TestCase.OffsetX + TestCase.WidgetWidth;
		const float ClientHeight = TestCase.OffsetY + TestCase.WidgetHeight;

		// Create the test widget with the specified size
		TSharedRef<STestWidgetWithKnownSize> TestWidget = SNew(STestWidgetWithKnownSize)
			.DesiredWidth(TestCase.WidgetWidth)
			.DesiredHeight(TestCase.WidgetHeight);

		// Wrap in offset container to achieve fractional positioning
		TSharedRef<SFractionalOffsetBox> OffsetBox = SNew(SFractionalOffsetBox)
			.OffsetX(TestCase.OffsetX)
			.OffsetY(TestCase.OffsetY)
			[
				TestWidget
			];

		// Create the test window at integral coordinates
		TSharedRef<SWindow> TestWindow = SNew(SWindow)
			.UseOSWindowBorder(bUseOSWindowBorder)
			.ClientSize(FVector2f(FMath::CeilToFloat(ClientWidth), FMath::CeilToFloat(ClientHeight)))
			.ScreenPosition(FVector2f(WindowOriginX, WindowOriginY))
			.AutoCenter(EAutoCenter::None)
			.CreateTitleBar(false)
			.SizingRule(ESizingRule::FixedSize)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.HasCloseButton(false)
			[
				OffsetBox
			];
		TSharedRef<FMockForegroundWindow> MockWindow = MakeShared<FMockForegroundWindow>();
		TestWindow->SetNativeWindow(MockWindow);

		FSlateApplication::Get().ForceRedrawWindow(TestWindow);

		TSharedRef<FMockCursorForLockTest> MockCursor = MakeShared<FMockCursorForLockTest>();
		TSharedPtr<FSlateUser> CursorUser = FSlateApplication::Get().GetCursorUser();
		if (!CursorUser.IsValid())
		{
			Test->AddError(FString::Printf(TEXT("%S %S: Could not get cursor user"), WindowDescription, TestCase.Description));
			FSlateApplication::Get().DestroyWindowImmediately(TestWindow);
			return false;
		}

		// Until we have a way of constructing SWindow from a custom native window AND registering it to Slate Application's window list
		bool bWasFastWidgetPathEnabled = GSlateFastWidgetPath;
		GSlateFastWidgetPath = true;

		// Store to restore later
		TSharedPtr<ICursor> OriginalCursor = CursorUser->GetCursor();

		CursorUser->OverrideCursor(MockCursor);
		CursorUser->LockCursor(TestWidget);

		bool bTestPassed = true;

		if (!MockCursor->bLockWasCalled)
		{
			Test->AddError(FString::Printf(TEXT("%S %S: Cursor Lock() was not called"), WindowDescription, TestCase.Description));
			bTestPassed = false;
		}
		else if (!MockCursor->bHasCapturedBounds)
		{
			Test->AddError(FString::Printf(TEXT("%S %S: Cursor Lock() was called with null bounds"), WindowDescription, TestCase.Description));
			bTestPassed = false;
		}
		else
		{
			const RECT& CapturedRect = MockCursor->CapturedBounds;

			const FDeprecateVector2DResult ClientLocation = OffsetBox->GetCachedGeometry().GetAbsolutePosition();
			const float InputLeft = ClientLocation.X + TestCase.OffsetX;
			const float InputTop = ClientLocation.Y + TestCase.OffsetY;
			const float InputRight = InputLeft + TestCase.WidgetWidth;
			const float InputBottom = InputTop + TestCase.WidgetHeight;
			int32 OffsetExpectedLeft = ClientLocation.X + TestCase.ExpectedLeft;
			int32 OffsetExpectedTop = ClientLocation.Y + TestCase.ExpectedTop;
			int32 OffsetExpectedRight = ClientLocation.X + TestCase.ExpectedRight;
			int32 OffsetExpectedBottom = ClientLocation.Y + TestCase.ExpectedBottom;

			// Verify the captured rect matches expectations
			if (CapturedRect.left != OffsetExpectedLeft)
			{
				Test->AddError(FString::Printf(TEXT("%S %S: Clip rect left mismatch. Expected %i, got %i (input: %.1f)"),
					WindowDescription, TestCase.Description, OffsetExpectedLeft, CapturedRect.left, InputLeft));
				bTestPassed = false;
			}
			if (CapturedRect.top != OffsetExpectedTop)
			{
				Test->AddError(FString::Printf(TEXT("%S %S: Clip rect top mismatch. Expected %i, got %i (input: %.1f)"),
					WindowDescription, TestCase.Description, OffsetExpectedTop, CapturedRect.top, InputTop));
				bTestPassed = false;
			}
			if (CapturedRect.right != OffsetExpectedRight)
			{
				Test->AddError(FString::Printf(TEXT("%S %S: Clip rect right mismatch. Expected %i, got %i (input: %.1f)"),
					WindowDescription, TestCase.Description, OffsetExpectedRight, CapturedRect.right, InputRight));
				bTestPassed = false;
			}
			if (CapturedRect.bottom != OffsetExpectedBottom)
			{
				Test->AddError(FString::Printf(TEXT("%S %S: Clip rect bottom mismatch. Expected %i, got %i (input: %.1f)"),
					WindowDescription, TestCase.Description, OffsetExpectedBottom, CapturedRect.bottom, InputBottom));
				bTestPassed = false;
			}
		}

		// Cleanup
		CursorUser->UnlockCursor();
		CursorUser->OverrideCursor(OriginalCursor);
		GSlateFastWidgetPath = bWasFastWidgetPathEnabled;
		FSlateApplication::Get().DestroyWindowImmediately(TestWindow);

		return bTestPassed;
	}

	/**
	 * Run all cursor lock rounding tests for a specific window configuration.
	 */
	bool RunCursorLockRoundingTestsForWindowConfig(
		FAutomationTestBase* Test,
		bool bUseOSWindowBorder,
		const ANSICHAR* WindowDescription)
	{
		// Test cases enumerated:
		// - All 16 combinations of .4/.6 for left, top, right, bottom
		// - 1 case with fully integral coordinates
		//
		// Expected rounding behavior (to stay inside the widget):
		// - Left/Top (min coords): Round UP (ceiling) - 100.4 -> 101, 100.6 -> 101
		// - Right/Bottom (max coords): Round DOWN (floor/truncate) - 200.4 -> 200, 200.6 -> 200
		//
		// Test case format: { OffsetX, OffsetY, Width, Height, ExpectedLeft, ExpectedTop, ExpectedRight, ExpectedBottom, Description }
		// Window is at (100, 100), so geometry bounds are:
		//   Left = 100 + OffsetX, Top = 100 + OffsetY
		//   Right = Left + Width, Bottom = Top + Height

		const TArray<FCursorLockTestCase> TestCases = {
			// Fully integral case - should remain unchanged
			// Bounds: (100, 100, 200, 200)
			{ 0.0f, 0.0f, 100.0f, 100.0f, 0, 0, 100, 100, "Integral bounds" },

			// Left=.4, Top=.4 (OffsetX=0.4, OffsetY=0.4)
			{ 0.4f, 0.4f, 100.0f, 100.0f, 1, 1, 100, 100, "L.4 T.4 R.4 B.4" },
			{ 0.4f, 0.4f, 100.0f, 100.2f, 1, 1, 100, 100, "L.4 T.4 R.4 B.6" },
			{ 0.4f, 0.4f, 100.2f, 100.0f, 1, 1, 100, 100, "L.4 T.4 R.6 B.4" },
			{ 0.4f, 0.4f, 100.2f, 100.2f, 1, 1, 100, 100, "L.4 T.4 R.6 B.6" },

			// Left=.4, Top=.6 (OffsetX=0.4, OffsetY=0.6)
			{ 0.4f, 0.6f,  99.8f,  99.8f, 1, 1, 100, 100, "L.4 T.6 R.4 B.4" },
			{ 0.4f, 0.6f,  99.8f, 100.0f, 1, 1, 100, 100, "L.4 T.6 R.4 B.6" },
			{ 0.4f, 0.6f, 100.0f,  99.8f, 1, 1, 100, 100, "L.4 T.6 R.6 B.4" },
			{ 0.4f, 0.6f, 100.0f, 100.0f, 1, 1, 100, 100, "L.4 T.6 R.6 B.6" },

			// Left=.6, Top=.4 (OffsetX=0.6, OffsetY=0.4)
			{ 0.6f, 0.4f,  99.8f,  99.8f, 1, 1, 100, 100, "L.6 T.4 R.4 B.4" },
			{ 0.6f, 0.4f,  99.8f, 100.0f, 1, 1, 100, 100, "L.6 T.4 R.4 B.6" },
			{ 0.6f, 0.4f, 100.0f,  99.8f, 1, 1, 100, 100, "L.6 T.4 R.6 B.4" },
			{ 0.6f, 0.4f, 100.0f, 100.0f, 1, 1, 100, 100, "L.6 T.4 R.6 B.6" },

			// Left=.6, Top=.6 (OffsetX=0.6, OffsetY=0.6)
			{ 0.6f, 0.6f,  99.8f,  99.8f, 1, 1, 100, 100, "L.6 T.6 R.4 B.4" },
			{ 0.6f, 0.6f,  99.8f, 100.0f, 1, 1, 100, 100, "L.6 T.6 R.4 B.6" },
			{ 0.6f, 0.6f, 100.0f,  99.8f, 1, 1, 100, 100, "L.6 T.6 R.6 B.4" },
			{ 0.6f, 0.6f, 100.0f, 100.0f, 1, 1, 100, 100, "L.6 T.6 R.6 B.6" },
		};

		bool bAllPassed = true;
		for (const FCursorLockTestCase& TestCase : TestCases)
		{
			if (!TestCursorLockRounding(Test, bUseOSWindowBorder, TestCase, WindowDescription))
			{
				bAllPassed = false;
			}
		}

		return bAllPassed;
	}
	
	constexpr EAutomationTestFlags SlateUserCursorLockTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext |
		EAutomationTestFlags::ProgramContext | EAutomationTestFlags::LowPriority | EAutomationTestFlags::EngineFilter;

}

namespace UE::Slate::Test
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateUserCursorLockBorderlessTest, "Slate.User.CursorLock.Borderless", SlateUserCursorLockTestFlags)

	bool FSlateUserCursorLockBorderlessTest::RunTest(const FString& Parameters)
	{
		return RunCursorLockRoundingTestsForWindowConfig(this, /*bUseOSWindowBorder*/ false, "Borderless");
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateUserCursorLockBorderedTest, "Slate.User.CursorLock.Bordered", SlateUserCursorLockTestFlags)

	bool FSlateUserCursorLockBorderedTest::RunTest(const FString& Parameters)
	{
		return RunCursorLockRoundingTestsForWindowConfig(this, /*bUseOSWindowBorder*/ true, "Bordered");
	}
} // namespace UE::Slate::Test

#endif // WITH_AUTOMATION_WORKER && PLATFORM_DESKTOP
