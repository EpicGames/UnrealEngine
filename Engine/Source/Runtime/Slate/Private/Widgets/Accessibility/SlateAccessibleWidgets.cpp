// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Text/STextBlock.h"

void FSlateAccessibleButton::Activate()
{
	if (Widget.IsValid())
	{
		StaticCastSharedPtr<SButton>(Widget.Pin())->ExecuteOnClick();
	}
}

void FSlateAccessibleCheckBox::Activate()
{
	if (Widget.IsValid())
	{
		StaticCastSharedPtr<SCheckBox>(Widget.Pin())->ToggleCheckedState();
	}
}

bool FSlateAccessibleCheckBox::IsCheckable() const {
	return true;
}

bool FSlateAccessibleCheckBox::GetCheckedState() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SCheckBox>(Widget.Pin())->IsChecked();
	}
	return false;
}

FString FSlateAccessibleCheckBox::GetValue() const
{
	bool bChecked = GetCheckedState();
	return FString::FromInt(bChecked ? 1 : 0);
}

FVariant FSlateAccessibleCheckBox::GetValueAsVariant() const
{
	bool bChecked = GetCheckedState();
	return FVariant(bChecked);
}

const FString& FSlateAccessibleEditableText::GetText() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SEditableText>(Widget.Pin())->GetText().ToString();
	}
	static FString EmptyString;
	return EmptyString;
}

bool FSlateAccessibleEditableText::IsPassword() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SEditableText>(Widget.Pin())->IsTextPassword();
	}
	return true;
}

bool FSlateAccessibleEditableText::IsReadOnly() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SEditableText>(Widget.Pin())->IsTextReadOnly();
	}
	return true;
}

FString FSlateAccessibleEditableText::GetValue() const
{
	return GetText();
}

FVariant FSlateAccessibleEditableText::GetValueAsVariant() const
{
	return FVariant(GetText());
}

void FSlateAccessibleEditableText::SetValue(const FString& Value)
{
	if (Widget.IsValid())
	{
		StaticCastSharedPtr<SEditableText>(Widget.Pin())->SetText(FText::FromString(Value));
	}
}

const FString& FSlateAccessibleEditableTextBox::GetText() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SEditableTextBox>(Widget.Pin())->GetText().ToString();
	}
	static FString EmptyString;
	return EmptyString;
}

bool FSlateAccessibleEditableTextBox::IsPassword() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SEditableTextBox>(Widget.Pin())->IsPassword();
	}
	return true;
}

bool FSlateAccessibleEditableTextBox::IsReadOnly() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SEditableTextBox>(Widget.Pin())->IsReadOnly();
	}
	return true;
}

FString FSlateAccessibleEditableTextBox::GetValue() const
{
	return GetText();
}

FVariant FSlateAccessibleEditableTextBox::GetValueAsVariant() const
{
	return FVariant(GetText());
}

void FSlateAccessibleEditableTextBox::SetValue(const FString& Value)
{
	if (Widget.IsValid())
	{
		StaticCastSharedPtr<SEditableTextBox>(Widget.Pin())->SetText(FText::FromString(Value));
	}
}

bool FSlateAccessibleSlider::IsReadOnly() const
{
	if (Widget.IsValid())
	{
		return !Widget.Pin()->IsEnabled() || !Widget.Pin()->IsInteractable();
	}
	return true;
}

float FSlateAccessibleSlider::GetStepSize() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SSlider>(Widget.Pin())->GetStepSize();
	}
	return 0.0f;
}

float FSlateAccessibleSlider::GetMaximum() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SSlider>(Widget.Pin())->GetMaxValue();
	}
	return 1.0f;
}

float FSlateAccessibleSlider::GetMinimum() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SSlider>(Widget.Pin())->GetMinValue();
	}
	return 0.0f;
}

FString FSlateAccessibleSlider::GetValue() const
{
	if (Widget.IsValid())
	{
		return FString::SanitizeFloat(StaticCastSharedPtr<SSlider>(Widget.Pin())->GetValue());
	}
	return FString::SanitizeFloat(0.0f);
}

FVariant FSlateAccessibleSlider::GetValueAsVariant() const
{
	if (Widget.IsValid())
	{
		return FVariant(StaticCastSharedPtr<SSlider>(Widget.Pin())->GetValue());
	}
	return FVariant();
}

void FSlateAccessibleSlider::SetValue(const FString& Value)
{
	if (Widget.IsValid())
	{
		StaticCastSharedPtr<SSlider>(Widget.Pin())->SetValue(FCString::Atof(*Value));
	}
}

const FString& FSlateAccessibleTextBlock::GetText() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<STextBlock>(Widget.Pin())->GetText().ToString();
	}
	else
	{
		static const FString EmptyString;
		return EmptyString;
	}
}

#endif
