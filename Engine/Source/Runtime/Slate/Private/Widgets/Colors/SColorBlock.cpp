// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SColorBlock.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"


SLATE_IMPLEMENT_WIDGET(SColorBlock)
void SColorBlock::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Color, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, AlphaBackgroundBrush, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, GradientCornerRadius, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ColorBlockSize, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, AlphaDisplayMode, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ColorIsHSV, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShowBackgroundForAlpha, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bUseSRGB, EInvalidateWidgetReason::Paint);
}

SColorBlock::SColorBlock()
	: Color(*this, FLinearColor::White)
	, AlphaBackgroundBrush(*this, nullptr)
	, GradientCornerRadius(*this, FVector4(0.0f))
	, ColorBlockSize(*this, FVector2D(16, 16))
	, AlphaDisplayMode(*this, EColorBlockAlphaDisplayMode::Combined)
	, ColorIsHSV(*this, false)
	, ShowBackgroundForAlpha(*this, false)
	, bUseSRGB(*this, true)
{
}

SColorBlock::~SColorBlock() = default;

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SColorBlock::Construct(const FArguments& InArgs)
{
	Color.Assign(*this, InArgs._Color);
	AlphaBackgroundBrush.Assign(*this, InArgs._AlphaBackgroundBrush);
	GradientCornerRadius.Assign(*this, InArgs._CornerRadius, FVector4(0.0f));
	ColorBlockSize.Assign(*this, InArgs._Size);
	MouseButtonDownHandler = InArgs._OnMouseButtonDown;
	AlphaDisplayMode.Assign(*this, InArgs._AlphaDisplayMode);
	ColorIsHSV.Assign(*this, InArgs._ColorIsHSV);
	ShowBackgroundForAlpha.Assign(*this, InArgs._ShowBackgroundForAlpha);
	bUseSRGB.Assign(*this, InArgs._UseSRGB);
}

int32 SColorBlock::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	EColorBlockAlphaDisplayMode DisplayMode = AlphaDisplayMode.Get();
	bool bIgnoreAlpha = DisplayMode == EColorBlockAlphaDisplayMode::Ignore;
	FLinearColor InColor = Color.Get();
	if (ColorIsHSV.Get())
	{
		InColor = InColor.HSVToLinearRGB();
	}

	if (ShowBackgroundForAlpha.Get() && InColor.A < 1.0f && !bIgnoreAlpha)
	{
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), AlphaBackgroundBrush.Get(), DrawEffects);
	}

	TArray<FSlateGradientStop> GradientStops;

	switch (DisplayMode)
	{
	default:
	case EColorBlockAlphaDisplayMode::Combined:
		MakeSection(GradientStops, FVector2D::ZeroVector, AllottedGeometry.GetLocalSize(), InColor, InWidgetStyle, false);
		break;
	case EColorBlockAlphaDisplayMode::Separate:
		MakeSection(GradientStops, FVector2D::ZeroVector, AllottedGeometry.GetLocalSize() * 0.5f, InColor, InWidgetStyle, false);
		MakeSection(GradientStops, AllottedGeometry.GetLocalSize() * 0.5f, AllottedGeometry.GetLocalSize(), InColor, InWidgetStyle, true);
		break;
	case EColorBlockAlphaDisplayMode::SeparateReverse:
		MakeSection(GradientStops, FVector2D::ZeroVector, AllottedGeometry.GetLocalSize() * 0.5f, InColor, InWidgetStyle, true);
		MakeSection(GradientStops, AllottedGeometry.GetLocalSize() * 0.5f, AllottedGeometry.GetLocalSize(), InColor, InWidgetStyle, false);
		break;
	case EColorBlockAlphaDisplayMode::Ignore:
		MakeSection(GradientStops, FVector2D::ZeroVector, AllottedGeometry.GetLocalSize(), InColor, InWidgetStyle, true);
		break;
	}

	FSlateDrawElement::MakeGradient(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(),
		MoveTemp(GradientStops),
		(AllottedGeometry.GetLocalSize().X >= AllottedGeometry.GetLocalSize().Y) ? Orient_Vertical : Orient_Horizontal,
		DrawEffects,
		(FVector4f)GradientCornerRadius.Get()
	);


	return LayerId + 1;
}

FReply SColorBlock::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseButtonDownHandler.IsBound())
	{
		// If a handler is assigned, call it.
		return MouseButtonDownHandler.Execute(MyGeometry, MouseEvent);
	}
	else
	{
		// otherwise the event is unhandled.
		return FReply::Unhandled();
	}
}

FVector2D SColorBlock::ComputeDesiredSize(float) const
{
	return ColorBlockSize.Get();
}

void SColorBlock::MakeSection(TArray<FSlateGradientStop>& OutGradientStops, FVector2D StartPt, FVector2D EndPt, FLinearColor InColor, const FWidgetStyle& InWidgetStyle, bool bIgnoreAlpha) const
{
	// determine if it is HDR
	const float MaxRGB = FMath::Max3(InColor.R, InColor.G, InColor.B);
	if (MaxRGB > 1.f)
	{
		const float Alpha = bIgnoreAlpha ? 1.0f : InColor.A;
		FLinearColor NormalizedLinearColor = InColor / MaxRGB;
		NormalizedLinearColor.A = Alpha;
		const FLinearColor DrawNormalizedColor = InWidgetStyle.GetColorAndOpacityTint() * NormalizedLinearColor.ToFColor(bUseSRGB.Get());

		FLinearColor ClampedLinearColor = InColor;
		ClampedLinearColor.A = Alpha * MaxRGB;
		const FLinearColor DrawClampedColor = InWidgetStyle.GetColorAndOpacityTint() * ClampedLinearColor.ToFColor(bUseSRGB.Get());

		OutGradientStops.Add(FSlateGradientStop(StartPt, DrawNormalizedColor));
		OutGradientStops.Add(FSlateGradientStop((StartPt + EndPt) * 0.5f, DrawClampedColor));
		OutGradientStops.Add(FSlateGradientStop(EndPt, DrawNormalizedColor));
	}
	else
	{
		FColor DrawColor = InColor.ToFColor(bUseSRGB.Get());
		if (bIgnoreAlpha)
		{
			DrawColor.A = 255;
		}
		OutGradientStops.Add(FSlateGradientStop(StartPt, InWidgetStyle.GetColorAndOpacityTint() * DrawColor));
		OutGradientStops.Add(FSlateGradientStop(EndPt, InWidgetStyle.GetColorAndOpacityTint() * DrawColor));
	}
}
