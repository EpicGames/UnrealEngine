// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/Colors/SColorGradingWheel.h"
#include "Styling/CoreStyle.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
SLATE_IMPLEMENT_WIDGET(SColorGradingWheel)
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SColorGradingWheel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SelectedColor", SelectedColorAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "DesiredWheelSize", DesiredWheelSizeAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ExponentDisplacement", ExponentDisplacementAttribute, EInvalidateWidgetReason::Paint);
}

SColorGradingWheel::SColorGradingWheel()
	: SelectedColorAttribute(*this, FLinearColor(ForceInit))
	, DesiredWheelSizeAttribute(*this)
	, ExponentDisplacementAttribute(*this)
	, Union_IsAttributeSet(0)
{
}

SColorGradingWheel::~SColorGradingWheel() = default;

/* SColorGradingWheel methods
 *****************************************************************************/

void SColorGradingWheel::Construct(const FArguments& InArgs)
{
	Image = FCoreStyle::Get().GetBrush("ColorGradingWheel.HueValueCircle");
	SelectorImage = FCoreStyle::Get().GetBrush("ColorGradingWheel.Selector");
	SetSelectedColorAttribute(InArgs._SelectedColor);
	SetDesiredWheelSizeAttribute(InArgs._DesiredWheelSize);
	SetExponentDisplacementAttribute(InArgs._ExponentDisplacement);
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	OnValueChanged = InArgs._OnValueChanged;
}

void SColorGradingWheel::SetSelectedColorAttribute(TAttribute<FLinearColor> InSelectedColor)
{
	SelectedColorAttribute.Assign(*this, MoveTemp(InSelectedColor));
}

void SColorGradingWheel::SetDesiredWheelSizeAttribute(TAttribute<int32> InDesiredWheelSize)
{
	const bool bDesiredWheelSizeSetChanged = (bIsAttributeDesiredWheelSizeSet != InDesiredWheelSize.IsSet());
	bIsAttributeDesiredWheelSizeSet = InDesiredWheelSize.IsSet();
	const bool bAttributeWasAssgined = DesiredWheelSizeAttribute.Assign(*this, MoveTemp(InDesiredWheelSize));

	// If the assign didn't invalidate the widget but the attribute set changed, then invalidate the widget.
	if (bDesiredWheelSizeSetChanged && !bAttributeWasAssgined)
	{
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SColorGradingWheel::SetExponentDisplacementAttribute(TAttribute<float> InExponentDisplacement)
{
	const bool bExponentDisplacementSetChanged = (bIsAttributeExponentDisplacementSet != InExponentDisplacement.IsSet());
	bIsAttributeExponentDisplacementSet = InExponentDisplacement.IsSet();
	const bool bAttributeWasAssgined = ExponentDisplacementAttribute.Assign(*this, MoveTemp(InExponentDisplacement), 1.f);

	// If the assign didn't invalidate the widget but the attribute set changed, then invalidate the widget.
	if (bExponentDisplacementSetChanged && !bAttributeWasAssgined)
	{
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}


/* SWidget overrides
 *****************************************************************************/

FVector2D SColorGradingWheel::ComputeDesiredSize(float) const
{
	if (bIsAttributeDesiredWheelSizeSet)
	{
		int32 CachedDesiredWheelSize = DesiredWheelSizeAttribute.Get();
		return FVector2D(CachedDesiredWheelSize, CachedDesiredWheelSize);
	}
	return FVector2D(Image->ImageSize);
}


FReply SColorGradingWheel::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Handled();
}


FReply SColorGradingWheel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (OnMouseCaptureBegin.IsBound())
		{
			SelectedColorAttribute.UpdateNow(*this);
			OnMouseCaptureBegin.Execute(SelectedColorAttribute.Get());
		}

		if (!ProcessMouseAction(MyGeometry, MouseEvent, false))
		{
			if (OnMouseCaptureEnd.IsBound())
			{
				SelectedColorAttribute.UpdateNow(*this);
				OnMouseCaptureEnd.Execute(SelectedColorAttribute.Get());
			}
			return FReply::Unhandled();
		}

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}


FReply SColorGradingWheel::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HasMouseCapture())
	{
		if (OnMouseCaptureEnd.IsBound())
		{
			SelectedColorAttribute.UpdateNow(*this);
			OnMouseCaptureEnd.Execute(SelectedColorAttribute.Get());
		}

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}


FReply SColorGradingWheel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		return FReply::Unhandled();
	}

	ProcessMouseAction(MyGeometry, MouseEvent, true);

	return FReply::Handled();
}


int32 SColorGradingWheel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FVector2f& SelectorSize = SelectorImage->ImageSize;
	FVector2f CircleSize = AllottedGeometry.GetLocalSize() - SelectorSize;
	FVector2f AllottedGeometrySize = AllottedGeometry.GetLocalSize();
	if (bIsAttributeDesiredWheelSizeSet)
	{
		int32 CachedDesiredWheelSize = DesiredWheelSizeAttribute.Get();
		CircleSize.X = (float)CachedDesiredWheelSize - SelectorSize.X;
		CircleSize.Y = (float)CachedDesiredWheelSize - SelectorSize.Y;
		AllottedGeometrySize.X = CachedDesiredWheelSize;
		AllottedGeometrySize.Y = CachedDesiredWheelSize;
	}
	
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(CircleSize, FSlateLayoutTransform(0.5f * SelectorSize)),
		Image,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * Image->GetTint(InWidgetStyle)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(SelectorSize, FSlateLayoutTransform(0.5f * (AllottedGeometrySize + CalcRelativePositionFromCenter() * CircleSize - SelectorSize))),
		SelectorImage,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * SelectorImage->GetTint(InWidgetStyle)
	);

	return LayerId + 1;
}


/* SColorGradingWheel implementation
 *****************************************************************************/

UE::Slate::FDeprecateVector2DResult SColorGradingWheel::CalcRelativePositionFromCenter() const
{
	float Hue = SelectedColorAttribute.Get().R;
	float Saturation = SelectedColorAttribute.Get().G;
	if (bIsAttributeExponentDisplacementSet && ExponentDisplacementAttribute.Get() != 1.0f && !FMath::IsNearlyEqual(ExponentDisplacementAttribute.Get(), 0.0f, 0.00001f))
	{
		//Use log curve to set the distance G value
		Saturation = FMath::Pow(Saturation, 1.0f / ExponentDisplacementAttribute.Get());
	}
	float Angle = Hue / 180.0f * PI;
	float Radius = Saturation;

	return FVector2f(FMath::Cos(Angle), FMath::Sin(Angle)) * Radius;
}


bool SColorGradingWheel::ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bProcessWhenOutsideColorWheel)
{
	FVector2f GeometrySize = MyGeometry.GetLocalSize();
	if (bIsAttributeDesiredWheelSizeSet)
	{
		DesiredWheelSizeAttribute.UpdateNow(*this);
		int32 CachedDesiredWheelSize = DesiredWheelSizeAttribute.Get();
		GeometrySize.X = CachedDesiredWheelSize;
		GeometrySize.Y = CachedDesiredWheelSize;
	}

	const FVector2f LocalMouseCoordinate = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const FVector2f RelativePositionFromCenter = (2.0f * LocalMouseCoordinate - GeometrySize) / (GeometrySize - SelectorImage->ImageSize);
	const float RelativeRadius = RelativePositionFromCenter.Size();

	if (RelativeRadius <= 1.0f || bProcessWhenOutsideColorWheel)
	{
		float Angle = FMath::Atan2(RelativePositionFromCenter.Y, RelativePositionFromCenter.X);

		if (Angle < 0.0f)
		{
			Angle += 2.0f * PI;
		}

		SelectedColorAttribute.UpdateNow(*this);
		FLinearColor NewColor = SelectedColorAttribute.Get();
		{
			NewColor.R = Angle * 180.0f * INV_PI;
			float LinearRadius = FMath::Min(RelativeRadius, 1.0f);
			if (bIsAttributeExponentDisplacementSet && ExponentDisplacementAttribute.Get() != 1.0f)
			{
				//Use log curve to set the distance G value
				float LogDistance = FMath::Pow(LinearRadius, ExponentDisplacementAttribute.Get());
				NewColor.G = LogDistance;
			}
			else
			{
				NewColor.G = LinearRadius;
			}
		}

		OnValueChanged.ExecuteIfBound(NewColor);
	}

	return (RelativeRadius <= 1.0f);
}
