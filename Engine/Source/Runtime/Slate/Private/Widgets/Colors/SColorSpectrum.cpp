// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SColorSpectrum.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Application/SlateApplicationBase.h"


SColorSpectrum::SColorSpectrum()
	: SelectedColor(*this, FLinearColor(ForceInit))
{
}

SColorSpectrum::~SColorSpectrum() = default;

/* SColorSpectrum methods
 *****************************************************************************/

void SColorSpectrum::Construct( const FArguments& InArgs )
{
	Image = FCoreStyle::Get().GetBrush("ColorSpectrum.Spectrum");
	SelectorImage = FCoreStyle::Get().GetBrush("ColorSpectrum.Selector");
	SelectedColor.Assign(*this, InArgs._SelectedColor);

	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	OnValueChanged = InArgs._OnValueChanged;
	bPreventThrottling = InArgs._PreventThrottling;
	bProcessMouseMoveImmediate = InArgs._ProcessMouseMoveImmediate;

	CtrlMultiplier = InArgs._CtrlMultiplier;
}


/* SWidget overrides
 *****************************************************************************/

FVector2D SColorSpectrum::ComputeDesiredSize( float ) const
{
	return FVector2D(Image->ImageSize);
}


FReply SColorSpectrum::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return FReply::Handled();
}


FReply SColorSpectrum::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bHighPrecision = MouseEvent.IsControlDown();
		ProcessMouseAction(MyGeometry, MouseEvent);
		OnMouseCaptureBegin.ExecuteIfBound();

		FReply ReturnReply = FReply::Handled().CaptureMouse(SharedThis(this)).SetUserFocus(SharedThis(this), EFocusCause::Mouse);

		if (bHighPrecision)
		{
			ReturnReply = ReturnReply.UseHighPrecisionMouseMovement(SharedThis(this));
		}

		if (bPreventThrottling)
		{
			ReturnReply.PreventThrottling();
		}

		return ReturnReply;
	}

	return FReply::Unhandled();
}


FReply SColorSpectrum::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && HasMouseCapture())
	{
		bDragging = false;

		OnMouseCaptureEnd.ExecuteIfBound();

		// Before showing the mouse position again, reset its position to the final location of the selector on the color spectrum
		const FVector2D FinalMousePosition = CalcRelativeSelectedPosition() * MyGeometry.Size;

		return FReply::Handled().ReleaseMouseCapture().SetMousePos(MyGeometry.LocalToAbsolute(FinalMousePosition).IntPoint());
	}

	return FReply::Unhandled();
}


FReply SColorSpectrum::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!HasMouseCapture())
	{
		return FReply::Unhandled();
	}

	if (!bDragging)
	{
		bDragging = true;
		LastSpectrumPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	}

	if (bProcessMouseMoveImmediate)
	{
		ProcessMouseAction(MyGeometry, MouseEvent);
	}
	else
	{
		LastGeometry = MyGeometry;
		LastMouseEvent = MouseEvent;
	}

	bHighPrecision = MouseEvent.IsControlDown();
	if (bHighPrecision && !FSlateApplicationBase::Get().GetPlatformApplication()->IsUsingHighPrecisionMouseMode())
	{
		return FReply::Handled().UseHighPrecisionMouseMovement(SharedThis(this));
	}

	return FReply::Handled();
}

void SColorSpectrum::OnFinishedPointerInput()
{
	if (bDragging && !LastMouseEvent.GetCursorDelta().IsNearlyZero() && !bProcessMouseMoveImmediate)
	{
		ProcessMouseAction(LastGeometry, LastMouseEvent);
		LastGeometry = FGeometry();
		LastMouseEvent = FPointerEvent();
	}
}

int32 SColorSpectrum::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
	// draw gradient
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		Image,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * Image->GetTint(InWidgetStyle));

	// ignore colors that can't be represented in spectrum
	const FLinearColor& Color = SelectedColor.Get();

	if ((Color.G < 1.0f) && (Color.B < 1.0f))
	{
		return LayerId;
	}

	// draw cursor
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(SelectorImage->ImageSize, FSlateLayoutTransform(CalcRelativeSelectedPosition() * AllottedGeometry.Size - SelectorImage->ImageSize * 0.5f)),
		SelectorImage,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * SelectorImage->GetTint(InWidgetStyle));

	return LayerId + 1;
}


/* SColorSpectrum implementation
 *****************************************************************************/

FVector2D SColorSpectrum::CalcRelativeSelectedPosition( ) const
{
	const FLinearColor& Color = SelectedColor.Get();

	if (Color.G == 1.0f)
	{
		return FVector2D(Color.R / 360.0f, 1.0f - 0.5f * Color.B);
	}

	return FVector2D(Color.R / 360.0f, 0.5f * Color.G);
}


void SColorSpectrum::ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2f LocalMouseCoordinate;
	if (bDragging)
	{
		constexpr float SpectrumSensitivity = 0.4f;
		FVector2f Delta = MouseEvent.GetCursorDelta() * SpectrumSensitivity;

		if (bHighPrecision)
		{
			Delta *= CtrlMultiplier.Get();
		}

		LocalMouseCoordinate = LastSpectrumPosition + Delta;

		// Clamp mouse position to spectrum geometry
		const FVector2f SpectrumSize = MyGeometry.GetLocalSize();
		LocalMouseCoordinate.X = FMath::Clamp(LocalMouseCoordinate.X, 0.0f, SpectrumSize.X);
		LocalMouseCoordinate.Y = FMath::Clamp(LocalMouseCoordinate.Y, 0.0f, SpectrumSize.Y);

		LastSpectrumPosition = LocalMouseCoordinate;
	}
	else
	{
		LocalMouseCoordinate = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	}

	FVector2D NormalizedMousePosition = LocalMouseCoordinate / MyGeometry.GetLocalSize();
	NormalizedMousePosition = NormalizedMousePosition.ClampAxes(0.0f, 1.0f);

	SelectedColor.UpdateNow(*this);
	FLinearColor NewColor = SelectedColor.Get();
	NewColor.R = 360.0f * NormalizedMousePosition.X;

	if (NormalizedMousePosition.Y > 0.5f)
	{
		NewColor.G = 1.0f;
		NewColor.B = 2.0f * (1.0f - NormalizedMousePosition.Y);
	}
	else
	{
		NewColor.G = 2.0f * NormalizedMousePosition.Y;
		NewColor.B = 1.0f;
	}

	OnValueChanged.ExecuteIfBound(NewColor);
}

FCursorReply SColorSpectrum::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return bDragging ?
		FCursorReply::Cursor(EMouseCursor::None) :
		FCursorReply::Cursor(EMouseCursor::Default);
}
