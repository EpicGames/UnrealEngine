// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SColorWheel.h"
#include "Rendering/DrawElements.h"
#include "Application/SlateApplicationBase.h"

SColorWheel::SColorWheel()
	: SelectedColor(*this, FLinearColor(ForceInit))
	, bShouldDrawSelector(true)
{}

SColorWheel::~SColorWheel() = default;

/* SColorWheel methods
 *****************************************************************************/

void SColorWheel::Construct(const FArguments& InArgs)
{
	Image = InArgs._ColorWheelBrush.Get();
	SelectorImage = FCoreStyle::Get().GetBrush("ColorWheel.Selector");
	SelectedColor.Assign(*this, InArgs._SelectedColor);

	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	OnValueChanged = InArgs._OnValueChanged;
	bPreventThrottling = InArgs._PreventThrottling;
	bProcessMouseMoveImmediate = InArgs._ProcessMouseMoveImmediate;

	CtrlMultiplier = InArgs._CtrlMultiplier;
}


/* SWidget overrides
 *****************************************************************************/

FVector2D SColorWheel::ComputeDesiredSize(float) const
{
	return FVector2D(Image->ImageSize + SelectorImage->ImageSize);
}


FReply SColorWheel::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Handled();
}


FReply SColorWheel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		OnMouseCaptureBegin.ExecuteIfBound();
		bHighPrecision = MouseEvent.IsControlDown();

		if (!ProcessMouseAction(MyGeometry, MouseEvent, false))
		{
			OnMouseCaptureEnd.ExecuteIfBound();
			return FReply::Unhandled();
		}

		FReply ReturnReply = FReply::Handled().CaptureMouse(SharedThis(this)).SetUserFocus(SharedThis(this), EFocusCause::Mouse);

		if (bHighPrecision)
		{
			ReturnReply = ReturnReply.UseHighPrecisionMouseMovement(SharedThis(this));
		}

		if (bPreventThrottling)
		{
			ReturnReply.PreventThrottling();
		}

		return ReturnReply;
	}

	return FReply::Unhandled();
}


FReply SColorWheel::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HasMouseCapture())
	{
		bDragging = false;
		bHighPrecision = false;

		OnMouseCaptureEnd.ExecuteIfBound();

		// Before showing the mouse position again, reset its position to the final location of the selector on the color wheel
		const FVector2f CircleSize = MyGeometry.GetLocalSize();
		const FVector2f FinalMousePosition = 0.5f * (MyGeometry.GetLocalSize() + CalcRelativePositionFromCenter() * CircleSize);

		return FReply::Handled().ReleaseMouseCapture().SetMousePos(MyGeometry.LocalToAbsolute(FinalMousePosition).IntPoint());
	}

	return FReply::Unhandled();
}


FReply SColorWheel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		return FReply::Unhandled();
	}

	if (!bDragging)
	{
		bDragging = true;
		LastWheelPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	}

	if (bProcessMouseMoveImmediate)
	{
		ProcessMouseAction(MyGeometry, MouseEvent, true);
	}
	else
	{
		LastGeometry = MyGeometry;
		LastMouseEvent = MouseEvent;
	}

	bHighPrecision = MouseEvent.IsControlDown();
	if (bHighPrecision && !FSlateApplicationBase::Get().GetPlatformApplication()->IsUsingHighPrecisionMouseMode())
	{
		return FReply::Handled().UseHighPrecisionMouseMovement(SharedThis(this));
	}

	return FReply::Handled();
}

void SColorWheel::OnFinishedPointerInput()
{
	if (bDragging && !LastMouseEvent.GetCursorDelta().IsNearlyZero() && !bProcessMouseMoveImmediate)
	{
		ProcessMouseAction(LastGeometry, LastMouseEvent, true);
		LastGeometry = FGeometry();
		LastMouseEvent = FPointerEvent();
	}
}

int32 SColorWheel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FVector2f SelectorSize = SelectorImage->ImageSize;
	const FVector2f CircleCenter = AllottedGeometry.GetLocalSize() * 0.5f;
	
	// Draw Color Wheel Image
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		Image,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * Image->GetTint(InWidgetStyle)
	);

	if (bShouldDrawSelector)
	{
		// Draw Selector Image
		const FVector2f SelectorImageLocation = CircleCenter + (CalcRelativePositionFromCenter() * CircleCenter) - (0.5f * SelectorSize);
		
		LayerId++;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(SelectorSize, FSlateLayoutTransform(SelectorImageLocation)),
			SelectorImage,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * SelectorImage->GetTint(InWidgetStyle)
		);
	}
	

	return LayerId;
}


/* SColorWheel implementation
 *****************************************************************************/

UE::Slate::FDeprecateVector2DResult SColorWheel::CalcRelativePositionFromCenter() const
{
	float Hue = SelectedColor.Get().R;
	float Saturation = SelectedColor.Get().G;
	float Angle = Hue / 180.0f * PI;
	float Radius = FMath::Clamp(Saturation, 0.0f, 1.0f);

	return FVector2f(FMath::Cos(Angle), FMath::Sin(Angle)) * Radius;
}


bool SColorWheel::ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bProcessWhenOutsideColorWheel)
{
	FVector2f LocalMouseCoordinate;

	if (bDragging)
	{
		constexpr float WheelSensitivity = 0.35f;
		FVector2f Delta = MouseEvent.GetCursorDelta() * WheelSensitivity;

		if (bHighPrecision)
		{
			Delta *= CtrlMultiplier.Get();
		}

		LocalMouseCoordinate = LastWheelPosition + Delta;

		// Clamp mouse position to the circle geometry
		const float CircleRadius = MyGeometry.GetLocalSize().X / 2.0f;

		const FVector2f CirclePosition = LocalMouseCoordinate - FVector2f(CircleRadius);
		const float DistanceFromCenter = FMath::Sqrt((CirclePosition.X * CirclePosition.X) + (CirclePosition.Y * CirclePosition.Y));

		if (DistanceFromCenter > CircleRadius)
		{
			const float Angle = FMath::Atan2(CirclePosition.Y, CirclePosition.X);
			LocalMouseCoordinate = FVector2f(FMath::Cos(Angle), FMath::Sin(Angle)) * CircleRadius + FVector2f(CircleRadius);
		}

		LastWheelPosition = LocalMouseCoordinate;
	}
	else
	{
		LocalMouseCoordinate = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	}
	
	const FVector2f RelativePositionFromCenter = (2.0f * LocalMouseCoordinate - MyGeometry.GetLocalSize()) / (MyGeometry.GetLocalSize());
	const float RelativeRadius = RelativePositionFromCenter.Size();

	if (RelativeRadius <= 1.0f || bProcessWhenOutsideColorWheel)
	{
		float Angle = FMath::Atan2(RelativePositionFromCenter.Y, RelativePositionFromCenter.X);

		if (Angle < 0.0f)
		{
			Angle += 2.0f * PI;
		}

		SelectedColor.UpdateNow(*this);
		FLinearColor NewColor = SelectedColor.Get();
		{
			NewColor.R = Angle * 180.0f * INV_PI;
			NewColor.G = FMath::Min(RelativeRadius, 1.0f);
		}

		OnValueChanged.ExecuteIfBound(NewColor);
	}

	return (RelativeRadius <= 1.0f);
}

FCursorReply SColorWheel::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return bDragging ?
		FCursorReply::Cursor(EMouseCursor::None) :
		FCursorReply::Cursor(EMouseCursor::Default);
}
