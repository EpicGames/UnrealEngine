// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SComplexGradient.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Application/SlateWindowHelper.h"


SComplexGradient::SComplexGradient()
	: GradientColors(*this)
	, DesiredSizeOverride(*this)
{
	SetCanTick(false);
}

SComplexGradient::~SComplexGradient() = default;

/* SComplexGradient interface
 *****************************************************************************/

void SComplexGradient::Construct( const FArguments& InArgs )
{
	DesiredSizeOverride.Assign(*this, InArgs._DesiredSizeOverride);
	GradientColors.Assign(*this, InArgs._GradientColors);
	bHasAlphaBackground = InArgs._HasAlphaBackground;
	Orientation = InArgs._Orientation;
}


/* SCompoundWidget overrides
 *****************************************************************************/

int32 SComplexGradient::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	ESlateDrawEffect DrawEffects = (bParentEnabled && IsEnabled()) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	if (bHasAlphaBackground)
	{
		const FSlateBrush* StyleInfo = FCoreStyle::Get().GetBrush("ColorPicker.AlphaBackground");

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			StyleInfo,
			DrawEffects
		);
	}

	const TArray<FLinearColor>& Colors = GradientColors.Get();
	int32 NumColors = Colors.Num();

	if (NumColors > 1)
	{
		TArray<FSlateGradientStop> GradientStops;

		for (int32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
		{
			GradientStops.Add(FSlateGradientStop(AllottedGeometry.GetLocalSize() * (float(ColorIndex) / (NumColors - 1)), Colors[ColorIndex]));
		}

		FSlateDrawElement::MakeGradient(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(),
			GradientStops,
			Orientation,
			DrawEffects
		);
	}

	return LayerId + 1;
}

FVector2D SComplexGradient::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const TOptional<FVector2D>& CurrentSizeOverride = DesiredSizeOverride.Get();
	if (CurrentSizeOverride.IsSet())
	{
		return CurrentSizeOverride.GetValue();
	}
	
	return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
}
