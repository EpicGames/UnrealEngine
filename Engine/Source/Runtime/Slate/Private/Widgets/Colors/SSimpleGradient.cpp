// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SSimpleGradient.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Application/SlateWindowHelper.h"

/* SSimpleGradient implementation
 *****************************************************************************/
SSimpleGradient::SSimpleGradient()
	: StartColor(*this, FLinearColor(0.0f, 0.0f, 0.0f))
	, EndColor(*this, FLinearColor(1.0f, 1.0f, 1.0f))
{
	SetCanTick(false);
}

/* SSimpleGradient interface
 *****************************************************************************/

void SSimpleGradient::Construct( const FArguments& InArgs )
{
	StartColor.Assign(*this, InArgs._StartColor);
	EndColor.Assign(*this, InArgs._EndColor);
	bHasAlphaBackground = InArgs._HasAlphaBackground.Get();
	Orientation = InArgs._Orientation.Get();
}


/* SCompoundWidget overrides
 *****************************************************************************/

int32 SSimpleGradient::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const ESlateDrawEffect DrawEffects = this->ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	if (bHasAlphaBackground)
	{
		const FSlateBrush* StyleInfo = FCoreStyle::Get().GetBrush("ColorPicker.AlphaBackground");

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			StyleInfo,
			DrawEffects
		);
	}

	TArray<FSlateGradientStop> GradientStops;

	GradientStops.Add(FSlateGradientStop(FVector2D::ZeroVector, StartColor.Get()));
	GradientStops.Add(FSlateGradientStop(AllottedGeometry.GetLocalSize(), EndColor.Get()));

	FSlateDrawElement::MakeGradient(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(),
		GradientStops,
		Orientation,
		DrawEffects | ESlateDrawEffect::NoGamma
	);

	return LayerId + 1;
}
