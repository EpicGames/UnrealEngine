// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Docking/SDockingTabWell.h"
#include "Framework/Docking/TabCommands.h"
#include "Framework/Docking/SDockingArea.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/Layout/SBox.h"
#include "Misc/NamePermissionList.h"

namespace SDockTabDefs
{
	/** Tab flash rate. Flashes per second */
	static const float TabFlashFrequency = 2.0f;

	/** Tab flash duration. Seconds*/
	static const float TabFlashDuration = 1.0f;

	/** The amount of time to pass before we switch tabs due to drag event */
	static const float DragTimerActivate = 0.75f;

	/** Overrides the tab padding if color overlays are enabled */
	static const float TabVerticalPaddingScaleOverride = 0.85f;
}


static float TotalDraggedDistance = 0;


FReply SDockTab::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( !HasMouseCapture() )
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			TotalDraggedDistance = 0;
			ActivateInParent(ETabActivationCause::UserClickedOnTab);

			return FReply::Handled().DetectDrag( SharedThis(this), EKeys::LeftMouseButton );
		}
		else if ( MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton )
		{
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
		else if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			// Show context menu for this tab otherwise let tab well handle it
			if (const TSharedPtr<SDockingTabStack> DockingTabStack = GetParentDockTabStack())
			{
				const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
				DockingTabStack->ShowContextMenu(SharedThis(this), WidgetPath);
				return FReply::Handled();
			}

			// let the tab well bring up the context menu or whatever it wants to do with the right click.
			return FReply::Unhandled();
		}
	}

	return FReply::Unhandled();
}

FReply SDockTab::OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton )
	{
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SDockTab::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent  )
{
	if (IsTabLocked())
	{
		return FReply::Unhandled();
	}

	// Need to remember where within a tab we grabbed
	const FVector2D TabGrabOffset = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
	const FVector2D TabSize = MyGeometry.GetLocalSize();
	const FVector2D TabGrabOffsetFraction = FVector2D(
		FMath::Clamp(TabGrabOffset.X / TabSize.X, 0.0f, 1.0f),
		FMath::Clamp(TabGrabOffset.Y / TabSize.Y, 0.0f, 1.0f) );
	
	if (TSharedPtr<SDockingTabWell> PinnedParent = ParentPtr.Pin())
	{
		//See if we can drag tabs contain in this manager
		TSharedPtr<FTabManager> TabManager = GetTabManagerPtr();
		if (TabManager.IsValid() && TabManager->GetCanDoDragOperation())
		{
			return PinnedParent->StartDraggingTab(SharedThis(this), TabGrabOffsetFraction, MouseEvent);
		}
		else
		{
			return FReply::Handled();
		}
	}
	else
	{
		// Should never get here (but sometimes does, under unknown circumstances)
		// TODO: investigate how the parent pointer can become invalid
		return FReply::Unhandled();
	}
}

FReply SDockTab::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( HasMouseCapture() )
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			return FReply::Handled().ReleaseMouseCapture();
		}
		else if ( MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton  )
		{
			if ( MyGeometry.IsUnderLocation( MouseEvent.GetScreenSpacePosition() ) )
			{
				RequestCloseTab();
			}
			
			return FReply::Handled().ReleaseMouseCapture();
		}
	}
	return FReply::Unhandled();
}

void SDockTab::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Register to activate the tab after a delay
	if ( !DragDropTimerHandle.IsValid() )
	{
		DragDropTimerHandle = RegisterActiveTimer( SDockTabDefs::DragTimerActivate, FWidgetActiveTimerDelegate::CreateSP( this, &SDockTab::TriggerActivateTab ) );
	}

	UpdateTabStyle();

	SBorder::OnDragEnter( MyGeometry, DragDropEvent );
}

void SDockTab::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	// Unregister the activation timer if it hasn't fired yet
	if ( DragDropTimerHandle.IsValid() )
	{
		UnRegisterActiveTimer( DragDropTimerHandle.Pin().ToSharedRef() );
	}

	SBorder::OnDragLeave( DragDropEvent );
}

FReply SDockTab::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Unregister the activation timer if it hasn't fired yet
	if ( DragDropTimerHandle.IsValid() )
	{
		UnRegisterActiveTimer( DragDropTimerHandle.Pin().ToSharedRef() );
	}

	return SBorder::OnDrop( MyGeometry, DragDropEvent );
}

FReply SDockTab::OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	if (!this->HasMouseCapture())
	{
		// We tapped on the tab, so it should be active
		ActivateInParent(ETabActivationCause::UserClickedOnTab);
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SDockTab::OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	if (this->HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

void SDockTab::SetContent( TSharedRef<SWidget> InContent )
{
	Content = InContent;
	if (ParentPtr.IsValid())
	{
		// This is critical to do, otherwise the content might remain if currently active even if expected to be destroyed
		ParentPtr.Pin()->RefreshParentContent();
	}
}

void SDockTab::SetLeftContent( TSharedRef<SWidget> InContent )
{
	this->TabWellContentLeft = InContent;
	if (ParentPtr.IsValid())
	{
		// This is critical to do, otherwise the content might remain if currently active even if expected to be destroyed
		ParentPtr.Pin()->RefreshParentContent();
	}
}

void SDockTab::SetRightContent( TSharedRef<SWidget> InContent )
{
	this->TabWellContentRight = InContent;
	if (ParentPtr.IsValid())
	{
		// This is critical to do, otherwise the content might remain if currently active even if expected to be destroyed
		ParentPtr.Pin()->RefreshParentContent();
	}
}

void SDockTab::SetTitleBarRightContent(TSharedRef<SWidget> InContent)
{
	this->TitleBarContentRight = InContent;
	if (ParentPtr.IsValid())
	{
		// This is critical to do, otherwise the content might remain if currently active even if expected to be destroyed
		ParentPtr.Pin()->RefreshParentContent();
	}
}

bool SDockTab::IsActive() const
{
	return FGlobalTabmanager::Get()->GetActiveTab() == SharedThis(this);
}

bool SDockTab::IsForeground() const
{
	return ParentPtr.IsValid() ? (ParentPtr.Pin()->GetForegroundTab() == SharedThis(this)) : true;
}

FSlateColor SDockTab::GetForegroundColor() const
{
	auto LocalForegroundColorAttribute = GetForegroundColorAttribute();
	if (LocalForegroundColorAttribute.IsBound() || LocalForegroundColorAttribute.Get() != FSlateColor::UseStyle())
	{
		return LocalForegroundColorAttribute.Get();
	}
	else if (IsActive())
	{
		return GetCurrentStyle().ActiveForegroundColor;
	}
	else if (IsHovered())
	{
		return GetCurrentStyle().HoveredForegroundColor;
	}
	else if (IsForeground())
	{
		return GetCurrentStyle().ForegroundForegroundColor;
	}
	else
	{
		return GetCurrentStyle().NormalForegroundColor;
	}
}

ETabRole SDockTab::GetTabRole() const
{
	return TabRole;
}

ETabRole SDockTab::GetVisualTabRole() const
{
	// If the tab role is NomadTab but is being visualized as a major tab
	if (this->TabRole == ETabRole::NomadTab)
	{
		bool bNomadMajorStyle = false;

		if (DraggedOverDockingArea.IsValid())
		{
			bNomadMajorStyle = DraggedOverDockingArea->GetTabManager() == FGlobalTabmanager::Get();
		}
		else if (GetParent().IsValid() && GetParent()->GetDockArea().IsValid())
		{
			bNomadMajorStyle = GetParent()->GetDockArea()->GetTabManager() == FGlobalTabmanager::Get();
		}
		// If we have a parent docking area but no docking tab well. We are in drawer. In this case we will not use the major style.
		else if (!GetParent().IsValid() && !ParentDockingAreaPtr.IsValid())
		{
			// We are dragging, but we are not dragging over anything, assume major
			bNomadMajorStyle = true;
		}

		if (bNomadMajorStyle)
		{
			return ETabRole::MajorTab;
		}
	}

	return GetTabRole();
}

const FSlateBrush* SDockTab::GetContentAreaBrush() const
{
	return &GetCurrentStyle().ContentAreaBrush;
}

const FSlateBrush* SDockTab::GetTabWellBrush() const
{
	return &GetCurrentStyle().TabWellBrush;
}

TSharedRef<SWidget> SDockTab::GetContent()
{
	return Content;
}

TSharedRef<SWidget> SDockTab::GetLeftContent()
{
	return TabWellContentLeft;
}

TSharedRef<SWidget> SDockTab::GetRightContent()
{
	return TabWellContentRight;
}

TSharedRef<SWidget> SDockTab::GetTitleBarRightContent()
{
	return TitleBarContentRight;
}

FMargin SDockTab::GetContentPadding() const
{
	return ContentAreaPadding.Get();
}

void SDockTab::SetLayoutIdentifier( const FTabId& TabId )
{
	LayoutIdentifier = TabId;
}

const FTabId SDockTab::GetLayoutIdentifier() const
{
	return LayoutIdentifier;
}

void SDockTab::SetParent(TSharedPtr<SDockingTabWell> Parent)
{
	ParentPtr = Parent;
	ParentDockingAreaPtr = nullptr;
	OnParentSet();
}

void SDockTab::SetParentDockingArea(TSharedPtr<SDockingArea> Parent)
{
	ParentDockingAreaPtr = Parent;
}

TSharedPtr<SDockingTabWell> SDockTab::GetParent() const
{
	return ParentPtr.IsValid() ? ParentPtr.Pin() : TSharedPtr<SDockingTabWell>();
}

TSharedPtr<SDockingTabStack> SDockTab::GetParentDockTabStack() const
{
	TSharedPtr<SDockingTabWell> ParentTabWell = GetParent();
	if( ParentTabWell.IsValid() )
	{
		return ParentTabWell->GetParentDockTabStack();
	}
	return NULL;
}

void SDockTab::RemoveTabFromParent()
{
	OnTabClosed.ExecuteIfBound(SharedThis(this));
	RemoveTabFromParent_Internal();
}

TSharedPtr<SDockingArea> SDockTab::GetDockArea() const
{
	if (TSharedPtr<SDockingArea> DockingArea = ParentDockingAreaPtr.Pin())
	{
		return DockingArea;
	}

	if (TSharedPtr<SDockingTabWell> Parent = ParentPtr.Pin())
	{
		return Parent->GetDockArea();
	}

	return {};
}

bool SDockTab::IsInSidebar() const
{
	if (TSharedPtr<SDockingArea> DockingArea = GetDockArea())
	{
		return DockingArea->IsTabInSidebar(SharedThis(const_cast<SDockTab*>(this)));
	}

	return false;
}

TSharedPtr<SWindow> SDockTab::GetParentWindow() const
{
	TSharedPtr<SDockingArea> DockingAreaPtr = this->GetDockArea();
	return DockingAreaPtr.IsValid() ? DockingAreaPtr->GetParentWindow() : TSharedPtr<SWindow>();
}

SDockTab::SDockTab()
	: Content(SNew(SSpacer))
	, TabWellContentLeft(SNullWidget::NullWidget)
	, TabWellContentRight(SNullWidget::NullWidget)
	, TitleBarContentRight(SNullWidget::NullWidget)
	, LayoutIdentifier(NAME_None)
	, TabRole(ETabRole::PanelTab)
	, ParentPtr()
	, TabLabel(NSLOCTEXT("DockTab", "DefaultTabTitle", "UNNAMED"))
	, TabLabelSuffix(FText::GetEmpty())
	, OnTabClosed()
	, OnTabActivated()
	, OnCanCloseTab()
	, ContentAreaPadding( 2.f )
	, AutosizePolicy(EDockTabAutosizePolicy::None)
	, TabColorScale(FLinearColor::Transparent)
	, LastActivationTime(0.0)
{
}

void SDockTab::ActivateInParent(ETabActivationCause InActivationCause)
{
	TSharedPtr<SDockingTabWell> ParentTabWell = ParentPtr.Pin();
	if (ParentTabWell.IsValid())
	{
		ParentTabWell->BringTabToFront(SharedThis(this));
	}

	OnTabActivated.ExecuteIfBound(SharedThis(this), InActivationCause);

	if (InActivationCause != UserClickedOnTab || !GetContent()->SupportsKeyboardFocus())
	{
		return;
	}

	FSlateApplication::Get().SetKeyboardFocus(GetContent(), EFocusCause::SetDirectly);
}

void SDockTab::SetTabManager( const TSharedPtr<FTabManager>& InTabManager)
{
	{
		TSharedPtr<FTabManager> Previous = MyTabManager.Pin();
		if (Previous && Previous != InTabManager)
		{
			Previous->GetTabPermissionList()->OnFilterChanged().RemoveAll(this);
		}
	}

	MyTabManager = InTabManager;

	if (InTabManager)
	{
		if (!InTabManager->GetTabPermissionList()->OnFilterChanged().IsBoundToObject(this))
		{
			InTabManager->GetTabPermissionList()->OnFilterChanged().AddSP(this, &SDockTab::CheckTabAllowed);
		}
	}
}

void SDockTab::SetOnPersistVisualState( const FOnPersistVisualState& Handler )
{
	OnPersistVisualState = Handler;
}

void SDockTab::SetOnExtendContextMenu( const FExtendContextMenu& Handler )
{
	OnExtendContextMenu = Handler;
}

void SDockTab::SetCanCloseTab( const FCanCloseTab& InOnTabClosing )
{
	OnCanCloseTab = InOnTabClosing;
}

void SDockTab::SetOnTabClosed( const FOnTabClosedCallback& InDelegate )
{
	OnTabClosed = InDelegate;
}

void SDockTab::SetOnTabActivated( const FOnTabActivatedCallback& InDelegate )
{
	OnTabActivated = InDelegate;
}

void SDockTab::SetOnTabRenamed(const FOnTabRenamed& InDelegate)
{
	OnTabRenamed = InDelegate;
}

void SDockTab::SetOnTabDrawerOpened(const FSimpleDelegate InDelegate)
{
	OnTabDrawerOpenedEvent = InDelegate;
}

void SDockTab::SetOnTabDrawerClosed(const FSimpleDelegate InDelegate)
{
	OnTabDrawerClosedEvent = InDelegate;
}

void SDockTab::SetOnTabRelocated(const FSimpleDelegate InDelegate)
{
	OnTabRelocated = InDelegate;
}

void SDockTab::SetOnTabDraggedOverDockArea(const FSimpleDelegate InDelegate)
{
	OnTabDraggedOverDockArea = InDelegate;
}

TSharedRef<FTabManager> SDockTab::GetTabManager() const
{
	return MyTabManager.Pin().ToSharedRef();
}

TSharedPtr<FTabManager> SDockTab::GetTabManagerPtr() const
{
	return MyTabManager.Pin();
}

void SDockTab::DrawAttention()
{
	TSharedPtr<FTabManager> TabManager = GetTabManagerPtr();
	if (TabManager.IsValid())
	{
		TabManager->DrawAttention(SharedThis(this));
	}
}

void SDockTab::ProvideDefaultLabel( const FText& InDefaultLabel )
{
	const bool UserProvidedLabel = TabLabel.IsBound() || !TabLabel.Get().IsEmpty();
	if ( !UserProvidedLabel )
	{
		TabLabel = InDefaultLabel;
	}
}

void SDockTab::ProvideDefaultIcon( const FSlateBrush* InDefaultIcon )
{
	const bool UserProvidedIcon = TabIcon.IsBound() || (TabIcon.Get() && TabIcon.Get() != FStyleDefaults::GetNoBrush());
	if( !UserProvidedIcon )
	{
		TabIcon = InDefaultIcon;
	}
}

void SDockTab::PlaySpawnAnim()
{
	SpawnAnimCurve.Play( this->AsShared() );
}

void SDockTab::FlashTab()
{
	FlashTabCurve = FCurveSequence(0, SDockTabDefs::TabFlashDuration, ECurveEaseFunction::Linear);
	FlashTabCurve.Play( this->AsShared() );
}

float SDockTab::GetFlashValue() const
{
	if(FlashTabCurve.IsPlaying())
	{
		const float Lerp = FlashTabCurve.GetLerp();

		const float SinRateMultiplier = 2.0f * PI * SDockTabDefs::TabFlashDuration * SDockTabDefs::TabFlashFrequency;
		const float SinTerm = 0.5f * (FMath::Sin(Lerp * SinRateMultiplier) + 1.0f);

		const float FadeTerm = 1.0f - Lerp;

		return SinTerm * FadeTerm;
	}

	return 0.0f;
}

void SDockTab::SetDraggedOverDockArea( const TSharedPtr<SDockingArea>& Area )
{
	DraggedOverDockingArea = Area;
	OnTabDraggedOverDockArea.ExecuteIfBound();
}

bool SDockTab::HasSiblingTab(const FTabId& SiblingTabId, const bool TreatIndexNoneAsWildcard) const
{
	TSharedPtr<SDockingTabStack> ParentTabStack = GetParentDockTabStack();
	return (ParentTabStack.IsValid()) ? ParentTabStack->HasTab(FTabMatcher(SiblingTabId, static_cast<ETabState::Type>(ETabState::ClosedTab | ETabState::OpenedTab | ETabState::SidebarTab), TreatIndexNoneAsWildcard)) : false;
}

void SDockTab::Construct( const FArguments& InArgs )
{
	SpawnAnimCurve = FCurveSequence(0, 0.15f);
	SpawnAnimCurve.JumpToEnd();

	// We are just holding on to the content via a referece; not actually presenting it.
	this->Content = InArgs._Content.Widget;
	this->TabWellContentLeft = InArgs._TabWellContentLeft.Widget;
	this->TabWellContentRight = InArgs._TabWellContentRight.Widget;
	this->TabRole = InArgs._TabRole;
	this->OnTabClosed = InArgs._OnTabClosed;
	this->OnTabActivated = InArgs._OnTabActivated;
	this->OnCanCloseTab = InArgs._OnCanCloseTab;
	this->bCanEverClose = InArgs._CanEverClose;
	this->OnPersistVisualState = InArgs._OnPersistVisualState;
	this->OnExtendContextMenu = InArgs._OnExtendContextMenu;
	this->TabLabel = InArgs._Label;
	this->TabLabelSuffix = InArgs._LabelSuffix;
	this->AutosizePolicy = (InArgs._ShouldAutosize && InArgs._AutosizePolicy == EDockTabAutosizePolicy::None)
		? EDockTabAutosizePolicy::LayoutAxis : InArgs._AutosizePolicy; // Map old arg to new arg
	this->TabColorScale = InArgs._TabColorScale;
	this->IconColor = InArgs._IconColor;

	OnTabDrawerClosedEvent = InArgs._OnTabDrawerClosed;
	OnTabRelocated = InArgs._OnTabRelocated;
	OnTabDraggedOverDockArea = InArgs._OnTabDraggedOverDockArea;

	MajorTabStyle = &FAppStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.MajorTab");
	GenericTabStyle = &FAppStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab");

	ContentAreaPadding = InArgs._ContentPadding;

	const FButtonStyle* const CloseButtonStyle = &GetCurrentStyle().CloseButtonStyle;
	const FTextBlockStyle* const TabTextStyle = &GetCurrentStyle().TabTextStyle;

	// This makes a gradient that displays whether or not a viewport is active
	static FLinearColor ActiveBorderColor = FAppStyle::Get().GetSlateColor("Docking.Tab.ActiveTabIndicatorColor").GetSpecifiedColor();
	static FLinearColor ActiveBorderColorTransparent = FLinearColor(ActiveBorderColor.R, ActiveBorderColor.G, ActiveBorderColor.B, 0.0f);
	static TArray<FLinearColor> GradientStops{ ActiveBorderColorTransparent, ActiveBorderColor, ActiveBorderColorTransparent };

	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FStyleDefaults::GetNoBrush() )
		.VAlign(VAlign_Bottom)
		.Padding( 0.0f )
		.ForegroundColor(InArgs._ForegroundColor)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( this, &SDockTab::GetImageBrush )
			]

			// Overlay for active tab indication.
			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			[
				SNew(SComplexGradient)
				.Visibility(this, &SDockTab::GetActiveTabIndicatorVisibility)
				.DesiredSizeOverride(FVector2D(1.0f, 1.0f))
				.GradientColors(GradientStops)
				.Orientation(EOrientation::Orient_Vertical)
			]

			// Overlay for flashing a tab for attention
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				// Don't allow flasher tab overlay to absorb mouse clicks
				.Visibility( EVisibility::HitTestInvisible )
				.Padding( this, &SDockTab::GetTabPadding )
				.BorderImage( this, &SDockTab::GetFlashOverlayImageBrush )
				.BorderBackgroundColor( this, &SDockTab::GetFlashColor )
			]

			+ SOverlay::Slot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDockTab::GetTabPadding)))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				.Visibility(EVisibility::Visible)
				.ToolTip(InArgs._ToolTip)
				.ToolTipText(InArgs._ToolTipText.IsSet() ? InArgs._ToolTipText : TAttribute<FText>(this, &SDockTab::GetTabLabel))

				// Tab Icon
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(IsTabNameHidden() ? FMargin(0) : FMargin(0, 0, 5, 0))
				[
					SNew(SBorder)
					// Don't allow active tab overlay to absorb mouse clicks
					.Padding(this, &SDockTab::GetTabIconBorderPadding)
					.Visibility(EVisibility::HitTestInvisible)
					// Overlay for color-coded tab effect
					.BorderImage(this, &SDockTab::GetColorOverlayImageBrush)
					.BorderBackgroundColor(this, &SDockTab::GetTabColor)
					[
						SAssignNew(IconWidget, SImage)
						.ColorAndOpacity(this, &SDockTab::GetIconColor)
						.Image(this, &SDockTab::GetTabIcon)
						.DesiredSizeOverride(this, &SDockTab::GetTabIconSize)
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f, 1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					// Label sub HBox
					SNew(SHorizontalBox)
					.ToolTip(InArgs._ToolTip)

					// Tab Label
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SAssignNew(LabelWidget, STextBlock)
						.TextStyle(&GetCurrentStyle().TabTextStyle)
						.Text(this, &SDockTab::GetTabLabel)
						.OverflowPolicy(InArgs._LabelOverflowPolicy)
					]

					// Tab Label Suffix
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SAssignNew(LabelSuffix, STextBlock)
						.TextStyle(&GetCurrentStyle().TabTextStyle)
						.Text(this, &SDockTab::GetTabLabelSuffix)
					]
				]
				
				// @todo toolkit major: Could inject inline content here into tab for standalone asset editing dropdown/dirty state, etc.

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(IsTabNameHidden() ? HAlign_Left : HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( CloseButtonStyle )
					.OnClicked( this, &SDockTab::OnCloseButtonClicked )
					.ContentPadding(FMargin(0.0, 1.5, 0.0, 0.0))
					.ToolTipText(this, &SDockTab::GetCloseButtonToolTipText)
					.Visibility(this, &SDockTab::HandleIsCloseButtonVisible)
					[
						SNew(SSpacer)
						.Size( CloseButtonStyle->Normal.ImageSize )
					]
				]
			]
		]
	);
}

EActiveTimerReturnType SDockTab::TriggerActivateTab( double InCurrentTime, float InDeltaTime )
{
	ActivateInParent( ETabActivationCause::UserClickedOnTab );
	return EActiveTimerReturnType::Stop;
}

EActiveTimerReturnType SDockTab::OnHandleUpdateStyle(double InCurrentTime, float InDeltaTime)
{
	if (GetParent().IsValid() && GetParent()->GetDockArea().IsValid())
	{
		UpdateTabStyle();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void SDockTab::OnParentSet()
{
	// Update the style once the parent is set as it could change the way things look for certian tab types.
	if (!UpdateStyleTimerHandle.IsValid())
	{
		UpdateStyleTimerHandle = RegisterActiveTimer(0, FWidgetActiveTimerDelegate::CreateSP(this, &SDockTab::OnHandleUpdateStyle));
	}
}

void SDockTab::UpdateTabStyle()
{
	const FTextBlockStyle& TabTextStyle = GetCurrentStyle().TabTextStyle;
	LabelSuffix->SetTextStyle(&TabTextStyle);
	LabelWidget->SetTextStyle(&TabTextStyle);
}

void SDockTab::OnTabDrawerOpened()
{
	OnTabDrawerOpenedEvent.ExecuteIfBound();
}

void SDockTab::OnTabDrawerClosed()
{
	OnTabDrawerClosedEvent.ExecuteIfBound();
}

void SDockTab::NotifyTabRelocated()
{
	OnTabRelocated.ExecuteIfBound();
}

void SDockTab::CheckTabAllowed()
{
	if (!LayoutIdentifier.TabType.IsNone())
	{
		TSharedPtr<FTabManager> Manager = MyTabManager.Pin();
		if (Manager)
		{
			if (!Manager->GetTabPermissionList()->PassesFilter(LayoutIdentifier.TabType))
			{
				UE_LOGF(LogSlate, Verbose, "Closing tab because type does not pass filter: %ls", *LayoutIdentifier.TabType.ToString());
				RequestCloseTab();
			}
		}
	}
}

void SDockTab::RemoveTabFromParent_Internal()
{
	// Keep this object alive until the end of this method since it may die by the time RemoveAndDestroyTab returns
	TSharedRef<SDockTab> This = SharedThis(this);
	if (ParentPtr.IsValid())
	{
		ParentPtr.Pin()->RemoveAndDestroyTab(This, SDockingNode::TabRemoval_Closed);
	}
	else if (TSharedPtr<SDockingArea> DockingArea = ParentDockingAreaPtr.Pin())
	{
		// Tab is hosted in a panel drawer
		DockingArea->ClosePanelDrawerForTransfer();
		DockingArea->RemoveHiddenInactivePanelDrawerTab(This);
	}

	ParentPtr.Reset();
	ParentDockingAreaPtr.Reset();
}

const FDockTabStyle& SDockTab::GetCurrentStyle() const
{
	if ( GetVisualTabRole() == ETabRole::MajorTab )
	{
		return *MajorTabStyle;
	}
	
	return *GenericTabStyle;
}

const FSlateBrush* SDockTab::GetImageBrush() const
{
	const FDockTabStyle& CurrentStyle = GetCurrentStyle();

	// Pick the right brush based on whether the tab is active or hovered.
	if ( this->IsForeground() )
	{
		return &CurrentStyle.ForegroundBrush;
	}
	else if ( this->IsHovered() )
	{
		return &CurrentStyle.HoveredBrush;
	}
	return &CurrentStyle.NormalBrush;
}


FMargin SDockTab::GetTabPadding() const
{
	FMargin NewPadding = GetCurrentStyle().TabPadding;
	NewPadding.Top *= SDockTabDefs::TabVerticalPaddingScaleOverride;
	NewPadding.Bottom *= SDockTabDefs::TabVerticalPaddingScaleOverride;
	return NewPadding;
}

FMargin SDockTab::GetTabIconBorderPadding() const
{
	return FMargin(GetCurrentStyle().IconBorderPadding);
}

const FSlateBrush* SDockTab::GetColorOverlayImageBrush() const
{
	if (this->TabColorScale.Get().A > 0.0f)
	{
		return &GetCurrentStyle().ColorOverlayIconBrush;
	}
	return FStyleDefaults::GetNoBrush();
}

EVisibility SDockTab::GetActiveTabIndicatorVisibility() const
{
	return IsActive() && GetVisualTabRole() != ETabRole::MajorTab ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

FSlateColor SDockTab::GetIconColor() const
{
	if (!IconColor.IsSet())
	{
		return FSlateColor::UseForeground();
	}

	if (this->IsForeground() || this->IsHovered())
	{
		return IconColor.Get();
	}
	else //  Dim to 30% if not active
	{
		return IconColor.Get().CopyWithNewOpacity(0.7f);
	}

}
FSlateColor SDockTab::GetTabColor() const
{
	return TabColorScale.Get();
}


const FSlateBrush* SDockTab::GetFlashOverlayImageBrush() const
{
	if (FlashTabCurve.IsPlaying())
	{
		// Flashing is really just applying a color overlay, so we can re-use the color overlay brush and apply our flash tint to it
		return &GetCurrentStyle().ColorOverlayTabBrush;
	}
	return FStyleDefaults::GetNoBrush();
}

void SDockTab::ExtendContextMenu(FMenuBuilder& MenuBuilder)
{
	OnExtendContextMenu.ExecuteIfBound(MenuBuilder);
}

FSlateColor SDockTab::GetFlashColor() const
{
	if ( GetCurrentStyle().FlashColor.IsColorSpecified() )
	{
		FLinearColor Color = GetCurrentStyle().FlashColor.GetSpecifiedColor();
		Color.A = GetFlashValue();

		return FSlateColor(Color);
	}

	return FLinearColor::White;
}

float SDockTab::GetOverlapWidth() const
{
	return GetCurrentStyle().OverlapWidth;
}

FText SDockTab::GetTabLabel() const
{
	return TabLabel.Get();
}

void SDockTab::SetLabel( const TAttribute<FText>& InTabLabel )
{
	TabLabel = InTabLabel;
	OnTabRenamed.ExecuteIfBound(SharedThis(this));
}

FText SDockTab::GetTabLabelSuffix() const
{
	return TabLabelSuffix.Get();
}

void SDockTab::SetTabLabelSuffix(const TAttribute<FText>& InTabLabelSuffix)
{
	TabLabelSuffix = InTabLabelSuffix;
}

void SDockTab::SetTabLabelOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy) const
{
	if (LabelWidget.IsValid())
	{
		LabelWidget->SetOverflowPolicy(InOverflowPolicy);
	}
}

const FSlateBrush* SDockTab::GetTabIcon() const
{
	return TabIcon.Get();
}

void SDockTab::SetTabToolTipWidget(TSharedPtr<SToolTip> InTabToolTipWidget)
{
	IconWidget->SetToolTip(InTabToolTipWidget);
	LabelWidget->SetToolTip(InTabToolTipWidget);	
}

void SDockTab::SetTabIcon( const TAttribute<const FSlateBrush*> InTabIcon )
{
	TabIcon = InTabIcon;
}

bool SDockTab::ShouldAutosize() const
{
	return AutosizePolicy != EDockTabAutosizePolicy::None;
}

void SDockTab::SetShouldAutosize(const bool bNewShouldAutosize)
{
	AutosizePolicy = bNewShouldAutosize ? EDockTabAutosizePolicy::LayoutAxis : EDockTabAutosizePolicy::None;
}

EDockTabAutosizePolicy SDockTab::GetAutosizePolicy() const
{
	return AutosizePolicy;
}

void SDockTab::SetAutosizePolicy(const EDockTabAutosizePolicy InNewPolicy)
{
	AutosizePolicy = InNewPolicy;
}

FReply SDockTab::OnCloseButtonClicked()
{
	RequestCloseTab();

	return FReply::Handled();
}

FText SDockTab::GetCloseButtonToolTipText() const
{
	TSharedPtr<FUICommandInfo> CloseCommand =
		GetVisualTabRole() == ETabRole::MajorTab ? FTabCommands::Get().CloseMajorTab : FTabCommands::Get().CloseMinorTab;

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Label"), NSLOCTEXT("DockTab", "CloseButtonToolTipLabel", "Close Tab"));

	FText InputText = CloseCommand->GetInputText();
	if (InputText.IsEmptyOrWhitespace())
	{
		Arguments.Add(TEXT("InputText"), InputText);
	}
	else
	{
		Arguments.Add(TEXT("InputText"), FText::Format(NSLOCTEXT("DockTab", "CloseButtonInputText", "({0})"), InputText));
	}

	return FText::Format(NSLOCTEXT("DockTab", "CloseButtonToolTipFormat", "{Label} {InputText}"), Arguments);
}

EVisibility SDockTab::HandleIsCloseButtonVisible() const
{
	TSharedPtr<FTabManager> Manager = MyTabManager.Pin();
	const bool bIsVisibled = IsHovered() || IsForeground();
	const bool bIsTabCloseable = Manager == nullptr || Manager->IsTabCloseable(SharedThis(this));
	return bCanEverClose && !IsTabLocked() && bIsVisibled && bIsTabCloseable ? EVisibility::Visible : EVisibility::Hidden;
}

TOptional<FVector2D> SDockTab::GetTabIconSize() const
{
	return FVector2D(GetCurrentStyle().IconSize);
}

bool SDockTab::CanCloseTab() const
{
	return CanCloseTab(false);
}

bool SDockTab::CanCloseTab(bool bIgnoreLockedTabs) const
{
	// Check the below conditions separately and early out to prevent the OnCanCloseTab delegate
	// from being called if we already know we cannot close the tab.

	if (!bCanEverClose)
	{
		return false;
	}

	if (TSharedPtr<FTabManager> Manager = MyTabManager.Pin();
		Manager && !Manager->IsTabCloseable(SharedThis(this)))
	{
		return false;
	}

	if (!bIgnoreLockedTabs)
	{
		if (IsTabLocked())
		{
			return false;
		}
	}

	if (OnCanCloseTab.IsBound() && !OnCanCloseTab.Execute())
	{
		return false;
	}
	
	return true;
}

bool SDockTab::RequestCloseTab()
{	
	this->PersistVisualState();

	// The tab can be closed if the delegate is not bound or if the delegate call indicates we cannot close it
	const bool bCanCloseTabNow = CanCloseTab();

	if (bCanCloseTabNow)
	{
		if (GetParentDockTabStack() &&
			GetParentDockTabStack()->GetDockArea() &&
			GetParentDockTabStack()->GetDockArea()->IsTabInSidebar(SharedThis(this)))
		{
			OnTabClosed.ExecuteIfBound(SharedThis(this));
			GetParentDockTabStack()->GetDockArea()->RemoveTabFromSidebar(SharedThis(this));
		}
		else
		{
			RemoveTabFromParent();
		}
	}

	return bCanCloseTabNow;
}

bool SDockTab::IsTabLocked() const
{
	const TSharedPtr<SDockingTabStack>& StackTab = GetParentDockTabStack();
	return StackTab.IsValid() ? StackTab->IsTabLocked(SharedThis(this)) : false;
}

void SDockTab::PersistVisualState()
{
	OnPersistVisualState.ExecuteIfBound();
}


UE::Slate::FDeprecateVector2DResult SDockTab::GetAnimatedScale() const
{
	static FVector2f FullyOpen = FVector2f::UnitVector;
	static FVector2f FullyClosed = FVector2f(1.0f, 0.0f);
	return FMath::Lerp(FullyClosed, FullyOpen, SpawnAnimCurve.GetLerp());
}

void SDockTab::UpdateActivationTime()
{
	if (FSlateApplication::IsInitialized())
	{
		LastActivationTime = FSlateApplication::Get().GetCurrentTime();
	}
}


void SDockTab::SetParentDockTabStackTabWellHidden(bool bIsTabWellHidden)
{
	if ( const TSharedPtr<SDockingTabStack> ParentStack = GetParentDockTabStack() )
	{
		if ( bIsTabWellHidden )
		{
			if ( ParentStack->CanHideTabWell() )
			{
				ParentStack->SetTabWellHidden(true);				
			}
		}
		else
		{
			ParentStack->SetTabWellHidden(false);
		}
	}
}

bool SDockTab::IsTabNameHidden() const
{
	return bIsTabNameHidden;
}

void SDockTab::SetIsTabNameHidden(bool bInIsTabNameHidden)
{
	bIsTabNameHidden = bInIsTabNameHidden;
}
