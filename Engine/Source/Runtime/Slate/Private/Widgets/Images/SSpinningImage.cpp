// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Images/SSpinningImage.h"
#include "Rendering/DrawElements.h"


void SSpinningImage::Construct(const FArguments& InArgs)
{
	SetImage(InArgs._Image);
	SetColorAndOpacity(InArgs._ColorAndOpacity);

	if (InArgs._OnMouseButtonDown.IsBound())
	{
		SetOnMouseButtonDown(InArgs._OnMouseButtonDown);
	}
	
	SpinAnimationSequence = FCurveSequence(0.f, InArgs._Period);
	SpinAnimationSequence.Play(AsShared(), true, 0.f, false);
}

// Override SImage's OnPaint to draw rotated
int32 SSpinningImage::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const FSlateBrush* ImageBrush = GetImageAttribute().Get();

	if ((ImageBrush != nullptr) && (ImageBrush->DrawAs != ESlateBrushDrawType::NoDrawType))
	{
		const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
		const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		const FLinearColor FinalColorAndOpacity( InWidgetStyle.GetColorAndOpacityTint() * GetColorAndOpacityAttribute().Get().GetColor(InWidgetStyle) * ImageBrush->GetTint( InWidgetStyle ) );
		
		const float Angle = SpinAnimationSequence.GetLerp() * 2.0f * PI;

		FSlateDrawElement::MakeRotatedBox( 
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			ImageBrush,
			DrawEffects,
			Angle,
			TOptional<FVector2f>(), // Will auto rotate about center
			FSlateDrawElement::RelativeToElement,
			FinalColorAndOpacity
			);
	}
	return LayerId;
}

bool SSpinningImage::ComputeVolatility() const
{
	return SImage::ComputeVolatility() || SpinAnimationSequence.IsPlaying();
}