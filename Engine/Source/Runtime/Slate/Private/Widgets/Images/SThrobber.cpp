// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Images/SThrobber.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"


void SThrobber::Construct(const FArguments& InArgs)
{
	PieceImage = InArgs._PieceImage;
	NumPieces = InArgs._NumPieces;
	Animate = InArgs._Animate;

	HBox = SNew(SHorizontalBox);

	this->ChildSlot
	[
		HBox.ToSharedRef()
	];

	ConstructPieces();
}

void SThrobber::ConstructPieces()
{
	ThrobberCurve.Reset(NumPieces);
	AnimCurves = FCurveSequence();
	HBox->ClearChildren();
	for (int32 PieceIndex = 0; PieceIndex < NumPieces; ++PieceIndex)
	{
		ThrobberCurve.Add(AnimCurves.AddCurve(PieceIndex*0.05f, 1.5f));

		HBox->AddSlot()
		.AutoWidth()
		[
			SNew(SBorder)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.ContentScale(this, &SThrobber::GetPieceScale, PieceIndex)
			.ColorAndOpacity(this, &SThrobber::GetPieceColor, PieceIndex)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SThrobber::GetPieceBrush)
			]
		];
	}
	AnimCurves.Play(AsShared(), true, 0.f, false);
}

bool SThrobber::ComputeVolatility() const
{
	return SCompoundWidget::ComputeVolatility() || Animate != EAnimation::None;
}

const FSlateBrush* SThrobber::GetPieceBrush() const
{
	return PieceImage;
}

void SThrobber::SetPieceImage(const FSlateBrush* InPieceImage)
{
	if (PieceImage != InPieceImage)
	{
		PieceImage = InPieceImage;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SThrobber::InvalidatePieceImage()
{
	Invalidate(EInvalidateWidgetReason::Paint);
}

void SThrobber::SetNumPieces(int InNumPieces)
{
	if (NumPieces != InNumPieces)
	{
		NumPieces = InNumPieces;
		ConstructPieces();
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SThrobber::SetAnimate(EAnimation InAnimate)
{
	if (Animate != InAnimate)
	{
		Animate = InAnimate;
		// invalidating the volatility will trigger a last paint, if Animate == 0
		Invalidate(EInvalidateWidgetReason::PaintAndVolatility);
	}
}

FVector2D SThrobber::GetPieceScale(int32 PieceIndex) const
{
	const float Value = FMath::Sin(2.f * PI * ThrobberCurve[PieceIndex].GetLerp());
	
	const bool bAnimateHorizontally = ((0 != (Animate & Horizontal)));
	const bool bAnimateVertically = (0 != (Animate & Vertical));
	
	return FVector2D(
		bAnimateHorizontally ? Value : 1.0f,
		bAnimateVertically ? Value : 1.0f
	);
}

FLinearColor SThrobber::GetPieceColor(int32 PieceIndex) const
{
	const bool bAnimateOpacity = (0 != (Animate & Opacity));
	if (bAnimateOpacity)
	{
		const float Value = FMath::Sin(2.f * PI * ThrobberCurve[PieceIndex].GetLerp());
		return FLinearColor(1.0f,1.0f,1.0f, Value);
	}
	else
	{
		return FLinearColor::White;
	}
}

// SCircularThrobber

const float SCircularThrobber::MinimumPeriodValue = SMALL_NUMBER;

SLATE_IMPLEMENT_WIDGET(SCircularThrobber)
void SCircularThrobber::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ColorAndOpacity, EInvalidateWidgetReason::Paint);
}


SCircularThrobber::SCircularThrobber()
	: ColorAndOpacity(*this)
{}

SCircularThrobber::~SCircularThrobber() = default;

void SCircularThrobber::Construct(const FArguments& InArgs)
{
	PieceImage = InArgs._PieceImage;
	NumPieces = InArgs._NumPieces;
	Period = InArgs._Period;
	Radius = InArgs._Radius;
	bColorAndOpacitySet = InArgs._ColorAndOpacity.IsSet();
	ColorAndOpacity.Assign(*this, InArgs._ColorAndOpacity);

	ConstructSequence();
}

void SCircularThrobber::SetPieceImage(const FSlateBrush* InPieceImage)
{
	if (PieceImage != InPieceImage)
	{
		PieceImage = InPieceImage;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SCircularThrobber::InvalidatePieceImage()
{
	Invalidate(EInvalidateWidgetReason::Paint);
}

void SCircularThrobber::SetNumPieces(const int32 InNumPieces)
{
	if (NumPieces != InNumPieces)
	{
		NumPieces = InNumPieces;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SCircularThrobber::SetPeriod(const float InPeriod)
{
	if (!FMath::IsNearlyEqual(Period, InPeriod))
	{
		Period = InPeriod;
		ConstructSequence();
	}
}

void SCircularThrobber::SetRadius(const float InRadius)
{
	if (!FMath::IsNearlyEqual(Radius, InRadius))
	{
		Radius = InRadius;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SCircularThrobber::ConstructSequence()
{
	Sequence = FCurveSequence();
	Curve = Sequence.AddCurve(0.f, FMath::Max(Period, MinimumPeriodValue));
	Sequence.Play(AsShared(), true, 0.0f, false);
}

int32 SCircularThrobber::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FLinearColor FinalColorAndOpacity;
	if (bColorAndOpacitySet)
	{
		FinalColorAndOpacity = ColorAndOpacity.Get().GetColor(InWidgetStyle);
	}
	else
	{
		FinalColorAndOpacity = InWidgetStyle.GetColorAndOpacityTint() * PieceImage->GetTint(InWidgetStyle);
	}

	const FVector2D LocalOffset = (AllottedGeometry.GetLocalSize() - PieceImage->ImageSize) * 0.5f;
	const float DeltaAngle = NumPieces > 0 ? 2 * PI / NumPieces : 0;
	const float Phase = Curve.GetLerp() * 2 * PI;

	for (int32 PieceIdx = 0; PieceIdx < NumPieces; ++PieceIdx)
	{
		const float Angle = DeltaAngle * PieceIdx + Phase;
		// scale each piece linearly until the last piece is full size
		FSlateLayoutTransform PieceLocalTransform(
			(PieceIdx + 1) / (float)NumPieces,
			LocalOffset + LocalOffset * FVector2D(FMath::Sin(Angle), FMath::Cos(Angle)));
		FPaintGeometry PaintGeom = AllottedGeometry.ToPaintGeometry(PieceImage->ImageSize, PieceLocalTransform);
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId, PaintGeom, PieceImage, ESlateDrawEffect::None, FinalColorAndOpacity);
	}
	
	return LayerId;
}

FVector2D SCircularThrobber::ComputeDesiredSize( float ) const
{
	return FVector2D(Radius, Radius) * 2;
}

bool SCircularThrobber::ComputeVolatility() const
{
	return Super::ComputeVolatility() || Sequence.IsPlaying();
}
