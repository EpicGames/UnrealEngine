// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SButton.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

static FName SButtonTypeName("SButton");

SLATE_IMPLEMENT_WIDGET(SButton)
void SButton::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "BaseBorderForegroundColor", BorderForegroundColorAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SButton&>(Widget).UpdateForegroundColor();
			}));

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ContentPadding", ContentPaddingAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			static_cast<SButton&>(Widget).UpdatePadding();
		}));

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "NormalPadding", NormalPaddingAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			static_cast<SButton&>(Widget).UpdatePadding();
		}));

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "PressedPadding", PressedPaddingAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			static_cast<SButton&>(Widget).UpdatePadding();
		}));
	
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "AppearPressed", AppearPressedAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SButton&>(Widget).UpdatePressStateChanged();
			}));

	AttributeInitializer.OverrideInvalidationReason("EnabledState", FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Layout|EInvalidateWidgetReason::Paint});
	AttributeInitializer.OverrideInvalidationReason("Hovered", FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Layout|EInvalidateWidgetReason::Paint});

	AttributeInitializer.OverrideOnValueChanged("EnabledState"
		, FSlateAttributeDescriptor::ECallbackOverrideType::ExecuteAfterPrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SButton&>(Widget).UpdateBorderImage();
			}));

	AttributeInitializer.OverrideOnValueChanged("Hovered"
		, FSlateAttributeDescriptor::ECallbackOverrideType::ExecuteAfterPrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SButton&>(Widget).UpdateBorderImage();
				static_cast<SButton&>(Widget).UpdateForegroundColor();
			}));
}

SButton::SButton()
	: BorderForegroundColorAttribute(*this, FSlateColor::UseStyle())
	, ContentPaddingAttribute(*this)
	, NormalPaddingAttribute(*this)
	, PressedPaddingAttribute(*this)
	, AppearPressedAttribute(*this)
{
#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

SButton::~SButton() = default;

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SButton::Construct( const FArguments& InArgs )
{
	bIsPressed = false;
	bIsFocusable = InArgs._IsFocusable;
	bAllowDragDrop = InArgs._AllowDragDrop;

	BorderForegroundColorAttribute.Assign(*this, InArgs._ForegroundColor);

	bIsStyleNormalPaddingOverridden = InArgs._NormalPaddingOverride.IsSet();
	if (bIsStyleNormalPaddingOverridden)
	{
		NormalPaddingAttribute.Assign(*this, InArgs._NormalPaddingOverride);
	}

	bIsStylePressedPaddingOverridden = InArgs._PressedPaddingOverride.IsSet();
	if (bIsStylePressedPaddingOverridden)
	{
		PressedPaddingAttribute.Assign(*this, InArgs._PressedPaddingOverride);
	}

	OnClicked = InArgs._OnClicked;
	OnPressed = InArgs._OnPressed;
	OnReleased = InArgs._OnReleased;
	OnHovered = InArgs._OnHovered;
	OnUnhovered = InArgs._OnUnhovered;
	OnReceivedFocus = InArgs._OnReceivedFocus;
	OnLostFocus = InArgs._OnLostFocus;

	OnSlateButtonDragDetected = InArgs._OnSlateButtonDragDetected;
	OnSlateButtonDragEnter = InArgs._OnSlateButtonDragEnter;
	OnSlateButtonDragLeave = InArgs._OnSlateButtonDragLeave;
	OnSlateButtonDragOver = InArgs._OnSlateButtonDragOver;
	OnSlateButtonDrop = InArgs._OnSlateButtonDrop;

	ClickMethod = InArgs._ClickMethod;
	TouchMethod = InArgs._TouchMethod;
	PressMethod = InArgs._PressMethod;

	HoveredSound = InArgs._HoveredSoundOverride.Get(InArgs._ButtonStyle->HoveredSlateSound);
	PressedSound = InArgs._PressedSoundOverride.Get(InArgs._ButtonStyle->PressedSlateSound);
	ClickedSound = InArgs._ClickedSoundOverride.Get(InArgs._ButtonStyle->ClickedSlateSound);

	// Text overrides button content. If nothing is specified, put an null widget in the button.
	// Null content makes the button enter a special mode where it will ask to be as big as the image used for its border.
	struct
	{
		TSharedRef<SWidget> operator()( const FArguments& InOpArgs ) const
		{
			if ((InOpArgs._Content.Widget == SNullWidget::NullWidget) && (InOpArgs._Text.IsBound() || !InOpArgs._Text.Get().IsEmpty()) )
			{
				return SNew(STextBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.Text( InOpArgs._Text )
					.TextStyle( InOpArgs._TextStyle )
					.TextShapingMethod( InOpArgs._TextShapingMethod )
					.TextFlowDirection( InOpArgs._TextFlowDirection );
			}
			else
			{
				return InOpArgs._Content.Widget;
			}
		}
	} DetermineContent; 

	SBorder::Construct( SBorder::FArguments()
		.ContentScale(InArgs._ContentScale)
		.DesiredSizeScale(InArgs._DesiredSizeScale)
		.BorderBackgroundColor(InArgs._ButtonColorAndOpacity)
		.HAlign(InArgs._HAlign)
		.VAlign(InArgs._VAlign)
		[
			DetermineContent(InArgs)
		]
	);

	SetContentPadding(InArgs._ContentPadding);
	SetButtonStyle(InArgs._ButtonStyle);

	// Only do this if we're exactly an SButton
	if (GetType() == SButtonTypeName)
	{
		SetCanTick(false);
	}
}

int32 SButton::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	bool bEnabled = ShouldBeEnabled(bParentEnabled);
	bool bShowDisabledEffect = GetShowDisabledEffect();

	const FSlateBrush* BrushResource = !bShowDisabledEffect && !bEnabled ? &Style->Disabled : GetBorderImage();
	
	ESlateDrawEffect DrawEffects = bShowDisabledEffect && !bEnabled ? ESlateDrawEffect::DisabledEffect : ESlateDrawEffect::None;

	if (BrushResource && BrushResource->DrawAs != ESlateBrushDrawType::NoDrawType)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			BrushResource,
			DrawEffects,
			BrushResource->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint() * GetBorderBackgroundColor().GetColor(InWidgetStyle)
		);
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled);
}

FMargin SButton::GetCombinedPadding() const
{
	return ( IsPressed() )
		? ContentPaddingAttribute.Get() + PressedPaddingAttribute.Get()
		: ContentPaddingAttribute.Get() + NormalPaddingAttribute.Get();
}

//~ Update when { ContentPaddingAttribute, PressedPaddingAttribute, NormalPaddingAttribute, IsPressed, Style }
void SButton::UpdatePadding()
{
	SetPadding(GetCombinedPadding());
}

bool SButton::GetShowDisabledEffect() const
{
	return Style->Disabled.DrawAs == ESlateBrushDrawType::NoDrawType;
}

//~ Update when { Style }
void SButton::UpdateShowDisabledEffect()
{
	// Needs to be called when the style changed
	SetShowEffectWhenDisabled(GetShowDisabledEffect());
}

//~ Update when { GetShowDisabledEffect(Style), IsEnable(EnabledState), Pressed, Hovered, Style }
void SButton::UpdateBorderImage()
{
	if (!GetShowDisabledEffect() && !IsInteractable())
	{
		SetBorderImage(&Style->Disabled);
	}
	else if (IsPressed())
	{
		SetBorderImage(&Style->Pressed);
	}
	else if (IsHovered())
	{
		SetBorderImage(&Style->Hovered);
	}
	else
	{
		SetBorderImage(&Style->Normal);
	}
}

//~ Update when { DefaultForegroundColorAttribute, Pressed, Hovered, Style }
void SButton::UpdateForegroundColor()
{
	if (BorderForegroundColorAttribute.Get() == FSlateColor::UseStyle())
	{
		if (IsPressed())
		{
			SetForegroundColor(Style->PressedForeground);
		}
		else if (IsHovered())
		{
			SetForegroundColor(Style->HoveredForeground);
		}
		else
		{
			SetForegroundColor(Style->NormalForeground);
		}
	}
	else
	{
		SetForegroundColor(BorderForegroundColorAttribute.Get());
	}
}

//~ Update when { Style }
void SButton::UpdateDisabledForegroundColor()
{
	Invalidate(EInvalidateWidgetReason::Paint);
}

bool SButton::SupportsKeyboardFocus() const
{
	// Buttons are focusable by default
	return bIsFocusable;
}

void SButton::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	SBorder::OnFocusLost(InFocusEvent);

	Release();
	OnLostFocus.ExecuteIfBound();
}

FReply SButton::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	FReply ReturnReply = SBorder::OnFocusReceived(MyGeometry, InFocusEvent);
	OnReceivedFocus.ExecuteIfBound();

	return ReturnReply;
}

FReply SButton::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	FReply Reply = FReply::Unhandled();
	if (IsEnabled() && FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		Press();

		if (PressMethod == EButtonPressMethod::ButtonPress)
		{
			//execute our "OnClicked" delegate, and get the reply
			Reply = ExecuteOnClick();

			//You should ALWAYS handle the OnClicked event.
			ensure(Reply.IsEventHandled() == true);
		}
		else
		{
			Reply = FReply::Handled();
		}
	}
	else
	{
		Reply = SBorder::OnKeyDown(MyGeometry, InKeyEvent);
	}

	//return the constructed reply
	return Reply;
}

FReply SButton::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsEnabled() && FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		const bool bWasPressed = bIsPressed;

		Release();

		//@Todo Slate: This should check focus, however we don't have that API yet, will be easier when focus is unified.
		if ( PressMethod == EButtonPressMethod::ButtonRelease || ( PressMethod == EButtonPressMethod::DownAndUp && bWasPressed ) )
		{
			//execute our "OnClicked" delegate, and get the reply
			Reply = ExecuteOnClick();

			//You should ALWAYS handle the OnClicked event.
			ensure(Reply.IsEventHandled() == true);
		}
		else
		{
			Reply = FReply::Handled();
		}
	}

	//return the constructed reply
	return Reply;
}

FReply SButton::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();
	if (IsEnabled() && (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.IsTouchEvent()))
	{
		Press();
		PressedScreenSpacePosition = MouseEvent.GetScreenSpacePosition();

		EButtonClickMethod::Type InputClickMethod = GetClickMethodFromInputType(MouseEvent);
		
		if(InputClickMethod == EButtonClickMethod::MouseDown)
		{
			//get the reply from the execute function
			Reply = ExecuteOnClick();

			//You should ALWAYS handle the OnClicked event.
			ensure(Reply.IsEventHandled() == true);
		}
		else if (InputClickMethod == EButtonClickMethod::PreciseClick)
		{
			// do not capture the pointer for precise taps or clicks
			// 
			Reply = FReply::Handled();
		}
		else
		{
			//we need to capture the mouse for MouseUp events
			Reply = FReply::Handled().CaptureMouse(AsShared());

			if (bAllowDragDrop)
			{
				Reply.DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
			}
		}
	}

	//return the constructed reply
	return Reply;
}

FReply SButton::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SBorder::OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
	if (Reply.IsEventHandled())
	{
		return Reply;
	}

	// We didn't handle the double click, treat it as single click
	return OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SButton::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();
	const EButtonClickMethod::Type InputClickMethod = GetClickMethodFromInputType(MouseEvent);
	const bool bMustBePressed = InputClickMethod == EButtonClickMethod::DownAndUp || InputClickMethod == EButtonClickMethod::PreciseClick;
	const bool bMeetsPressedRequirements = (!bMustBePressed || (bIsPressed && bMustBePressed));

	if (bMeetsPressedRequirements && ( ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.IsTouchEvent())))
	{
		Release();

		if ( IsEnabled() )
		{
			if ( InputClickMethod == EButtonClickMethod::MouseDown )
			{
				// NOTE: If we're configured to click on mouse-down/precise-tap, then we never capture the mouse thus
				//       may never receive an OnMouseButtonUp() call.  We make sure that our bIsPressed
				//       state is reset by overriding OnMouseLeave().
			}
			else
			{
				bool bEventOverButton = IsHovered();

				if ( !bEventOverButton && MouseEvent.IsTouchEvent() )
				{
					bEventOverButton = MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());
				}

				if ( bEventOverButton )
				{
					// If we asked for a precise tap, all we need is for the user to have not moved their pointer very far.
					const bool bTriggerForTouchEvent = InputClickMethod == EButtonClickMethod::PreciseClick;

					// If we were asked to allow the button to be clicked on mouse up, regardless of whether the user
					// pressed the button down first, then we'll allow the click to proceed without an active capture
					const bool bTriggerForMouseEvent = (InputClickMethod == EButtonClickMethod::MouseUp || HasMouseCapture() );

					if ( ( bTriggerForTouchEvent || bTriggerForMouseEvent ) )
					{
						Reply = ExecuteOnClick();
					}
				}
			}
		}
		
		//If the user of the button didn't handle this click, then the button's
		//default behavior handles it.
		if ( Reply.IsEventHandled() == false )
		{
			Reply = FReply::Handled();
		}
	}

	//If the user hasn't requested a new mouse captor and the button still has mouse capture,
	//then the default behavior of the button is to release mouse capture.
	if ( Reply.GetMouseCaptor().IsValid() == false && HasMouseCapture() )
	{
		Reply.ReleaseMouseCapture();
	}

	return Reply;
}

FReply SButton::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( IsPressed() && IsPreciseTapOrClick(MouseEvent) && FSlateApplication::Get().HasTraveledFarEnoughToTriggerDrag(MouseEvent, PressedScreenSpacePosition) )
	{
		Release();
	}

	return FReply::Unhandled();
}

void SButton::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const bool bWasHovered = IsHovered();

	SBorder::OnMouseEnter( MyGeometry, MouseEvent );

	if (!bWasHovered && IsHovered())
	{
		ExecuteHoverStateChanged(true);
	}
}
/**/
SLATE_API FReply SButton::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	//	If the drag detected handler is bound, we want to use the response up the chain
	if (OnSlateButtonDragDetected.IsBound())
	{
		return OnSlateButtonDragDetected.Execute(MyGeometry, MouseEvent);
	}
	else
	{
		return SBorder::OnDragDetected(MyGeometry, MouseEvent);
	}
}

SLATE_API void SButton::OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent)
{
	if (OnSlateButtonDragEnter.IsBound())
	{
		OnSlateButtonDragEnter.Execute(MyGeometry, DragDropEvent);
	}
	SBorder::OnDragEnter(MyGeometry, DragDropEvent);
}

SLATE_API void SButton::OnDragLeave(FDragDropEvent const& DragDropEvent)
{
	if (OnSlateButtonDragLeave.IsBound())
	{
			OnSlateButtonDragLeave.Execute(DragDropEvent);
	}
	SBorder::OnDragLeave(DragDropEvent);
}

SLATE_API FReply SButton::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnSlateButtonDragOver.IsBound())
	{
		return OnSlateButtonDragOver.Execute(MyGeometry, DragDropEvent);
	}
	else
	{
		return SBorder::OnDragOver(MyGeometry, DragDropEvent);
	}
}

SLATE_API FReply SButton::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnSlateButtonDrop.IsBound())
	{
		return OnSlateButtonDrop.Execute(MyGeometry, DragDropEvent);
	}
	else
	{
		return SBorder::OnDrop(MyGeometry, DragDropEvent);
	}
}


void SButton::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	const bool bWasHovered = IsHovered();
	
	// Call parent implementation
	SBorder::OnMouseLeave( MouseEvent );

	// If we're setup to click on mouse-down, then we never capture the mouse and may not receive a
	// mouse up event, so we need to make sure our pressed state is reset properly here
	if ( ClickMethod == EButtonClickMethod::MouseDown || IsPreciseTapOrClick(MouseEvent) )
	{
		Release();
	}

	if (bWasHovered && !IsHovered())
	{
		ExecuteHoverStateChanged(true);
	}
}

void SButton::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	Release();
}

FReply SButton::ExecuteOnClick()
{
	PlayClickedSound();
	if (OnClicked.IsBound())
	{
		FReply Reply = OnClicked.Execute();
#if WITH_ACCESSIBILITY
		// @TODOAccessibility: This should pass the Id of the user that clicked the button but we don't want to change the regular Slate API just yet
		FSlateApplicationBase::Get().GetAccessibleMessageHandler()->OnWidgetEventRaised(FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(AsShared(), EAccessibleEvent::Activate));
#endif
		return Reply;
	}
	else
	{
		return FReply::Handled();
	}
}

void SButton::Press()
{
	if ( !bIsPressed )
	{
		bIsPressed = true;
		PlayPressedSound();
		OnPressed.ExecuteIfBound();
		UpdatePressStateChanged();
	}
}

void SButton::Release()
{
	if ( bIsPressed )
	{
		bIsPressed = false;
		OnReleased.ExecuteIfBound();
		UpdatePressStateChanged();
	}
}

void SButton::UpdatePressStateChanged()
{
	UpdatePadding();
	UpdateBorderImage();
	UpdateForegroundColor();
}

bool SButton::IsInteractable() const
{
	return IsEnabled();
}

TEnumAsByte<EButtonClickMethod::Type> SButton::GetClickMethodFromInputType(const FPointerEvent& MouseEvent) const
{
	if (MouseEvent.IsTouchEvent())
	{
		switch (TouchMethod)
		{
			case EButtonTouchMethod::Down:
				return EButtonClickMethod::MouseDown;
			case EButtonTouchMethod::DownAndUp:
				return EButtonClickMethod::DownAndUp;
			case EButtonTouchMethod::PreciseTap:
				return EButtonClickMethod::PreciseClick;
		}
	}

	return ClickMethod;
}

bool SButton::IsPreciseTapOrClick(const FPointerEvent& MouseEvent) const
{
	return GetClickMethodFromInputType(MouseEvent) == EButtonClickMethod::PreciseClick;
}

void SButton::PlayPressedSound() const
{
	FSlateApplication::Get().PlaySound( PressedSound );
}

void SButton::PlayClickedSound() const
{
	FSlateApplication::Get().PlaySound( ClickedSound );
}

void SButton::PlayHoverSound() const
{
	FSlateApplication::Get().PlaySound( HoveredSound );
}

FVector2D SButton::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	// When there is no widget in the button, it sizes itself based on
	// the border image specified by the style.
	if (ChildSlot.GetWidget() == SNullWidget::NullWidget)
	{
		return FVector2D(GetBorderImage()->ImageSize);
	}
	else
	{
		return SBorder::ComputeDesiredSize(LayoutScaleMultiplier);
	}
}

void SButton::SetContentPadding(TAttribute<FMargin> InContentPadding)
{
	ContentPaddingAttribute.Assign(*this, MoveTemp(InContentPadding));
}

void SButton::SetHoveredSound(TOptional<FSlateSound> InHoveredSound)
{
	HoveredSound = InHoveredSound.Get(Style->HoveredSlateSound);
}

void SButton::SetPressedSound(TOptional<FSlateSound> InPressedSound)
{
	PressedSound = InPressedSound.Get(Style->PressedSlateSound);
}

void SButton::SetClickedSound(TOptional<FSlateSound> InClickedSound)
{
	ClickedSound = InClickedSound.Get(Style->ClickedSlateSound);
}

void SButton::SetOnClicked(FOnClicked InOnClicked)
{
	OnClicked = InOnClicked;
}

void SButton::SetOnHovered(FSimpleDelegate InOnHovered)
{
	OnHovered = InOnHovered;
}

void SButton::SetOnUnhovered(FSimpleDelegate InOnUnhovered)
{
	OnUnhovered = InOnUnhovered;
}

void SButton::SetOnFocusReceived(FSimpleDelegate InOnFocusReceived)
{
	OnReceivedFocus = InOnFocusReceived;
}

void SButton::SetOnFocusLost(FSimpleDelegate InOnFocusLost)
{
	OnLostFocus = InOnFocusLost;
}

void SButton::ExecuteHoverStateChanged(bool bPlaySound)
{
	if (IsHovered())
	{
		if (bPlaySound)
		{
			PlayHoverSound();
		}
		OnHovered.ExecuteIfBound();
	}
	else
	{
		OnUnhovered.ExecuteIfBound();
	}
}

void SButton::SetButtonStyle(const FButtonStyle* InButtonStyle)
{
	if (InButtonStyle == nullptr)
	{
		ensureAlwaysMsgf(false, TEXT("The Style is not valid."));
		return;
	}

	/* Get pointer to the button Style */
	Style = InButtonStyle;

	if (!bIsStyleNormalPaddingOverridden)
	{
		NormalPaddingAttribute.Set(*this, Style->NormalPadding);
	}

	if (!bIsStylePressedPaddingOverridden)
	{
		PressedPaddingAttribute.Set(*this, Style->PressedPadding);
	}

	HoveredSound = Style->HoveredSlateSound;
	PressedSound = Style->PressedSlateSound;
	ClickedSound = Style->ClickedSlateSound;

	UpdatePadding();
	UpdateShowDisabledEffect();
	UpdateBorderImage(); // Must be after UpdateShowDisabledEffect()
	UpdateForegroundColor();
	UpdateDisabledForegroundColor();
}

void SButton::SetClickMethod(EButtonClickMethod::Type InClickMethod)
{
	ClickMethod = InClickMethod;
}

void SButton::SetTouchMethod(EButtonTouchMethod::Type InTouchMethod)
{
	TouchMethod = InTouchMethod;
}

void SButton::SetPressMethod(EButtonPressMethod::Type InPressMethod)
{
	PressMethod = InPressMethod;
}

SLATE_API void SButton::SetAllowDragDrop(bool bInAllowDragDrop)
{
	bAllowDragDrop = bInAllowDragDrop;
}

#if !UE_BUILD_SHIPPING
void SButton::SimulateClick()
{
	ExecuteOnClick();
}
#endif // !UE_BUILD_SHIPPING

FSlateColor SButton::GetDisabledForegroundColor() const
{
	return Style->DisabledForeground;
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SButton::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleButton(SharedThis(this)));
}
#endif
