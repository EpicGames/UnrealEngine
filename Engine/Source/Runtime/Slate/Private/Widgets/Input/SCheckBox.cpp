// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

SLATE_IMPLEMENT_WIDGET(SCheckBox)
void SCheckBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, PaddingOverrideAttribute, EInvalidateWidgetReason::None);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ForegroundColorOverrideAttribute, EInvalidateWidgetReason::None);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BorderBackgroundColorOverrideAttribute, EInvalidateWidgetReason::None);
}

SCheckBox::SCheckBox()
	: PaddingOverrideAttribute(*this)
	, ForegroundColorOverrideAttribute(*this, FSlateColor::UseStyle())
	, BorderBackgroundColorOverrideAttribute(*this)
{
#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

SCheckBox::~SCheckBox() = default;

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SCheckBox::Construct( const SCheckBox::FArguments& InArgs )
{
	check(InArgs._Style != nullptr);
	Style = InArgs._Style;

	UncheckedImage = InArgs._UncheckedImage;
	UncheckedHoveredImage = InArgs._UncheckedHoveredImage;
	UncheckedPressedImage = InArgs._UncheckedPressedImage;

	CheckedImage = InArgs._CheckedImage;
	CheckedHoveredImage = InArgs._CheckedHoveredImage;
	CheckedPressedImage = InArgs._CheckedPressedImage;

	UndeterminedImage = InArgs._UndeterminedImage;
	UndeterminedHoveredImage = InArgs._UndeterminedHoveredImage;
	UndeterminedPressedImage = InArgs._UndeterminedPressedImage;

	BackgroundImage = InArgs._BackgroundImage;
	BackgroundHoveredImage = InArgs._BackgroundHoveredImage;
	BackgroundPressedImage = InArgs._BackgroundPressedImage;

	SetPaddingOverride(InArgs._Padding);
	SetForegroundColorOverride(InArgs._ForegroundColor);
	SetBorderBackgroundColorOverride(InArgs._BorderBackgroundColor);
	CheckBoxTypeOverride = InArgs._Type;

	HorizontalAlignment = InArgs._HAlign;
	CheckBoxImageVAlign = InArgs._CheckBoxImageVAlign;
	bCheckBoxContentUsesAutoWidth = InArgs._CheckBoxContentUsesAutoWidth;

	bIsPressed = false;
	bIsFocusable = InArgs._IsFocusable;

	BuildCheckBox(InArgs._Content.Widget);

	SetIsChecked(InArgs._IsChecked);
	OnCheckStateChanged = InArgs._OnCheckStateChanged;

	ClickMethod = InArgs._ClickMethod;
	TouchMethod = InArgs._TouchMethod;
	PressMethod = InArgs._PressMethod;

	OnGetMenuContent = InArgs._OnGetMenuContent;

	SetHoveredSound(InArgs._HoveredSoundOverride);
	SetCheckedSound(InArgs._CheckedSoundOverride);
	SetUncheckedSound(InArgs._UncheckedSoundOverride);
	SetForegroundColor(TAttribute<FSlateColor>::CreateRaw(this, &SCheckBox::GetForegroundColorInternal));
}

/**
 * See SWidget::SupportsKeyboardFocus().
 *
 * @return  True if this widget can take keyboard focus
 */
bool SCheckBox::SupportsKeyboardFocus() const
{
	// Buttons are focusable by default
	return bIsFocusable;
}

FReply SCheckBox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		bIsPressed = true;

		if (PressMethod == EButtonPressMethod::ButtonPress)
		{
			ToggleCheckedState();

			const ECheckBoxState State = GetCheckedState();
			if (State == ECheckBoxState::Checked)
			{
				PlayCheckedSound();
			}
			else if (State == ECheckBoxState::Unchecked)
			{
				PlayUncheckedSound();
			}
		}

		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);

}

FReply SCheckBox::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		const bool bWasPressed = bIsPressed;
		bIsPressed = false;

		if (PressMethod == EButtonPressMethod::ButtonRelease || (PressMethod == EButtonPressMethod::DownAndUp && bWasPressed))
		{
			ToggleCheckedState();

			const ECheckBoxState State = GetCheckedState();
			if (State == ECheckBoxState::Checked)
			{
				PlayCheckedSound();
			}
			else if (State == ECheckBoxState::Unchecked)
			{
				PlayUncheckedSound();
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/**
 * See SWidget::OnMouseButtonDown.
 *
 * @param MyGeometry The Geometry of the widget receiving the event
 * @param MouseEvent Information about the input event
 *
 * @return Whether the event was handled along with possible requests for the system to take action.
 */
FReply SCheckBox::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = true;

		EButtonClickMethod::Type InputClickMethod = GetClickMethodFromInputType(MouseEvent);

		if(InputClickMethod == EButtonClickMethod::MouseDown )
		{
			ToggleCheckedState();
			const ECheckBoxState State = GetCheckedState();
			if(State == ECheckBoxState::Checked)
			{
				PlayCheckedSound();
			}
			else if(State == ECheckBoxState::Unchecked)
			{
				PlayUncheckedSound();
			}

			// Set focus to this button, but don't capture the mouse
			return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Mouse);
		}
		else
		{
			// Capture the mouse, and also set focus to this button
			return FReply::Handled().CaptureMouse(AsShared()).SetUserFocus(AsShared(), EFocusCause::Mouse);
		}
	}
	else if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnGetMenuContent.IsBound() )
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			OnGetMenuContent.Execute(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

/**
 * See SWidget::OnMouseButtonDoubleClick.
 *
 * @param MyGeometry The Geometry of the widget receiving the event
 * @param MouseEvent Information about the input event
 *
 * @return Whether the event was handled along with possible requests for the system to take action.
 */
FReply SCheckBox::OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return OnMouseButtonDown( MyGeometry, MouseEvent );
}

/**
 * See SWidget::OnMouseButtonUp.
 *
 * @param MyGeometry The Geometry of the widget receiving the event
 * @param MouseEvent Information about the input event
 *
 * @return Whether the event was handled along with possible requests for the system to take action.
 */
FReply SCheckBox::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const EButtonClickMethod::Type InputClickMethod = GetClickMethodFromInputType(MouseEvent);
	const bool bMustBePressed = InputClickMethod == EButtonClickMethod::DownAndUp || InputClickMethod == EButtonClickMethod::PreciseClick;
	const bool bMeetsPressedRequirements = (!bMustBePressed || (bIsPressed && bMustBePressed));

	if (bMeetsPressedRequirements && ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.IsTouchEvent())))
	{
		bIsPressed = false;

		if(InputClickMethod == EButtonClickMethod::MouseDown )
		{
			// NOTE: If we're configured to click on mouse-down/precise-tap, then we never capture the mouse thus
			//       may never receive an OnMouseButtonUp() call.  We make sure that our bIsPressed
			//       state is reset by overriding OnMouseLeave().
		}
		else
		{
			const bool IsUnderMouse = MyGeometry.IsUnderLocation( MouseEvent.GetScreenSpacePosition() );
			if ( IsUnderMouse )
			{
				// If we were asked to allow the button to be clicked on mouse up, regardless of whether the user
				// pressed the button down first, then we'll allow the click to proceed without an active capture
				if(InputClickMethod == EButtonClickMethod::MouseUp || HasMouseCapture() )
				{
					ToggleCheckedState();
					const ECheckBoxState State = GetCheckedState();
					if(State == ECheckBoxState::Checked)
					{
						PlayCheckedSound();
					}
					else if(State == ECheckBoxState::Unchecked)
					{
						PlayUncheckedSound();
					}
				}
			}
		}

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void SCheckBox::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	PlayHoverSound();
	SCompoundWidget::OnMouseEnter( MyGeometry, MouseEvent );
}


void SCheckBox::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	// Call parent implementation
	SWidget::OnMouseLeave( MouseEvent );

	// If we're setup to click on mouse-down, then we never capture the mouse and may not receive a
	// mouse up event, so we need to make sure our pressed state is reset properly here
	if( ClickMethod == EButtonClickMethod::MouseDown || IsPreciseTapOrClick(MouseEvent) )
	{
		bIsPressed = false;
	}
}


bool SCheckBox::IsInteractable() const
{
	return IsEnabled();
}

FSlateColor SCheckBox::GetForegroundColorInternal() const
{
	FSlateColor UserColor = ForegroundColorOverrideAttribute.Get();

	if (UserColor == FSlateColor::UseStyle())
	{
		ECheckBoxState State = GetCheckedState();

		switch (State)
		{
		case ECheckBoxState::Unchecked:
			return bIsPressed ? Style->PressedForeground : IsHovered() ? Style->HoveredForeground : Style->ForegroundColor;
		case ECheckBoxState::Checked:
			return bIsPressed ? Style->CheckedPressedForeground : IsHovered() ? Style->CheckedHoveredForeground : Style->CheckedForeground;
		default:
		case ECheckBoxState::Undetermined:
			return Style->UndeterminedForeground;
		}
	}

	return UserColor;
}

/**
 * Gets the check image to display for the current state of the check box
 * @return	The name of the image to display
 */
const FSlateBrush* SCheckBox::OnGetCheckImage() const
{
	ECheckBoxState State = GetCheckedState();

	const FSlateBrush* ImageToUse;
	switch( State )
	{
		case ECheckBoxState::Unchecked:
			ImageToUse = IsPressed() ? GetUncheckedPressedImage() : ( IsHovered() ? GetUncheckedHoveredImage() : GetUncheckedImage() );
			break;
	
		case ECheckBoxState::Checked:
			ImageToUse = IsPressed() ? GetCheckedPressedImage() : ( IsHovered() ? GetCheckedHoveredImage() : GetCheckedImage() );
			break;
	
		default:
		case ECheckBoxState::Undetermined:
			ImageToUse = IsPressed() ? GetUndeterminedPressedImage() : ( IsHovered() ? GetUndeterminedHoveredImage() : GetUndeterminedImage() );
			break;
	}

	return ImageToUse;
}

const FSlateBrush* SCheckBox::OnGetBackgroundImage() const 
{
	return IsPressed() ? GetBackgroundPressedImage() : ( IsHovered() ? GetBackgroundHoveredImage() : GetBackgroundImage() );
}


ECheckBoxState SCheckBox::GetCheckedState() const
{
	return IsCheckboxChecked.Get();
}

/**
 * Toggles the checked state for this check box, fire events as needed
 */
void SCheckBox::ToggleCheckedState()
{
	const ECheckBoxState State = GetCheckedState();

	// If the current check box state is checked OR undetermined we set the check box to unchecked.
	if( State == ECheckBoxState::Checked || State == ECheckBoxState::Undetermined )
	{
		if ( !IsCheckboxChecked.IsBound() )
		{
			// When we are not bound, just toggle the current state.
			SetIsChecked( ECheckBoxState::Unchecked );
		}

		// The state of the check box changed.  Execute the delegate to notify users
		OnCheckStateChanged.ExecuteIfBound( ECheckBoxState::Unchecked );
	}
	else if( State == ECheckBoxState::Unchecked )
	{
		if ( !IsCheckboxChecked.IsBound() )
		{
			// When we are not bound, just toggle the current state.
			SetIsChecked( ECheckBoxState::Checked );
		}

		// The state of the check box changed.  Execute the delegate to notify users
		OnCheckStateChanged.ExecuteIfBound( ECheckBoxState::Checked );
	}

#if WITH_ACCESSIBILITY
	// @TODOAccessibility: Technically we should pass the Id of the user that toggled the checkbox, but we don't want to change the Slate API as much as possible
	FSlateApplicationBase::Get().GetAccessibleMessageHandler()->OnWidgetEventRaised(
		FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(
			AsShared(),
			EAccessibleEvent::Activate,
			State == ECheckBoxState::Checked,
			IsChecked()
		));
#endif
}

void SCheckBox::SetIsChecked(TAttribute<ECheckBoxState> InIsChecked)
{
	IsCheckboxChecked = MoveTemp(InIsChecked);
}

TEnumAsByte<EButtonClickMethod::Type> SCheckBox::GetClickMethodFromInputType(const FPointerEvent& MouseEvent) const
{
	if (MouseEvent.IsTouchEvent())
	{
		switch (TouchMethod)
		{
		case EButtonTouchMethod::Down:
			return EButtonClickMethod::MouseDown;
		case EButtonTouchMethod::DownAndUp:
			return EButtonClickMethod::DownAndUp;
		case EButtonTouchMethod::PreciseTap:
			return EButtonClickMethod::PreciseClick;
		}
	}

	return ClickMethod;
}

bool SCheckBox::IsPreciseTapOrClick(const FPointerEvent& MouseEvent) const
{
	return GetClickMethodFromInputType(MouseEvent) == EButtonClickMethod::PreciseClick;
}

void SCheckBox::PlayCheckedSound() const
{
	FSlateApplication::Get().PlaySound( GetCheckedSound() );
}

void SCheckBox::PlayUncheckedSound() const
{
	FSlateApplication::Get().PlaySound( GetUncheckedSound() );
}

void SCheckBox::PlayHoverSound() const
{
	FSlateApplication::Get().PlaySound( GetHoveredSound() );
}

void SCheckBox::SetContent(const TSharedRef< SWidget >& InContent)
{
	ContentContainer->SetContent(InContent);
}

void SCheckBox::SetStyle(const FCheckBoxStyle* InStyle)
{
	Style = InStyle;

	if (Style == nullptr)
	{
		FArguments Defaults;
		Style = Defaults._Style;
	}

	check(Style);

	BuildCheckBox(ContentContainer->GetContent());
}

void SCheckBox::SetUncheckedImage(const FSlateBrush* Brush)
{
	UncheckedImage = Brush;
}

void SCheckBox::SetUncheckedHoveredImage(const FSlateBrush* Brush)
{
	UncheckedHoveredImage = Brush;
}

void SCheckBox::SetUncheckedPressedImage(const FSlateBrush* Brush)
{
	UncheckedPressedImage = Brush;
}

void SCheckBox::SetCheckedImage(const FSlateBrush* Brush)
{
	CheckedImage = Brush;
}

void SCheckBox::SetCheckedHoveredImage(const FSlateBrush* Brush)
{
	CheckedHoveredImage = Brush;
}

void SCheckBox::SetCheckedPressedImage(const FSlateBrush* Brush)
{
	CheckedPressedImage = Brush;
}

void SCheckBox::SetUndeterminedImage(const FSlateBrush* Brush)
{
	UndeterminedImage = Brush;
}

void SCheckBox::SetUndeterminedHoveredImage(const FSlateBrush* Brush)
{
	UndeterminedHoveredImage = Brush;
}

void SCheckBox::SetUndeterminedPressedImage(const FSlateBrush* Brush)
{
	UndeterminedPressedImage = Brush;
}

void SCheckBox::BuildCheckBox(TSharedRef<SWidget> InContent)
{
	if (ContentContainer.IsValid())
	{
		ContentContainer->SetContent(SNullWidget::NullWidget);
	}

	ESlateCheckBoxType::Type CheckBoxType = OnGetCheckBoxType();

	if (CheckBoxType == ESlateCheckBoxType::CheckBox)
	{
		// Check boxes use a separate check button to the side of the user's content (often, a text label or icon.)
		SHorizontalBox::FSlot* ContentSlot;
		this->ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(CheckBoxImageVAlign)
			.HAlign(HAlign_Center)
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SCheckBox::OnGetBackgroundImage)
				]
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SCheckBox::OnGetCheckImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]	
			]
			+ SHorizontalBox::Slot()
			.Padding(TAttribute<FMargin>(this, &SCheckBox::OnGetPadding))
			.VAlign(VAlign_Center)
			.Expose(ContentSlot)
			[
				SAssignNew(ContentContainer, SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.Padding(0.0f)
				[
					MoveTemp(InContent)
				]
			]
		];
		if (bCheckBoxContentUsesAutoWidth)
		{
			ContentSlot->SetAutoWidth();
		}
	}
	else if (ensure(CheckBoxType == ESlateCheckBoxType::ToggleButton))
	{
		// Toggle buttons have a visual appearance that is similar to a Slate button
		this->ChildSlot
		[
			SAssignNew(ContentContainer, SBorder)
			.BorderImage(this, &SCheckBox::OnGetCheckImage)
			.Padding(this, &SCheckBox::OnGetPadding)
			.BorderBackgroundColor(this, &SCheckBox::OnGetBorderBackgroundColor)
			.HAlign(HorizontalAlignment)
			[
				MoveTemp(InContent)
			]
		];
	}
}


FMargin SCheckBox::OnGetPadding() const
{
	return bIsPaddingAttrSet ? PaddingOverrideAttribute.Get() : Style->Padding;
}

FSlateColor SCheckBox::OnGetBorderBackgroundColor() const
{
	return bIsBorderBackgroundColorOverrideAttrSet ? BorderBackgroundColorOverrideAttribute.Get() : Style->BorderBackgroundColor;
}

ESlateCheckBoxType::Type SCheckBox::OnGetCheckBoxType() const
{
	return CheckBoxTypeOverride.IsSet() ? CheckBoxTypeOverride.GetValue() : Style->CheckBoxType.GetValue();
}

const FSlateBrush* SCheckBox::GetUncheckedImage() const
{
	return UncheckedImage ? UncheckedImage : &Style->UncheckedImage;
}

const FSlateBrush* SCheckBox::GetUncheckedHoveredImage() const
{
	return UncheckedHoveredImage ? UncheckedHoveredImage : &Style->UncheckedHoveredImage;
}

const FSlateBrush* SCheckBox::GetUncheckedPressedImage() const
{
	return UncheckedPressedImage ? UncheckedPressedImage : &Style->UncheckedPressedImage;
}

const FSlateBrush* SCheckBox::GetCheckedImage() const
{
	return CheckedImage ? CheckedImage : &Style->CheckedImage;
}

const FSlateBrush* SCheckBox::GetCheckedHoveredImage() const
{
	return CheckedHoveredImage ? CheckedHoveredImage : &Style->CheckedHoveredImage;
}

const FSlateBrush* SCheckBox::GetCheckedPressedImage() const
{
	return CheckedPressedImage ? CheckedPressedImage : &Style->CheckedPressedImage;
}

const FSlateBrush* SCheckBox::GetUndeterminedImage() const
{
	return UndeterminedImage ? UndeterminedImage : &Style->UndeterminedImage;
}

const FSlateBrush* SCheckBox::GetUndeterminedHoveredImage() const
{
	return UndeterminedHoveredImage ? UndeterminedHoveredImage : &Style->UndeterminedHoveredImage;
}

const FSlateBrush* SCheckBox::GetUndeterminedPressedImage() const
{
	return UndeterminedPressedImage ? UndeterminedPressedImage : &Style->UndeterminedPressedImage;
}

const FSlateBrush* SCheckBox::GetBackgroundImage() const
{
	return BackgroundImage ? BackgroundImage : &Style->BackgroundImage;
}

const FSlateBrush* SCheckBox::GetBackgroundHoveredImage() const
{
	return BackgroundHoveredImage ? BackgroundHoveredImage : &Style->BackgroundHoveredImage;
}

const FSlateBrush* SCheckBox::GetBackgroundPressedImage() const
{
	return BackgroundPressedImage ? BackgroundPressedImage : &Style->BackgroundPressedImage;
}

void SCheckBox::SetPaddingOverride(TAttribute<FMargin> InPaddingOverride)
{
	bIsPaddingAttrSet = InPaddingOverride.IsSet();
	PaddingOverrideAttribute.Assign(*this, MoveTemp(InPaddingOverride));
}

void SCheckBox::SetForegroundColorOverride(TAttribute<FSlateColor> InForegroundColorOverride)
{
	ForegroundColorOverrideAttribute.Assign(*this, MoveTemp(InForegroundColorOverride));
}

void SCheckBox::SetBorderBackgroundColorOverride(TAttribute<FSlateColor> InBorderBackgroundColorOverride)
{
	bIsBorderBackgroundColorOverrideAttrSet = InBorderBackgroundColorOverride.IsSet();
	BorderBackgroundColorOverrideAttribute.Assign(*this, MoveTemp(InBorderBackgroundColorOverride));
}

void SCheckBox::SetHoveredSound(TOptional<FSlateSound> InHoveredSound)
{
	HoveredSoundOverride = MoveTemp(InHoveredSound);
}

FSlateSound SCheckBox::GetHoveredSound() const
{
	return HoveredSoundOverride.Get(Style->HoveredSlateSound);
}

void SCheckBox::SetCheckedSound(TOptional<FSlateSound> InCheckedSound)
{
	CheckedSoundOverride = MoveTemp(InCheckedSound);
}

FSlateSound SCheckBox::GetCheckedSound() const
{
	return CheckedSoundOverride.Get(Style->CheckedSlateSound);
}

void SCheckBox::SetUncheckedSound(TOptional<FSlateSound> InUncheckedSound)
{
	UncheckedSoundOverride = MoveTemp(InUncheckedSound);
}

FSlateSound SCheckBox::GetUncheckedSound() const
{
	return UncheckedSoundOverride.Get(Style->UncheckedSlateSound);
}

void SCheckBox::SetClickMethod(EButtonClickMethod::Type InClickMethod)
{
	ClickMethod = InClickMethod;
}

void SCheckBox::SetTouchMethod(EButtonTouchMethod::Type InTouchMethod)
{
	TouchMethod = InTouchMethod;
}

void SCheckBox::SetPressMethod(EButtonPressMethod::Type InPressMethod)
{
	PressMethod = InPressMethod;
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SCheckBox::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleCheckBox(SharedThis(this)));
}
#endif
