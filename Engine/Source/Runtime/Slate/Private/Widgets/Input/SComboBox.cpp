// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SComboBox.h"

#include "HAL/IConsoleManager.h"

namespace SlateComboRow
{
	bool bMouseButtonHandlingV2 = true;
}

FAutoConsoleVariableRef CVarComboRowMouseButtonDownHandlingV2(
	TEXT("Slate.ComboRowMouseButtonHandlingV2"),
	SlateComboRow::bMouseButtonHandlingV2,
	TEXT("Whether to enable new mouse button event handling on SComboRow")
);
