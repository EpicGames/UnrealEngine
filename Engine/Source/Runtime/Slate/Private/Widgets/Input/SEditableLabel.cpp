// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SEditableLabel.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableText.h"


#define LOCTEXT_NAMESPACE "SEditableLabel"


/* SEditableLabel interface
 *****************************************************************************/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SEditableLabel::Construct(const FArguments& InArgs)
{
	CanEditAttribute = InArgs._CanEdit;
	OnTextChanged = InArgs._OnTextChanged;
	TextAttribute = InArgs._Text;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SAssignNew(TextBlock, STextBlock)
					.ColorAndOpacity(InArgs._ColorAndOpacity)
					.Font(InArgs._Font)
					.HighlightColor(InArgs._HighlightColor)
					.HighlightShape(InArgs._HighlightShape)
					.HighlightText(InArgs._HighlightText)
					.MinDesiredWidth(InArgs._MinDesiredWidth)
					.ShadowColorAndOpacity(InArgs._ShadowColorAndOpacity)
					.ShadowOffset(InArgs._ShadowOffset)
					.TextStyle(InArgs._TextStyle)
					.Text(InArgs._Text)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center) 
			[
				SAssignNew(EditableText, SEditableText)
					.ClearKeyboardFocusOnCommit(true)
					.ColorAndOpacity(InArgs._ColorAndOpacity)
					.Font(InArgs._Font)
					.MinDesiredWidth(InArgs._MinDesiredWidth)
					.OnTextCommitted(this, &SEditableLabel::HandleEditableTextTextCommitted)
					.RevertTextOnEscape(true)
					.SelectAllTextOnCommit(false)
					.SelectAllTextWhenFocused(true)
					.Style(InArgs._EditableTextStyle)
					.Text(InArgs._Text)
					.VirtualKeyboardType(EKeyboardType::Keyboard_Number)
					.Visibility(EVisibility::Collapsed)
			]
	];
}


void SEditableLabel::EnterTextMode()
{
	if (!CanEditAttribute.Get())
	{
		return;
	}

	TextBlock->SetVisibility(EVisibility::Collapsed);
	EditableText->SetVisibility(EVisibility::Visible);
	FSlateApplication::Get().SetAllUserFocus(EditableText);
}
	

void SEditableLabel::ExitTextMode()
{
	TextBlock->SetVisibility(EVisibility::Visible);
	EditableText->SetVisibility(EVisibility::Collapsed);
	FSlateApplication::Get().SetAllUserFocus(AsShared());
}


/* SWidget interface
 *****************************************************************************/

bool SEditableLabel::HasKeyboardFocus() const
{
	// this label is considered focused when we are typing it text.
	return SCompoundWidget::HasKeyboardFocus() ||
		(EditableText.IsValid() && EditableText->HasKeyboardFocus());
}


FReply SEditableLabel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();

	if (Key == EKeys::Escape)
	{
		ExitTextMode();
		return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Navigation);
	}

	if ((Key == EKeys::F2) && CanEditAttribute.Get())
	{
		EnterTextMode();
		return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), EFocusCause::Navigation);
	}

	return FReply::Unhandled();
}


FReply SEditableLabel::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		return HandleTextBlockDoubleClicked();
	}

	return FReply::Unhandled();
}

bool SEditableLabel::SupportsKeyboardFocus() const
{
	return CanEditAttribute.Get();
}


/* SEditableLabel callbacks
 *****************************************************************************/

void SEditableLabel::HandleEditableTextTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	ExitTextMode();
	OnTextChanged.ExecuteIfBound(NewText);
}


FReply SEditableLabel::HandleTextBlockDoubleClicked()
{
	EnterTextMode();
	return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), EFocusCause::Navigation);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
