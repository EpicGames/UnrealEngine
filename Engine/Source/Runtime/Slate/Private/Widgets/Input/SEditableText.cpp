// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SEditableText.h"
#include "Framework/Text/TextEditHelper.h"
#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Widgets/Text/SlateEditableTextLayout.h"
#include "Widgets/Text/SlateTextBlockLayout.h"
#include "Types/ReflectionMetadata.h"
#include "Types/TrackedMetaData.h"

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

SLATE_IMPLEMENT_WIDGET(SEditableText)
void SEditableText::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	struct FInvalidation
	{
		static void SynchronizeTextStyle(SWidget& OwningWidget)
		{
			SEditableText& EditableText = static_cast<SEditableText&>(OwningWidget);
			EditableText.SynchronizeTextStyle();
		}
		static void UpdateMarshaller(SWidget& OwningWidget)
		{
			SEditableText& EditableText = static_cast<SEditableText&>(OwningWidget);
			EditableText.PlainTextMarshaller->MakeDirty();
		}
	};

	FSlateEditableTextLayout::RegisterNestedAttributes<&SEditableText::SlateEditableTextLayout>(AttributeInitializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredWidthAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bIsPasswordAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateMarshaller));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, FontAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::SynchronizeTextStyle));

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ColorAndOpacityAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::SynchronizeTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BackgroundImageSelectedAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::SynchronizeTextStyle));
}

SEditableText::SEditableText()
	: SlateEditableTextLayout(*this)
	, FontAttribute(*this)
	, ColorAndOpacityAttribute(*this)
	, BackgroundImageSelectedAttribute(*this)
	, bIsPasswordAttribute(*this, false)
	, MinDesiredWidthAttribute(*this, 0.0f)
{
#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = false;
#endif
}

void SEditableText::Construct( const FArguments& InArgs )
{
	bIsCaretMovedWhenGainFocus = InArgs._IsCaretMovedWhenGainFocus;
	bSelectAllTextWhenFocused = InArgs._SelectAllTextWhenFocused;
	bRevertTextOnEscape = InArgs._RevertTextOnEscape;
	bClearKeyboardFocusOnCommit = InArgs._ClearKeyboardFocusOnCommit;
	bAllowContextMenu = InArgs._AllowContextMenu;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	OnIsTypedCharValid = InArgs._OnIsTypedCharValid;
	OnBeginTextEditCallback = InArgs._OnBeginTextEdit;
	OnTextChangedCallback = InArgs._OnTextChanged;
	OnTextCommittedCallback = InArgs._OnTextCommitted;
	OnCursorMovedWithSelectionCallback = InArgs._OnCursorMovedWithSelection;
	SetMinDesiredWidth(InArgs._MinDesiredWidth);
	bSelectAllTextOnCommit = InArgs._SelectAllTextOnCommit;
	bSelectWordOnMouseDoubleClick = InArgs._SelectWordOnMouseDoubleClick;
	VirtualKeyboardType = InArgs._VirtualKeyboardType;
	VirtualKeyboardOptions = InArgs._VirtualKeyboardOptions;
	VirtualKeyboardTrigger = InArgs._VirtualKeyboardTrigger;
	VirtualKeyboardDismissAction = InArgs._VirtualKeyboardDismissAction;
	OnKeyCharHandler = InArgs._OnKeyCharHandler;
	OnKeyDownHandler = InArgs._OnKeyDownHandler;
	bEnableIntegratedKeyboard = InArgs._EnableIntegratedKeyboard;

	// We use the given style when creating the text layout as it may not be safe to call the override delegates until we've finished being constructed
	// The first call to SynchronizeTextStyle will apply the correct overrides, and that will happen before the first paint
	check(InArgs._Style);
	FTextBlockStyle TextStyle = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	TextStyle.Font = InArgs._Style->Font;
	TextStyle.ColorAndOpacity = InArgs._Style->ColorAndOpacity;
	TextStyle.HighlightShape = InArgs._Style->BackgroundImageSelected;

	PlainTextMarshaller = FPlainTextLayoutMarshaller::Create();

	// We use a separate marshaller for the hint text, as that should never be displayed as a password
	TSharedRef<FPlainTextLayoutMarshaller> HintTextMarshaller = FPlainTextLayoutMarshaller::Create();

	SlateEditableTextLayout.Construct(InArgs._Text, TextStyle, InArgs._TextShapingMethod, InArgs._TextFlowDirection, FCreateSlateTextLayout(), PlainTextMarshaller.ToSharedRef(), MoveTemp(HintTextMarshaller));
	SlateEditableTextLayout.SetCursorBrush(InArgs._CaretImage.IsSet() ? InArgs._CaretImage : &InArgs._Style->CaretImage);
	SlateEditableTextLayout.SetCompositionBrush(InArgs._BackgroundImageComposing.IsSet() ? InArgs._BackgroundImageComposing : &InArgs._Style->BackgroundImageComposing);
	SlateEditableTextLayout.SetDebugSourceInfo(TAttribute<FString>::Create(TAttribute<FString>::FGetter::CreateLambda([this]{ return FReflectionMetaData::GetWidgetDebugInfo(this); })));
	SlateEditableTextLayout.SetOverflowPolicy(InArgs._OverflowPolicy);
	SlateEditableTextLayout.SetFontFacesLoadingPaintPolicy(InArgs._FontFacesLoadingPaintPolicy);
	if (InArgs._OnAllFontFacesFinishLoading.IsBound())
	{
		SlateEditableTextLayout.SetOnAllFontFacesFinishLoadingDelegate(InArgs._OnAllFontFacesFinishLoading);
	}

	SetIsReadOnly(InArgs._IsReadOnly);
	SetIsPassword(InArgs._IsPassword);
	SetHintText(InArgs._HintText);
	SetHintTextOpacity(InArgs._HintTextOpacity);
	SetSearchText(InArgs._SearchText);
	SetFont(InArgs._Font);
	SetBackgroundImageSelected(InArgs._BackgroundImageSelected);
	SetColorAndOpacity(InArgs._ColorAndOpacity);
	SetJustification(InArgs._Justification);

	// build context menu extender
	MenuExtender = MakeShareable(new FExtender());
	MenuExtender->AddMenuExtension("EditText", EExtensionHook::Before, TSharedPtr<FUICommandList>(), InArgs._ContextMenuExtender);

	AddMetadata(MakeShared<FTrackedMetaData>(this, FName("EditableText")));
}

void SEditableText::SetText(TAttribute<FText> InNewText)
{
	SlateEditableTextLayout.SetText(MoveTemp(InNewText));
}

FText SEditableText::GetText() const
{
	return SlateEditableTextLayout.GetText();
}

bool SEditableText::SetEditableText(const FText& InNewText)
{
	return SlateEditableTextLayout.SetEditableText(InNewText);
}

void SEditableText::SetFont(TAttribute<FSlateFontInfo> InNewFont)
{
	bIsFontAttrSet = InNewFont.IsSet();
	FontAttribute.Assign(*this, MoveTemp(InNewFont));
}

FSlateFontInfo SEditableText::GetFont() const
{
	return bIsFontAttrSet ? FontAttribute.Get() : SlateEditableTextLayout.GetTextStyle().Font;
}

void SEditableText::SetTextStyle( const FEditableTextStyle& InNewTextStyle )
{
	bIsFontAttrSet = true;
	FontAttribute.Assign(*this, InNewTextStyle.Font);
	bIsColorAndOpacityAttrSet = true;
	ColorAndOpacityAttribute.Assign(*this, InNewTextStyle.ColorAndOpacity);
	bIsBackgroundImageSelectedAttrSet = true;
	BackgroundImageSelectedAttribute.Assign(*this, &InNewTextStyle.BackgroundImageSelected);
}

void SEditableText::SetTextBlockStyle(const FTextBlockStyle* InTextStyle)
{
	if (InTextStyle)
	{
		SlateEditableTextLayout.SetTextStyle(*InTextStyle);
	}
}

void SEditableText::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SlateEditableTextLayout.Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

int32 SEditableText::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const FTextBlockStyle& EditableTextStyle = SlateEditableTextLayout.GetTextStyle();
	const FLinearColor ForegroundColor = EditableTextStyle.ColorAndOpacity.GetColor(InWidgetStyle);

	FWidgetStyle TextWidgetStyle = FWidgetStyle(InWidgetStyle)
		.SetForegroundColor(ForegroundColor);

	LayerId = SlateEditableTextLayout.OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, TextWidgetStyle, ShouldBeEnabled(bParentEnabled));

	return LayerId;
}

void SEditableText::CacheDesiredSize(float LayoutScaleMultiplier)
{
	SynchronizeTextStyle();
	SlateEditableTextLayout.CacheDesiredSize(LayoutScaleMultiplier);
	SWidget::CacheDesiredSize(LayoutScaleMultiplier);
}

FVector2D SEditableText::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D TextLayoutSize = SlateEditableTextLayout.ComputeDesiredSize(LayoutScaleMultiplier);
	TextLayoutSize.X = FMath::Max(TextLayoutSize.X, MinDesiredWidthAttribute.Get());
	return TextLayoutSize;
}

FChildren* SEditableText::GetChildren()
{
	return SlateEditableTextLayout.GetChildren();
}

void SEditableText::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	SlateEditableTextLayout.OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

FReply SEditableText::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if ( DragDropOp.IsValid() )
	{
		if ( DragDropOp->HasText() )
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SEditableText::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if ( DragDropOp.IsValid() )
	{
		if ( DragDropOp->HasText() )
		{
			SlateEditableTextLayout.SetText(FText::FromString(DragDropOp->GetText()));
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SEditableText::SupportsKeyboardFocus() const
{
	return true;
}

FReply SEditableText::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	SlateEditableTextLayout.HandleFocusReceived(InFocusEvent);
	return FReply::Handled();
}

void SEditableText::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	SlateEditableTextLayout.HandleFocusLost(InFocusEvent);
}

FReply SEditableText::OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent )
{
	FReply Reply = FReply::Unhandled();

	// First call the user defined key handler, there might be overrides to normal functionality
	if (OnKeyCharHandler.IsBound())
	{
		Reply = OnKeyCharHandler.Execute(MyGeometry, InCharacterEvent);
	}

	if (!Reply.IsEventHandled())
	{
		Reply = SlateEditableTextLayout.HandleKeyChar(InCharacterEvent);
	}

	return Reply;
}

FReply SEditableText::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	FReply Reply = FReply::Unhandled();

	// First call the user defined key handler, there might be overrides to normal functionality
	if (OnKeyDownHandler.IsBound())
	{
		Reply = OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
	}

	if (!Reply.IsEventHandled())
	{
		Reply = SlateEditableTextLayout.HandleKeyDown(InKeyEvent);

		if (!Reply.IsEventHandled())
		{
			Reply = SWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}
	}

	return Reply;
}

FReply SEditableText::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	return SlateEditableTextLayout.HandleKeyUp(InKeyEvent);
}

FReply SEditableText::OnMouseButtonDown( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return SlateEditableTextLayout.HandleMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply SEditableText::OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return SlateEditableTextLayout.HandleMouseButtonUp(InMyGeometry, InMouseEvent);
}

FReply SEditableText::OnMouseMove( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return SlateEditableTextLayout.HandleMouseMove(InMyGeometry, InMouseEvent);
}

FReply SEditableText::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return SlateEditableTextLayout.HandleMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

FCursorReply SEditableText::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return FCursorReply::Cursor( EMouseCursor::TextEditBeam );
}

const FSlateBrush* SEditableText::GetFocusBrush() const
{
	return nullptr;
}

bool SEditableText::IsInteractable() const
{
	return IsEnabled();
}

bool SEditableText::ComputeVolatility() const
{
	return SWidget::ComputeVolatility()
		|| SlateEditableTextLayout.ComputeVolatility();
}

void SEditableText::SetHintText(TAttribute<FText> InHintText )
{
	SlateEditableTextLayout.SetHintText(MoveTemp(InHintText));
}

FText SEditableText::GetHintText() const
{
	return SlateEditableTextLayout.GetHintText();
}

void SEditableText::SetHintTextOpacity(TAttribute<float> InHintTextOpacity)
{
	SlateEditableTextLayout.SetHintTextOpacity(MoveTemp(InHintTextOpacity));
}

float SEditableText::GetHintTextOpacity() const
{
	return SlateEditableTextLayout.GetHintTextOpacity();
}

void SEditableText::SetSearchText(TAttribute<FText> InSearchText)
{
	SlateEditableTextLayout.SetSearchText(MoveTemp(InSearchText));
}

FText SEditableText::GetSearchText() const
{
	return SlateEditableTextLayout.GetSearchText();
}

void SEditableText::SetIsReadOnly(TAttribute<bool> InIsReadOnly)
{
	SlateEditableTextLayout.SetIsReadOnly(MoveTemp(InIsReadOnly));
}

void SEditableText::SetIsPassword(TAttribute<bool> InIsPassword)
{
	PlainTextMarshaller->SetIsPassword(InIsPassword);
	bIsPasswordAttribute.Assign(*this, MoveTemp(InIsPassword));
}

void SEditableText::SetColorAndOpacity(TAttribute<FSlateColor> Color)
{
	bIsColorAndOpacityAttrSet = Color.IsSet();
	ColorAndOpacityAttribute.Assign(*this, MoveTemp(Color));
}

void SEditableText::SetBackgroundImageSelected(TAttribute<const FSlateBrush*> InBackgroundImageSelected)
{
	bIsBackgroundImageSelectedAttrSet = InBackgroundImageSelected.IsSet();
	BackgroundImageSelectedAttribute.Assign(*this, MoveTemp(InBackgroundImageSelected));
}

void SEditableText::SetMinDesiredWidth(TAttribute<float> InMinDesiredWidth)
{
	MinDesiredWidthAttribute.Assign(*this, MoveTemp(InMinDesiredWidth));
}

void SEditableText::SetIsCaretMovedWhenGainFocus(TAttribute<bool> InIsCaretMovedWhenGainFocus)
{
	bIsCaretMovedWhenGainFocus = MoveTemp(InIsCaretMovedWhenGainFocus);
}

void SEditableText::SetSelectAllTextWhenFocused(TAttribute<bool> InSelectAllTextWhenFocused)
{
	bSelectAllTextWhenFocused = MoveTemp(InSelectAllTextWhenFocused);
}

void SEditableText::SetRevertTextOnEscape(TAttribute<bool> InRevertTextOnEscape)
{
	bRevertTextOnEscape = MoveTemp(InRevertTextOnEscape);
}

void SEditableText::SetClearKeyboardFocusOnCommit(TAttribute<bool> InClearKeyboardFocusOnCommit)
{
	bClearKeyboardFocusOnCommit = MoveTemp(InClearKeyboardFocusOnCommit);
}

void SEditableText::SetSelectAllTextOnCommit(TAttribute<bool> InSelectAllTextOnCommit)
{
	bSelectAllTextOnCommit = MoveTemp(InSelectAllTextOnCommit);
}

void SEditableText::SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick)
{
	bSelectWordOnMouseDoubleClick = MoveTemp(InSelectWordOnMouseDoubleClick);
}

void SEditableText::SetJustification(TAttribute<ETextJustify::Type> InJustification)
{
	SlateEditableTextLayout.SetJustification(MoveTemp(InJustification));
}

void SEditableText::SetAllowContextMenu(TAttribute<bool> InAllowContextMenu)
{
	bAllowContextMenu = MoveTemp(InAllowContextMenu);
}

void SEditableText::SetEnableIntegratedKeyboard(TAttribute<bool> InEnableIntegratedKeyboard)
{
	bEnableIntegratedKeyboard = MoveTemp(InEnableIntegratedKeyboard);
}

void SEditableText::SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction)
{
	VirtualKeyboardDismissAction = MoveTemp(InVirtualKeyboardDismissAction);
}

void SEditableText::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	SlateEditableTextLayout.SetTextShapingMethod(InTextShapingMethod);
}

void SEditableText::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	SlateEditableTextLayout.SetTextFlowDirection(InTextFlowDirection);
}

void SEditableText::SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy)
{
	SlateEditableTextLayout.SetOverflowPolicy(InOverflowPolicy);
}

void SEditableText::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	SlateEditableTextLayout.SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
}

bool SEditableText::AnyTextSelected() const
{
	return SlateEditableTextLayout.AnyTextSelected();
}

void SEditableText::SelectAllText()
{
	SlateEditableTextLayout.SelectAllText();
}

void SEditableText::ClearSelection()
{
	SlateEditableTextLayout.ClearSelection();
}

FText SEditableText::GetSelectedText() const
{
	return SlateEditableTextLayout.GetSelectedText();
}

void SEditableText::GoTo(const FTextLocation& NewLocation)
{
	SlateEditableTextLayout.GoTo(NewLocation);
}

void SEditableText::GoTo(const ETextLocation NewLocation)
{
	SlateEditableTextLayout.GoTo(NewLocation);
}

void SEditableText::ScrollTo(const FTextLocation& NewLocation)
{
	SlateEditableTextLayout.ScrollTo(NewLocation);
}

void SEditableText::ScrollTo(const ETextLocation NewLocation)
{
	SlateEditableTextLayout.ScrollTo(NewLocation);
}

void SEditableText::BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase, const bool InReverse)
{
	SlateEditableTextLayout.BeginSearch(InSearchText, InSearchCase, InReverse);
}

void SEditableText::AdvanceSearch(const bool InReverse)
{
	SlateEditableTextLayout.AdvanceSearch(InReverse);
}

void SEditableText::EnableTextInputMethodContext()
{
	SlateEditableTextLayout.EnableTextInputMethodContext();
}

FTextSelection SEditableText::GetSelection() const
{
	return SlateEditableTextLayout.GetSelection();
}

void SEditableText::SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation)
{
	SlateEditableTextLayout.SelectText(InSelectionStart, InCursorLocation);
}

void SEditableText::SynchronizeTextStyle()
{
	// Has the style used for this editable text changed?
	bool bTextStyleChanged = false;
	FTextBlockStyle NewTextStyle = SlateEditableTextLayout.GetTextStyle();

	// Sync from the font override
	if (bIsFontAttrSet)
	{
		const FSlateFontInfo& NewFontInfo = FontAttribute.Get();
		if (!NewTextStyle.Font.IsIdenticalTo(NewFontInfo))
		{
			NewTextStyle.Font = NewFontInfo;
			bTextStyleChanged = true;
		}
	}

	// Sync from the color override
	if (bIsColorAndOpacityAttrSet)
	{
		const FSlateColor& NewColorAndOpacity = ColorAndOpacityAttribute.Get();
		if (NewTextStyle.ColorAndOpacity != NewColorAndOpacity)
		{
			NewTextStyle.ColorAndOpacity = NewColorAndOpacity;
			bTextStyleChanged = true;
		}
	}

	// Sync from the highlight shape override
	if (bIsBackgroundImageSelectedAttrSet)
	{
		const FSlateBrush* NewSelectionBrush = BackgroundImageSelectedAttribute.Get();
		if (NewSelectionBrush && NewTextStyle.HighlightShape != *NewSelectionBrush)
		{
			NewTextStyle.HighlightShape = *NewSelectionBrush;
			bTextStyleChanged = true;
		}
	}

	if (bTextStyleChanged)
	{
		SlateEditableTextLayout.SetTextStyle(NewTextStyle);
		SlateEditableTextLayout.ForceRefreshTextLayout(SlateEditableTextLayout.GetEditableText());
	}
}

bool SEditableText::IsWidgetConstructed() const
{
	return IsConstructed();
}

bool SEditableText::IsTextReadOnly() const
{
	return SlateEditableTextLayout.GetIsReadOnly();
}

bool SEditableText::IsTextPassword() const
{
	return bIsPasswordAttribute.Get();
}

bool SEditableText::IsMultiLineTextEdit() const
{
	return false;
}

bool SEditableText::IsIntegratedKeyboardEnabled() const
{
	return bEnableIntegratedKeyboard.Get(false);
}

bool SEditableText::ShouldJumpCursorToEndWhenFocused() const
{
	return bIsCaretMovedWhenGainFocus.Get(false);
}

bool SEditableText::ShouldSelectAllTextWhenFocused() const
{
	return bSelectAllTextWhenFocused.Get(false);
}

bool SEditableText::ShouldClearTextSelectionOnFocusLoss() const
{
	return true;
}

bool SEditableText::ShouldRevertTextOnEscape() const
{
	return bRevertTextOnEscape.Get(false);
}

bool SEditableText::ShouldClearKeyboardFocusOnCommit() const
{
	return bClearKeyboardFocusOnCommit.Get(false);
}

bool SEditableText::ShouldSelectAllTextOnCommit() const
{
	return bSelectAllTextOnCommit.Get(false);
}

bool SEditableText::ShouldSelectWordOnMouseDoubleClick() const
{
	return bSelectWordOnMouseDoubleClick.Get(true);
}

bool SEditableText::CanInsertCarriageReturn() const
{
	return false;
}

bool SEditableText::CanTypeCharacter(const TCHAR InChar) const
{
	if (OnIsTypedCharValid.IsBound())
	{
		return OnIsTypedCharValid.Execute(InChar);
	}

	return InChar != TEXT('\t');
}

void SEditableText::EnsureActiveTick()
{
	TSharedPtr<FActiveTimerHandle> ActiveTickTimerPin = ActiveTickTimer.Pin();
	if (ActiveTickTimerPin.IsValid())
	{
		return;
	}

	auto DoActiveTick = [this](double InCurrentTime, float InDeltaTime) -> EActiveTimerReturnType
	{
		// Continue if we still have focus, otherwise treat as a fire-and-forget Tick() request
		const bool bShouldAppearFocused = HasKeyboardFocus() || SlateEditableTextLayout.HasActiveContextMenu();
		return (bShouldAppearFocused) ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
	};

	const float TickPeriod = EditableTextDefs::BlinksPerSecond * 0.5f;
	ActiveTickTimer = RegisterActiveTimer(TickPeriod, FWidgetActiveTimerDelegate::CreateLambda(DoActiveTick));
}

EKeyboardType SEditableText::GetVirtualKeyboardType() const
{
	return VirtualKeyboardType.Get();
}

FVirtualKeyboardOptions SEditableText::GetVirtualKeyboardOptions() const
{
	return VirtualKeyboardOptions;
}

EVirtualKeyboardTrigger SEditableText::GetVirtualKeyboardTrigger() const
{
	return VirtualKeyboardTrigger.Get();
}

EVirtualKeyboardDismissAction SEditableText::GetVirtualKeyboardDismissAction() const
{
	return VirtualKeyboardDismissAction.Get();
}

TSharedRef<SWidget> SEditableText::GetSlateWidget()
{
	return AsShared();
}

TSharedPtr<SWidget> SEditableText::GetSlateWidgetPtr()
{
	if (DoesSharedInstanceExist())
	{
		return AsShared();
	}
	return nullptr;
}

TSharedPtr<SWidget> SEditableText::BuildContextMenuContent() const
{
	if (!bAllowContextMenu.Get())
	{
		return nullptr;
	}

	if (OnContextMenuOpening.IsBound())
	{
		return OnContextMenuOpening.Execute();
	}

	return SlateEditableTextLayout.BuildDefaultContextMenu(MenuExtender);
}

void SEditableText::OnBeginTextEdit(const FText& InText)
{
	OnBeginTextEditCallback.ExecuteIfBound(InText);
}

void SEditableText::OnTextChanged(const FText& InText)
{
	OnTextChangedCallback.ExecuteIfBound(InText);
}

void SEditableText::OnTextCommitted(const FText& InText, const ETextCommit::Type InTextAction)
{
	OnTextCommittedCallback.ExecuteIfBound(InText, InTextAction);
}

void SEditableText::OnCursorMoved(const FTextLocation& InLocation)
{
	FTextSelection TextSelection = GetSelection();

	OnCursorMovedWithSelectionCallback.ExecuteIfBound(InLocation, TextSelection);
	Invalidate(EInvalidateWidgetReason::Layout);
}

float SEditableText::UpdateAndClampHorizontalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride)
{
	return SlateEditableTextLayout.GetScrollOffset().X;
}

float SEditableText::UpdateAndClampVerticalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride)
{
	return 0.0f;
}

ETextJustify::Type SEditableText::GetJustification() const
{
	return SlateEditableTextLayout.GetJustification();
}

FSlateColor SEditableText::GetColorAndOpacity() const
{
	return ColorAndOpacityAttribute.Get();
}

const FSlateBrush* SEditableText::GetBackgroundImageSelected() const
{
	return BackgroundImageSelectedAttribute.Get();
}

float SEditableText::GetMinDesiredWidth() const
{
	return MinDesiredWidthAttribute.Get();
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SEditableText::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleEditableText(SharedThis(this)));
}

TOptional<FText> SEditableText::GetDefaultAccessibleText(EAccessibleType AccessibleType) const
{
	return GetHintText();
}
#endif

void SEditableText::ToggleVirtualKeyboard(bool bShow, const int32 UserIndex)
{
	SlateEditableTextLayout.ToggleVirtualKeyboard(bShow, UserIndex);
}
