// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SEditableTextBox.h"
#include "Framework/Text/TextEditHelper.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SPopUpErrorText.h"

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

SEditableTextBox::SEditableTextBox()
{
#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = false;
#endif
}

SEditableTextBox::~SEditableTextBox() = default;

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SEditableTextBox::Construct( const FArguments& InArgs )
{
	check (InArgs._Style);

	PaddingOverride = InArgs._Padding;
	FontOverride = InArgs._Font;
	ForegroundColorOverride = InArgs._ForegroundColor;
	BackgroundColorOverride = InArgs._BackgroundColor;
	ReadOnlyForegroundColorOverride = InArgs._ReadOnlyForegroundColor;
	FocusedForegroundColorOverride = InArgs._FocusedForegroundColor;
	MaximumLength = InArgs._MaximumLength;
	OnTextChanged = InArgs._OnTextChanged;
	OnVerifyTextChanged = InArgs._OnVerifyTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;
	OnCursorMovedWithSelection = InArgs._OnCursorMovedWithSelection;

	SBorder::Construct( SBorder::FArguments()
		.BorderImage( this, &SEditableTextBox::DetermineBorderImage )
		.BorderBackgroundColor( this, &SEditableTextBox::DetermineBackgroundColor )
		.ForegroundColor( this, &SEditableTextBox::DetermineForegroundColor )
		.Padding(0.f)
		[
			SAssignNew( Box, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.FillWidth(1)
			[
				SAssignNew(PaddingBox, SBox)
				.Padding( this, &SEditableTextBox::DeterminePadding )
				.VAlign(VAlign_Center)
				[
					SAssignNew( EditableText, SEditableText )
					.Text( InArgs._Text )
					.HintText( InArgs._HintText )
					.SearchText( InArgs._SearchText )
					.Font( this, &SEditableTextBox::DetermineFont )
					.IsReadOnly( InArgs._IsReadOnly )
					.IsPassword( InArgs._IsPassword )
					.IsCaretMovedWhenGainFocus( InArgs._IsCaretMovedWhenGainFocus )
					.SelectAllTextWhenFocused( InArgs._SelectAllTextWhenFocused )
					.RevertTextOnEscape( InArgs._RevertTextOnEscape )
					.ClearKeyboardFocusOnCommit( InArgs._ClearKeyboardFocusOnCommit )
					.Justification( InArgs._Justification )
					.AllowContextMenu( InArgs._AllowContextMenu )
					.OnContextMenuOpening( InArgs._OnContextMenuOpening )
					.ContextMenuExtender( InArgs._ContextMenuExtender )
					.OnBeginTextEdit(InArgs._OnBeginTextEdit)
					.OnTextChanged(this, &SEditableTextBox::OnEditableTextChanged)
					.OnTextCommitted(this, &SEditableTextBox::OnEditableTextCommitted)
					.OnCursorMovedWithSelection(this, &SEditableTextBox::OnEditableTextCursorMovedWithSelection)
					.MinDesiredWidth( InArgs._MinDesiredWidth )
					.SelectAllTextOnCommit( InArgs._SelectAllTextOnCommit )
					.SelectWordOnMouseDoubleClick(InArgs._SelectWordOnMouseDoubleClick)
					.OnKeyCharHandler( InArgs._OnKeyCharHandler )			
					.OnKeyDownHandler( InArgs._OnKeyDownHandler )
					.VirtualKeyboardType( InArgs._VirtualKeyboardType )
					.VirtualKeyboardOptions( InArgs._VirtualKeyboardOptions )
					.VirtualKeyboardTrigger( InArgs._VirtualKeyboardTrigger )
					.VirtualKeyboardDismissAction( InArgs._VirtualKeyboardDismissAction )
					.TextShapingMethod(InArgs._TextShapingMethod)
					.TextFlowDirection( InArgs._TextFlowDirection )
					.OverflowPolicy(InArgs._OverflowPolicy)
					.FontFacesLoadingPaintPolicy(InArgs._FontFacesLoadingPaintPolicy)
					.OnAllFontFacesFinishLoading(InArgs._OnAllFontFacesFinishLoading)
				]
			]
		]
	);

	SetStyle(InArgs._Style);

	ErrorReporting = InArgs._ErrorReporting;
	if ( ErrorReporting.IsValid() )
	{
		Box->AddSlot()
		.AutoWidth()
		.Padding(3,0)
		[
			ErrorReporting->AsWidget()
		];
	}
}

void SEditableTextBox::SetStyle(const FEditableTextBoxStyle* InStyle)
{
	Style = InStyle;

	if ( Style == nullptr )
	{
		FArguments Defaults;
		Style = Defaults._Style;
	}

	check(Style);

	BorderImageNormal = &Style->BackgroundImageNormal;
	BorderImageHovered = &Style->BackgroundImageHovered;
	BorderImageFocused = &Style->BackgroundImageFocused;
	BorderImageReadOnly = &Style->BackgroundImageReadOnly;

	SetTextBlockStyle(&Style->TextStyle);
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SEditableTextBox::SetTextBlockStyle(const FTextBlockStyle* InTextStyle)
{
	EditableText->SetTextBlockStyle(InTextStyle);
}

void SEditableTextBox::SetText(TAttribute<FText> InNewText)
{
	EditableText->SetText(MoveTemp(InNewText));
}


void SEditableTextBox::SetError( const FText& InError )
{
	SetError( InError.ToString() );
}


void SEditableTextBox::SetError( const FString& InError )
{
	const bool bHaveError = !InError.IsEmpty();

	if (!ErrorReporting)
	{
		// If we are clearing an error but none was set before, we can leave.
		// If we have an active timer, we might have called this function previously, so we still need to register a timer
		if (!bHaveError && !HasActiveTimers())
		{
			return;
		}
		RegisterActiveTimer( 0.f,
			FWidgetActiveTimerDelegate::CreateLambda([this, InError](double, float)
			{
				if (!ErrorReporting) // SetError might have been called a few times, so ErrorReporting might be set
				{
					// No error reporting was specified; make a default one
					TSharedPtr<SPopupErrorText> ErrorTextWidget;
					Box->AddSlot()
					.AutoWidth()
					.Padding(3,0)
					[
						SAssignNew( ErrorTextWidget, SPopupErrorText )
					];
					ErrorReporting = ErrorTextWidget;
				}
				SetError(InError);
				return EActiveTimerReturnType::Stop;
			})
		);
		return;
	}

	ErrorReporting->AsWidget()->SetVisibility(bHaveError ? EVisibility::Visible : EVisibility::Collapsed);
	ErrorReporting->SetError(InError);
}


void SEditableTextBox::SetOnKeyCharHandler(FOnKeyChar InOnKeyCharHandler)
{
	EditableText->SetOnKeyCharHandler(MoveTemp(InOnKeyCharHandler));
}


void SEditableTextBox::SetOnKeyDownHandler(FOnKeyDown InOnKeyDownHandler)
{
	EditableText->SetOnKeyDownHandler(MoveTemp(InOnKeyDownHandler));
}


void SEditableTextBox::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	EditableText->SetTextShapingMethod(InTextShapingMethod);
}


void SEditableTextBox::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	EditableText->SetTextFlowDirection(InTextFlowDirection);
}


void SEditableTextBox::SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy)
{
	EditableText->SetOverflowPolicy(InOverflowPolicy);
}

void SEditableTextBox::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	EditableText->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
}

bool SEditableTextBox::AnyTextSelected() const
{
	return EditableText->AnyTextSelected();
}


void SEditableTextBox::SelectAllText()
{
	EditableText->SelectAllText();
}


void SEditableTextBox::ClearSelection()
{
	EditableText->ClearSelection();
}


FText SEditableTextBox::GetSelectedText() const
{
	return EditableText->GetSelectedText();
}

void SEditableTextBox::GoTo(const FTextLocation& NewLocation)
{
	EditableText->GoTo(NewLocation);
}

void SEditableTextBox::ScrollTo(const FTextLocation& NewLocation)
{
	EditableText->ScrollTo(NewLocation);
}

void SEditableTextBox::BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase, const bool InReverse)
{
	EditableText->BeginSearch(InSearchText, InSearchCase, InReverse);
}

void SEditableTextBox::AdvanceSearch(const bool InReverse)
{
	EditableText->AdvanceSearch(InReverse);
}

bool SEditableTextBox::HasError() const
{
	return ErrorReporting.IsValid() && ErrorReporting->HasError();
}

const FSlateBrush* SEditableTextBox::DetermineBorderImage() const
{
	if ( EditableText->IsTextReadOnly() )
	{
		return BorderImageReadOnly;
	}
	else if ( EditableText->HasKeyboardFocus() )
	{
		return BorderImageFocused;
	}
	else
	{
		if ( EditableText->IsHovered() )
		{
			return BorderImageHovered;
		}
		else
		{
			return BorderImageNormal;
		}
	}
}


bool SEditableTextBox::SupportsKeyboardFocus() const
{
	return StaticCastSharedPtr<SWidget>(EditableText)->SupportsKeyboardFocus();
}


bool SEditableTextBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return SBorder::HasKeyboardFocus() || EditableText->HasKeyboardFocus();
}


FReply SEditableTextBox::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	FReply Reply = FReply::Handled();

	if ( InFocusEvent.GetCause() != EFocusCause::Cleared )
	{
		// Forward keyboard focus to our editable text widget
		Reply.SetUserFocus(EditableText.ToSharedRef(), InFocusEvent.GetCause());
	}

	return Reply;
}


FReply SEditableTextBox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FKey Key = InKeyEvent.GetKey();

	if (Key == EKeys::Escape && EditableText->HasKeyboardFocus())
	{
		// Clear focus
		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Cleared);
	}

	return FReply::Unhandled();
}

FMargin SEditableTextBox::DeterminePadding() const
{
	check(Style);
	return PaddingOverride.IsSet() ? PaddingOverride.Get() : Style->Padding;
}

FSlateFontInfo SEditableTextBox::DetermineFont() const
{
	check(Style);
	return FontOverride.IsSet() ? FontOverride.Get() : Style->TextStyle.Font;
}

FSlateColor SEditableTextBox::DetermineBackgroundColor() const
{
	check(Style);
	return BackgroundColorOverride.IsSet() ? BackgroundColorOverride.Get() : Style->BackgroundColor;
}

FSlateColor SEditableTextBox::DetermineForegroundColor() const
{
	check(Style);  
	
	if (EditableText->IsTextReadOnly())
	{
		if (ReadOnlyForegroundColorOverride.IsSet())
		{
			return ReadOnlyForegroundColorOverride.Get();
		}
		if (ForegroundColorOverride.IsSet())
		{
			return ForegroundColorOverride.Get();
		}

		return Style->ReadOnlyForegroundColor;
	}
	else if(HasKeyboardFocus())
	{
		return FocusedForegroundColorOverride.IsSet() ? FocusedForegroundColorOverride.Get() : Style->FocusedForegroundColor;
	}
	else
	{
		return ForegroundColorOverride.IsSet() ? ForegroundColorOverride.Get() : Style->ForegroundColor;
	}
}

void SEditableTextBox::SetHintText(TAttribute<FText> InHintText)
{
	EditableText->SetHintText(MoveTemp(InHintText));
}

void SEditableTextBox::SetSearchText(TAttribute<FText> InSearchText)
{
	EditableText->SetSearchText(MoveTemp(InSearchText));
}

FText SEditableTextBox::GetSearchText() const
{
	return EditableText->GetSearchText();
}

void SEditableTextBox::SetIsReadOnly(TAttribute<bool> InIsReadOnly)
{
	EditableText->SetIsReadOnly(MoveTemp(InIsReadOnly));
}

void SEditableTextBox::SetIsPassword(TAttribute<bool> InIsPassword)
{
	EditableText->SetIsPassword(MoveTemp(InIsPassword));
}

void SEditableTextBox::SetFont(TAttribute<FSlateFontInfo> InFont)
{
	FontOverride = MoveTemp(InFont);
}

void SEditableTextBox::SetTextBoxForegroundColor(TAttribute<FSlateColor> InForegroundColor)
{
	ForegroundColorOverride = MoveTemp(InForegroundColor);
}

void SEditableTextBox::SetTextBoxBackgroundColor(TAttribute<FSlateColor> InBackgroundColor)
{
	BackgroundColorOverride = MoveTemp(InBackgroundColor);
}

void SEditableTextBox::SetReadOnlyForegroundColor(TAttribute<FSlateColor> InReadOnlyForegroundColor)
{
	ReadOnlyForegroundColorOverride = MoveTemp(InReadOnlyForegroundColor);
}

void SEditableTextBox::SetFocusedForegroundColor(TAttribute<FSlateColor> InFocusedForegroundColor)
{
	FocusedForegroundColorOverride = MoveTemp(InFocusedForegroundColor);
}

void SEditableTextBox::SetMaximumLength(TAttribute<int32> InMaximumLength)
{
	MaximumLength = MoveTemp(InMaximumLength);
}

void SEditableTextBox::SetMinimumDesiredWidth(TAttribute<float> InMinimumDesiredWidth)
{
	EditableText->SetMinDesiredWidth(MoveTemp(InMinimumDesiredWidth));
}


void SEditableTextBox::SetIsCaretMovedWhenGainFocus(TAttribute<bool> InIsCaretMovedWhenGainFocus)
{
	EditableText->SetIsCaretMovedWhenGainFocus(MoveTemp(InIsCaretMovedWhenGainFocus));
}


void SEditableTextBox::SetSelectAllTextWhenFocused(TAttribute<bool> InSelectAllTextWhenFocused)
{
	EditableText->SetSelectAllTextWhenFocused(MoveTemp(InSelectAllTextWhenFocused));
}


void SEditableTextBox::SetRevertTextOnEscape(TAttribute<bool> InRevertTextOnEscape)
{
	EditableText->SetRevertTextOnEscape(MoveTemp(InRevertTextOnEscape));
}


void SEditableTextBox::SetClearKeyboardFocusOnCommit(TAttribute<bool> InClearKeyboardFocusOnCommit)
{
	EditableText->SetClearKeyboardFocusOnCommit(MoveTemp(InClearKeyboardFocusOnCommit));
}


void SEditableTextBox::SetSelectAllTextOnCommit(TAttribute<bool> InSelectAllTextOnCommit)
{
	EditableText->SetSelectAllTextOnCommit(MoveTemp(InSelectAllTextOnCommit));
}

void SEditableTextBox::SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick)
{
	EditableText->SetSelectWordOnMouseDoubleClick(MoveTemp(InSelectWordOnMouseDoubleClick));
}

void SEditableTextBox::SetJustification(TAttribute<ETextJustify::Type> InJustification)
{
	EditableText->SetJustification(MoveTemp(InJustification));
}


void SEditableTextBox::SetAllowContextMenu(TAttribute<bool> InAllowContextMenu)
{
	EditableText->SetAllowContextMenu(MoveTemp(InAllowContextMenu));
}

void SEditableTextBox::SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction)
{
	EditableText->SetVirtualKeyboardDismissAction(MoveTemp(InVirtualKeyboardDismissAction));
}

void SEditableTextBox::EnableTextInputMethodContext()
{
	EditableText->EnableTextInputMethodContext();
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SEditableTextBox::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleEditableTextBox(SharedThis(this)));
}

TOptional<FText> SEditableTextBox::GetDefaultAccessibleText(EAccessibleType AccessibleType) const
{
	// The parent Construct() function will call this before EditableText exists,
	// so we need a guard here to ignore that function call.
	if (EditableText.IsValid())
	{
		return EditableText->GetHintText();
	}
	return TOptional<FText>();
}
#endif

void SEditableTextBox::OnEditableTextChanged(const FText& InText)
{
	OnTextChanged.ExecuteIfBound(InText);

	const int32 MaximumLengthValue = MaximumLength.Get();
	if (OnVerifyTextChanged.IsBound() || MaximumLengthValue >= 0)
	{
		FText OutErrorMessage;
		if (!FTextEditHelper::VerifyTextLength(InText, OutErrorMessage, MaximumLengthValue) ||
			(OnVerifyTextChanged.IsBound() && !OnVerifyTextChanged.Execute(InText, OutErrorMessage)))
		{
			// Display as an error.
			SetError(OutErrorMessage);
		}
		else
		{
			SetError(FText::GetEmpty());
		}
	}
}

void SEditableTextBox::OnEditableTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	FText OutErrorMessage;
	if (!FTextEditHelper::VerifyTextLength(InText, OutErrorMessage, MaximumLength.Get(0)) ||
		(OnVerifyTextChanged.IsBound() && !OnVerifyTextChanged.Execute(InText, OutErrorMessage)))
	{
		// Display as an error.
		if (InCommitType == ETextCommit::OnEnter)
		{
			SetError(OutErrorMessage);
		}
		return;
	}

	// Text commited without errors, so clear error text
	SetError(FText::GetEmpty());

	OnTextCommitted.ExecuteIfBound(InText, InCommitType);
}

void SEditableTextBox::OnEditableTextCursorMovedWithSelection(const FTextLocation& InLocation, const FTextSelection& TextSelection)
{
	OnCursorMovedWithSelection.ExecuteIfBound(InLocation, TextSelection);
}
