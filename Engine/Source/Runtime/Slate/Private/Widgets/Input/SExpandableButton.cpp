// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SExpandableButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"


//////////////////////////////////////////////////////////////////////////
// SExpandableButton
SLATE_IMPLEMENT_WIDGET(SExpandableButton)
void SExpandableButton::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IsExpanded, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SExpandableButton&>(Widget).UpdateVisibility();
			}));
}

SExpandableButton::SExpandableButton()
	: IsExpanded(*this, false)
{}

EVisibility SExpandableButton::GetCollapsedVisibility() const
{
	return IsExpanded.Get() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SExpandableButton::GetExpandedVisibility() const
{
	return IsExpanded.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SExpandableButton::Construct(const FArguments& InArgs)
{
	IsExpanded.Assign(*this, InArgs._IsExpanded);
	ExpandedChildContent = InArgs._ExpandedChildContent.Widget;

	// Set up the collapsed button content
	TSharedRef<SWidget> CollapsedButtonContent = (InArgs._CollapsedButtonContent.Widget == SNullWidget::NullWidget)
		? StaticCastSharedRef<SWidget>( SNew(STextBlock) .Text( InArgs._CollapsedText ) )
		: InArgs._CollapsedButtonContent.Widget;

	// Set up the expanded button content
	TSharedRef<SWidget> ExpandedButtonContent = (InArgs._ExpandedButtonContent.Widget == SNullWidget::NullWidget)
		? StaticCastSharedRef<SWidget>( SNew(STextBlock) .Text( InArgs._ExpandedText ) )
		: InArgs._ExpandedButtonContent.Widget;

	SBorder::Construct(SBorder::FArguments()
		.BorderImage( FCoreStyle::Get().GetBrush( "ExpandableButton.Background" ) )
		.Padding( FCoreStyle::Get().GetMargin("ExpandableButton.Padding") )
		[
			SNew(SHorizontalBox)

			// Toggle button (closed)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SAssignNew(ToggleButtonClosed, SButton)
				.VAlign(VAlign_Center)
				.OnClicked( InArgs._OnExpansionClicked )
				.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
				.ContentPadding( 0.f )
				[
					CollapsedButtonContent
				]
			]

			// Toggle button (expanded)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ToggleButtonExpanded, SButton)
				.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
				.ContentPadding( 0.f )
				.VAlign(VAlign_Center)
				[
					ExpandedButtonContent
				]
			]

			// Expansion-only box
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				InArgs._ExpandedChildContent.Widget
			]

			// Right side of expansion arrow
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Right)
			[
				// Close expansion button
				SAssignNew(CloseExpansionButton, SButton)
				.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
				.ContentPadding( 0.f )
				.OnClicked(InArgs._OnCloseClicked)
				[
					SNew(SImage)
					.Image( FCoreStyle::Get().GetBrush("ExpandableButton.CloseButton") )
				]
			]
		]
	);

	UpdateVisibility();
}

void SExpandableButton::UpdateVisibility()
{
	// The child content will be optionally visible depending on the state of the expandable button.
	if (ExpandedChildContent)
	{
		ExpandedChildContent->SetVisibility(GetExpandedVisibility());
	}
	ToggleButtonClosed->SetVisibility(GetCollapsedVisibility());
	ToggleButtonExpanded->SetVisibility(GetExpandedVisibility());
	CloseExpansionButton->SetVisibility(GetExpandedVisibility());
}
