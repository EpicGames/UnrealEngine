// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SInputKeySelector.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"


void SInputKeySelector::Construct( const FArguments& InArgs )
{
	SelectedKey = InArgs._SelectedKey;
	KeySelectionText = InArgs._KeySelectionText;
	NoKeySpecifiedText = InArgs._NoKeySpecifiedText;
	OnKeySelected = InArgs._OnKeySelected;
	OnIsSelectingKeyChanged = InArgs._OnIsSelectingKeyChanged;
	bAllowModifierKeys = InArgs._AllowModifierKeys;
	bAllowGamepadKeys = InArgs._AllowGamepadKeys;
	bEscapeCancelsSelection = InArgs._EscapeCancelsSelection;
	EscapeKeys = InArgs._EscapeKeys;
	bIsFocusable = InArgs._IsFocusable;

	bIsSelectingKey = false;

	ChildSlot
	[
		SAssignNew(Button, SButton)
		.ButtonStyle(InArgs._ButtonStyle)
		.IsFocusable(bIsFocusable)
		.OnClicked(this, &SInputKeySelector::OnClicked)
		[
			SAssignNew(TextBlock, STextBlock)
			.Text(this, &SInputKeySelector::GetSelectedKeyText)
			.TextStyle(InArgs._TextStyle)
			.Margin(Margin)
			.Justification(ETextJustify::Center)
		]
	];
}

FText SInputKeySelector::GetSelectedKeyText() const
{
	if ( bIsSelectingKey )
	{
		return KeySelectionText;
	}
	else if ( SelectedKey.IsSet() )
	{
		if(SelectedKey.Get().Key.IsValid())
		{
			// If the key in the chord is a modifier key, print it's display name directly since the FInputChord
					// displays these as empty text.
			return SelectedKey.Get().Key.IsModifierKey()
				? SelectedKey.Get().Key.GetDisplayName()
				: SelectedKey.Get().GetInputText();
		}
	}
	return NoKeySpecifiedText;
}

FInputChord SInputKeySelector::GetSelectedKey() const
{
	return SelectedKey.IsSet() ? SelectedKey.Get() : EKeys::Invalid;
}

void SInputKeySelector::SetSelectedKey( TAttribute<FInputChord> InSelectedKey )
{
	if ( SelectedKey.IdenticalTo(InSelectedKey) == false)
	{
		SelectedKey = InSelectedKey;
		OnKeySelected.ExecuteIfBound( SelectedKey.IsSet() ? SelectedKey.Get() : FInputChord( EKeys::Invalid ) );
	}
}

FMargin SInputKeySelector::GetMargin() const
{
	return Margin.Get();
}

void SInputKeySelector::SetMargin( TAttribute<FMargin> InMargin )
{
	Margin = InMargin;
}

void SInputKeySelector::SetButtonStyle(const FButtonStyle* ButtonStyle )
{
	if (Button.IsValid())
	{
		Button->SetButtonStyle(ButtonStyle);
	}
}

void SInputKeySelector::SetTextStyle(const FTextBlockStyle* InTextStyle)
{
	if (TextBlock.IsValid())
	{
		TextBlock->SetTextStyle(InTextStyle);
	}
}

FReply SInputKeySelector::OnClicked()
{
	if ( bIsSelectingKey == false )
	{
		SetIsSelectingKey(true);
		return FReply::Handled()
			.SetUserFocus(SharedThis(this), EFocusCause::SetDirectly);
	}
	return FReply::Handled();
}

void SInputKeySelector::SelectKey( FKey Key, bool bShiftDown, bool bControllDown, bool bAltDown, bool bCommandDown )
{
	FInputChord NewSelectedKey = bAllowModifierKeys 
		? FInputChord( Key, bShiftDown, bControllDown, bAltDown, bCommandDown ) 
		: FInputChord( Key );
	if ( SelectedKey.IsBound() == false )
	{
		SelectedKey.Set( NewSelectedKey );
	}
	OnKeySelected.ExecuteIfBound( NewSelectedKey );
}

void SInputKeySelector::SetIsSelectingKey( bool bInIsSelectingKey )
{
	if ( bIsSelectingKey != bInIsSelectingKey )
	{
		bIsSelectingKey = bInIsSelectingKey;
		// Prevents certain inputs from being consumed by the button
		if (Button.IsValid())
		{
			Button->SetEnabled(!bIsSelectingKey);
		}
		OnIsSelectingKeyChanged.ExecuteIfBound();
	}
}

bool SInputKeySelector::IsEscapeKey(const FKey& InKey) const
{
	return EscapeKeys.Contains(InKey);
}

FReply SInputKeySelector::OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( bIsSelectingKey && (bAllowGamepadKeys || InKeyEvent.GetKey().IsGamepadKey() == false) )
	{
		// While selecting keys handle all key downs to prevent contained controls from
		// interfering with key selection.
		return FReply::Handled();
	}
	return SCompoundWidget::OnPreviewKeyDown( MyGeometry, InKeyEvent );
}

FReply SInputKeySelector::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (!bIsSelectingKey)
	{
		if (SelectedKey.IsSet() && SelectedKey.Get().Key.IsValid() && (bAllowGamepadKeys && InKeyEvent.GetKey() == EKeys::Gamepad_FaceButton_Left))
		{
			SetSelectedKey(FInputChord());
			return FReply::Handled();
		}
		else if (Button.IsValid())
		{
			return Button->OnKeyDown(MyGeometry, InKeyEvent);
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SInputKeySelector::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	FKey KeyUp = InKeyEvent.GetKey();
	EModifierKey::Type ModifierKey = EModifierKey::FromBools(
		InKeyEvent.IsControlDown() && KeyUp != EKeys::LeftControl && KeyUp != EKeys::RightControl,
		InKeyEvent.IsAltDown() && KeyUp != EKeys::LeftAlt && KeyUp != EKeys::RightAlt,
		InKeyEvent.IsShiftDown() && KeyUp != EKeys::LeftShift && KeyUp != EKeys::RightShift,
		InKeyEvent.IsCommandDown() && KeyUp != EKeys::LeftCommand && KeyUp != EKeys::RightCommand );

	// Don't allow chords consisting of just modifier keys.
	if ( bIsSelectingKey && (bAllowGamepadKeys || KeyUp.IsGamepadKey() == false) && ( KeyUp.IsModifierKey() == false || ModifierKey == EModifierKey::None ) )
	{
		SetIsSelectingKey( false );

		if (bEscapeCancelsSelection && (KeyUp == EKeys::Escape || IsEscapeKey(KeyUp)))
		{
			return FReply::Handled();
		}

		SelectKey(
			KeyUp,
			(ModifierKey & EModifierKey::Shift) != EModifierKey::None,
			(ModifierKey & EModifierKey::Control) != EModifierKey::None,
			(ModifierKey & EModifierKey::Alt) != EModifierKey::None,
			(ModifierKey & EModifierKey::Command) != EModifierKey::None);
		return FReply::Handled();
	}
	else if (!bIsSelectingKey && Button.IsValid())
	{
		return Button->OnKeyUp(MyGeometry, InKeyEvent);
	}

	return SCompoundWidget::OnKeyUp( MyGeometry, InKeyEvent );
}

FReply SInputKeySelector::OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( bIsSelectingKey )
	{
		SetIsSelectingKey(false);
		// TODO: Add options for enabling mouse modifiers.
		SelectKey(MouseEvent.GetEffectingButton(), false, false, false, false);
		return FReply::Handled();
	}
	return SCompoundWidget::OnPreviewMouseButtonDown( MyGeometry, MouseEvent );
}

FReply SInputKeySelector::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bIsSelectingKey && SelectedKey.IsSet() && SelectedKey.Get().Key.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		SetSelectedKey(FInputChord());
		return FReply::Handled();
	}
	return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FNavigationReply SInputKeySelector::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	if (Button.IsValid())
	{
		return Button->OnNavigation(MyGeometry, InNavigationEvent);
	}

	return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
}

void SInputKeySelector::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	if ( bIsSelectingKey )
	{
		SetIsSelectingKey(false);
	}
}

void SInputKeySelector::SetTextBlockVisibility(EVisibility InVisibility)
{
	if (TextBlock.IsValid())
	{
		TextBlock->SetVisibility(InVisibility);
	}
}
