// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Widgets/Input/SMenuAnchor.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/Menu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/LayoutUtils.h"

#if WITH_EDITOR
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/ToolMenuBase.h"
#endif

namespace SMenuAnchor_CVars
{
	// This has been disabled since it no longer appears necessary, and it triggers widget tree traversals during destruction, causing intermittent crashes
	static bool bExecuteOnMenuOpenChangedInDestructor = false;
	static FAutoConsoleVariableRef CVarExecuteOnMenuOpenChangedInDestructor(
		TEXT("SMenuAnchor.ExecuteOnMenuOpenChangedInDestructor"),
		bExecuteOnMenuOpenChangedInDestructor,
		TEXT("If true, we run the original code which execute OnMenuOpenChanged.")
	);
}

SLATE_IMPLEMENT_WIDGET(SMenuAnchor)
void SMenuAnchor::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, PlacementAttribute, EInvalidateWidgetReason::Layout);
}

static FVector2D GetMenuOffsetForPlacement(const FGeometry& AllottedGeometry, EMenuPlacement PlacementMode, FVector2D PopupSizeLocalSpace)
{
	switch (PlacementMode)
	{
		case MenuPlacement_BelowAnchor:
			return FVector2D(0.0f, AllottedGeometry.GetLocalSize().Y);
		case MenuPlacement_CenteredBelowAnchor:
			return FVector2D(-((PopupSizeLocalSpace.X / 2) - (AllottedGeometry.GetLocalSize().X / 2)), AllottedGeometry.GetLocalSize().Y);
		case MenuPlacement_BelowRightAnchor:
			return FVector2D( -( PopupSizeLocalSpace.X ) + ( AllottedGeometry.GetLocalSize().X ), AllottedGeometry.GetLocalSize().Y );
		case MenuPlacement_ComboBox:
			return FVector2D(0.0f, AllottedGeometry.GetLocalSize().Y);
		case MenuPlacement_ComboBoxRight:
			return FVector2D(AllottedGeometry.GetLocalSize().X - PopupSizeLocalSpace.X, AllottedGeometry.GetLocalSize().Y);
		case MenuPlacement_MenuRight:
			return FVector2D(AllottedGeometry.GetLocalSize().X, 0.0f);
		case MenuPlacement_AboveAnchor:
			return FVector2D(0.0f, -PopupSizeLocalSpace.Y);
		case MenuPlacement_CenteredAboveAnchor:
			return FVector2D(-((PopupSizeLocalSpace.X / 2) - (AllottedGeometry.GetLocalSize().X / 2)), -PopupSizeLocalSpace.Y);
		case MenuPlacement_AboveRightAnchor:
			return FVector2D( -( PopupSizeLocalSpace.X ) + ( AllottedGeometry.GetLocalSize().X ), -PopupSizeLocalSpace.Y );
		case MenuPlacement_MenuLeft:
			return FVector2D(-PopupSizeLocalSpace.X, 0.0f);
		case MenuPlacement_Center:
			return FVector2D( -( ( PopupSizeLocalSpace.X / 2 ) - ( AllottedGeometry.GetLocalSize().X / 2 ) ), -( ( PopupSizeLocalSpace.Y / 2 ) - ( AllottedGeometry.GetLocalSize().Y / 2 ) ) );
		case MenuPlacement_RightLeftCenter:
			return FVector2D( AllottedGeometry.GetLocalSize().X, - ( ( PopupSizeLocalSpace.Y / 2 ) - ( AllottedGeometry.GetLocalSize().Y / 2 ) ) );
		case MenuPlacement_MatchBottomLeft:
			return FVector2D(0.0f, (AllottedGeometry.GetLocalSize().Y - PopupSizeLocalSpace.Y));
		default:
			ensureMsgf( false, TEXT("Unhandled placement mode: %d"), PlacementMode );
			return FVector2D::ZeroVector;
	}
}

SMenuAnchor::FPopupPlacement::FPopupPlacement(const FGeometry& PlacementGeometry, const FVector2D& PopupDesiredSize, EMenuPlacement PlacementMode)
{
	// Compute the popup size, offset, and anchor rect  in local space
	const bool bIsComboBoxPopup = (PlacementMode == MenuPlacement_ComboBox || PlacementMode == MenuPlacement_ComboBoxRight);
	LocalPopupSize = bIsComboBoxPopup ? FVector2D(FMath::Max(PlacementGeometry.Size.X, PopupDesiredSize.X), PopupDesiredSize.Y) : PopupDesiredSize;
	LocalPopupOffset = GetMenuOffsetForPlacement(PlacementGeometry, PlacementMode, LocalPopupSize);
	AnchorLocalSpace = FSlateRect::FromPointAndExtent(FVector2D::ZeroVector, PlacementGeometry.GetLocalSize());
	Orientation = (PlacementMode == MenuPlacement_MenuRight || PlacementMode == MenuPlacement_MenuLeft) ? Orient_Horizontal : Orient_Vertical;
}

/*static*/ TArray<TWeakPtr<IMenu>> SMenuAnchor::OpenApplicationMenus;

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SMenuAnchor::Construct( const FArguments& InArgs )
{
	Children.AddSlot(MoveTemp(FBasicLayoutWidgetSlot::FSlotArguments(MakeUnique<FBasicLayoutWidgetSlot>())
		.Padding(InArgs._Padding)
		[
			InArgs._Content.Widget
		]));
	Children.AddSlot(FBasicLayoutWidgetSlot::FSlotArguments(MakeUnique<FBasicLayoutWidgetSlot>()));

	MenuContent                            = InArgs._MenuContent;
	WrappedContent                         = InArgs._MenuContent;
	OnGetMenuContent                       = InArgs._OnGetMenuContent;
	OnMenuOpenChanged                      = InArgs._OnMenuOpenChanged;
	bFitInWindow						   = InArgs._FitInWindow;
	Method                                 = InArgs._Method;
	bShouldDeferPaintingAfterWindowContent = InArgs._ShouldDeferPaintingAfterWindowContent;
	bUseApplicationMenuStack               = InArgs._UseApplicationMenuStack;
	bShowMenuBackground					   = InArgs._ShowMenuBackground;
	bIsCollapsedByParent                   = InArgs._IsCollapsedByParent;
	bApplyWidgetStyleToMenu                = InArgs._ApplyWidgetStyleToMenu;

	SetMenuPlacement(InArgs._Placement);
}

FGeometry SMenuAnchor::ComputeNewWindowMenuPlacement(const FGeometry& AllottedGeometry, const FVector2D& PopupDesiredSize, EMenuPlacement PlacementMode) const
{
	// Compute the popup size, offset, and anchor rect  in local space
	const FPopupPlacement PopupPlacement(AllottedGeometry, PopupDesiredSize, PlacementMode);

	// already handled
	const bool bAutoAdjustForDPIScale = false;

	// ask the application to compute the proper desktop offset for the anchor. This requires the offsets to be in desktop space.
	const FVector2D NewPositionDesktopSpace = FSlateApplication::Get().CalculatePopupWindowPosition(
		TransformRect(AllottedGeometry.GetAccumulatedLayoutTransform(), PopupPlacement.AnchorLocalSpace),
		TransformVector(AllottedGeometry.GetAccumulatedLayoutTransform(), PopupPlacement.LocalPopupSize),
		bAutoAdjustForDPIScale,
		TransformPoint(AllottedGeometry.GetAccumulatedLayoutTransform(), PopupPlacement.LocalPopupOffset),
		PopupPlacement.Orientation);

	// transform the desktop offset into local space and use that as the layout transform for the child content.
	return AllottedGeometry.MakeChild(
		PopupPlacement.LocalPopupSize,
		FSlateLayoutTransform(TransformPoint(Inverse(AllottedGeometry.GetAccumulatedLayoutTransform()), NewPositionDesktopSpace)));
}

EVisibility SMenuAnchor::GetMenuContentVisibility(TAttribute<EVisibility> OriginalVisibilityAttribute) const
{
	FWidgetProxyHandle ProxyHandle = GetProxyHandle();
	if (ProxyHandle.IsValid(this))
	{
		FSlateInvalidationWidgetVisibility Visibility = ProxyHandle.GetWidgetVisibility(this);
		if (!Visibility.IsVisible())
		{
			return EVisibility::Collapsed;
		}
	}
	else // if we don't have access to the proxy handle, we need to walk the hierarchy
	{
		TSharedPtr<const SWidget> Widget = AsShared();
		while (Widget)
		{
			if (!Widget->GetVisibility().IsVisible())
			{
				return EVisibility::Collapsed;
			}
			Widget = Widget->GetParentWidget();
		}
	}

	return OriginalVisibilityAttribute.Get(EVisibility::Collapsed);
}

void SMenuAnchor::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	TSharedPtr<SWindow> PopupWindow = PopupWindowPtr.Pin();
	if ( PopupWindow.IsValid() && IsOpenViaCreatedWindow() )
	{
		// Figure out where our attached pop-up window should be placed.
		const FVector2D PopupContentDesiredSize = PopupWindow->GetContent()->GetDesiredSize();
		FGeometry PopupGeometry = ComputeNewWindowMenuPlacement( AllottedGeometry, PopupContentDesiredSize, PlacementAttribute.Get() );
		const FVector2D NewPosition = PopupGeometry.LocalToAbsolute(FVector2D::ZeroVector);
		// NOTE: In order to get the right size of the window, we need to take whatever the incoming scale of the menu anchor,
		// then divide out the popup window's DPI scale.  Finally, we need to divide that remainder to the draw size.
		// The idea here is to divide out any "extra" scale that's not associated with the DPI scale, since ComputeNewWindowMenuPlacement
		// makes a child transform for the new window, based on the geometry of it, which if the menu anchor is inside a zoom panel
		// that would translate to a menu that had a smaller size window, if the scale was tiny, which we don't want - we only want
		// the DPI scale if any, to be factored into the size.
		// NOTE: We only do this for "New Window" popups.  Because games use "Current Window", we can't do this same trick, as they have
		// to be concerned with the viewport scale as well, which Slate knows nothing about.  Perhaps DPI Scale of the viewports should be
		// passed down in FGeometry, along with the window DPI Scale, as one extra value code can take into account if it needs to.
		const FVector2D NewSize = PopupGeometry.GetDrawSize() / ( AllottedGeometry.GetAccumulatedLayoutTransform().GetScale() / PopupWindow->GetLocalToWindowTransform().GetScale() );

		// When in a ComboBox, we want the submenu to be at minimum the size of the combo box, so we update the Popupwindow minimum size.
		const EMenuPlacement PlacementMode = PlacementAttribute.Get();
		if (PlacementMode == MenuPlacement_ComboBox)
		{
			// Set the new size limits
			FWindowSizeLimits SizeLimits = PopupWindow->GetSizeLimits();
			SizeLimits.SetMinWidth(NewSize.X);
			PopupWindow->SetSizeLimits(SizeLimits);
		}

		// We made a window for showing the popup.
		// Update the window's position!
		PopupWindow->ReshapeWindow(NewPosition, NewSize);
	}
	else if (PopupWindow.IsValid() && IsOpenAndReusingWindow())
	{
		// Ideally, do this in OnArrangeChildren(); currently not possible because OnArrangeChildren()
		// can be called in DesktopSpace or WindowSpace, and we will not know which version of the Window
		// geometry to use. Tick() is always in DesktopSpace, so cache the position here and just use
		// it in OnArrangeChildren().
		const FPopupPlacement LocalPlacement(AllottedGeometry, Children[1].GetWidget()->GetDesiredSize(), PlacementAttribute.Get());

		FVector2D FittedPlacement;

		if (bFitInWindow)
		{
			const FSlateRect WindowRectLocalSpace = TransformRect(Inverse(AllottedGeometry.GetAccumulatedLayoutTransform()), PopupWindow->GetClientRectInScreen());
			FittedPlacement = ComputePopupFitInRect(
				LocalPlacement.AnchorLocalSpace,
				FSlateRect(LocalPlacement.LocalPopupOffset, LocalPlacement.LocalPopupOffset + LocalPlacement.LocalPopupSize),
				LocalPlacement.Orientation, WindowRectLocalSpace);
		}
		else
		{
			FittedPlacement = LocalPlacement.LocalPopupOffset;
		}

		LocalPopupPosition = FittedPlacement;
		ScreenPopupPosition = AllottedGeometry.GetAccumulatedLayoutTransform().TransformPoint(LocalPopupPosition);
	}

	/** The tick is ending, so the window was not dismissed this tick. */
	bDismissedThisTick = false;
}

bool SMenuAnchor::ComputeVolatility() const
{
	return SPanel::ComputeVolatility() || IsOpen();
}

void SMenuAnchor::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	ArrangeSingleChild(GSlateFlowDirection, AllottedGeometry, ArrangedChildren, Children[0], FVector2D::UnitVector);
	const TSharedPtr<SWindow> PresentingWindow = PopupWindowPtr.Pin();
	if (IsOpenAndReusingWindow() && PresentingWindow.IsValid())
	{
		const FPopupPlacement LocalPlacement(AllottedGeometry, Children[1].GetWidget()->GetDesiredSize(), PlacementAttribute.Get());
		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(Children[1].GetWidget(), LocalPlacement.LocalPopupSize, FSlateLayoutTransform(LocalPopupPosition)));
	}
}

FVector2D SMenuAnchor::ComputeDesiredSize( float ) const
{
	const FVector2D DesiredWidgetSize = Children[0].GetWidget()->GetDesiredSize();

	// Menu anchors might be created with null content, in which case they must still get drawn in order to 
	// draw pop-up content, therefore it must lie and always request a desired size of at least 1,1, otherwise 
	// a panel may filter it from drawing thinking the it doesn't have anything to draw.
	return FVector2D(FMath::Max(DesiredWidgetSize.X, 1.0f), FMath::Max(DesiredWidgetSize.Y, 1.0f));
}

FChildren* SMenuAnchor::GetChildren()
{
	return &Children;
}

int32 SMenuAnchor::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren( EVisibility::Visible );
	ArrangeChildren( AllottedGeometry, ArrangedChildren );
	
	// There may be zero elements in this array if our child collapsed/hidden
	if ( ArrangedChildren.Num() > 0 )
	{
		const FArrangedWidget& FirstChild = ArrangedChildren[0];

		// In the case where the user doesn't provide content to the menu anchor, the null widget
		// wont appear in the visible set of arranged children, so only immediately paint the first child,
		// if it's visible and matches the first slot content.
		const bool bHasArrangedAnchorContent = FirstChild.Widget == Children[0].GetWidget();
		if ( bHasArrangedAnchorContent )
		{
			LayerId = FirstChild.Widget->Paint(Args.WithNewParent(this), FirstChild.Geometry, MyCullingRect, OutDrawElements, LayerId + 1, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
		}

		const bool bIsOpen = IsOpen();

		if ( bIsOpen )
		{
			// In the case where the anchor content is present and visible, it's the 1 index child, in the case
			// where the anchor content is invisible, it's the 0 index child.
			FArrangedWidget* PopupChild = nullptr;
			if ( bHasArrangedAnchorContent && ArrangedChildren.Num() > 1 )
			{
				PopupChild = &ArrangedChildren[1];
			}
			else if ( !bHasArrangedAnchorContent && ArrangedChildren.Num() == 1 )
			{
				PopupChild = &ArrangedChildren[0];
			}

			if ( PopupChild != nullptr )
			{
				if (bShouldDeferPaintingAfterWindowContent)
				{
					OutDrawElements.QueueDeferredPainting(
						FSlateWindowElementList::FDeferredPaint(PopupChild->Widget, Args, PopupChild->Geometry, bApplyWidgetStyleToMenu ? InWidgetStyle : FWidgetStyle(), bParentEnabled));
				}
				else
				{
					const TSharedPtr<SWindow> PresentingWindow = PopupWindowPtr.Pin();
					if (PresentingWindow.IsValid())
					{
						PopupChild->Widget->Paint(Args.WithNewParent(this), PopupChild->Geometry, PresentingWindow->GetClippingRectangleInWindow(), OutDrawElements, LayerId + 1, bApplyWidgetStyleToMenu ? InWidgetStyle : FWidgetStyle(), ShouldBeEnabled(bParentEnabled));
					}
				}
			}
		} 
	}

	return LayerId;
}

bool SMenuAnchor::IsOpenAndReusingWindow() const
{
	return MethodInUse.IsSet() && MethodInUse.GetPopupMethod() == EPopupMethod::UseCurrentWindow;
}

bool SMenuAnchor::IsOpenViaCreatedWindow() const
{
	return MethodInUse.IsSet() && MethodInUse.GetPopupMethod() == EPopupMethod::CreateNewWindow;
}

void SMenuAnchor::SetContent(TSharedRef<SWidget> InContent)
{
	Children[0].SetPadding(0.f);
	Children[0].AttachWidget(InContent);
}

void SMenuAnchor::SetMenuContent(TSharedRef<SWidget> InMenuContent)
{
	MenuContent = InMenuContent;
	WrappedContent = InMenuContent;	// wrapping, if any will happen when the menu is opened
}

FPopupMethodReply QueryPopupMethod(const FWidgetPath& PathToQuery)
{
	for (int32 WidgetIndex = PathToQuery.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		FPopupMethodReply PopupMethod = PathToQuery.Widgets[WidgetIndex].Widget->OnQueryPopupMethod();
		if (PopupMethod.IsEventHandled())
		{
			return PopupMethod;
		}
	}

	return FPopupMethodReply::UseMethod(EPopupMethod::CreateNewWindow);
}

void SMenuAnchor::SetIsOpen( bool InIsOpen, const bool bFocusMenu, const int32 FocusUserIndex )
{
	// Prevent redundant opens/closes
	if ( IsOpen() != InIsOpen )
	{
		if ( InIsOpen )
		{
			if ( OnGetMenuContent.IsBound() )
			{
				SetMenuContent(OnGetMenuContent.Execute());
			}

			if ( MenuContent.IsValid() )
			{
				// OPEN POPUP
				if ( OnMenuOpenChanged.IsBound() )
				{
					OnMenuOpenChanged.Execute(true);
				}

				// Figure out where the menu anchor is on the screen, so we can set the initial position of our pop-up window
				// This can be called at any time so we use the push menu override that explicitly allows us to specify our parent
				// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
				FWidgetPath MyWidgetPath;
				FSlateApplication::Get().GeneratePathToWidgetUnchecked(AsShared(), MyWidgetPath);
				if (MyWidgetPath.IsValid())
				{
					const FGeometry& MyGeometry = MyWidgetPath.Widgets.Last().Geometry;
					const float LayoutScaleMultiplier =  MyGeometry.GetAccumulatedLayoutTransform().GetScale();

					SlatePrepass(LayoutScaleMultiplier);

					// Figure out how big the content widget is so we can set the window's initial size properly
					TSharedRef< SWidget > MenuContentRef = MenuContent.ToSharedRef();
					MenuContentRef->SlatePrepass(LayoutScaleMultiplier);

					// Combo-boxes never size down smaller than the widget that spawned them, but all
					// other pop-up menus are currently auto-sized
					const FVector2D DesiredContentSize = MenuContentRef->GetDesiredSize();  // @todo slate: This is ignoring any window border size!
					const EMenuPlacement PlacementMode = PlacementAttribute.Get();

					const FVector2D NewPosition = FVector2D(MyGeometry.AbsolutePosition);
					FVector2D NewWindowSize = DesiredContentSize;

					FPopupTransitionEffect TransitionEffect( FPopupTransitionEffect::None );
					if ( PlacementMode == MenuPlacement_ComboBox || PlacementMode == MenuPlacement_ComboBoxRight )
					{
						TransitionEffect = FPopupTransitionEffect( FPopupTransitionEffect::ComboButton );
						NewWindowSize = FVector2D( FMath::Max( MyGeometry.Size.X, DesiredContentSize.X ), DesiredContentSize.Y );
					}
					else if ( PlacementMode == MenuPlacement_BelowAnchor )
					{
						TransitionEffect = FPopupTransitionEffect( FPopupTransitionEffect::TopMenu );
					}
					else if ( PlacementMode == MenuPlacement_MenuRight )
					{
						TransitionEffect = FPopupTransitionEffect( FPopupTransitionEffect::SubMenu );
						NewWindowSize = MyGeometry.GetAbsoluteSize();
					}

					MethodInUse = Method.IsSet()
						? FPopupMethodReply::UseMethod(Method.GetValue())
						: QueryPopupMethod(MyWidgetPath);

					// "Normal" menus are created and managed by the application's menu stack functions
					if (bUseApplicationMenuStack)
					{
						if (MethodInUse.GetPopupMethod() == EPopupMethod::CreateNewWindow)
						{
							// Open the pop-up
							TSharedPtr<IMenu> NewMenu = FSlateApplication::Get().PushMenu(AsShared(), MyWidgetPath, MenuContentRef, NewPosition, TransitionEffect, bFocusMenu, NewWindowSize, MethodInUse.GetPopupMethod(), bIsCollapsedByParent, FocusUserIndex);
							
							if (ensure(NewMenu.IsValid()))
							{
#if WITH_EDITOR
								// Reporting more information for UE-81655
								if (!NewMenu->GetOwnedWindow().IsValid())
								{
									UE_LOGF(LogSlate, Error, "!NewMenu->GetOwnedWindow().IsValid(), WidgetPath:\n%ls", *MyWidgetPath.ToString());

									static const FName SMultiBoxWidgetTypeName = "SMultiBoxWidget";
									for (int32 WidgetIndex = MyWidgetPath.Widgets.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
									{
										if (MyWidgetPath.Widgets[WidgetIndex].Widget->GetType() == SMultiBoxWidgetTypeName)
										{
											TSharedRef<SMultiBoxWidget> MultiBoxWidget = StaticCastSharedRef<SMultiBoxWidget>(MyWidgetPath.Widgets[WidgetIndex].Widget);
											TSharedRef<const FMultiBox> MultiBox = MultiBoxWidget->GetMultiBox();
											FString BlockText;
											const TArray< TSharedRef< const FMultiBlock > >& Blocks = MultiBox->GetBlocks();
											for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); ++BlockIndex)
											{
												const bool bIsFinalBlock = (BlockIndex == (Blocks.Num() - 1));
												const TSharedRef< const FMultiBlock >& Block = Blocks[BlockIndex];
												BlockText += FString::Printf(TEXT("%s (%d)%s"), *Block->GetExtensionHook().ToString(), (int32)Block->GetType(), bIsFinalBlock ? TEXT("") : TEXT(", "));
											}
											UE_LOGF(LogSlate, Error, " Blocks: %ls", *BlockText);
											break;
										}
									}

									UE_LOGF(LogSlate, Error, " MenuContentRef: %ls", *MenuContentRef->ToString());
								}
#endif
								// We want to be sure the menu content ends up hidden if this SMenuAnchor is also hidden
								NewMenu->GetContent()->SetVisibility(TAttribute<EVisibility>::CreateSP(this,
									&SMenuAnchor::GetMenuContentVisibility,
									AccessWidgetVisibilityAttribute(NewMenu->GetContent().ToSharedRef())
								));

								if (NewMenu->GetOwnedWindow().IsValid())
								{
									PopupMenuPtr = NewMenu;
									NewMenu->GetOnMenuDismissed().AddSP(this, &SMenuAnchor::OnMenuClosed);
									PopupWindowPtr = NewMenu->GetOwnedWindow();
								}
								else
								{
									UE_LOGF(LogSlate, Error, " Menu '%ls' could not open '%ls'", *ToString(), *MenuContentRef->ToString());
									if (TSharedPtr<IMenu> Pinned = PopupMenuPtr.Pin())
									{
										Pinned->Dismiss();
									}

									ResetPopupMenuContent();
								}
							}
						}
						else
						{
							// We are re-using the current window instead of creating a new one.
							// The popup will be presented as a child of this widget.
							ensure(MethodInUse.GetPopupMethod() == EPopupMethod::UseCurrentWindow);
							// We get the deepest window so that it works correctly inside a widget component
							// though we may need to come up with a more complex setup if we ever need
							// parents to be in a virtual window, but not the popup.
							PopupWindowPtr = MyWidgetPath.GetDeepestWindow();

							if (bFocusMenu)
							{
								FSlateApplication::Get().ReleaseAllPointerCapture(FocusUserIndex);
							}

							TSharedRef<SMenuAnchor> SharedThis = StaticCastSharedRef<SMenuAnchor>(AsShared());

							TSharedPtr<IMenu> NewMenu = FSlateApplication::Get().PushHostedMenu(
								SharedThis, MyWidgetPath, SharedThis, MenuContentRef, WrappedContent, TransitionEffect, MethodInUse.GetShouldThrottle(), bIsCollapsedByParent);

							PopupMenuPtr = NewMenu;
							check(NewMenu.IsValid());
							//check(NewMenu->GetParentWindow().ToSharedRef() == PopupWindow);
							check(WrappedContent.IsValid());

							Children[1]
							[
								WrappedContent.ToSharedRef()
							];

							if (bFocusMenu)
							{
								FSlateApplication::Get().SetUserFocus(FocusUserIndex, MenuContentRef, EFocusCause::SetDirectly);
							}
						}
					}
					else // !bUseApplicationMenuStack
					{
						// Anchor's menu doesn't participate in the application's menu stack.
						// Lifetime is managed by this anchor
						if (MethodInUse.GetPopupMethod() == EPopupMethod::CreateNewWindow)
						{
							// Start pop-up windows out transparent, then fade them in over time
							const EWindowTransparency Transparency(EWindowTransparency::PerWindow);

							FSlateRect Anchor(NewPosition, NewPosition + MyGeometry.GetLocalSize());
							EOrientation Orientation = (TransitionEffect.SlideDirection == FPopupTransitionEffect::SubMenu) ? Orient_Horizontal : Orient_Vertical;
		
							// @todo slate: Assumes that popup is not Scaled up or down from application scale.
							MenuContentRef->SlatePrepass(FSlateApplication::Get().GetApplicationScale());
							// @todo slate: Doesn't take into account potential window border size
							FVector2D ExpectedSize = MenuContentRef->GetDesiredSize();

							// already handled
							const bool bAutoAdjustForDPIScale = false;

							const FVector2D ScreenPosition = FSlateApplication::Get().CalculatePopupWindowPosition(Anchor, ExpectedSize, bAutoAdjustForDPIScale, FVector2D::ZeroVector, Orientation);

							// Release the mouse so that context can be properly restored upon closing menus.  See CL 1411833 before changing this.
							if (bFocusMenu)
							{
								FSlateApplication::Get().ReleaseAllPointerCapture(FocusUserIndex);
							}

							// Create a new window for the menu
							TSharedRef<SWindow> NewMenuWindow = SNew(SWindow)
								.Type(EWindowType::Menu)
								.IsPopupWindow(true)
								.SizingRule(ESizingRule::Autosized)
								.ScreenPosition(ScreenPosition)
								.AutoCenter(EAutoCenter::None)
								.ClientSize(ExpectedSize)
								.InitialOpacity(1.0f)
								.SupportsTransparency(Transparency)
								.FocusWhenFirstShown(bFocusMenu)
								.ActivationPolicy(bFocusMenu ? EWindowActivationPolicy::Always : EWindowActivationPolicy::Never)
								[
									MenuContentRef
								];

							if (bFocusMenu)
							{
								// Focus the unwrapped content rather than just the window
								NewMenuWindow->SetWidgetToFocusOnActivate(MenuContentRef);
							}

							TSharedPtr<IMenu> NewMenu = MakeShareable(new FMenuInWindow(NewMenuWindow, MenuContentRef, bIsCollapsedByParent));
							FSlateApplication::Get().AddWindowAsNativeChild(NewMenuWindow, MyWidgetPath.GetWindow(), true);

							PopupMenuPtr = OwnedMenuPtr = NewMenu;
							check(NewMenu.IsValid());
							NewMenu->GetOnMenuDismissed().AddSP(this, &SMenuAnchor::OnMenuClosed);
							PopupWindowPtr = NewMenuWindow;
						}
						else
						{
							// We are re-using the current window instead of creating a new one.
							// The popup will be presented as a child of this widget.
							ensure(MethodInUse.GetPopupMethod() == EPopupMethod::UseCurrentWindow);
							PopupWindowPtr = MyWidgetPath.GetWindow();

							if (bFocusMenu)
							{
								FSlateApplication::Get().ReleaseAllPointerCapture(FocusUserIndex);
							}

							TSharedRef<SMenuAnchor> SharedThis = StaticCastSharedRef<SMenuAnchor>(AsShared());
							TSharedPtr<IMenu> NewMenu = MakeShareable(new FMenuInHostWidget(SharedThis, MenuContentRef, bIsCollapsedByParent));

							PopupMenuPtr = OwnedMenuPtr = NewMenu;
							check(NewMenu.IsValid());
							//check(NewMenu->GetParentWindow().ToSharedRef() == PopupWindow);

							Children[1]
							[
								MenuContentRef
							];

							if (bFocusMenu)
							{
								FSlateApplication::Get().SetUserFocus(FocusUserIndex, MenuContentRef, EFocusCause::SetDirectly);
							}

							OpenApplicationMenus.Add(NewMenu);
						}
					}
				}
			}

			Invalidate(EInvalidateWidget::ChildOrder | EInvalidateWidget::Volatility);
		}
		else
		{
			// CLOSE POPUP
			if (PopupMenuPtr.IsValid())
			{
				PopupMenuPtr.Pin()->Dismiss();
			}

			ResetPopupMenuContent();
		}
	}
}

void SMenuAnchor::OnMenuClosed(TSharedRef<IMenu> InMenu)
{
	bDismissedThisTick = true;

	ResetPopupMenuContent();

	if (OnMenuOpenChanged.IsBound())
	{
		OnMenuOpenChanged.Execute(false);
	}

	if ( OnGetMenuContent.IsBound() )
	{
		SetMenuContent(SNullWidget::NullWidget);
	}
}

void SMenuAnchor::ResetPopupMenuContent()
{
	MethodInUse = FPopupMethodReply::Unhandled();
	PopupMenuPtr.Reset();
	OwnedMenuPtr.Reset();
	PopupWindowPtr.Reset();

	// Always clear out the menu content children slot to prevent prepass and other hierarchy queries from considering the
	// hidden menu content as content they should be concerned with.
	Children[1]
	[
		SNullWidget::NullWidget
	];

	Invalidate(EInvalidateWidget::ChildOrder | EInvalidateWidget::Volatility);
}

bool SMenuAnchor::IsOpen() const
{
	return MethodInUse.IsSet() && PopupMenuPtr.IsValid();
}

bool SMenuAnchor::ShouldOpenDueToClick() const
{
	return !IsOpen() && !bDismissedThisTick;
}

FVector2D SMenuAnchor::GetMenuPosition() const
{
	FVector2D Pos(0,0);

	if (IsOpenViaCreatedWindow() && PopupWindowPtr.IsValid())
	{
		Pos = PopupWindowPtr.Pin()->GetPositionInScreen();
	}
	else if (IsOpenAndReusingWindow() && PopupMenuPtr.IsValid())
	{
		Pos = ScreenPopupPosition;
	}
	
	return Pos;
}

void SMenuAnchor::SetMenuPlacement(TAttribute<EMenuPlacement> InMenuPlacement)
{
	PlacementAttribute.Assign(*this, MoveTemp(InMenuPlacement));
}

void SMenuAnchor::SetFitInWindow(bool bFit)
{
	if (bFitInWindow != bFit)
	{
		bFitInWindow = bFit;
		Invalidate(EInvalidateWidget::Layout);
	}
}

bool SMenuAnchor::HasOpenSubMenus() const
{
	bool Result = false;
	if (PopupMenuPtr.IsValid())
	{
		Result = FSlateApplication::Get().HasOpenSubMenus(PopupMenuPtr.Pin());
	}

	return Result;
}

TSharedPtr<SWindow> SMenuAnchor::GetMenuWindow() const
{
	return IsOpen() ? PopupWindowPtr.Pin() : TSharedPtr<SWindow>();
}

void SMenuAnchor::OnMenuDismissed()
{
	if (PopupMenuPtr.IsValid())
	{
		OnMenuClosed(PopupMenuPtr.Pin().ToSharedRef());
	}
}

bool SMenuAnchor::UsingApplicationMenuStack() const
{
	return bUseApplicationMenuStack;
}

/*static*/ void SMenuAnchor::DismissAllApplicationMenus()
{
	for (int32 i = 0; i < OpenApplicationMenus.Num(); ++i)
	{
		TSharedPtr<IMenu> Iter = OpenApplicationMenus[i].IsValid() ? OpenApplicationMenus[i].Pin() : nullptr;
		if (Iter.IsValid() && Iter->UsingApplicationMenuStack())
		{
			Iter->Dismiss();
			OpenApplicationMenus.RemoveAtSwap(i--);
		}
	}
}

SMenuAnchor::SMenuAnchor()
	: MenuContent( SNullWidget::NullWidget )
	, WrappedContent(SNullWidget::NullWidget)
	, bDismissedThisTick( false )
	, Method()
	, MethodInUse()
	, LocalPopupPosition( FVector2D::ZeroVector )
	, Children(this)
	, PlacementAttribute(*this)
{
}

SMenuAnchor::~SMenuAnchor()
{
	if (PopupMenuPtr.IsValid())
	{
		PopupMenuPtr.Pin()->Dismiss();

		// If the menu hasn't been dismissed, then dismiss it now
		if (!bDismissedThisTick)
		{
			bDismissedThisTick = true;
			// This has been disabled since it no longer appears necessary, and it triggers widget tree traversals during destruction, causing intermittent crashes
			if (SMenuAnchor_CVars::bExecuteOnMenuOpenChangedInDestructor)
			{
				if (OnMenuOpenChanged.IsBound())
				{
					OnMenuOpenChanged.Execute(false);
				}
			}
		}
	}
		
	// We no longer have a popup open, so reset all the tracking state associated.
	PopupMenuPtr.Reset();
	OwnedMenuPtr.Reset();
	PopupWindowPtr.Reset();
	MethodInUse = FPopupMethodReply::Unhandled();
}
