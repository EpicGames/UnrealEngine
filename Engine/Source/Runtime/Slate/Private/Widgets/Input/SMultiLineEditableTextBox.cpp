// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Framework/Text/TextEditHelper.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SPopUpErrorText.h"

#if WITH_FANCY_TEXT

namespace
{
	/**
     * Helper function to solve some issues with ternary operators inside construction of a widget.
	 */
	TSharedRef< SWidget > AsWidgetRef( const TSharedPtr< SWidget >& InWidget )
	{
		if ( InWidget.IsValid() )
		{
			return InWidget.ToSharedRef();
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}
}

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SMultiLineEditableTextBox::Construct(const FArguments& InArgs)
{
	check (InArgs._Style);

	PaddingOverride = InArgs._Padding;
	HScrollBarPaddingOverride = InArgs._HScrollBarPadding;
	VScrollBarPaddingOverride = InArgs._VScrollBarPadding;
	FontOverride = InArgs._Font;
	ForegroundColorOverride = InArgs._ForegroundColor;
	BackgroundColorOverride = InArgs._BackgroundColor;
	MaximumLength = InArgs._MaximumLength;
	ReadOnlyForegroundColorOverride = InArgs._ReadOnlyForegroundColor;
	FocusedForegroundColorOverride = InArgs._FocusedForegroundColor;
	bSelectWordOnMouseDoubleClick = InArgs._SelectWordOnMouseDoubleClick;

	OnTextChanged = InArgs._OnTextChanged;
	OnVerifyTextChanged = InArgs._OnVerifyTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;

	bHasExternalHScrollBar = InArgs._HScrollBar.IsValid();
	HScrollBar = InArgs._HScrollBar;
	if (!HScrollBar.IsValid())
	{
		// Create and use our own scrollbar
		HScrollBar = SNew(SScrollBar)
			.Style(&InArgs._Style->ScrollBarStyle)
			.Orientation(Orient_Horizontal)
			.AlwaysShowScrollbar(InArgs._AlwaysShowScrollbars)
			.Thickness(FVector2D(9.0f, 9.0f));
	}
	
	bHasExternalVScrollBar = InArgs._VScrollBar.IsValid();
	VScrollBar = InArgs._VScrollBar;
	if (!VScrollBar.IsValid())
	{
		// Create and use our own scrollbar
		VScrollBar = SNew(SScrollBar)
			.Style(&InArgs._Style->ScrollBarStyle)
			.Orientation(Orient_Vertical)
			.AlwaysShowScrollbar(InArgs._AlwaysShowScrollbars)
			.Thickness(FVector2D(9.0f, 9.0f));
	}

	SBorder::Construct( SBorder::FArguments()
		.BorderImage( this, &SMultiLineEditableTextBox::DetermineBorderImage )
		.BorderBackgroundColor( this, &SMultiLineEditableTextBox::DetermineBackgroundColor )
		.ForegroundColor( this, &SMultiLineEditableTextBox::DetermineForegroundColor )
		.Padding( this, &SMultiLineEditableTextBox::DeterminePadding )
		[
			SAssignNew( Box, SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.FillWidth(1)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.FillHeight(1)
				[
					SAssignNew( EditableText, SMultiLineEditableText )
					.Text( InArgs._Text )
					.HintText( InArgs._HintText )
					.SearchText( InArgs._SearchText )
					.TextStyle( &InArgs._Style->TextStyle )
					.Marshaller( InArgs._Marshaller )
					.Font( this, &SMultiLineEditableTextBox::DetermineFont )
					.IsReadOnly( InArgs._IsReadOnly )
					.AllowMultiLine( InArgs._AllowMultiLine )
					.OnContextMenuOpening( InArgs._OnContextMenuOpening )
					.OnIsTypedCharValid( InArgs._OnIsTypedCharValid )
					.OnBeginTextEdit( InArgs._OnBeginTextEdit )
					.OnTextChanged(this, &SMultiLineEditableTextBox::OnEditableTextChanged)
					.OnTextCommitted(this, &SMultiLineEditableTextBox::OnEditableTextCommitted)
					.OnCursorMoved( InArgs._OnCursorMoved )
					.ContextMenuExtender( InArgs._ContextMenuExtender )
					.CreateSlateTextLayout( InArgs._CreateSlateTextLayout )
					.Justification(InArgs._Justification)
					.RevertTextOnEscape(InArgs._RevertTextOnEscape)
					.SelectAllTextWhenFocused(InArgs._SelectAllTextWhenFocused)
					.SelectWordOnMouseDoubleClick(InArgs._SelectWordOnMouseDoubleClick)
					.ClearTextSelectionOnFocusLoss(InArgs._ClearTextSelectionOnFocusLoss)
					.ClearKeyboardFocusOnCommit(InArgs._ClearKeyboardFocusOnCommit)
					.LineHeightPercentage(InArgs._LineHeightPercentage)
					.Margin(InArgs._Margin)
					.WrapTextAt(InArgs._WrapTextAt)
					.AutoWrapText(InArgs._AutoWrapText)
					.WrappingPolicy(InArgs._WrappingPolicy)
					.HScrollBar(HScrollBar)
					.VScrollBar(VScrollBar)
					.OnHScrollBarUserScrolled(InArgs._OnHScrollBarUserScrolled)
					.OnVScrollBarUserScrolled(InArgs._OnVScrollBarUserScrolled)
					.OnKeyCharHandler(InArgs._OnKeyCharHandler)
					.OnKeyDownHandler(InArgs._OnKeyDownHandler)
					.ModiferKeyForNewLine(InArgs._ModiferKeyForNewLine)
					.VirtualKeyboardOptions( InArgs._VirtualKeyboardOptions )
					.VirtualKeyboardTrigger( InArgs._VirtualKeyboardTrigger )
					.VirtualKeyboardDismissAction( InArgs._VirtualKeyboardDismissAction )
					.TextShapingMethod(InArgs._TextShapingMethod)
					.TextFlowDirection(InArgs._TextFlowDirection)
					.AllowContextMenu(InArgs._AllowContextMenu)
					.OverflowPolicy(InArgs._OverflowPolicy)
					.FontFacesLoadingPaintPolicy(InArgs._FontFacesLoadingPaintPolicy)
					.OnAllFontFacesFinishLoading(InArgs._OnAllFontFacesFinishLoading)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(HScrollBarPaddingBox, SBox)
					.Padding( this, &SMultiLineEditableTextBox::DetermineHScrollBarPadding )
					[
						AsWidgetRef( HScrollBar )
					]
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(VScrollBarPaddingBox, SBox)
				.Padding( this, &SMultiLineEditableTextBox::DetermineVScrollBarPadding )
				[
					AsWidgetRef( VScrollBar )
				]
			]
		]
	);

	SetStyle(InArgs._Style);

	ErrorReporting = InArgs._ErrorReporting;
	if ( ErrorReporting.IsValid() )
	{
		Box->AddSlot()
		.AutoWidth()
		.Padding(3,0)
		[
			ErrorReporting->AsWidget()
		];
	}

}

void SMultiLineEditableTextBox::GetCurrentTextLine(FString& OutTextLine) const
{
	EditableText->GetCurrentTextLine(OutTextLine);
}

void SMultiLineEditableTextBox::SetStyle(const FEditableTextBoxStyle* InStyle)
{
	if (InStyle)
	{
		Style = InStyle;
	}
	else
	{
		FArguments Defaults;
		Style = Defaults._Style;
	}

	check(Style);

	if (!bHasExternalHScrollBar && HScrollBar.IsValid())
	{
		HScrollBar->SetStyle(&Style->ScrollBarStyle);
	}

	if (!bHasExternalVScrollBar && VScrollBar.IsValid())
	{
		VScrollBar->SetStyle(&Style->ScrollBarStyle);
	}

	BorderImageNormal = &Style->BackgroundImageNormal;
	BorderImageHovered = &Style->BackgroundImageHovered;
	BorderImageFocused = &Style->BackgroundImageFocused;
	BorderImageReadOnly = &Style->BackgroundImageReadOnly;

	SetTextStyle(&Style->TextStyle);
}

void SMultiLineEditableTextBox::SetTextStyle(const FTextBlockStyle* InTextStyle)
{
	EditableText->SetTextStyle(InTextStyle);
}

FMargin SMultiLineEditableTextBox::DeterminePadding() const
{
	check(Style);
	return PaddingOverride.IsSet() ? PaddingOverride.Get() : Style->Padding;
}

FMargin SMultiLineEditableTextBox::DetermineHScrollBarPadding() const
{
	check(Style);
	return HScrollBarPaddingOverride.IsSet() ? HScrollBarPaddingOverride.Get() : Style->HScrollBarPadding;
}

FMargin SMultiLineEditableTextBox::DetermineVScrollBarPadding() const
{
	check(Style);
	return VScrollBarPaddingOverride.IsSet() ? VScrollBarPaddingOverride.Get() : Style->VScrollBarPadding;
}

FSlateFontInfo SMultiLineEditableTextBox::DetermineFont() const
{
	return FontOverride.IsSet() ? FontOverride.Get() : Style->TextStyle.Font;
}

FSlateColor SMultiLineEditableTextBox::DetermineBackgroundColor() const
{
	check(Style);  
	return BackgroundColorOverride.IsSet() ? BackgroundColorOverride.Get() : Style->BackgroundColor;
}

FSlateColor SMultiLineEditableTextBox::DetermineForegroundColor() const
{
	check(Style);

	FSlateColor Result = FSlateColor::UseStyle();

	if (EditableText->IsTextReadOnly())
	{
		if (ReadOnlyForegroundColorOverride.IsSet())
		{
			Result = ReadOnlyForegroundColorOverride.Get();
		}
		else if (ForegroundColorOverride.IsSet())
		{
			Result = ForegroundColorOverride.Get();
		}

		if (Result == FSlateColor::UseStyle())
		{
			return Style->ReadOnlyForegroundColor;
		}
		else
		{
			return Result;
		}
	}
	else if (HasKeyboardFocus())
	{
		if (FocusedForegroundColorOverride.IsSet())
		{
			Result = FocusedForegroundColorOverride.Get();
		}
		else if (ForegroundColorOverride.IsSet())
		{
			Result = ForegroundColorOverride.Get();
		}

		if (Result == FSlateColor::UseStyle())
		{
			return Style->FocusedForegroundColor;
		}
		else
		{
			return Result;
		}
	}
	else
	{
		if (ForegroundColorOverride.IsSet())
		{
			Result = ForegroundColorOverride.Get();
		}

		if (Result == FSlateColor::UseStyle())
		{
			return Style->ForegroundColor;
		}
		else
		{
			return Result;
		}

	}
}

void SMultiLineEditableTextBox::SetText(TAttribute<FText> InNewText)
{
	EditableText->SetText(MoveTemp(InNewText));
}

void SMultiLineEditableTextBox::SetHintText(TAttribute<FText> InHintText)
{
	EditableText->SetHintText(MoveTemp(InHintText));
}

void SMultiLineEditableTextBox::SetSearchText(TAttribute<FText> InSearchText)
{
	EditableText->SetSearchText(MoveTemp(InSearchText));
}

FText SMultiLineEditableTextBox::GetSearchText() const
{
	return EditableText->GetSearchText();
}

void SMultiLineEditableTextBox::SetTextBoxForegroundColor(TAttribute<FSlateColor> InForegroundColor)
{
	ForegroundColorOverride = MoveTemp(InForegroundColor);
}

void SMultiLineEditableTextBox::SetTextBoxBackgroundColor(TAttribute<FSlateColor> InBackgroundColor)
{
	BackgroundColorOverride = MoveTemp(InBackgroundColor);
}

void SMultiLineEditableTextBox::SetReadOnlyForegroundColor(TAttribute<FSlateColor> InReadOnlyForegroundColor)
{
	ReadOnlyForegroundColorOverride = MoveTemp(InReadOnlyForegroundColor);
}

void SMultiLineEditableTextBox::SetMaximumLength(TAttribute<int32> InMaximumLength)
{
	MaximumLength = MoveTemp(InMaximumLength);
}

void SMultiLineEditableTextBox::SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick)
{
	EditableText->SetSelectWordOnMouseDoubleClick(MoveTemp(InSelectWordOnMouseDoubleClick));
}

void SMultiLineEditableTextBox::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	EditableText->SetTextShapingMethod(InTextShapingMethod);
}

void SMultiLineEditableTextBox::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	EditableText->SetTextFlowDirection(InTextFlowDirection);
}

void SMultiLineEditableTextBox::SetWrapTextAt(TAttribute<float> InWrapTextAt)
{
	EditableText->SetWrapTextAt(MoveTemp(InWrapTextAt));
}

void SMultiLineEditableTextBox::SetAutoWrapText(TAttribute<bool> InAutoWrapText)
{
	EditableText->SetAutoWrapText(MoveTemp(InAutoWrapText));
}

void SMultiLineEditableTextBox::SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy)
{
	EditableText->SetWrappingPolicy(MoveTemp(InWrappingPolicy));
}

void SMultiLineEditableTextBox::SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage)
{
	EditableText->SetLineHeightPercentage(MoveTemp(InLineHeightPercentage));
}

void SMultiLineEditableTextBox::SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine)
{
	EditableText->SetApplyLineHeightToBottomLine(MoveTemp(InApplyLineHeightToBottomLine));
}

void SMultiLineEditableTextBox::SetMargin(TAttribute<FMargin> InMargin)
{
	EditableText->SetMargin(MoveTemp(InMargin));
}

void SMultiLineEditableTextBox::SetJustification(TAttribute<ETextJustify::Type> InJustification)
{
	EditableText->SetJustification(MoveTemp(InJustification));
}

void SMultiLineEditableTextBox::SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy)
{
	EditableText->SetOverflowPolicy(InOverflowPolicy);
}

void SMultiLineEditableTextBox::SetAllowContextMenu(TAttribute<bool> InAllowContextMenu)
{
	EditableText->SetAllowContextMenu(MoveTemp(InAllowContextMenu));
}

void SMultiLineEditableTextBox::SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction)
{
	EditableText->SetVirtualKeyboardDismissAction(MoveTemp(InVirtualKeyboardDismissAction));
}

void SMultiLineEditableTextBox::SetIsReadOnly(TAttribute<bool> InIsReadOnly)
{
	EditableText->SetIsReadOnly(MoveTemp(InIsReadOnly));
}

void SMultiLineEditableTextBox::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	EditableText->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
}

void SMultiLineEditableTextBox::SetError(const FText& InError)
{
	SetError( InError.ToString() );
}

void SMultiLineEditableTextBox::SetError(const FString& InError)
{
	const bool bHaveError = !InError.IsEmpty();

	if (!ErrorReporting)
	{
		// If we are clearing an error but none was set before, we can leave.
		// If we have an active timer, we might have called this function previously, so we still need to register a timer
		if (!bHaveError && !HasActiveTimers())
		{
			return;
		}
		RegisterActiveTimer( 0.f,
			FWidgetActiveTimerDelegate::CreateLambda([this, InError](double, float)
			{
				if (!ErrorReporting) // SetError might have been called a few times, so ErrorReporting might be set
				{
					// No error reporting was specified; make a default one
					TSharedPtr<SPopupErrorText> ErrorTextWidget;
					Box->AddSlot()
					.AutoWidth()
					.Padding(3,0)
					[
						SAssignNew( ErrorTextWidget, SPopupErrorText )
					];
					ErrorReporting = ErrorTextWidget;
				}
				SetError(InError);
				return EActiveTimerReturnType::Stop;
			})
		);
		return;
	}

	ErrorReporting->AsWidget()->SetVisibility(bHaveError ? EVisibility::Visible : EVisibility::Collapsed);
	ErrorReporting->SetError(InError);
}

bool SMultiLineEditableTextBox::HasError() const
{
	return ErrorReporting && ErrorReporting->HasError();
}

/** @return Border image for the text box based on the hovered and focused state */
const FSlateBrush* SMultiLineEditableTextBox::DetermineBorderImage() const
{
	if ( EditableText->IsTextReadOnly() )
	{
		return BorderImageReadOnly;
	}
	else if ( EditableText->HasKeyboardFocus() )
	{
		return BorderImageFocused;
	}
	else
	{
		if ( EditableText->IsHovered() )
		{
			return BorderImageHovered;
		}
		else
		{
			return BorderImageNormal;
		}
	}
}

bool SMultiLineEditableTextBox::SupportsKeyboardFocus() const
{
	return StaticCastSharedPtr<SWidget>(EditableText)->SupportsKeyboardFocus();
}

bool SMultiLineEditableTextBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return SBorder::HasKeyboardFocus() || EditableText->HasKeyboardFocus();
}

FReply SMultiLineEditableTextBox::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	FReply Reply = FReply::Handled();

	if ( InFocusEvent.GetCause() != EFocusCause::Cleared )
	{
		// Forward keyboard focus to our editable text widget
		Reply.SetUserFocus(EditableText.ToSharedRef(), InFocusEvent.GetCause());
	}

	return Reply;
}

FReply SMultiLineEditableTextBox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FKey Key = InKeyEvent.GetKey();

	if (Key == EKeys::Escape && EditableText->HasKeyboardFocus())
	{
		// Clear focus
		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Cleared);
	}

	return FReply::Unhandled();
}

bool SMultiLineEditableTextBox::AnyTextSelected() const
{
	return EditableText->AnyTextSelected();
}

void SMultiLineEditableTextBox::SelectAllText()
{
	EditableText->SelectAllText();
}

void SMultiLineEditableTextBox::ClearSelection()
{
	EditableText->ClearSelection();
}

FText SMultiLineEditableTextBox::GetSelectedText() const
{
	return EditableText->GetSelectedText();
}

void SMultiLineEditableTextBox::InsertTextAtCursor(const FText& InText)
{
	EditableText->InsertTextAtCursor(InText);
}

void SMultiLineEditableTextBox::InsertTextAtCursor(const FString& InString)
{
	EditableText->InsertTextAtCursor(InString);
}

void SMultiLineEditableTextBox::InsertRunAtCursor(TSharedRef<IRun> InRun)
{
	EditableText->InsertRunAtCursor(MoveTemp(InRun));
}

void SMultiLineEditableTextBox::GoTo(const FTextLocation& NewLocation)
{
	EditableText->GoTo(NewLocation);
}

void SMultiLineEditableTextBox::ScrollTo(const FTextLocation& NewLocation)
{
	EditableText->ScrollTo(NewLocation);
}

void SMultiLineEditableTextBox::ApplyToSelection(const FRunInfo& InRunInfo, const FTextBlockStyle& InStyle)
{
	EditableText->ApplyToSelection(InRunInfo, InStyle);
}

void SMultiLineEditableTextBox::BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase, const bool InReverse)
{
	EditableText->BeginSearch(InSearchText, InSearchCase, InReverse);
}

void SMultiLineEditableTextBox::AdvanceSearch(const bool InReverse)
{
	EditableText->AdvanceSearch(InReverse);
}

TSharedPtr<const IRun> SMultiLineEditableTextBox::GetRunUnderCursor() const
{
	return EditableText->GetRunUnderCursor();
}

TArray<TSharedRef<const IRun>> SMultiLineEditableTextBox::GetSelectedRuns() const
{
	return EditableText->GetSelectedRuns();
}

FTextLocation SMultiLineEditableTextBox::GetCursorLocation() const
{
	return EditableText->GetCursorLocation();
}

TSharedPtr<const SScrollBar> SMultiLineEditableTextBox::GetHScrollBar() const
{
	return EditableText->GetHScrollBar();
}

TSharedPtr<const SScrollBar> SMultiLineEditableTextBox::GetVScrollBar() const
{
	return EditableText->GetVScrollBar();
}

void SMultiLineEditableTextBox::Refresh()
{
	return EditableText->Refresh();
}

void SMultiLineEditableTextBox::SetOnKeyCharHandler(FOnKeyChar InOnKeyCharHandler)
{
	EditableText->SetOnKeyCharHandler(MoveTemp(InOnKeyCharHandler));
}

void SMultiLineEditableTextBox::SetOnKeyDownHandler(FOnKeyDown InOnKeyDownHandler)
{
	EditableText->SetOnKeyDownHandler(MoveTemp(InOnKeyDownHandler));
}

void SMultiLineEditableTextBox::ForceScroll(int32 UserIndex, float ScrollAxisMagnitude)
{
	EditableText->ForceScroll(UserIndex, ScrollAxisMagnitude);
}

void SMultiLineEditableTextBox::OnEditableTextChanged(const FText& InText)
{
	OnTextChanged.ExecuteIfBound(InText);

	const int32 MaximumLengthValue = MaximumLength.Get();
	if (OnVerifyTextChanged.IsBound() || MaximumLengthValue >= 0)
	{
		FText OutErrorMessage;
		if (!FTextEditHelper::VerifyTextLength(InText, OutErrorMessage, MaximumLengthValue) ||
			(OnVerifyTextChanged.IsBound() && !OnVerifyTextChanged.Execute(InText, OutErrorMessage)))
		{
			// Display as an error.
			SetError(OutErrorMessage);
		}
		else
		{
			SetError(FText::GetEmpty());
		}
	}
}

void SMultiLineEditableTextBox::OnEditableTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	FText OutErrorMessage;
	if (!FTextEditHelper::VerifyTextLength(InText, OutErrorMessage, MaximumLength.Get()) ||
		(OnVerifyTextChanged.IsBound() && !OnVerifyTextChanged.Execute(InText, OutErrorMessage)))
	{
		// Display as an error.
		if (InCommitType == ETextCommit::OnEnter)
		{
			SetError(OutErrorMessage);
		}
		return;
	}

	// Text commited without errors, so clear error text
	SetError(FText::GetEmpty());

	OnTextCommitted.ExecuteIfBound(InText, InCommitType);
}

#endif //WITH_FANCY_TEXT
