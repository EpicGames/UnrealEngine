// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SSpacer.h"

void SSearchBox::Construct( const FArguments& InArgs )
{
	check(InArgs._Style);

	SearchResultData = InArgs._SearchResultData;
	bIsSearching = InArgs._IsSearching;
	OnSearchDelegate = InArgs._OnSearch;
	OnTextChangedDelegate = InArgs._OnTextChanged;
	OnTextCommittedDelegate = InArgs._OnTextCommitted;
	DelayChangeNotificationsWhileTyping = InArgs._DelayChangeNotificationsWhileTyping;
	DelayChangeNotificationsWhileTypingSeconds = InArgs._DelayChangeNotificationsWhileTypingSeconds;

	InactiveFont = InArgs._Style->TextBoxStyle.TextStyle.Font;
	ActiveFont = InArgs._Style->ActiveFontInfo;

	SEditableTextBox::Construct( SEditableTextBox::FArguments()
		.Style( &InArgs._Style->TextBoxStyle )
		.Font( this, &SSearchBox::GetWidgetFont )
		.Text( InArgs._InitialText.Get() )
		.HintText( InArgs._HintText )
		.SelectAllTextWhenFocused( InArgs._SelectAllTextWhenFocused )
		.RevertTextOnEscape( true )
		.ClearKeyboardFocusOnCommit( false )
		.OnTextChanged( this, &SSearchBox::HandleTextChanged )
		.OnTextCommitted( this, &SSearchBox::HandleTextCommitted )
		.OnBeginTextEdit(InArgs._OnBeginTextEdit)
		.OnVerifyTextChanged( InArgs._OnVerifyTextChanged )
		.MinDesiredWidth( InArgs._MinDesiredWidth )
		.OnKeyDownHandler( InArgs._OnKeyDownHandler )
	);

	// If we want to have the search result buttons appear to the left of the text box we have to insert the slots instead of add them
	int32 SlotIndex = InArgs._Style->bLeftAlignSearchResultButtons ? 0 : Box->NumSlots();

	// Add a throbber to show if there is a search running.
	Box->InsertSlot(SlotIndex++)
	.AutoWidth()
	.Padding(0, 0, 2, 0)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(SCircularThrobber)
		.Radius(9.0f)
		.Visibility(this, &SSearchBox::GetIsSearchingThrobberVisibility)
		.ToolTipText(NSLOCTEXT("SearchBox", "Searching", "Searching..."))
		.ColorAndOpacity(FSlateColor::UseForeground())
	];

	// If a search delegate was bound, add a previous and next button
	if (OnSearchDelegate.IsBound())
	{
		FMargin SearchResultPadding = InArgs._Style->bLeftAlignSearchResultButtons ? FMargin(5, 0, 2, 0) : FMargin(2, 0, 2, 0);

		// Search result data text
		Box->InsertSlot(SlotIndex++)
		.AutoWidth()
		.Padding(SearchResultPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Visibility(this, &SSearchBox::GetSearchResultDataVisibility)
			.Text(this, &SSearchBox::GetSearchResultText)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		];

		// Previous result button
		Box->InsertSlot(SlotIndex++)
		.AutoWidth()
		.Padding(InArgs._Style->ImagePadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SNew(SButton)
			.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
			.ContentPadding( FMargin(5, 0) )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked( this, &SSearchBox::OnClickedSearch, SSearchBox::Previous )
			.ForegroundColor( FSlateColor::UseForeground() )
			.IsFocusable(false)
			.Visibility(this, &SSearchBox::GetSearchResultNavigationButtonVisibility)
			[
				SNew(SImage)
				.Image( &InArgs._Style->UpArrowImage )
				.ColorAndOpacity( FSlateColor::UseForeground() )
				.DesiredSizeOverride( InArgs._Style->ImageSizeOverride )
			]
		];
		// Next result button
		Box->InsertSlot(SlotIndex++)
		.AutoWidth()
		.Padding(InArgs._Style->ImagePadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SNew(SButton)
			.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
			.ContentPadding( FMargin(5, 0) )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked( this, &SSearchBox::OnClickedSearch, SSearchBox::Next )
			.ForegroundColor( FSlateColor::UseForeground() )
			.IsFocusable(false)
			.Visibility(this, &SSearchBox::GetSearchResultNavigationButtonVisibility)
			[
				SNew(SImage)
				.Image( &InArgs._Style->DownArrowImage )
				.ColorAndOpacity( FSlateColor::UseForeground() )
				.DesiredSizeOverride( InArgs._Style->ImageSizeOverride )
			]
		];
	}

	// Add a search glass image so that the user knows this text box is for searching
	int GlassImageSlotIndex = InArgs._Style->bLeftAlignGlassImageAndClearButton ? 0 : Box->NumSlots();

	if (InArgs._Style->bLeftAlignSearchResultButtons == InArgs._Style->bLeftAlignGlassImageAndClearButton)
	{
		// if they are on the same side, Glass Image follows the search result buttons
		GlassImageSlotIndex = SlotIndex;
	}

	const int32 ClearButtonIndex = GlassImageSlotIndex + 1;
	
	Box->InsertSlot(GlassImageSlotIndex)
		.AutoWidth()
		.Padding(InArgs._Style->ImagePadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Visibility(this, &SSearchBox::GetSearchGlassVisibility)
			.Image(&InArgs._Style->GlassImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.DesiredSizeOverride(InArgs._Style->ImageSizeOverride)
		];

	// Add an X to clear the search whenever there is some text typed into it
	Box->InsertSlot(ClearButtonIndex)
	.AutoWidth()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.Visibility(this, &SSearchBox::GetXVisibility)
		.ButtonStyle(FAppStyle::Get(), "ThinHoverOnlyButton")
		.ContentPadding(InArgs._Style->ImagePadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked(this, &SSearchBox::OnClearSearch)
		.AddMetaData<FTagMetaData>(TEXT("SearchClearButton"))
		// Allow the button to steal focus so that the search text will be automatically committed. Afterwards focus will be returned to the text box.
		// If the user is keyboard-centric, they'll "ctrl+a, delete" to clear the search
		.IsFocusable(true)
		[
			SNew(SImage)
			.Image(&InArgs._Style->ClearImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.DesiredSizeOverride(InArgs._Style->ImageSizeOverride)
		]
	];

	// Padding at the end so that the text box grows long enough to host all buttons
	// the text box needs extra space because it is rounded on both ends
	Box->AddSlot()
	.AutoWidth()
	[
		SNew(SSpacer)
		.Size(FVector2d(7,0))
	];
}

EActiveTimerReturnType SSearchBox::TriggerOnTextChanged( double, float, FText NewText )
{
	// Reset the flag first in case the delegate winds up triggering HandleTextChanged
	ActiveTimerHandle.Reset();

	OnTextChangedDelegate.ExecuteIfBound( NewText );
	return EActiveTimerReturnType::Stop;
}

void SSearchBox::HandleTextChanged(const FText& NewText)
{
	// Remove the existing registered tick if necessary
	if ( ActiveTimerHandle.IsValid() )
	{
		UnRegisterActiveTimer( ActiveTimerHandle.Pin().ToSharedRef() );
	}

	if ( DelayChangeNotificationsWhileTyping.Get() && HasKeyboardFocus() )
	{
		ActiveTimerHandle = RegisterActiveTimer( DelayChangeNotificationsWhileTypingSeconds.Get(), FWidgetActiveTimerDelegate::CreateSP( this, &SSearchBox::TriggerOnTextChanged, NewText ) );
	}
	else
	{
		OnTextChangedDelegate.ExecuteIfBound( NewText );
	}
}

void SSearchBox::HandleTextCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	if ( ActiveTimerHandle.IsValid() )
	{
		UnRegisterActiveTimer( ActiveTimerHandle.Pin().ToSharedRef() );
		
		// If there was a pending text change we need to fire it in case someone was cache the last value and
		// ignoring changes during commit, or if they expected a change always before the commit.
		OnTextChangedDelegate.ExecuteIfBound(NewText);
	}

	OnTextCommittedDelegate.ExecuteIfBound( NewText, CommitType );
}

FText SSearchBox::GetSearchResultText() const
{
	TOptional<FSearchResultData> CurrentSearchResultData = SearchResultData.Get();
	if (CurrentSearchResultData.IsSet())
	{
		return FText::Format(NSLOCTEXT("SearchBox", "SearchResultFormat", "{0} / {1}"), 
			CurrentSearchResultData.GetValue().CurrentSearchResultIndex, CurrentSearchResultData.GetValue().NumSearchResults);
	}
	else
	{
		return FText();
	}
}

EVisibility SSearchBox::GetSearchResultNavigationButtonVisibility() const
{
	if (SearchResultData.IsBound())
	{
		return SearchResultData.Get().IsSet()
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}
	else
	{
		return EVisibility::Visible;
	}
}

EVisibility SSearchBox::GetXVisibility() const
{
	return (EditableText->GetText().IsEmpty())
		? EVisibility::Collapsed
		: EVisibility::Visible;
}

EVisibility SSearchBox::GetSearchResultDataVisibility() const
{
	return SearchResultData.Get().IsSet()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SSearchBox::GetIsSearchingThrobberVisibility() const
{
	return (bIsSearching.Get())
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SSearchBox::GetSearchGlassVisibility() const
{
	return (EditableText->GetText().IsEmpty())
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

FReply SSearchBox::OnClickedSearch(SSearchBox::SearchDirection Direction)
{
	OnSearchDelegate.ExecuteIfBound(Direction);
	return FReply::Handled();
}

FReply SSearchBox::OnClearSearch()
{
	// When we get here, the button will already have stolen focus, thus committing any unset values in the search box.
	// This will have allowed any widgets which depend on its state to update themselves prior to the search box being cleared,
	// which happens now. This is important as the act of clearing the search text may also destroy those widgets (for example,
	// if the search box is being used as a filter).
	this->SetText( FText::GetEmpty() );

	// Finally set focus back to the editable text
	return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), EFocusCause::SetDirectly);
}

FSlateFontInfo SSearchBox::GetWidgetFont() const
{
	return EditableText->GetText().IsEmpty() ? InactiveFont : ActiveFont;
}

FTextSelection SSearchBox::GetSelection() const
{
	return EditableText->GetSelection();
}

void SSearchBox::SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation)
{
	EditableText->SelectText(InSelectionStart, InCursorLocation);
}
