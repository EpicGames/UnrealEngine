// Copyright Epic Games, Inc. All Rights Reservekd.

#include "Widgets/Input/SSearchableComboBox.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SearchableComboBox"

void SSearchableComboBox::Construct(const FArguments& InArgs)
{
	check(InArgs._ComboBoxStyle);

	ItemStyle = InArgs._ItemStyle;
	MenuRowPadding = InArgs._ComboBoxStyle->MenuRowPadding;

	// Work out which values we should use based on whether we were given an override, or should use the style's version
	const FComboButtonStyle& OurComboButtonStyle = InArgs._ComboBoxStyle->ComboButtonStyle;
	const FButtonStyle* const OurButtonStyle = InArgs._ButtonStyle ? InArgs._ButtonStyle : &OurComboButtonStyle.ButtonStyle;

	this->OnComboBoxOpening = InArgs._OnComboBoxOpening;
	this->OnSelectionChanged = InArgs._OnSelectionChanged;
	this->bAlwaysSelectItem = InArgs._bAlwaysSelectItem;
	this->OnGenerateWidget = InArgs._OnGenerateWidget;

	OptionsSource = InArgs._OptionsSource;
	CustomScrollbar = InArgs._CustomScrollbar;

	FilteredOptionsSource.Append(*OptionsSource);

	TAttribute<EVisibility> SearchVisibility = InArgs._SearchVisibility;
	const EVisibility CurrentSearchVisibility = SearchVisibility.Get();

	TSharedRef<SWidget> ComboBoxMenuContent =
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxListHeight)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(this->SearchField, SEditableTextBox)
				.HintText(LOCTEXT("Search", "Search"))
				.OnTextChanged(this, &SSearchableComboBox::OnSearchTextChanged)
				.OnTextCommitted(this, &SSearchableComboBox::OnSearchTextCommitted)
				.Visibility(SearchVisibility)
			]

			+ SVerticalBox::Slot()
			[
				SAssignNew(this->ComboListView, SComboListType)
				.ListItemsSource(&FilteredOptionsSource)
				.OnGenerateRow(this, &SSearchableComboBox::GenerateMenuItemRow)
				.OnSelectionChanged(this, &SSearchableComboBox::OnSelectionChanged_Internal)
				.OnKeyDownHandler(this, &SSearchableComboBox::OnKeyDownHandler)
				.SelectionMode(ESelectionMode::Single)
				.ExternalScrollbar(InArgs._CustomScrollbar)
			]
		];

	// Set up content
	TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
	if (InArgs._Content.Widget == SNullWidget::NullWidget)
	{
		SAssignNew(ButtonContent, STextBlock)
			.Text(NSLOCTEXT("SSearchableComboBox", "ContentWarning", "No Content Provided"))
			.ColorAndOpacity(FLinearColor::Red);
	}


	SComboButton::Construct(SComboButton::FArguments()
		.ComboButtonStyle(&OurComboButtonStyle)
		.ButtonStyle(OurButtonStyle)
		.Method(InArgs._Method)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
		.MenuContent()
		[
			ComboBoxMenuContent
		]
		.HasDownArrow(InArgs._HasDownArrow)
		.ContentPadding(InArgs._ContentPadding)
		.ForegroundColor(InArgs._ForegroundColor)
		.OnMenuOpenChanged(this, &SSearchableComboBox::OnMenuOpenChanged)
		.IsFocusable(true)
		);

	if (CurrentSearchVisibility == EVisibility::Visible)
	{
		SetMenuContentWidgetToFocus(SearchField);
	}
	else
	{
		SetMenuContentWidgetToFocus(ComboListView);
	}

	// Need to establish the selected item at point of construction so its available for querying
	// NB: If you need a selection to fire use SetItemSelection rather than setting an IntiallySelectedItem
	SelectedItem = InArgs._InitiallySelectedItem;
	if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(SelectedItem))
	{
		ComboListView->Private_SetItemSelection(SelectedItem, true);
	}

}

void SSearchableComboBox::ClearSelection()
{
	ComboListView->ClearSelection();
	SelectedItem = nullptr;
}

void SSearchableComboBox::SetSelectedItem(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(InSelectedItem))
	{
		ComboListView->SetSelection(InSelectedItem, InSelectInfo);
	}
	else
	{
		ComboListView->SetSelection(SelectedItem, InSelectInfo);
	}
}

TSharedPtr<FString> SSearchableComboBox::GetSelectedItem()
{
	return SelectedItem;
}

void SSearchableComboBox::RefreshOptions()
{
	// Need to refresh filtered list whenever options change
	FilteredOptionsSource.Reset();

	if (SearchText.IsEmpty())
	{
		FilteredOptionsSource.Append(*OptionsSource);
	}
	else
	{
		TArray<FString> SearchTokens;
		SearchText.ToString().ParseIntoArrayWS(SearchTokens);

		for (const TSharedPtr<FString>& Option : *OptionsSource)
		{
			bool bAllTokensMatch = true;
			for (const FString& SearchToken : SearchTokens)
			{
				if (Option->Find(SearchToken, ESearchCase::Type::IgnoreCase) == INDEX_NONE)
				{
					bAllTokensMatch = false;
					break;
				}
			}

			if (bAllTokensMatch)
			{
				FilteredOptionsSource.Add(Option);
			}
		}
	}

	ComboListView->RequestListRefresh();
}

void SSearchableComboBox::SetOptionsSource(const TArray<TSharedPtr<FString>>* InOptionsSource)
{
	OptionsSource = InOptionsSource;

	FilteredOptionsSource.Empty();
	FilteredOptionsSource.Append(*OptionsSource);
}

TSharedRef<ITableRow> SSearchableComboBox::GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (OnGenerateWidget.IsBound())
	{
		return SNew(SComboRow<TSharedPtr<FString>>, OwnerTable)
			.Style(ItemStyle)
			.Padding(MenuRowPadding)
			[
				OnGenerateWidget.Execute(InItem)
			];
	}
	else
	{
		return SNew(SComboRow<TSharedPtr<FString>>, OwnerTable)
			[
				SNew(STextBlock).Text(NSLOCTEXT("SlateCore", "ComboBoxMissingOnGenerateWidgetMethod", "Please provide a .OnGenerateWidget() handler."))
			];

	}
}

void SSearchableComboBox::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen == false)
	{
		if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(SelectedItem))
		{
			// Ensure the ListView selection is set back to the last committed selection
			ComboListView->SetSelection(SelectedItem, ESelectInfo::OnNavigation);
		}

		// Set focus back to ComboBox for users focusing the ListView that just closed
		FSlateApplication::Get().ForEachUser([this](FSlateUser& User) 
		{
			TSharedRef<SWidget> ThisRef = this->AsShared();
			if (User.IsWidgetInFocusPath(this->ComboListView))
			{
				User.SetFocus(ThisRef);
			}
		});

	}
}


void SSearchableComboBox::OnSelectionChanged_Internal(TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo)
{
	if (!ProposedSelection)
	{
		return;
	}

	// Ensure that the proposed selection is different from selected
	if (ProposedSelection != SelectedItem || bAlwaysSelectItem)
	{
		SelectedItem = ProposedSelection;
		OnSelectionChanged.ExecuteIfBound(ProposedSelection, SelectInfo);
	}

	// close combo as long as the selection wasn't from navigation
	if (SelectInfo != ESelectInfo::OnNavigation)
	{
		this->SetIsOpen(false);
	}
	else
	{
		ComboListView->RequestScrollIntoView(SelectedItem, 0);
	}
}

void SSearchableComboBox::OnSearchTextChanged(const FText& ChangedText)
{
	SearchText = ChangedText;

	RefreshOptions();
}

void SSearchableComboBox::OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if ((InCommitType == ETextCommit::Type::OnEnter) && FilteredOptionsSource.Num() > 0)
	{
		ComboListView->SetSelection(FilteredOptionsSource[0], ESelectInfo::OnKeyPress);
	}
}

FReply SSearchableComboBox::OnButtonClicked()
{
	// if user clicked to close the combo menu
	if (this->IsOpen())
	{
		// Re-select first selected item, just in case it was selected by navigation previously
		TArray<TSharedPtr<FString>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::Direct);
		}
	}
	else
	{
		SearchField->SetText(FText::GetEmpty());
		OnComboBoxOpening.ExecuteIfBound();
	}

	return SComboButton::OnButtonClicked();
}

FReply SSearchableComboBox::OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Select the first selected item on hitting enter
		TArray<TSharedPtr<FString>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::OnKeyPress);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
