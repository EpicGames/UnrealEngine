// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SSlider.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

SLATE_IMPLEMENT_WIDGET(SSlider)
void SSlider::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Value", ValueSlateAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "IndentHandle", IndentHandleSlateAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Locked", LockedSlateAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SliderBarColor", SliderBarColorSlateAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SliderHandleColor", SliderHandleColorSlateAttribute, EInvalidateWidgetReason::Paint);

	AttributeInitializer.OverrideInvalidationReason("EnabledState", FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Paint});
	AttributeInitializer.OverrideInvalidationReason("Hovered", FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Paint});
}

SSlider::SSlider()
	: Style(nullptr)
	, PressedScreenSpaceTouchDownPosition(FVector2f(0, 0))
	, ValueSlateAttribute(*this, 1.f)
	, IndentHandleSlateAttribute(*this, true)
	, LockedSlateAttribute(*this, false)
	, SliderBarColorSlateAttribute(*this, FLinearColor::White)
	, SliderHandleColorSlateAttribute(*this, FLinearColor::White)
{
#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

SSlider::~SSlider() = default;

void SSlider::Construct( const SSlider::FArguments& InDeclaration )
{
	check(InDeclaration._Style);

	Style = InDeclaration._Style;

	IndentHandleSlateAttribute.Assign(*this, InDeclaration._IndentHandle);
	bMouseUsesStep = InDeclaration._MouseUsesStep;
	bRequiresControllerLock = InDeclaration._RequiresControllerLock;
	LockedSlateAttribute.Assign(*this, InDeclaration._Locked);
	Orientation = InDeclaration._Orientation;
	StepSize = InDeclaration._StepSize;
	ValueSlateAttribute.Assign(*this, InDeclaration._Value);
	MinValue = InDeclaration._MinValue;
	MaxValue = InDeclaration._MaxValue;
	SliderBarColorSlateAttribute.Assign(*this, InDeclaration._SliderBarColor);
	SliderHandleColorSlateAttribute.Assign(*this, InDeclaration._SliderHandleColor);
	bIsFocusable = InDeclaration._IsFocusable;
	OnMouseCaptureBegin = InDeclaration._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InDeclaration._OnMouseCaptureEnd;
	OnControllerCaptureBegin = InDeclaration._OnControllerCaptureBegin;
	OnControllerCaptureEnd = InDeclaration._OnControllerCaptureEnd;
	OnValueChanged = InDeclaration._OnValueChanged;
	bPreventThrottling = InDeclaration._PreventThrottling;

	bControllerInputCaptured = false;
}

int32 SSlider::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// we draw the slider like a horizontal slider regardless of the orientation, and apply a render transform to make it display correctly.
	// However, the AllottedGeometry is computed as it will be rendered, so we have to use the "horizontal orientation" when doing drawing computations.
	const float AllottedWidth = Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().X : AllottedGeometry.GetLocalSize().Y;
	const float AllottedHeight = Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X;

	float HandleRotation;
	FVector2f HandleTopLeftPoint;
	FVector2f SliderStartPoint;
	FVector2f SliderEndPoint;

	// calculate slider geometry as if it's a horizontal slider (we'll rotate it later if it's vertical)
	const FVector2f HandleSize = GetThumbImage()->ImageSize;
	const FVector2f HalfHandleSize = 0.5f * HandleSize;
	const float Indentation = IndentHandleSlateAttribute.Get() ? HandleSize.X : 0.0f;

	// We clamp to make sure that the slider cannot go out of the slider Length.
	const float SliderPercent = FMath::Clamp(GetNormalizedValue(), 0.0f, 1.0f); 
	const float SliderLength = AllottedWidth - (Indentation + HandleSize.X);
	const float SliderHandleOffset = SliderPercent * SliderLength;
	const float SliderY = 0.5f * AllottedHeight;

	HandleRotation = 0.0f;
	HandleTopLeftPoint = FVector2f(SliderHandleOffset + (0.5f * Indentation), SliderY - HalfHandleSize.Y);

	SliderStartPoint = FVector2f(HalfHandleSize.X, SliderY);
	SliderEndPoint = FVector2f(AllottedWidth - HalfHandleSize.X, SliderY);

	FGeometry SliderGeometry = AllottedGeometry;
	
	// rotate the slider 90deg if it's vertical. The 0 side goes on the bottom, the 1 side on the top.
	if (Orientation == Orient_Vertical)
	{
		// Do this by translating along -X by the width of the geometry, then rotating 90 degreess CCW (left-hand coords)
		FSlateRenderTransform SlateRenderTransform = TransformCast<FSlateRenderTransform>(Concatenate(Inverse(FVector2f(AllottedWidth, 0)), FQuat2D(FMath::DegreesToRadians(-90.0f))));
		// create a child geometry matching this one, but with the render transform.
		SliderGeometry = AllottedGeometry.MakeChild(
			FVector2f(AllottedWidth, AllottedHeight), 
			FSlateLayoutTransform(), 
			SlateRenderTransform, FVector2f::ZeroVector);
	}

	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	// draw slider bar
	auto BarTopLeft = FVector2f(SliderStartPoint.X, SliderStartPoint.Y - Style->BarThickness * 0.5f);
	auto BarSize = FVector2f(SliderEndPoint.X - SliderStartPoint.X, Style->BarThickness);
	auto BarImage = GetBarImage();
	auto ThumbImage = GetThumbImage();
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(BarSize, FSlateLayoutTransform(BarTopLeft)),
		BarImage,
		DrawEffects,
		BarImage->GetTint(InWidgetStyle) * SliderBarColorSlateAttribute.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

	++LayerId;

	// draw slider thumb
	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(GetThumbImage()->ImageSize, FSlateLayoutTransform(HandleTopLeftPoint)),
		ThumbImage,
		DrawEffects,
		ThumbImage->GetTint(InWidgetStyle) * SliderHandleColorSlateAttribute.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return LayerId;
}

FVector2D SSlider::ComputeDesiredSize( float ) const
{
	static const FVector2D SSliderDesiredSize(16.0f, 16.0f);

	if ( Style == nullptr )
	{
		return SSliderDesiredSize;
	}

	const float Thickness = FMath::Max(Style->BarThickness, 
		FMath::Max(Style->NormalThumbImage.ImageSize.Y, Style->HoveredThumbImage.ImageSize.Y));

	if (Orientation == Orient_Vertical)
	{
		return FVector2D(Thickness, SSliderDesiredSize.Y);
	}

	return FVector2D(SSliderDesiredSize.X, Thickness);
}

void SSlider::SetStyle(const FSliderStyle* InStyle)
{
	Style = InStyle;
	Invalidate(EInvalidateWidgetReason::Layout);
}

bool SSlider::IsLocked() const
{
	return LockedSlateAttribute.Get();
}

bool SSlider::IsInteractable() const
{
	return IsEnabled() && !IsLocked() && SupportsKeyboardFocus();
}

bool SSlider::SupportsKeyboardFocus() const
{
	return bIsFocusable;
}

void SSlider::ResetControllerState()
{
	if (bControllerInputCaptured)
	{
		OnControllerCaptureEnd.ExecuteIfBound();
		bControllerInputCaptured = false;
	}
}

FNavigationReply SSlider::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	if (bControllerInputCaptured || !bRequiresControllerLock)
	{
		FNavigationReply Reply = FNavigationReply::Escape();

		float NewValue = ValueSlateAttribute.Get();
		if (Orientation == EOrientation::Orient_Horizontal)
		{
			if (InNavigationEvent.GetNavigationType() == EUINavigation::Left)
			{
				NewValue -= StepSize.Get();
				Reply = FNavigationReply::Stop();
			}
			else if (InNavigationEvent.GetNavigationType() == EUINavigation::Right)
			{
				NewValue += StepSize.Get();
				Reply = FNavigationReply::Stop();
			}
		}
		else
		{
			if (InNavigationEvent.GetNavigationType() == EUINavigation::Down)
			{
				NewValue -= StepSize.Get();
				Reply = FNavigationReply::Stop();
			}
			else if (InNavigationEvent.GetNavigationType() == EUINavigation::Up)
			{
				NewValue += StepSize.Get();
				Reply = FNavigationReply::Stop();
			}
		}
		if (ValueSlateAttribute.Get() != NewValue)
		{
			CommitValue(FMath::Clamp(NewValue, MinValue, MaxValue));
			return Reply;
		}
	}

	return SLeafWidget::OnNavigation(MyGeometry, InNavigationEvent);
}

FReply SSlider::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsInteractable())
	{
		// The controller's bottom face button must be pressed once to begin manipulating the slider's value.
		// Navigation away from the widget is prevented until the button has been pressed again or focus is lost.
		// The value can be manipulated by using the game pad's directional arrows ( relative to slider orientation ).
		if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept && bRequiresControllerLock)
		{
			if (bControllerInputCaptured == false)
			{
				// Begin capturing controller input and allow user to modify the slider's value.
				bControllerInputCaptured = true;
				OnControllerCaptureBegin.ExecuteIfBound();
				Reply = FReply::Handled();
			}
			else
			{
				ResetControllerState();
				Reply = FReply::Handled();
			}
		}
		else
		{
			Reply = SLeafWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}
	}
	else
	{
		Reply = SLeafWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	return Reply;
}

FReply SSlider::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	if (bControllerInputCaptured)
	{
		Reply = FReply::Handled();
	}
	return Reply;
}

void SSlider::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if (bControllerInputCaptured)
	{
		// Commit and reset state
		CommitValue(ValueSlateAttribute.Get());
		ResetControllerState();
	}
}

FReply SSlider::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{
		CachedCursor = GetCursor().Get(EMouseCursor::Default);
		OnMouseCaptureBegin.ExecuteIfBound();
		CommitValue(PositionToValue(MyGeometry, MouseEvent.GetScreenSpacePosition()));
		
		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		FReply ReturnReply = FReply::Handled().CaptureMouse(SharedThis(this));
		if (bPreventThrottling)
		{
			ReturnReply.PreventThrottling();
		}
		return ReturnReply;
	}

	return FReply::Unhandled();
}

FReply SSlider::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && HasMouseCaptureByUser(MouseEvent.GetUserIndex(), MouseEvent.GetPointerIndex()))
	{
		SetCursor(CachedCursor);
		
		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();
		
		return FReply::Handled().ReleaseMouseCapture();	
	}

	return FReply::Unhandled();
}

void SSlider::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	OnMouseCaptureEnd.ExecuteIfBound();
	SLeafWidget::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SSlider::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (HasMouseCaptureByUser(MouseEvent.GetUserIndex(), MouseEvent.GetPointerIndex()) && !IsLocked())
	{
		SetCursor((Orientation == Orient_Horizontal) ? EMouseCursor::ResizeLeftRight : EMouseCursor::ResizeUpDown);
		CommitValue(PositionToValue(MyGeometry, MouseEvent.GetScreenSpacePosition()));
		
		// Release capture for controller/keyboard when switching to mouse
		ResetControllerState();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSlider::OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (!IsLocked())
	{
		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		PressedScreenSpaceTouchDownPosition = InTouchEvent.GetScreenSpacePosition();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSlider::OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (HasMouseCaptureByUser(InTouchEvent.GetUserIndex(), InTouchEvent.GetPointerIndex()))
	{
		CommitValue(PositionToValue(MyGeometry, InTouchEvent.GetScreenSpacePosition()));

		// Release capture for controller/keyboard when switching to mouse
		ResetControllerState();

		return FReply::Handled();
	}
	else if (!HasMouseCapture())
	{
		if (FSlateApplication::Get().HasTraveledFarEnoughToTriggerDrag(InTouchEvent, PressedScreenSpaceTouchDownPosition, Orientation))
		{
			CachedCursor = GetCursor().Get(EMouseCursor::Default);
			OnMouseCaptureBegin.ExecuteIfBound();

			CommitValue(PositionToValue(MyGeometry, InTouchEvent.GetScreenSpacePosition()));

			// Release capture for controller/keyboard when switching to mouse
			ResetControllerState();

			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}

	return FReply::Unhandled();
}

FReply SSlider::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (HasMouseCaptureByUser(InTouchEvent.GetUserIndex(), InTouchEvent.GetPointerIndex()))
	{
		SetCursor(CachedCursor);
		OnMouseCaptureEnd.ExecuteIfBound();

		CommitValue(PositionToValue(MyGeometry, InTouchEvent.GetScreenSpacePosition()));

		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void SSlider::CommitValue(float NewValue)
{
	const float OldValue = GetValue();

	if (NewValue != OldValue)
	{
		if (!ValueSlateAttribute.IsBound(*this))
		{
			ValueSlateAttribute.Assign(*this, NewValue);
		}

		Invalidate(EInvalidateWidgetReason::Paint);

		OnValueChanged.ExecuteIfBound(NewValue);
	}
}

float SSlider::PositionToValue( const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& AbsolutePosition )
{
	const FVector2f LocalPosition = MyGeometry.AbsoluteToLocal(AbsolutePosition);

	float RelativeValue;
	float Denominator;
	// Only need X as we rotate the thumb image when rendering vertically
	const float Indentation = GetThumbImage()->ImageSize.X * (IndentHandleSlateAttribute.Get() ? 2.f : 1.f);
	const float HalfIndentation = 0.5f * Indentation;

	if (Orientation == Orient_Horizontal)
	{
		Denominator = MyGeometry.Size.X - Indentation;
		RelativeValue = (Denominator != 0.f) ? (LocalPosition.X - HalfIndentation) / Denominator : 0.f;
	}
	else
	{
		Denominator = MyGeometry.Size.Y - Indentation;
		// Inverse the calculation as top is 0 and bottom is 1
		RelativeValue = (Denominator != 0.f) ? ((MyGeometry.Size.Y - LocalPosition.Y) - HalfIndentation) / Denominator : 0.f;
	}

	RelativeValue = FMath::Clamp(RelativeValue, 0.0f, 1.0f) * (MaxValue - MinValue) + MinValue;
	if (bMouseUsesStep)
	{
		float direction = ValueSlateAttribute.Get() - RelativeValue;
		float CurrentStepSize = StepSize.Get();
		if (CurrentStepSize <= 0)
		{
			// Invalid step size, keep current value
			return ValueSlateAttribute.Get();
		}
		float Steps = FMath::Abs(direction) / CurrentStepSize;
		Steps = FMath::RoundHalfFromZero(Steps);
		const float ClampedDist = Steps * CurrentStepSize;
		if (direction > CurrentStepSize / 2.0f)
		{
			return FMath::Clamp(ValueSlateAttribute.Get() - ClampedDist, MinValue, MaxValue);
		}
		else if (direction < CurrentStepSize / -2.0f)
		{
			return FMath::Clamp(ValueSlateAttribute.Get() + ClampedDist, MinValue, MaxValue);
		}
		else
		{
			return ValueSlateAttribute.Get();
		}
	}
	return RelativeValue;
}

const FSlateBrush* SSlider::GetBarImage() const
{
	if (!IsEnabled() || LockedSlateAttribute.Get())
	{
		return &Style->DisabledBarImage;
	}
	else if (IsHovered())
	{
		return &Style->HoveredBarImage;
	}
	else
	{
		return &Style->NormalBarImage;
	}
}

const FSlateBrush* SSlider::GetThumbImage() const
{
	if (!IsEnabled() || LockedSlateAttribute.Get())
	{
		return &Style->DisabledThumbImage;
	}
	else if (IsHovered())
	{
		return &Style->HoveredThumbImage;
	}
	else
	{
		return &Style->NormalThumbImage;
	}
}

float SSlider::GetValue() const
{
	return ValueSlateAttribute.Get();
}

float SSlider::GetNormalizedValue() const
{
	if (MaxValue == MinValue)
	{
		return 1.0f;
	}
	else
	{
		return (ValueSlateAttribute.Get() - MinValue) / (MaxValue - MinValue);
	}
}

void SSlider::SetValue(TAttribute<float> InValueAttribute)
{
	ValueSlateAttribute.Assign(*this, MoveTemp(InValueAttribute));
}

void SSlider::SetMinAndMaxValues(float InMinValue, float InMaxValue)
{
	if (MinValue != InMinValue || MaxValue != InMaxValue)
	{
		MinValue = InMinValue;
		MaxValue = InMaxValue;
		if (MinValue > MaxValue)
		{
			MaxValue = MinValue;
		}
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SSlider::SetIndentHandle(TAttribute<bool> InIndentHandle)
{
	IndentHandleSlateAttribute.Assign(*this, MoveTemp(InIndentHandle));
}

void SSlider::SetLocked(TAttribute<bool> InLocked)
{
	LockedSlateAttribute.Assign(*this, MoveTemp(InLocked));
}

void SSlider::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SSlider::SetSliderBarColor(TAttribute<FSlateColor> InSliderBarColor)
{
	SliderBarColorSlateAttribute.Assign(*this, MoveTemp(InSliderBarColor));
}

void SSlider::SetSliderHandleColor(TAttribute<FSlateColor> InSliderHandleColor)
{
	SliderHandleColorSlateAttribute.Assign(*this, MoveTemp(InSliderHandleColor));
}

float SSlider::GetStepSize() const
{
	return StepSize.Get();
}

void SSlider::SetStepSize(TAttribute<float> InStepSize)
{
	StepSize = MoveTemp(InStepSize);
}

void SSlider::SetMouseUsesStep(bool MouseUsesStep)
{
	bMouseUsesStep = MouseUsesStep;
}

void SSlider::SetRequiresControllerLock(bool RequiresControllerLock)
{
	bRequiresControllerLock = RequiresControllerLock;
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SSlider::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleSlider(SharedThis(this)));
}
#endif
