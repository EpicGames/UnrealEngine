// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/Input/SSpinBox.h"

namespace SpinBoxPrivate
{
	bool bUseSpinBoxMouseMoveOptimization = true;
	FAutoConsoleVariableRef CVar
		(
			TEXT("Slate.Spinbox.MouseMoveOptimization"),
			bUseSpinBoxMouseMoveOptimization,
			TEXT("")
		);
}

float SpinBoxComputeExponentSliderFraction(float FractionFilled, float StartFractionFilled, float SliderExponent)
{
	if (FractionFilled <= StartFractionFilled)
	{
		float DeltaFraction = (StartFractionFilled - FractionFilled)/StartFractionFilled;
		float LeftFractionFilled = 1.0f - FMath::Pow(1.0f - DeltaFraction, SliderExponent);
		FractionFilled = StartFractionFilled - (StartFractionFilled*LeftFractionFilled);
	}
	else
	{
		float DeltaFraction = (FractionFilled - StartFractionFilled)/(1.0f - StartFractionFilled);
		float RightFractionFilled = 1.0f - FMath::Pow(1.0f - DeltaFraction, SliderExponent);
		FractionFilled = StartFractionFilled + (1.0f - StartFractionFilled) * RightFractionFilled;
	}
	return FractionFilled;
}


template<typename NumericType>
inline void SSpinBox<NumericType>::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bDragging, EInvalidateWidgetReason::Paint);
	AttributeInitializer.OverrideInvalidationReason("EnabledState", FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Paint});
	AttributeInitializer.OverrideInvalidationReason("Hovered", FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Paint});

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ValueAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Delta, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinValue, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxValue, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinSliderValue, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxSliderValue, EInvalidateWidgetReason::Paint);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, SliderExponent, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, SliderExponentNeutralValue, EInvalidateWidgetReason::Paint);
}

template <typename NumericType>
SSpinBox<NumericType>::SSpinBox()
	: ValueAttribute(*this, 0)
	, Delta(*this, 0)
	, SliderExponent(*this, 1.f)
	, SliderExponentNeutralValue(*this)
	, MinValue(*this, TOptional<NumericType>{})
	, MaxValue(*this, TOptional<NumericType>{})
	, MinSliderValue(*this, TOptional<NumericType>{})
	, MaxSliderValue(*this, TOptional<NumericType>{})
	, bDragging(*this, false)
{
}

template <typename NumericType>
SSpinBox<NumericType>::~SSpinBox()
{
	if (bDragging.Get() || PointerDraggingSliderIndex != INDEX_NONE)
	{
		CancelMouseCapture();
	}
}

template <typename NumericType>
void SSpinBox<NumericType>::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);

	Style = InArgs._Style;

	SetForegroundColor(InArgs._Style->ForegroundColor);
	InterfaceAttr = InArgs._TypeInterface;

	if (!InterfaceAttr.IsBound() && InterfaceAttr.Get() == nullptr)
	{
		InterfaceAttr = MakeShared<TDefaultNumericTypeInterface<NumericType>>();
	}

	TSharedPtr<INumericTypeInterface<NumericType>> Interface = InterfaceAttr.Get();
	if (Interface->GetOnSettingChanged())
	{
		Interface->GetOnSettingChanged()->AddSP(this, &SSpinBox::ResetCachedValueString);
	}

	ValueAttribute.Assign(*this, InArgs._Value, ESlateAttributeBindAction::Initialize);
	OnValueChanged = InArgs._OnValueChanged;
	OnValueCommitted = InArgs._OnValueCommitted;
	OnBeginSliderMovement = InArgs._OnBeginSliderMovement;
	OnEndSliderMovement = InArgs._OnEndSliderMovement;
	MinDesiredWidth = InArgs._MinDesiredWidth;

	MinValue.Assign(*this, InArgs._MinValue, ESlateAttributeBindAction::Initialize);
	MaxValue.Assign(*this, InArgs._MaxValue, ESlateAttributeBindAction::Initialize);
	MinSliderValue.Assign(*this, (InArgs._MinSliderValue.Get().IsSet()) ? InArgs._MinSliderValue : MinValue.Get(), ESlateAttributeBindAction::Initialize);
	MaxSliderValue.Assign(*this, (InArgs._MaxSliderValue.Get().IsSet()) ? InArgs._MaxSliderValue : MaxValue.Get(), ESlateAttributeBindAction::Initialize);

	MinFractionalDigits = (InArgs._MinFractionalDigits.Get().IsSet()) ? InArgs._MinFractionalDigits : DefaultMinFractionalDigits;
	MaxFractionalDigits = (InArgs._MaxFractionalDigits.Get().IsSet()) ? InArgs._MaxFractionalDigits : DefaultMaxFractionalDigits;
	SetMaxFractionalDigits(MaxFractionalDigits);
	SetMinFractionalDigits(MinFractionalDigits);

	AlwaysUsesDeltaSnap = InArgs._AlwaysUsesDeltaSnap;
	EnableSlider = InArgs._EnableSlider;

	SupportDynamicSliderMaxValue = InArgs._SupportDynamicSliderMaxValue;
	SupportDynamicSliderMinValue = InArgs._SupportDynamicSliderMinValue;
	OnDynamicSliderMaxValueChanged = InArgs._OnDynamicSliderMaxValueChanged;
	OnDynamicSliderMinValueChanged = InArgs._OnDynamicSliderMinValueChanged;
	
	OnGetDisplayValue = InArgs._OnGetDisplayValue;
	OnEditEditableText = InArgs._OnEditEditableText;

	bEnableWheel = InArgs._EnableWheel;
	bBroadcastValueChangesPerKey = InArgs._BroadcastValueChangesPerKey;
	WheelStep = InArgs._WheelStep;

	bPreventThrottling = InArgs._PreventThrottling;

	CachedExternalValue = ValueAttribute.Get();
	CachedValueString = Interface->ToString(CachedExternalValue);
	bCachedValueStringDirty = false;

	InternalValue = (double)CachedExternalValue;

	if (SupportDynamicSliderMaxValue.Get() && CachedExternalValue > GetMaxSliderValue())
	{
		ApplySliderMaxValueChanged(float(CachedExternalValue - GetMaxSliderValue()), true);
	}
	else if (SupportDynamicSliderMinValue.Get() && CachedExternalValue < GetMinSliderValue())
	{
		ApplySliderMinValueChanged(float(CachedExternalValue - GetMinSliderValue()), true);
	}

	UpdateIsSpinRangeUnlimited();

	SliderExponent.Assign(*this, InArgs._SliderExponent);

	bIsSliderExponentNeutralValueSet = InArgs._SliderExponentNeutralValue.IsSet();
	SliderExponentNeutralValue.Assign(*this, InArgs._SliderExponentNeutralValue);

	DistanceDragged = 0.0f;
	PreDragValue = NumericType();

	Delta.Assign(*this, InArgs._Delta);
	ShiftMultiplier = InArgs._ShiftMultiplier;
	CtrlMultiplier = InArgs._CtrlMultiplier;
	ShiftAltMultiplier = InArgs._ShiftAltMultiplier;
	CtrlAltMultiplier = InArgs._CtrlAltMultiplier;
	LinearDeltaSensitivity = InArgs._LinearDeltaSensitivity;

	BackgroundHoveredBrush = &InArgs._Style->HoveredBackgroundBrush;
	BackgroundBrush = &InArgs._Style->BackgroundBrush;
	BackgroundActiveBrush = InArgs._Style->ActiveBackgroundBrush.IsSet() ? &InArgs._Style->ActiveBackgroundBrush : BackgroundHoveredBrush;

	ActiveFillBrush = &InArgs._Style->ActiveFillBrush;
	HoveredFillBrush = InArgs._Style->HoveredFillBrush.IsSet() ? &InArgs._Style->HoveredFillBrush : ActiveFillBrush;
	InactiveFillBrush = &InArgs._Style->InactiveFillBrush;

	const FMargin& TextMargin = InArgs._Style->TextPadding;

	PointerDraggingSliderIndex = INDEX_NONE;

	bIsTextChanging = false;

	this->ChildSlot
	    .Padding(InArgs._ContentPadding)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		  .FillWidth(1.0f)
		  .Padding(TextMargin)
		  .HAlign(HAlign_Fill)
		  .VAlign(VAlign_Center)
		[
			SAssignNew(TextBlock, STextBlock)
				.Font(InArgs._Font)
				.Text(this, &SSpinBox<NumericType>::GetDisplayValue)
				.MinDesiredWidth(this, &SSpinBox<NumericType>::GetTextMinDesiredWidth)
				.Justification(InArgs._Justification)
		]
		+ SHorizontalBox::Slot()
		  .FillWidth(1.0f)
		  .Padding(TextMargin)
		  .HAlign(HAlign_Fill)
		  .VAlign(VAlign_Center)
		[
			SAssignNew(EditableText, SEditableText)
				.Visibility(EVisibility::Collapsed)
				.Font(InArgs._Font)
				.SelectAllTextWhenFocused(true)
				.Text(this, &SSpinBox<NumericType>::GetValueAsText)
				.RevertTextOnEscape(InArgs._RevertTextOnEscape)
				.OnIsTypedCharValid(this, &SSpinBox<NumericType>::IsCharacterValid)
				.OnTextChanged(this, &SSpinBox<NumericType>::TextField_OnTextChanged)
				.OnTextCommitted(this, &SSpinBox<NumericType>::TextField_OnTextCommitted)
				.ClearKeyboardFocusOnCommit(InArgs._ClearKeyboardFocusOnCommit)
				.SelectAllTextOnCommit(InArgs._SelectAllTextOnCommit)
				.MinDesiredWidth(this, &SSpinBox<NumericType>::GetTextMinDesiredWidth)
				.VirtualKeyboardType(InArgs._KeyboardType)
				.VirtualKeyboardDismissAction(InArgs._VirtualKeyboardDismissAction)
				.Justification(InArgs._Justification)
				.VirtualKeyboardTrigger(EVirtualKeyboardTrigger::OnAllFocusEvents)
				.ContextMenuExtender(InArgs._ContextMenuExtender)
		]
		+ SHorizontalBox::Slot()
		  .AutoWidth()
		  .HAlign(HAlign_Fill)
		  .VAlign(VAlign_Center)
		[
			SNew(SImage)
				.Image(&InArgs._Style->ArrowsImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}

template <typename NumericType>
int32 SSpinBox<NumericType>::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{

	const bool bActiveFeedback = bDragging.Get() || IsInTextMode();

	const FSlateBrush* BackgroundImage = bActiveFeedback ? BackgroundActiveBrush : IsHovered() ?
		                                     BackgroundHoveredBrush :
		                                     BackgroundBrush;

	const FSlateBrush* FillImage = bActiveFeedback ?
		                               ActiveFillBrush : IsHovered() ? HoveredFillBrush : 
		                               InactiveFillBrush;

	const int32 BackgroundLayer = LayerId;

	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		BackgroundLayer,
		AllottedGeometry.ToPaintGeometry(),
		BackgroundImage,
		DrawEffects,
		BackgroundImage->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	const int32 FilledLayer = BackgroundLayer + 1;

	//if there is a spin range limit, draw the filler bar
	if (!bUnlimitedSpinRange)
	{
		NumericType Value = ValueAttribute.Get();
		NumericType CurrentDelta = Delta.Get();
		if (CurrentDelta != NumericType())
		{
			Value = FMath::GridSnap(Value, CurrentDelta); // snap value to nearest Delta
		}

		float FractionFilled = Fraction((double)Value, (double)GetMinSliderValue(), (double)GetMaxSliderValue());
		const float CachedSliderExponent = SliderExponent.Get();
		if (!FMath::IsNearlyEqual(CachedSliderExponent, 1.f))
		{
			if (bIsSliderExponentNeutralValueSet && SliderExponentNeutralValue.Get() > GetMinSliderValue() && SliderExponentNeutralValue.Get() < GetMaxSliderValue())
			{
				//Compute a log curve on both side of the neutral value
				float StartFractionFilled = Fraction((double)SliderExponentNeutralValue.Get(), (double)GetMinSliderValue(), (double)GetMaxSliderValue());
				FractionFilled = SpinBoxComputeExponentSliderFraction(FractionFilled, StartFractionFilled, CachedSliderExponent);
			}
			else
			{
				FractionFilled = 1.0f - FMath::Pow(1.0f - FractionFilled, CachedSliderExponent);
			}
		}
		const FVector2D FillSize(AllottedGeometry.GetLocalSize().X * FractionFilled, AllottedGeometry.GetLocalSize().Y);

		if (!IsInTextMode())
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				FilledLayer,
				AllottedGeometry.ToPaintGeometry(FillSize - FVector2D(Style->InsetPadding.GetTotalSpaceAlong<Orient_Horizontal>(), Style->InsetPadding.GetTotalSpaceAlong<Orient_Vertical>()), FSlateLayoutTransform(Style->InsetPadding.GetTopLeft())),
				FillImage,
				DrawEffects,
				FillImage->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}
	}

	return FMath::Max(FilledLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, FilledLayer, InWidgetStyle, bEnabled));
}

template <typename NumericType>
void SSpinBox<NumericType>::Tick(const FGeometry& AlottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (PendingCommitValue)
	{
		const NumericType RoundedNewValue = RoundIfIntegerValue(PendingCommitValue->NewValue);
		CommitValue(RoundedNewValue, PendingCommitValue->NewValue, PendingCommitValue->CommitMethod, ETextCommit::OnEnter);
		PendingCommitValue.Reset();
	}
}

template <typename NumericType>
const bool SSpinBox<NumericType>::CommitWithMultiplier(const FPointerEvent& MouseEvent)
{
	return MouseEvent.IsShiftDown() || MouseEvent.IsControlDown();
}

template <typename NumericType>
FReply SSpinBox<NumericType>::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && PointerDraggingSliderIndex == INDEX_NONE)
	{
		DistanceDragged = 0.f;
		PreDragValue = ValueAttribute.Get();
		InternalValue = (double)PreDragValue;
		PointerDraggingSliderIndex = MouseEvent.GetPointerIndex();
		CachedMousePosition = MouseEvent.GetScreenSpacePosition().IntPoint();

		FReply ReturnReply = FReply::Handled().CaptureMouse(SharedThis(this)).UseHighPrecisionMouseMovement(SharedThis(this)).SetUserFocus(SharedThis(this), EFocusCause::Mouse);
		if (bPreventThrottling)
		{
			ReturnReply.PreventThrottling();
		}
		return ReturnReply;
	}
	else
	{

		return FReply::Unhandled();
	}
}

template <typename NumericType>
FReply SSpinBox<NumericType>::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && PointerDraggingSliderIndex == MouseEvent.GetPointerIndex())
	{
		if (!this->HasMouseCapture())
		{
			// Lost Capture - ensure reset
			bDragging.Assign(*this, false);
			PointerDraggingSliderIndex = INDEX_NONE;

			return FReply::Unhandled();
		}

		if (bDragging.Get())
		{
			NumericType CurrentDelta = Delta.Get();
			if (CurrentDelta != NumericType() && !CommitWithMultiplier(MouseEvent))
			{
				InternalValue = FMath::GridSnap(InternalValue, (double)CurrentDelta);
			}

			const NumericType CurrentValue = RoundIfIntegerValue(InternalValue);
			NotifyValueCommitted(CurrentValue);
		}

		bDragging.Assign(*this, false);
		PointerDraggingSliderIndex = INDEX_NONE;

		FReply Reply = FReply::Handled().ReleaseMouseCapture();

		if (!MouseEvent.IsTouchEvent())
		{
			Reply.SetMousePos(CachedMousePosition);
		}

		if (DistanceDragged < FSlateApplication::Get().GetDragTriggerDistance())
		{
			EnterTextMode();
			Reply.SetUserFocus(EditableText.ToSharedRef(), EFocusCause::SetDirectly);
		}

		return Reply;
	}

	return FReply::Unhandled();
}

template <typename NumericType>
void SSpinBox<NumericType>::ApplySliderMaxValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfHigher)
{
	check(SupportDynamicSliderMaxValue.Get());

	NumericType NewMaxSliderValue = std::numeric_limits<NumericType>::min();

	if (MaxSliderValue.Get().IsSet())
	{
		NewMaxSliderValue = GetMaxSliderValue();

		if ((NewMaxSliderValue + (NumericType)SliderDeltaToAdd > GetMaxSliderValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
		{
			NewMaxSliderValue += (NumericType)SliderDeltaToAdd;

			if (!MaxSliderValue.IsBound(*this)) // simple value so we can update it without breaking the mechanic otherwise it must be handle by the callback implementer
			{
				SetMaxSliderValue(NewMaxSliderValue);
			}
		}
	}

	if (OnDynamicSliderMaxValueChanged.IsBound())
	{
		OnDynamicSliderMaxValueChanged.Execute(NewMaxSliderValue, TWeakPtr<SWidget>(AsShared()), true, UpdateOnlyIfHigher);
	}
}

template <typename NumericType>
void SSpinBox<NumericType>::ApplySliderMinValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfLower)
{
	check(SupportDynamicSliderMinValue.Get());

	NumericType NewMinSliderValue = std::numeric_limits<NumericType>::min();

	if (MinSliderValue.Get().IsSet())
	{
		NewMinSliderValue = GetMinSliderValue();

		if ((NewMinSliderValue + (NumericType)SliderDeltaToAdd < GetMinSliderValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
		{
			NewMinSliderValue += (NumericType)SliderDeltaToAdd;

			if (!MinSliderValue.IsBound(*this)) // simple value so we can update it without breaking the mechanic otherwise it must be handle by the callback implementer
			{
				SetMinSliderValue(NewMinSliderValue);
			}
		}
	}

	if (OnDynamicSliderMinValueChanged.IsBound())
	{
		OnDynamicSliderMinValueChanged.Execute(NewMinSliderValue, TWeakPtr<SWidget>(AsShared()), true, UpdateOnlyIfLower);
	}
}

template <typename NumericType>
FReply SSpinBox<NumericType>::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bEnableSlider = GetEnableSlider();
	if (PointerDraggingSliderIndex == MouseEvent.GetPointerIndex() && bEnableSlider)
	{
		if (!this->HasMouseCapture())
		{
			// Lost the mouse capture - ensure reset
			bDragging.Assign(*this, false);
			PointerDraggingSliderIndex = INDEX_NONE;

			return FReply::Unhandled();
		}

		if (!bDragging.Get())
		{
			DistanceDragged += (float)FMath::Abs(MouseEvent.GetCursorDelta().X);
			if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
			{
				ExitTextMode();
				bDragging.Assign(*this, true);
				OnBeginSliderMovement.ExecuteIfBound();
			}

			// Cache the mouse, even if not dragging cache it.
			CachedMousePosition = MouseEvent.GetScreenSpacePosition().IntPoint();
		}
		else
		{
			double NewValue = 0.0;

			// Increments the spin based on delta mouse movement.

			// A minimum slider width to use for calculating deltas in the slider-range space
			const float MinSliderWidth = 100.f;
			float SliderWidthInSlateUnits = FMath::Max((float)MyGeometry.GetDrawSize().X, MinSliderWidth);

			if (MouseEvent.IsAltDown())
			{
				float DeltaToAdd = (float)MouseEvent.GetCursorDelta().X / SliderWidthInSlateUnits;

				if (SupportDynamicSliderMaxValue.Get() && (NumericType)InternalValue == GetMaxSliderValue())
				{
					ApplySliderMaxValueChanged(DeltaToAdd, false);
				}
				else if (SupportDynamicSliderMinValue.Get() && (NumericType)InternalValue == GetMinSliderValue())
				{
					ApplySliderMinValueChanged(DeltaToAdd, false);
				}
			}

			ECommitMethod CommitMethod = MouseEvent.IsControlDown() || MouseEvent.IsShiftDown() ? CommittedViaSpinMultiplier : CommittedViaSpin;
			double Step = GetDefaultStepSize(MouseEvent);

			//if we have a range to draw in
			if (!bUnlimitedSpinRange)
			{
				bool HasValidExponentNeutralValue = bIsSliderExponentNeutralValueSet && SliderExponentNeutralValue.Get() > GetMinSliderValue() && SliderExponentNeutralValue.Get() < GetMaxSliderValue();

				const float CachedSliderExponent = SliderExponent.Get();
				// The amount currently filled in the spinbox, needs to be calculated to do deltas correctly.
				float FractionFilled = Fraction(InternalValue, (double)GetMinSliderValue(), (double)GetMaxSliderValue());

				if (!FMath::IsNearlyEqual(CachedSliderExponent, 1.0f))
				{
					if (HasValidExponentNeutralValue)
					{
						//Compute a log curve on both side of the neutral value
						float StartFractionFilled = Fraction((double)SliderExponentNeutralValue.Get(), (double)GetMinSliderValue(), (double)GetMaxSliderValue());
						FractionFilled = SpinBoxComputeExponentSliderFraction(FractionFilled, StartFractionFilled, CachedSliderExponent);
					}
					else
					{
						FractionFilled = 1.0f - FMath::Pow(1.0f - FractionFilled, CachedSliderExponent);
					}
				}
				FractionFilled *= SliderWidthInSlateUnits;

				// Now add the delta to the fraction filled, this causes the spin.
				FractionFilled += (float)(MouseEvent.GetCursorDelta().X * Step);

				// Clamp the fraction to be within the bounds of the geometry.
				FractionFilled = FMath::Clamp(FractionFilled, 0.0f, SliderWidthInSlateUnits);

				// Convert the fraction filled to a percent.
				float Percent = FMath::Clamp(FractionFilled / SliderWidthInSlateUnits, 0.0f, 1.0f);
				if (!FMath::IsNearlyEqual(CachedSliderExponent, 1.0f))
				{
					// Have to convert the percent to the proper value due to the exponent component to the spin.
					if (HasValidExponentNeutralValue)
					{
						//Compute a log curve on both side of the neutral value
						float StartFractionFilled = Fraction(SliderExponentNeutralValue.Get(), GetMinSliderValue(), GetMaxSliderValue());
						Percent = SpinBoxComputeExponentSliderFraction(Percent, StartFractionFilled, 1.0f / CachedSliderExponent);
					}
					else
					{
						Percent = 1.0f - FMath::Pow(1.0f - Percent, 1.0f / CachedSliderExponent);
					}


				}

				NewValue = FMath::LerpStable<double>((double)GetMinSliderValue(), (double)GetMaxSliderValue(), Percent);
			}
			else
			{
				// If this control has a specified delta and sensitivity then we use that instead of the current value for determining how much to change.
				const double Sign = (MouseEvent.GetCursorDelta().X > 0) ? 1.0 : -1.0;

				if (LinearDeltaSensitivity.IsSet() && LinearDeltaSensitivity.Get() != 0 && Delta.Get() > 0)
				{
					const double MouseDelta = FMath::Abs(MouseEvent.GetCursorDelta().X / (float)LinearDeltaSensitivity.Get());
					NewValue = InternalValue + (Sign * MouseDelta * FMath::Pow((double)Delta.Get(), (double)SliderExponent.Get())) * Step;
				}
				else
				{
					const double MouseDelta = FMath::Abs(MouseEvent.GetCursorDelta().X / SliderWidthInSlateUnits);
					const double CurrentValue = FMath::Clamp<double>(FMath::Abs(InternalValue), 1.0, (double)std::numeric_limits<NumericType>::max());
					NewValue = InternalValue + (Sign * MouseDelta * FMath::Pow((double)CurrentValue, (double)SliderExponent.Get())) * Step;
				}
			}

			if (SpinBoxPrivate::bUseSpinBoxMouseMoveOptimization)
			{
				if (CommitMethod == ECommitMethod::CommittedViaSpin)
				{
					NewValue = FMath::Clamp<double>(NewValue, (double)GetMinSliderValue(), (double)GetMaxSliderValue());
				}
				NewValue = FMath::Clamp<double>(NewValue, (double)GetMinValue(), (double)GetMaxValue());
				InternalValue = NewValue;

				PendingCommitValue.Emplace(FPendingCommitValue
					{
						.NewValue = NewValue,
						.CommitMethod = CommitMethod
					}); 
			}
			else
			{
				NumericType RoundedNewValue = RoundIfIntegerValue(NewValue);
				CommitValue(RoundedNewValue, NewValue, CommitMethod, ETextCommit::OnEnter);
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

template <typename NumericType>
FReply SSpinBox<NumericType>::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bEnableWheel && PointerDraggingSliderIndex == INDEX_NONE && HasKeyboardFocus())
	{
		// If there is no WheelStep defined, we use StepSize (Or SmallStepSize if slider range is <= SmallStepSizeMax)
		constexpr bool bIsIntegral = TIsIntegral<NumericType>::Value;
		const bool bIsSmallStep = !bIsIntegral && (GetMaxSliderValue() - GetMinSliderValue()) <= SmallStepSizeMax;
		double Step = WheelStep.IsSet() && WheelStep.Get().IsSet() ? WheelStep.Get().GetValue() : (bIsSmallStep ? SmallStepSize : StepSize);

		// Apply multiplier based on mouse event
		Step *= GetInputEventMultiplier(MouseEvent);

		const double Sign = (MouseEvent.GetWheelDelta() > 0) ? 1.0 : -1.0;
		const double NewValue = InternalValue + (Sign * Step);
		const NumericType RoundedNewValue = RoundIfIntegerValue(NewValue);

		TSharedPtr<INumericTypeInterface<NumericType>> Interface = InterfaceAttr.Get();

		// First SetEditableText is to update the value before calling CommitValue. Otherwise, when the text lose
		// focus from the CommitValue, it will override the value we just committed.
		// The second SetEditableText is to update the text to the InternalValue since it could have been clamped.
		EditableText->SetEditableText(FText::FromString(Interface->ToString((NumericType)NewValue)));
		CommitValue(RoundedNewValue, NewValue, CommittedViaSpin, ETextCommit::OnEnter);
		EditableText->SetEditableText(FText::FromString(Interface->ToString((NumericType)InternalValue)));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

template <typename NumericType>
FCursorReply SSpinBox<NumericType>::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	const bool bEnableSlider = GetEnableSlider();

	if (!bEnableSlider)
	{
		return FCursorReply::Cursor(EMouseCursor::Default);
	}

	return bDragging.Get() ?
		       FCursorReply::Cursor(EMouseCursor::None) :
		       FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
}

template <typename NumericType>
bool SSpinBox<NumericType>::SupportsKeyboardFocus() const
{
	// SSpinBox is focusable.
	return true;
}

template <typename NumericType>
FReply SSpinBox<NumericType>::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if (!bDragging.Get() && (InFocusEvent.GetCause() == EFocusCause::Navigation || InFocusEvent.GetCause() == EFocusCause::SetDirectly))
	{
		EnterTextMode();
		return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), InFocusEvent.GetCause());
	}
	else
	{
		return FReply::Unhandled();
	}
}

template <typename NumericType>
FReply SSpinBox<NumericType>::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();
	if (Key == EKeys::Escape && HasMouseCapture())
	{
		CancelMouseCapture();
		return FReply::Handled().ReleaseMouseCapture().SetMousePos(CachedMousePosition);
	}
	else if (Key == EKeys::Up || Key == EKeys::Right)
	{
		const NumericType LocalValueAttribute = ValueAttribute.Get();
		const NumericType LocalDelta = Delta.Get() != 0 ? Delta.Get() : (NumericType)GetDefaultStepSize(InKeyEvent);
		InternalValue = (double)LocalValueAttribute;
		CommitValue(LocalValueAttribute + LocalDelta, InternalValue + (double)LocalDelta, CommittedViaArrowKey, ETextCommit::OnEnter);
		ExitTextMode();
		return FReply::Handled();
	}
	else if (Key == EKeys::Down || Key == EKeys::Left)
	{
		const NumericType LocalValueAttribute = ValueAttribute.Get();
		const NumericType LocalDelta = Delta.Get() != 0 ? Delta.Get() : (NumericType)GetDefaultStepSize(InKeyEvent);
		InternalValue = (double)LocalValueAttribute;
		CommitValue(LocalValueAttribute - LocalDelta, InternalValue + (double)LocalDelta, CommittedViaArrowKey, ETextCommit::OnEnter);
		ExitTextMode();
		return FReply::Handled();
	}
	else if (Key == EKeys::Enter)
	{
		InternalValue = (double)ValueAttribute.Get();
		EnterTextMode();
		return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), EFocusCause::Navigation);
	}
	else
	{
		return FReply::Unhandled();
	}
}

template <typename NumericType>
bool SSpinBox<NumericType>::HasKeyboardFocus() const
{
	// The spinbox is considered focused when we are typing it text.
	return SCompoundWidget::HasKeyboardFocus() || (EditableText.IsValid() && EditableText->HasKeyboardFocus());
}

template <typename NumericType>
TAttribute<NumericType> SSpinBox<NumericType>::GetValueAttribute() const
{ return ValueAttribute.ToAttribute(*this); }

template <typename NumericType>
NumericType SSpinBox<NumericType>::GetValue() const
{ return ValueAttribute.Get(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetValue(TAttribute<NumericType> InValueAttribute, const bool bShouldCommit)
{
	ValueAttribute.Assign(*this, MoveTemp(InValueAttribute), ESlateAttributeBindAction::Initialize);
	const NumericType LocalValueAttribute = ValueAttribute.Get();
	CommitValue(LocalValueAttribute, (double)LocalValueAttribute, ECommitMethod::CommittedViaCode, ETextCommit::Default, bShouldCommit);
}

template <typename NumericType>
NumericType SSpinBox<NumericType>::GetMinValue() const
{ return MinValue.Get().Get(std::numeric_limits<NumericType>::lowest()); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMinValue(TAttribute<TOptional<NumericType>> InMinValue)
{
	MinValue.Assign(*this, MoveTemp(InMinValue), ESlateAttributeBindAction::Initialize);
	UpdateIsSpinRangeUnlimited();
}

template <typename NumericType>
NumericType SSpinBox<NumericType>::GetMaxValue() const
{ return MaxValue.Get().Get(std::numeric_limits<NumericType>::max()); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMaxValue(TAttribute<TOptional<NumericType>> InMaxValue)
{
	MaxValue.Assign(*this, MoveTemp(InMaxValue), ESlateAttributeBindAction::Initialize);
	UpdateIsSpinRangeUnlimited();
}

template <typename NumericType>
bool SSpinBox<NumericType>::IsMinSliderValueBound() const
{ return MinSliderValue.IsBound(*this); }

template <typename NumericType>
NumericType SSpinBox<NumericType>::GetMinSliderValue() const
{ return MinSliderValue.Get().Get(std::numeric_limits<NumericType>::lowest()); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMinSliderValue(TAttribute<TOptional<NumericType>> InMinSliderValue)
{
	MinSliderValue.Assign(*this, InMinSliderValue.Get().IsSet() ? MoveTemp(InMinSliderValue) : MinValue.Get(), ESlateAttributeBindAction::Initialize);
	UpdateIsSpinRangeUnlimited();
}

template <typename NumericType>
bool SSpinBox<NumericType>::IsMaxSliderValueBound() const
{ return MaxSliderValue.IsBound(*this); }

template <typename NumericType>
NumericType SSpinBox<NumericType>::GetMaxSliderValue() const
{ return MaxSliderValue.Get().Get(std::numeric_limits<NumericType>::max()); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMaxSliderValue(TAttribute<TOptional<NumericType>> InMaxSliderValue)
{
	MaxSliderValue.Assign(*this, InMaxSliderValue.Get().IsSet() ? MoveTemp(InMaxSliderValue) : MaxValue.Get(), ESlateAttributeBindAction::Initialize);
	UpdateIsSpinRangeUnlimited();
}

template <typename NumericType>
int32 SSpinBox<NumericType>::GetMinFractionalDigits() const
{ return InterfaceAttr.Get()->GetMinFractionalDigits(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMinFractionalDigits(const TAttribute<TOptional<int32>>& InMinFractionalDigits)
{
	InterfaceAttr.Get()->SetMinFractionalDigits((InMinFractionalDigits.Get().IsSet()) ? InMinFractionalDigits.Get() : MinFractionalDigits);
	bCachedValueStringDirty = true;
}

template <typename NumericType>
int32 SSpinBox<NumericType>::GetMaxFractionalDigits() const
{ return InterfaceAttr.Get()->GetMaxFractionalDigits(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& InMaxFractionalDigits)
{
	InterfaceAttr.Get()->SetMaxFractionalDigits((InMaxFractionalDigits.Get().IsSet()) ? InMaxFractionalDigits.Get() : MaxFractionalDigits);
	bCachedValueStringDirty = true;
}

template <typename NumericType>
bool SSpinBox<NumericType>::GetAlwaysUsesDeltaSnap() const
{ return AlwaysUsesDeltaSnap.Get(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetAlwaysUsesDeltaSnap(bool bNewValue)
{ AlwaysUsesDeltaSnap.Set(bNewValue); }

template <typename NumericType>
bool SSpinBox<NumericType>::GetEnableSlider() const
{ return EnableSlider.Get(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetEnableSlider(bool bNewValue)
{ EnableSlider.Set(bNewValue); }

template <typename NumericType>
NumericType SSpinBox<NumericType>::GetDelta() const
{ return Delta.Get(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetDelta(NumericType InDelta)
{ Delta.Assign(*this, InDelta); }

template <typename NumericType>
float SSpinBox<NumericType>::GetSliderExponent() const
{ return SliderExponent.Get(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetSliderExponent(TAttribute<float> InSliderExponent)
{ SliderExponent.Assign(*this, MoveTemp(InSliderExponent)); }

template <typename NumericType>
float SSpinBox<NumericType>::GetMinDesiredWidth() const
{ return MinDesiredWidth.Get(); }

template <typename NumericType>
void SSpinBox<NumericType>::SetMinDesiredWidth(TAttribute<float> InMinDesiredWidth)
{ MinDesiredWidth = MoveTemp(InMinDesiredWidth); }

template <typename NumericType>
const FSpinBoxStyle* SSpinBox<NumericType>::GetWidgetStyle() const
{ return Style; }

template <typename NumericType>
void SSpinBox<NumericType>::SetWidgetStyle(const FSpinBoxStyle* InStyle)
{ Style = InStyle; }

template <typename NumericType>
void SSpinBox<NumericType>::InvalidateStyle()
{ Invalidate(EInvalidateWidgetReason::Layout); }

template <typename NumericType>
void SSpinBox<NumericType>::SetTextBlockFont(FSlateFontInfo InFont)
{ EditableText->SetFont(InFont); TextBlock->SetFont(InFont); }

template <typename NumericType>
void SSpinBox<NumericType>::SetTextJustification(ETextJustify::Type InJustification)
{ EditableText->SetJustification(InJustification); TextBlock->SetJustification(InJustification);  }

template <typename NumericType>
void SSpinBox<NumericType>::SetTextClearKeyboardFocusOnCommit(bool bNewValue)
{ EditableText->SetClearKeyboardFocusOnCommit(bNewValue); }

template <typename NumericType>
void SSpinBox<NumericType>::SetTextSelectAllTextOnCommit(bool bNewValue)
{ EditableText->SetSelectAllTextOnCommit(bNewValue); }

template <typename NumericType>
void SSpinBox<NumericType>::SetTextRevertTextOnEscape(bool bNewValue)
{ EditableText->SetRevertTextOnEscape(bNewValue); }

template <typename NumericType>
void SSpinBox<NumericType>::EnterTextMode()
{
	TextBlock->SetVisibility(EVisibility::Collapsed);
	EditableText->SetVisibility(EVisibility::Visible);

	if (OnEditEditableText.IsBound())
	{
		if (const TOptional<FText> OptionalOverrideEditableText = OnEditEditableText.Execute(ValueAttribute.Get()))
		{
			const FText OverrideEditableText = OptionalOverrideEditableText.GetValue();
			
			EditableText->SetText(OverrideEditableText);
			EditableText->SetEditableText(OverrideEditableText);
		}
	}
	Invalidate(EInvalidateWidgetReason::Paint);
}

template <typename NumericType>
void SSpinBox<NumericType>::ExitTextMode()
{
	TextBlock->SetVisibility(EVisibility::Visible);
	EditableText->SetVisibility(EVisibility::Collapsed);
	Invalidate(EInvalidateWidgetReason::Paint);
}

template <typename NumericType>
FString SSpinBox<NumericType>::GetValueAsString() const
{
	NumericType CurrentValue = ValueAttribute.Get();
	if (!bCachedValueStringDirty && CurrentValue == CachedExternalValue)
	{
		return CachedValueString;
	}

	bCachedValueStringDirty = false;
	return InterfaceAttr.Get()->ToString(CurrentValue);
}

template <typename NumericType>
FText SSpinBox<NumericType>::GetValueAsText() const
{
	return FText::FromString(GetValueAsString());
}

template <typename NumericType>
FText SSpinBox<NumericType>::GetDisplayValue() const
{
	if (OnGetDisplayValue.IsBound())
	{
		const TOptional<FText> OverrideValue = OnGetDisplayValue.Execute(ValueAttribute.Get());
		if (OverrideValue.IsSet())
		{
			return OverrideValue.GetValue();
		}
	}
	return FText::FromString(GetValueAsString());
}

template <typename NumericType>
void SSpinBox<NumericType>::TextField_OnTextChanged(const FText& NewText)
{
	if (!bIsTextChanging)
	{
		TGuardValue<bool> TextChangedGuard(bIsTextChanging, true);

		// Validate the text on change, and only accept text up until the first invalid character
		FString Data = NewText.ToString();
		int32 NumValidChars = Data.Len();

		TSharedPtr<INumericTypeInterface<NumericType>> Interface = InterfaceAttr.Get();

		for (int32 Index = 0; Index < Data.Len(); ++Index)
		{
			if (!Interface->IsCharacterValid(Data[Index]))
			{
				NumValidChars = Index;
				break;
			}
		}

		if (NumValidChars < Data.Len())
		{
			FString ValidData = NumValidChars > 0 ? Data.Left(NumValidChars) : GetValueAsString();
			EditableText->SetEditableText(FText::FromString(ValidData));
		}

		// we check that the input is numeric, as we don't want to commit the new value on every change when an expression like *= is entered
		if (bBroadcastValueChangesPerKey && FCString::IsNumeric(*Data))
		{
			TOptional<NumericType> NewValue = Interface->FromString(Data, ValueAttribute.Get());
			if (NewValue.IsSet())
			{
				CommitValue(NewValue.GetValue(), static_cast<double>(NewValue.GetValue()), CommittedViaCode, ETextCommit::Default);
			}
		}
	}
}

template <typename NumericType>
void SSpinBox<NumericType>::TextField_OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo != ETextCommit::OnEnter)
	{
		ExitTextMode();
	}

	TSharedPtr<INumericTypeInterface<NumericType>> Interface = InterfaceAttr.Get();
	TOptional<NumericType> NewValue = Interface->FromString(NewText.ToString(), ValueAttribute.Get());
	if (NewValue.IsSet())
	{
		CommitValue(NewValue.GetValue(), (double)NewValue.GetValue(), CommittedViaTypeIn, CommitInfo);
	}
}

template <typename NumericType>
void SSpinBox<NumericType>::CommitValue(NumericType NewValue, double NewSpinValue, ECommitMethod CommitMethod, ETextCommit::Type OriginalCommitInfo, const bool bShouldCommit)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSpinBox_CommitValue);
	if (CommitMethod == CommittedViaSpin || CommitMethod == CommittedViaArrowKey)
	{
		const NumericType LocalMinSliderValue = GetMinSliderValue();
		const NumericType LocalMaxSliderValue = GetMaxSliderValue();
		NewValue = FMath::Clamp<NumericType>(NewValue, LocalMinSliderValue, LocalMaxSliderValue);
		NewSpinValue = FMath::Clamp<double>(NewSpinValue, (double)LocalMinSliderValue, (double)LocalMaxSliderValue);
	}

	{
		const NumericType LocalMinValue = GetMinValue();
		const NumericType LocalMaxValue = GetMaxValue();
		NewValue = FMath::Clamp<NumericType>(NewValue, LocalMinValue, LocalMaxValue);
		NewSpinValue = FMath::Clamp<double>(NewSpinValue, (double)LocalMinValue, (double)LocalMaxValue);
	}

	if (!ValueAttribute.IsBound(*this))
	{
		ValueAttribute.Assign(*this, NewValue);
	}

	ValueAttribute.UpdateNow(*this);

	// If not in spin mode, there is no need to jump to the value from the external source, continue to use the committed value.
	if (CommitMethod == CommittedViaSpin)
	{
		const NumericType CurrentValue = ValueAttribute.Get();
		// This will detect if an external force has changed the value. Internally it will abandon the delta calculated this tick and update the internal value instead.
		if (CurrentValue != CachedExternalValue)
		{
			NewValue = CurrentValue;
			NewSpinValue = (double)CurrentValue;
		}
	}

	// Update the internal value, this needs to be done before rounding.
	InternalValue = NewSpinValue;

	const bool bAlwaysUsesDeltaSnap = GetAlwaysUsesDeltaSnap();
	// If needed, round this value to the delta. Internally the value is not held to the Delta but externally it appears to be.
	if (CommitMethod == CommittedViaSpin || CommitMethod == CommittedViaArrowKey || bAlwaysUsesDeltaSnap)
	{
		NumericType CurrentDelta = Delta.Get();
		if (CurrentDelta != NumericType())
		{
			NewValue = FMath::GridSnap<NumericType>(NewValue, CurrentDelta); // snap numeric point value to nearest Delta
		}
	}

	// Update the max slider value based on the current value if we're in dynamic mode
	if (SupportDynamicSliderMaxValue.Get() && ValueAttribute.Get() > GetMaxSliderValue())
	{
		ApplySliderMaxValueChanged(float(ValueAttribute.Get() - GetMaxSliderValue()), true);
	}
	else if (SupportDynamicSliderMinValue.Get() && ValueAttribute.Get() < GetMinSliderValue())
	{
		ApplySliderMinValueChanged(float(ValueAttribute.Get() - GetMinSliderValue()), true);
	}

	if (CommitMethod == CommittedViaTypeIn || CommitMethod == CommittedViaArrowKey || bShouldCommit)
	{
		OnValueCommitted.ExecuteIfBound(NewValue, OriginalCommitInfo);
	}

	OnValueChanged.ExecuteIfBound(NewValue);

	if (!ValueAttribute.IsBound(*this))
	{
		ValueAttribute.Assign(*this, NewValue);
	}
	ValueAttribute.UpdateNow(*this);

	// Update the cache of the external value to what the user believes the value is now.
	const NumericType CurrentValue = ValueAttribute.Get();
	if (CachedExternalValue != CurrentValue || bCachedValueStringDirty)
	{
		TSharedPtr<INumericTypeInterface<NumericType>> Interface = InterfaceAttr.Get();

		CachedExternalValue = ValueAttribute.Get();
		CachedValueString = Interface->ToString(CachedExternalValue);
		bCachedValueStringDirty = false;
	}

	// This ensures that dragging is cleared if focus has been removed from this widget in one of the delegate calls, such as when spawning a modal dialog.
	if (!this->HasMouseCapture())
	{
		bDragging.Assign(*this, false);
		PointerDraggingSliderIndex = INDEX_NONE;
	}
}

template <typename NumericType>
void SSpinBox<NumericType>::NotifyValueCommitted(NumericType CurrentValue) const
{
	// The internal value will have been clamped and rounded to the delta at this point, but integer values may still need to be rounded
	// if the delta is 0.
	OnValueCommitted.ExecuteIfBound(CurrentValue, ETextCommit::OnEnter);
	OnEndSliderMovement.ExecuteIfBound(CurrentValue);
}

template <typename NumericType>
bool SSpinBox<NumericType>::IsInTextMode() const
{
	return (EditableText->GetVisibility() == EVisibility::Visible);
}

template <typename NumericType>
float SSpinBox<NumericType>::Fraction(double InValue, double InMinValue, double InMaxValue)
{
	const double HalfMax = InMaxValue * 0.5;
	const double HalfMin = InMinValue * 0.5;
	const double HalfVal = InValue * 0.5;

	return (float)FMath::Clamp((HalfVal - HalfMin) / (HalfMax - HalfMin), 0.0, 1.0);
}

template <typename NumericType>
float SSpinBox<NumericType>::GetInputEventMultiplier(const FInputEvent& InEvent) const
{
	float Multiplier = 1.f;

	if (InEvent.IsShiftDown())
	{
		if (InEvent.IsAltDown())
		{
			Multiplier = ShiftAltMultiplier.Get();
		}
		else
		{
			Multiplier = ShiftMultiplier.Get();
		}
	}
	else if (InEvent.IsControlDown())
	{
		if (InEvent.IsAltDown())
		{
			Multiplier = CtrlAltMultiplier.Get();
		}
		else
		{
			Multiplier = CtrlMultiplier.Get();
		}
	}

	return Multiplier;
}

template <typename NumericType>
void SSpinBox<NumericType>::UpdateIsSpinRangeUnlimited()
{
	bUnlimitedSpinRange = !((MinValue.Get().IsSet() && MaxValue.Get().IsSet()) || (MinSliderValue.Get().IsSet() && MaxSliderValue.Get().IsSet()));
}

template <typename NumericType>
float SSpinBox<NumericType>::GetTextMinDesiredWidth() const
{
	return FMath::Max(0.0f, MinDesiredWidth.Get() - (float)Style->ArrowsImage.ImageSize.X);
}

template <typename NumericType>
bool SSpinBox<NumericType>::IsCharacterValid(TCHAR InChar) const
{
	return InterfaceAttr.Get()->IsCharacterValid(InChar);
}

template <typename NumericType>
NumericType SSpinBox<NumericType>::RoundIfIntegerValue(double ValueToRound) const
{
	constexpr bool bIsIntegral = TIsIntegral<NumericType>::Value;
	constexpr bool bCanBeRepresentedInDouble = std::numeric_limits<double>::digits >= std::numeric_limits<NumericType>::digits;
	if (bIsIntegral && !bCanBeRepresentedInDouble)
	{
		return (NumericType)FMath::Clamp<double>(FMath::FloorToDouble(ValueToRound + 0.5), -1.0 * (double)(1ll << std::numeric_limits<double>::digits), (double)(1ll << std::numeric_limits<double>::digits));
	}
	else if (bIsIntegral)
	{
		return (NumericType)FMath::Clamp<double>(FMath::FloorToDouble(ValueToRound + 0.5), (double)std::numeric_limits<NumericType>::lowest(), (double)std::numeric_limits<NumericType>::max());
	}
	else
	{
		return (NumericType)FMath::Clamp<double>(ValueToRound, (double)std::numeric_limits<NumericType>::lowest(), (double)std::numeric_limits<NumericType>::max());
	}
}

template <typename NumericType>
void SSpinBox<NumericType>::CancelMouseCapture()
{
	bDragging.Assign(*this, false);
	PointerDraggingSliderIndex = INDEX_NONE;

	InternalValue = (double)PreDragValue;
	NotifyValueCommitted(PreDragValue);
}

template<typename NumericType>
double SSpinBox<NumericType>::GetDefaultStepSize(const FInputEvent& InputEvent)
{
	const bool bIsSmallStep = (GetMaxSliderValue() - GetMinSliderValue()) <= SmallStepSizeMax;
	double Step = bIsSmallStep ? SmallStepSize : StepSize;

	// Apply multiplier based on mouse event
	Step *= GetInputEventMultiplier(InputEvent);

	return Step;
}

template <typename NumericType>
void SSpinBox<NumericType>::ResetCachedValueString()
{ 
	const NumericType CurrentValue = ValueAttribute.Get();
	CachedExternalValue = CurrentValue;
	CachedValueString = InterfaceAttr.Get()->ToString(CachedExternalValue);
}

// Explicit instantiation of the numeric types valid for this template
template class SSpinBox<double>;
template class SSpinBox<float>;
template class SSpinBox<uint64>;
template class SSpinBox<uint32>;
template class SSpinBox<uint16>;
template class SSpinBox<uint8>;
template class SSpinBox<int64>;
template class SSpinBox<int32>;
template class SSpinBox<int16>;
template class SSpinBox<int8>;
// template struct SSpinBox<double>::FArguments;
// template struct SSpinBox<float>::FArguments;
// template struct SSpinBox<uint64>::FArguments;
// template struct SSpinBox<uint32>::FArguments;
// template struct SSpinBox<uint16>::FArguments;
// template struct SSpinBox<uint8>::FArguments;
// template struct SSpinBox<int64>::FArguments;
// template struct SSpinBox<int32>::FArguments;
// template struct SSpinBox<int16>::FArguments;
// template struct SSpinBox<int8>::FArguments;
