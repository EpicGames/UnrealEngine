// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SSubMenuHandler.h"
#include "Framework/MultiBox/MultiBox.h"

void SSubMenuHandler::Construct(const FArguments& InArgs, TWeakPtr<SMenuOwner> InMenuOwner)
{
	MenuOwnerWidget = InMenuOwner;

	TSharedPtr< SWidget > ChildSlotWidget;

	if (InArgs._MenuAnchor.IsValid())
	{
		MenuAnchor = InArgs._MenuAnchor;
		ChildSlotWidget = InArgs._Content.Widget;
	}
	else
	{
		// If no way for a sub-menu is provided, do not use a MenuAnchor
		if (InArgs._OnGetMenuContent.IsBound() || InArgs._MenuContent.IsValid())
		{
			ChildSlotWidget = SAssignNew(MenuAnchor, SMenuAnchor)
				.Placement(InArgs._Placement)
				.OnGetMenuContent(InArgs._OnGetMenuContent)
				.MenuContent(InArgs._MenuContent)
				[
					InArgs._Content.Widget
				];
		}
		else
		{
			ChildSlotWidget = InArgs._Content.Widget;
		}
		
	}

	this->ChildSlot
	[
		ChildSlotWidget.ToSharedRef()
	];
}

void SSubMenuHandler::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr< SMenuOwner > PinnedMenuOwnerWidget( MenuOwnerWidget.Pin() );
	check( PinnedMenuOwnerWidget.IsValid() );

	// Never dismiss another entry's submenu while the cursor is potentially moving toward that menu.  It's
	// not fun to try to keep the mouse in the menu entry bounds while moving towards the actual menu!
	TSharedPtr< const SMenuAnchor > OpenedMenuAnchor = PinnedMenuOwnerWidget->GetOpenMenu();
	const bool bSubMenuAlreadyOpen = ( OpenedMenuAnchor.IsValid() && OpenedMenuAnchor->IsOpen() );
	bool bMouseEnteredTowardSubMenu = false;
	{
		if( bSubMenuAlreadyOpen )
		{
			const FVector2D& SubMenuPosition = OpenedMenuAnchor->GetMenuPosition();
			const bool bIsMenuTowardRight = MouseEvent.GetScreenSpacePosition().X < SubMenuPosition.X;
			const bool bDidMouseEnterTowardRight = MouseEvent.GetCursorDelta().X >= 0.0f;	// NOTE: Intentionally inclusive of zero here.
			bMouseEnteredTowardSubMenu = ( bIsMenuTowardRight == bDidMouseEnterTowardRight );
		}
	}

	if (MenuAnchor.IsValid())
	{
		check( PinnedMenuOwnerWidget.IsValid() );

		// Do we have a different pull-down menu open?
		TSharedPtr< SMenuAnchor > PinnedMenuAnchor( MenuAnchor.Pin() );
		if( PinnedMenuOwnerWidget->GetOpenMenu() != PinnedMenuAnchor )
		{
			const bool bClobber = bSubMenuAlreadyOpen && bMouseEnteredTowardSubMenu;
			RequestSubMenuToggle( true, bClobber );
		}
	}
	else
	{
		// Hovering over a menu item that is not a sub-menu, we need to close any sub-menus that are open
		const bool bClobber = bSubMenuAlreadyOpen && bMouseEnteredTowardSubMenu;
		RequestSubMenuToggle( false, bClobber );
	}
}

void SSubMenuHandler::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave( MouseEvent );

	// Reset any pending sub-menus that may be opening when we stop hovering over it
	CancelPendingSubMenu();
}

bool SSubMenuHandler::ShouldSubMenuAppearHovered() const
{
	// The sub-menu entry should appear hovered if the sub-menu is open.  Except in the case that the user is actually interacting with this menu.  
	// In that case we need to show what the user is selecting
	return MenuAnchor.IsValid() && MenuAnchor.Pin()->IsOpen() && !MenuOwnerWidget.Pin()->IsHovered();
}

void SSubMenuHandler::RequestSubMenuToggle( bool bOpenMenu, const bool bClobber, const bool bImmediate )
{
	if (MenuAnchor.IsValid())
	{
		if (bImmediate)
		{
			UpdateSubMenuState(0, 0, true);
		}
		else
		{
			// Reset the time before the menu opens
			float TimeToSubMenuOpen = bClobber ? MultiBoxConstants::SubMenuClobberTime : MultiBoxConstants::SubMenuOpenTime;
			if (!ActiveTimerHandle.IsValid())
			{
				ActiveTimerHandle = RegisterActiveTimer(TimeToSubMenuOpen, FWidgetActiveTimerDelegate::CreateSP(this, &SSubMenuHandler::UpdateSubMenuState, bOpenMenu));
			}
		}
	}
}

void SSubMenuHandler::CancelPendingSubMenu()
{
	// Reset any pending sub-menu openings
	//SubMenuRequestState = Idle;

	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
}

bool SSubMenuHandler::IsSubMenuOpen() const
{
	return MenuAnchor.IsValid() && MenuAnchor.Pin()->IsOpen();
}

EActiveTimerReturnType SSubMenuHandler::UpdateSubMenuState(double InCurrentTime, float InDeltaTime, bool bWantsOpen)
{
	TSharedPtr< SMenuOwner > PinnedMenuOwnerWidget(MenuOwnerWidget.Pin());
	check(PinnedMenuOwnerWidget.IsValid());

	if (bWantsOpen)
	{
		// For menu bar entries, we also need to handle mouse enter/leave events, so we can show and hide
		// the pull-down menu appropriately
		check(MenuAnchor.IsValid());

		// Close other open pull-down menus from this menu bar
		// Do we have a different pull-down menu open?
		TSharedPtr< SMenuAnchor > PinnedMenuAnchor(MenuAnchor.Pin());
		if (PinnedMenuOwnerWidget->GetOpenMenu() != PinnedMenuAnchor)
		{
			PinnedMenuOwnerWidget->CloseSummonedMenus();

			// Summon the new pull-down menu!
			if (PinnedMenuAnchor.IsValid())
			{
				PinnedMenuAnchor->SetIsOpen(true);
			}

			// Also tell the MenuOwner about this open pull-down menu, so it can be closed later if we need to
			PinnedMenuOwnerWidget->SetSummonedMenu(PinnedMenuAnchor.ToSharedRef());
		}
	}
	else
	{
		PinnedMenuOwnerWidget->CloseSummonedMenus();
	}

	return EActiveTimerReturnType::Stop;
}
