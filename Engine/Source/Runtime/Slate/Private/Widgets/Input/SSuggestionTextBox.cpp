// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SSuggestionTextBox.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Views/SListView.h"


/* SSuggestionTextBox structors
 *****************************************************************************/


SSuggestionTextBox::SSuggestionTextBox( )
	: IgnoreUIUpdate(false)
	, SelectedSuggestion(-1)
{ }


/* SSuggestionTextBox interface
 *****************************************************************************/

void SSuggestionTextBox::Construct( const FArguments& InArgs )
{
	SuggestionTextStyle = InArgs._SuggestionTextStyle;

	OnShowingHistory = InArgs._OnShowingHistory;
	OnShowingSuggestions = InArgs._OnShowingSuggestions;
	OnTextChanged = InArgs._OnTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;

	CommitsOnSuggestionSelected = InArgs._CommitOnSuggestionSelected.Get();

	FSlateApplication::Get().OnFocusChanging().AddSP(this, &SSuggestionTextBox::OnGlobalFocusChanging);

	ChildSlot
	[
		SAssignNew(MenuAnchor, SMenuAnchor)
			.Placement(MenuPlacement_ComboBox)
			[
				SAssignNew(TextBox, SEditableTextBox)
					.BackgroundColor(InArgs._BackgroundColor)
					.ClearKeyboardFocusOnCommit(InArgs._ClearKeyboardFocusOnCommit)
					.ErrorReporting(InArgs._ErrorReporting)
					.Font(InArgs._Font)
					.ForegroundColor(InArgs._ForegroundColor)
					.HintText(InArgs._HintText)
					.IsCaretMovedWhenGainFocus(InArgs._IsCaretMovedWhenGainFocus)
					.MinDesiredWidth(InArgs._MinDesiredWidth)
					.RevertTextOnEscape(InArgs._RevertTextOnEscape)
					.SelectAllTextOnCommit(InArgs._SelectAllTextOnCommit)
					.SelectAllTextWhenFocused(InArgs._SelectAllTextWhenFocused)
					.Style(InArgs._TextStyle)
					.Text(InArgs._Text)
					.OnKeyDownHandler(this, &SSuggestionTextBox::OnKeyDown)
					.OnTextChanged(this, &SSuggestionTextBox::HandleTextBoxTextChanged)
					.OnTextCommitted(this, &SSuggestionTextBox::HandleTextBoxTextCommitted)
			]
			.MenuContent
			(
				SNew(SBorder)
					.BorderImage(InArgs._BackgroundImage)
					.Padding(FMargin(2))
					[
						SAssignNew(VerticalBox, SVerticalBox)
						
						+ SVerticalBox::Slot()
							.AutoHeight()
							.MaxHeight(InArgs._SuggestionListMaxHeight)
							[
								SAssignNew(SuggestionListView, SListView< TSharedPtr<FString> >)
								.ListItemsSource(&Suggestions)
								.SelectionMode(ESelectionMode::Single)
								.OnGenerateRow(this, &SSuggestionTextBox::HandleSuggestionListViewGenerateRow)
								.OnSelectionChanged(this, &SSuggestionTextBox::HandleSuggestionListViewSelectionChanged)
							]
					]
			)
	];
}


void SSuggestionTextBox::SetText( const TAttribute<FText>& InNewText )
{
	IgnoreUIUpdate = true;

	TextBox->SetText(InNewText);

	IgnoreUIUpdate = false;
}


/* SSuggestionTextBox implementation
 *****************************************************************************/

void SSuggestionTextBox::ClearSuggestions( )
{
	SelectedSuggestion = -1;

	MenuAnchor->SetIsOpen(false);
	Suggestions.Empty();
}


void SSuggestionTextBox::FocusTextBox( )
{
	FWidgetPath TextBoxPath;

	FSlateApplication::Get().GeneratePathToWidgetUnchecked(TextBox.ToSharedRef(), TextBoxPath);
	FSlateApplication::Get().SetKeyboardFocus(TextBoxPath, EFocusCause::SetDirectly);
}


FString SSuggestionTextBox::GetSelectedSuggestionString( ) const
{
	FString SuggestionString = *Suggestions[SelectedSuggestion];

	return SuggestionString.Replace(TEXT("\t"), TEXT(""));
}


void SSuggestionTextBox::MarkActiveSuggestion( )
{
	IgnoreUIUpdate = true;

	if (SelectedSuggestion >= 0)
	{
		TSharedPtr<FString> Suggestion = Suggestions[SelectedSuggestion];

		SuggestionListView->SetSelection(Suggestion);

		if (!SuggestionListView->IsItemVisible(Suggestion))
		{
			SuggestionListView->RequestScrollIntoView(Suggestion);
		}

		TextBox->SetText(FText::FromString(GetSelectedSuggestionString()));
	}
	else
	{
		SuggestionListView->ClearSelection();
	}

	IgnoreUIUpdate = false;
}


void SSuggestionTextBox::SetSuggestions( TArray<FString>& SuggestionStrings, bool InHistoryMode )
{
	FString SelectionText;

	// cache previously selected suggestion
	if ((SelectedSuggestion >= 0) && (SelectedSuggestion < Suggestions.Num()))
	{
		SelectionText = *Suggestions[SelectedSuggestion];
	}

	SelectedSuggestion = -1;

	// refill suggestions
	Suggestions.Empty();

	for (int32 i = 0; i < SuggestionStrings.Num(); ++i)
	{
		Suggestions.Add(MakeShareable(new FString(SuggestionStrings[i])));

		if (SuggestionStrings[i] == SelectionText)
		{
			SelectedSuggestion = i;
		}
	}

	if (Suggestions.Num())
	{
		// @todo Slate: make the window title not flicker when the box toggles visibility
		MenuAnchor->SetIsOpen(true, false);

		int32 SuggestionIndexToScrollIntoView = FMath::Max(0, SelectedSuggestion);
		SuggestionListView->RequestScrollIntoView(Suggestions[SuggestionIndexToScrollIntoView]);

		FocusTextBox();
	}
	else
	{
		MenuAnchor->SetIsOpen(false);
	}
}


/* SWidget overrides
 *****************************************************************************/

void SSuggestionTextBox::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	//	MenuAnchor->SetIsOpen(false);
}


FReply SSuggestionTextBox::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent )
{
	const FString& InputTextStr = TextBox->GetText().ToString();

	if (MenuAnchor->IsOpen())
	{
		if (KeyEvent.GetKey() == EKeys::Up)
		{
			// backward navigate the list of suggestions
			if (SelectedSuggestion < 0)
			{
				SelectedSuggestion = Suggestions.Num() - 1;
			}
			else
			{
				--SelectedSuggestion;
			}

			MarkActiveSuggestion();

			return FReply::Handled();
		}
		else if(KeyEvent.GetKey() == EKeys::Down)
		{
			// forward navigate the list of suggestions
			if (SelectedSuggestion < Suggestions.Num() - 1)
			{
				++SelectedSuggestion;
			}
			else
			{
				SelectedSuggestion = -1;
			}

			MarkActiveSuggestion();

			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Enter && CommitsOnSuggestionSelected && SuggestionListView->GetNumItemsSelected() > 0)
		{
			HandleTextBoxTextCommitted(FText::FromString(*Suggestions[SelectedSuggestion]), ETextCommit::OnEnter);
		}
		else if (KeyEvent.GetKey() == EKeys::Tab)
		{
			// auto-complete the highlighted suggestion
			if (Suggestions.Num())
			{
				if ((SelectedSuggestion >= 0) && (SelectedSuggestion < Suggestions.Num()))
				{
					MarkActiveSuggestion();
					HandleTextBoxTextCommitted(TextBox->GetText(), ETextCommit::OnEnter);
				}
				else
				{
					SelectedSuggestion = 0;

					MarkActiveSuggestion();
				}
			}

			return FReply::Handled();
		}
	}
	else
	{
		if ((KeyEvent.GetKey() == EKeys::Up) || (KeyEvent.GetKey() == EKeys::Down))
		{
			// show the input history
			if (OnShowingHistory.IsBound())
			{
				TArray<FString> HistoryStrings;

				OnShowingHistory.ExecuteIfBound(HistoryStrings);

				if (HistoryStrings.Num() > 0)
				{
					SetSuggestions(HistoryStrings, true);

					if (KeyEvent.GetKey() == EKeys::Up)
					{
						SelectedSuggestion = HistoryStrings.Num() - 1;
					}
					else
					{
						SelectedSuggestion = 0;
					}

					MarkActiveSuggestion();
				}
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}


bool SSuggestionTextBox::SupportsKeyboardFocus() const
{
	return true;
}


/* SSuggestionTextBox callbacks
 *****************************************************************************/

TSharedRef<ITableRow> SSuggestionTextBox::HandleSuggestionListViewGenerateRow( TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable )
{
	FString Left, Right, Combined;

	if (Text->Split(TEXT("\t"), &Left, &Right))
	{
		Combined = Left + Right;
	}
	else
	{
		Combined = *Text;
	}

	return SNew(STableRow<TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SBox)
				[
					SNew(STextBlock)
						.HighlightText(this, &SSuggestionTextBox::HandleSuggestionListWidgetHighlightText)
						.TextStyle( SuggestionTextStyle )
						.Text( FText::FromString(Combined) )						
				]
		];
}


void SSuggestionTextBox::HandleSuggestionListViewSelectionChanged( TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo )
{
	if (!IgnoreUIUpdate)
	{
		for (int32 i = 0; i < Suggestions.Num(); ++i)
		{
			if (NewValue == Suggestions[i])
			{
				SelectedSuggestion = i;

				MarkActiveSuggestion();

				if (CommitsOnSuggestionSelected)
				{
					HandleTextBoxTextCommitted(FText::FromString(*NewValue.Get()), ETextCommit::OnEnter);
				}

				FocusTextBox();

				break;
			}
		}
	}
}

FText SSuggestionTextBox::HandleSuggestionListWidgetHighlightText( ) const
{
	return TextBox->GetText();
}

void SSuggestionTextBox::HandleTextBoxTextChanged( const FText& InText )
{
	OnTextChanged.ExecuteIfBound(InText);

	if (!IgnoreUIUpdate)
	{
		const FString& InputTextStr = TextBox->GetText().ToString();

		if (!InputTextStr.IsEmpty() && OnShowingSuggestions.IsBound())
		{
			TArray<FString> SuggestionStrings;

			OnShowingSuggestions.ExecuteIfBound(InText.ToString(), SuggestionStrings);

			for (int32 Index = 0; Index < SuggestionStrings.Num(); ++Index)
			{
				FString& StringRef = SuggestionStrings[Index];

				StringRef = StringRef.Left(InputTextStr.Len()) + TEXT("\t") + StringRef.RightChop(InputTextStr.Len());
			}

			SetSuggestions(SuggestionStrings, false);
		}
		else
		{
			ClearSuggestions();
		}
	}
}


void SSuggestionTextBox::HandleTextBoxTextCommitted( const FText& InText, ETextCommit::Type CommitInfo )
{
	if (!MenuAnchor->IsOpen())
	{
		OnTextCommitted.ExecuteIfBound(InText, CommitInfo);
	}

	if ((CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnCleared))
	{
		ClearSuggestions();
	}
}


void SSuggestionTextBox::OnGlobalFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget)
{
	if (TextBox.IsValid() && OldFocusedWidgetPath.ContainsWidget(TextBox.Get()) && FocusEvent.GetCause() == EFocusCause::Mouse)
	{
		// If the textbox has lost focus and the SuggestionList has not gained it, then we assume the user clicked somewhere else in the app and clear the SuggestionList.
		if (VerticalBox.IsValid() && !NewFocusedWidgetPath.ContainsWidget(VerticalBox.Get()))
		{
			ClearSuggestions();
			OnTextCommitted.ExecuteIfBound(TextBox->GetText(), ETextCommit::Type::OnUserMovedFocus);
		}
	}
}
