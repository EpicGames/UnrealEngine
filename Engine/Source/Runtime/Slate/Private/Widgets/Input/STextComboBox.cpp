// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/STextComboBox.h"


void STextComboBox::Construct( const FArguments& InArgs )
{
	SelectionChanged = InArgs._OnSelectionChanged;
	GetTextLabelForItem = InArgs._OnGetTextLabelForItem;
	Font = InArgs._Font;

	// Then make widget
	this->ChildSlot
	[
		SAssignNew(StringCombo, SComboBox< TSharedPtr<FString> > )
		.ComboBoxStyle(InArgs._ComboBoxStyle)
		.ButtonStyle(InArgs._ButtonStyle)
		.OptionsSource(InArgs.GetOptionsSource())
		.OnGenerateWidget(this, &STextComboBox::MakeItemWidget)
		.OnSelectionChanged(this, &STextComboBox::OnSelectionChanged)
		.OnComboBoxOpening(InArgs._OnComboBoxOpening)
		.InitiallySelectedItem(InArgs._InitiallySelectedItem)
		.ContentPadding(InArgs._ContentPadding)
		[
			SNew(STextBlock)
				.ColorAndOpacity(InArgs._ColorAndOpacity)
				.Text(this, &STextComboBox::GetSelectedTextLabel)
				.Font(InArgs._Font)
		]
	];
	SelectedItem = StringCombo->GetSelectedItem();
}

FText STextComboBox::GetItemTextLabel(TSharedPtr<FString> StringItem) const
{
	if (!StringItem.IsValid())
	{
		return FText::GetEmpty();
	}

	// todo: jdale - should this be using FText natively?
	return FText::FromString(
		(GetTextLabelForItem.IsBound())
		? GetTextLabelForItem.Execute(StringItem)
		: *StringItem
		);
}

FText STextComboBox::GetSelectedTextLabel() const
{
	TSharedPtr<FString> StringItem = StringCombo->GetSelectedItem();
	return GetItemTextLabel(StringItem);
}

TSharedRef<SWidget> STextComboBox::MakeItemWidget( TSharedPtr<FString> StringItem ) 
{
	check( StringItem.IsValid() );

	return SNew(STextBlock)
		.Text(this, &STextComboBox::GetItemTextLabel, StringItem)
		.Font(Font);
}

void STextComboBox::OnSelectionChanged (TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		SelectedItem = Selection;
	}
	SelectionChanged.ExecuteIfBound(Selection, SelectInfo);
}

void STextComboBox::SetSelectedItem(TSharedPtr<FString> NewSelection)
{
	StringCombo->SetSelectedItem(NewSelection);
}

void STextComboBox::RefreshOptions()
{
	StringCombo->RefreshOptions();
}

void STextComboBox::ClearSelection( )
{
	StringCombo->ClearSelection();
}
