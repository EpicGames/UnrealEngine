// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/STextComboPopup.h"
#include "Widgets/Input/SButton.h"


void STextComboPopup::Construct( const FArguments& InArgs )
{
	OnTextChosen = InArgs._OnTextChosen;

	// First make array of pointers to strings
	for(int32 i=0; i<InArgs._TextOptions.Num(); i++)
	{
		Strings.Add( MakeShareable( new FString( InArgs._TextOptions[i] ) ) );
	}

	// Then make widget
	this->ChildSlot
	[
		SNew(SBorder)
		. BorderImage(FCoreStyle::Get().GetBrush("PopupText.Background"))
		. Padding(10.f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text( InArgs._Label )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(StringCombo, SComboBox< TSharedPtr<FString> >)
				.OptionsSource( &Strings )
				.OnGenerateWidget( this, &STextComboPopup::MakeItemWidget )
				.OnSelectionChanged(this, &STextComboPopup::OnSelectionChanged)
				[
					SNew (STextBlock)
					.Text(this, &STextComboPopup::GetSelectedItem)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(OkButton, SButton)
				.Text( NSLOCTEXT("UnrealEd", "OK", "OK"))
				.OnClicked( this, &STextComboPopup::OnOK )
			]
		]
	];

	// Start showing the first interface 
	SelectedItem = (Strings.Num() > 0)
		? Strings[0]
		: MakeShareable(new FString());

	// Update the combo box with the new options
	StringCombo->RefreshOptions();

	StringCombo->SetSelectedItem(SelectedItem);

	if (InArgs._AutoFocus)
	{
		RegisterActiveTimer(0.016f, FWidgetActiveTimerDelegate::CreateSP(this, &STextComboPopup::TickAutoFocus));
	}
}

EActiveTimerReturnType STextComboPopup::TickAutoFocus(double InCurrentTime, float InDeltaTime)
{
	TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (!OwnerWindow.IsValid())
	{
		return EActiveTimerReturnType::Stop;
	}
	else if (FSlateApplication::Get().HasFocusedDescendants(OwnerWindow.ToSharedRef()))
	{
		FocusDefaultWidget();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> STextComboPopup::MakeItemWidget( TSharedPtr<FString> StringItem ) 
{
	// todo: jdale - should this be using FText natively?
	check( StringItem.IsValid() );
	return	SNew(STextBlock)
		.Text(FText::FromString(*StringItem));
}

FText STextComboPopup::GetSelectedItem() const
{
	// todo: jdale - should this be using FText natively?
	return FText::FromString(*SelectedItem);
}

void STextComboPopup::OnSelectionChanged (TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		SelectedItem = Selection;
	}
}

FReply STextComboPopup::OnOK()
{
	if(SelectedItem.IsValid() && OnTextChosen.IsBound())
	{
		OnTextChosen.Execute(*SelectedItem);
	}
	return FReply::Handled();
}

void STextComboPopup::FocusDefaultWidget()
{
	FWidgetPath FocusMe;
	FSlateApplication::Get().GeneratePathToWidgetChecked( StringCombo.ToSharedRef(), FocusMe );
	FSlateApplication::Get().SetKeyboardFocus( FocusMe, EFocusCause::SetDirectly );
}

FReply STextComboPopup::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if(InKeyEvent.GetKey() == EKeys::Enter)
	{
		return OnOK();
	}
	
	return FReply::Unhandled();
}
