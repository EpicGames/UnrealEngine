// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"


void STextEntryPopup::Construct( const FArguments& InArgs )
{

	WidgetWithDefaultFocus.Reset();

	this->ChildSlot
	[
		SNew(SBorder)
		. BorderImage(FCoreStyle::Get().GetBrush("PopupText.Background"))
		. Padding(10.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.MaxWidth(InArgs._MaxWidth)
			[
				SAssignNew(Box, SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text( InArgs._Label )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(WidgetWithDefaultFocus, SEditableTextBox)
					.MinDesiredWidth(10.0f)
					.Text( InArgs._DefaultText )
					.OnTextCommitted( InArgs._OnTextCommitted )
					.OnTextChanged( InArgs._OnTextChanged )
					.OnVerifyTextChanged( InArgs._OnVerifyTextChanged )
					.HintText( InArgs._HintText )
					.SelectAllTextWhenFocused( InArgs._SelectAllTextWhenFocused )
					.ClearKeyboardFocusOnCommit( InArgs._ClearKeyboardFocusOnCommit )
				]
			]
		]
	];

	ErrorReporting = InArgs._ErrorReporting;
	if ( ErrorReporting.IsValid() )
	{
		Box->AddSlot()
		.AutoHeight()
		.Padding(3,0)
		[
			ErrorReporting->AsWidget()
		];
	}

	if (InArgs._AutoFocus)
	{
		RegisterActiveTimer(0.016f, FWidgetActiveTimerDelegate::CreateSP(this, &STextEntryPopup::TickAutoFocus));
	}
}

EActiveTimerReturnType STextEntryPopup::TickAutoFocus(double InCurrentTime, float InDeltaTime)
{
	TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (!OwnerWindow.IsValid())
	{
		return EActiveTimerReturnType::Stop;
	}
	else if (FSlateApplication::Get().HasFocusedDescendants(OwnerWindow.ToSharedRef()))
	{
		FocusDefaultWidget();
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void STextEntryPopup::FocusDefaultWidget()
{
	// Focus the text widget
	FWidgetPath FocusMe;
	FSlateApplication::Get().GeneratePathToWidgetChecked( WidgetWithDefaultFocus.ToSharedRef(), FocusMe );
	FSlateApplication::Get().SetKeyboardFocus( FocusMe, EFocusCause::SetDirectly );
}

void STextEntryPopup::SetError( const FText& InError )
{
	SetError( InError.ToString() );
}

void STextEntryPopup::SetError (const FString& InError)
{
	if ( !ErrorReporting.IsValid() )
	{
		// No error reporting was specified; make a default one
		TSharedPtr<SErrorText> ErrorTextWidget;
		Box->AddSlot()
		.AutoHeight()
		.Padding(3,0)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3,1)
			[
				SAssignNew( ErrorTextWidget, SErrorText )
			]
		];
		ErrorReporting = ErrorTextWidget;
	}
	ErrorReporting->SetError( InError );
}
