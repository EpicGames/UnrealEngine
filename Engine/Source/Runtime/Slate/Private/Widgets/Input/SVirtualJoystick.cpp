// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SVirtualJoystick.h"
#include "Rendering/DrawElements.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

const float OPACITY_LERP_RATE = 3.f;


static FORCEINLINE float GetScaleFactor(const FGeometry& Geometry)
{
	const float DesiredWidth = 1024.0f;

	float UndoDPIScaling = 1.0f / Geometry.Scale;
	return (Geometry.GetDrawSize().GetMax() / DesiredWidth) * UndoDPIScaling;
}

FORCEINLINE float SVirtualJoystick::GetBaseOpacity()
{
	return (State == State_Active || State == State_CountingDownToInactive) ? ActiveOpacity : InactiveOpacity;
}

void SVirtualJoystick::FControlData::Reset()
{
	// snap the visual center back to normal (for controls that have a center on touch)
	VisualCenter = CorrectedCenter;
}

void SVirtualJoystick::Construct(const FArguments& InArgs)
{
	bVisible = true;
	bPreventReCenter = false;
	
	// listen for displaymetrics changes to reposition controls
	FSlateApplication::Get().GetPlatformApplication()->OnDisplayMetricsChanged().AddSP(this, &SVirtualJoystick::HandleDisplayMetricsChanged);
}

void SVirtualJoystick::HandleDisplayMetricsChanged(const FDisplayMetrics& NewDisplayMetric)
{
	// Mark all controls to be repositioned on next tick
	for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
	{
		Controls[ControlIndex].bHasBeenPositioned = false;
	}
}

void SVirtualJoystick::SetGlobalParameters(float InActiveOpacity, float InInactiveOpacity, float InTimeUntilDeactive, float InTimeUntilReset, float InActivationDelay, bool InbPreventReCenter, float InStartupDelay)
{
	ActiveOpacity = InActiveOpacity;
	InactiveOpacity = InInactiveOpacity;
	TimeUntilDeactive = InTimeUntilDeactive;
	TimeUntilReset = InTimeUntilReset;
	ActivationDelay = InActivationDelay;
	StartupDelay = InStartupDelay;

	bPreventReCenter = InbPreventReCenter;

	if (StartupDelay > 0.f)
	{
		State = State_WaitForStart;
	}
}

bool SVirtualJoystick::ShouldDisplayTouchInterface()
{
	bool bAlwaysShowTouchInterface = false;
	GConfig->GetBool(TEXT("/Script/Engine.InputSettings"), TEXT("bAlwaysShowTouchInterface"), bAlwaysShowTouchInterface, GInputIni);

	// do we want to show virtual joysticks?
	return FPlatformMisc::GetUseVirtualJoysticks() || bAlwaysShowTouchInterface || (FSlateApplication::Get().IsFakingTouchEvents() && FPlatformMisc::ShouldDisplayTouchInterfaceOnFakingTouchEvents());
}

static float ResolveRelativePosition(float Position, float RelativeTo, float ScaleFactor)
{
	// absolute from edge
	if (Position < -1.0f)
	{
		return RelativeTo + Position * ScaleFactor;
	}
	// relative from edge
	else if (Position < 0.0f)
	{
		return RelativeTo + Position * RelativeTo;
	}
	// relative from 0
	else if (Position <= 1.0f)
	{
		return Position * RelativeTo;
	}
	// absolute from 0
	else
	{
		return Position * ScaleFactor;
	}

}

static bool PositionIsInside(const FVector2D& Center, const FVector2D& Position, const FVector2D& BoxSize)
{
	return
		Position.X >= Center.X - BoxSize.X * 0.5f &&
		Position.X <= Center.X + BoxSize.X * 0.5f &&
		Position.Y >= Center.Y - BoxSize.Y * 0.5f &&
		Position.Y <= Center.Y + BoxSize.Y * 0.5f;
}

int32 SVirtualJoystick::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 RetLayerId = LayerId;

	if (bVisible)
	{
		FLinearColor ColorAndOpacitySRGB = InWidgetStyle.GetColorAndOpacityTint();
		ColorAndOpacitySRGB.A = CurrentOpacity;

		for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
		{
			const FControlData& Control = Controls[ControlIndex];

			if (Control.Info.Image2.IsValid())
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					RetLayerId++,
					AllottedGeometry.ToPaintGeometry(
						Control.CorrectedVisualSize,
						FSlateLayoutTransform(Control.VisualCenter - FVector2D(Control.CorrectedVisualSize.X * 0.5f, Control.CorrectedVisualSize.Y * 0.5f))
					),
					Control.Info.Image2->GetSlateBrush(),
					ESlateDrawEffect::None,
					ColorAndOpacitySRGB
					);
			}

			if (Control.Info.Image1.IsValid())
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					RetLayerId++,
					AllottedGeometry.ToPaintGeometry(
						Control.CorrectedThumbSize,
						FSlateLayoutTransform(Control.VisualCenter + Control.ThumbPosition - FVector2D(Control.CorrectedThumbSize.X * 0.5f, Control.CorrectedThumbSize.Y * 0.5f))
					),
					Control.Info.Image1->GetSlateBrush(),
					ESlateDrawEffect::None,
					ColorAndOpacitySRGB
					);
			}
		}
	}
	
	return RetLayerId;
}

FVector2D SVirtualJoystick::ComputeDesiredSize( float ) const
{
	return FVector2D(100, 100);
}

bool SVirtualJoystick::SupportsKeyboardFocus() const
{
	return false;
}

FReply SVirtualJoystick::OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
//	UE_LOGF(LogTemp, Log, "Pointer index: %d", Event.GetPointerIndex());

	FVector2D LocalCoord = MyGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());

	for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
	{
		FControlData& Control = Controls[ControlIndex];

		// skip controls already in use
		if (Control.CapturedPointerIndex == -1)
		{
			if (PositionIsInside(Control.CorrectedCenter, LocalCoord, Control.CorrectedInteractionSize))
			{
				// Align Joystick inside of Screen
				AlignBoxIntoScreen(LocalCoord, Control.CorrectedVisualSize, MyGeometry.GetLocalSize());

				Control.CapturedPointerIndex = Event.GetPointerIndex();

				if (ActivationDelay == 0.f)
				{
					CurrentOpacity = ActiveOpacity;

					if (!bPreventReCenter)
					{
						Control.VisualCenter = LocalCoord;
					}

					if (HandleTouch(ControlIndex, LocalCoord, MyGeometry.GetLocalSize())) // Never fail!
					{
						return FReply::Handled().CaptureMouse(SharedThis(this));
					}
				}
				else
				{
					Control.bNeedUpdatedCenter = true;
					Control.ElapsedTime = 0.f;
					Control.NextCenter = LocalCoord;

					return FReply::Unhandled();
				}
			}
		}
	}

//	CapturedPointerIndex[CapturedJoystick] = -1;

	return FReply::Unhandled();
}

FReply SVirtualJoystick::OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	FVector2D LocalCoord = MyGeometry.AbsoluteToLocal( Event.GetScreenSpacePosition() );

	for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
	{
		FControlData& Control = Controls[ControlIndex];

		// is this control the one captured to this pointer?
		if (Control.CapturedPointerIndex == Event.GetPointerIndex())
		{
			if (Control.bNeedUpdatedCenter)
			{
				return FReply::Unhandled();
			}
			else if (HandleTouch(ControlIndex, LocalCoord, MyGeometry.GetLocalSize()))
			{
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SVirtualJoystick::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
	{
		FControlData& Control = Controls[ControlIndex];

		// is this control the one captured to this pointer?
		if (Control.CapturedPointerIndex == Event.GetPointerIndex())
		{
			// release and center the joystick
			Control.ThumbPosition = FVector2D(0, 0);
			Control.CapturedPointerIndex = -1;

			// send one more joystick update for the centering
			Control.bSendOneMoreEvent = true;

			// Pass event as unhandled if time is too short
			if (Control.bNeedUpdatedCenter)
			{
				Control.bNeedUpdatedCenter = false;
				return FReply::Unhandled();
			}

			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return FReply::Unhandled();
}

FVector2D SVirtualJoystick::ComputeThumbPosition(int32 ControlIndex, const FVector2D& LocalCoord, float* OutDistanceToTouchSqr, float* OutDistanceToEdgeSqr)
{
	float DistanceToTouchSqr = 0.0f;
	float DistanceToEdgeSqr = 0.0f;
	FVector2D Position;
	const FControlData& Control = Controls[ControlIndex];

	// figure out position around center
	FVector2D Offset = LocalCoord - Control.VisualCenter;
	// only do work if we aren't at the center
	if (Offset == FVector2D(0, 0))
	{
		Position = Offset;
	}
	else
	{
		// clamp to the ellipse of the stick (snaps to the visual size, so, the art should go all the way to the edge of the texture)
		DistanceToTouchSqr = Offset.SizeSquared();
		float Angle = FMath::Atan2(Offset.Y, Offset.X);

		// length along line to ellipse: L = 1.0 / sqrt(((sin(T)/Rx)^2 + (cos(T)/Ry)^2))
		float CosAngle = FMath::Cos(Angle);
		float SinAngle = FMath::Sin(Angle);
		float XTerm = CosAngle / (Control.CorrectedVisualSize.X * 0.5f);
		float YTerm = SinAngle / (Control.CorrectedVisualSize.Y * 0.5f);
		float XYTermSqr = XTerm * XTerm + YTerm * YTerm;
		DistanceToEdgeSqr = 1.0f / XYTermSqr;

		// only clamp 
		if (DistanceToTouchSqr > DistanceToEdgeSqr)
		{
			float DistanceToEdge = FMath::InvSqrt(XYTermSqr);
			Position = FVector2D(DistanceToEdge * CosAngle, DistanceToEdge * SinAngle);
		}
		else
		{
			Position = Offset;
		}
	}

	if (OutDistanceToTouchSqr != nullptr)
	{
		*OutDistanceToTouchSqr = DistanceToTouchSqr;
	}

	if (OutDistanceToEdgeSqr != nullptr)
	{
		*OutDistanceToEdgeSqr = DistanceToEdgeSqr;
	}

	return Position;
}

bool SVirtualJoystick::HandleTouch(int32 ControlIndex, const FVector2D& LocalCoord, const FVector2D& ScreenSize)
{
	FControlData& Control = Controls[ControlIndex];
	Control.ThumbPosition = ComputeThumbPosition(ControlIndex, LocalCoord);

	FVector2D AbsoluteThumbPos = Control.ThumbPosition + Controls[ControlIndex].VisualCenter;
	AlignBoxIntoScreen(AbsoluteThumbPos, Control.CorrectedThumbSize, ScreenSize);
	Control.ThumbPosition = AbsoluteThumbPos - Controls[ControlIndex].VisualCenter;

	return true;
}

void SVirtualJoystick::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (State == State_WaitForStart || State == State_CountingDownToStart)
	{
		CurrentOpacity = 0.f;
	}
	else
	{
		// lerp to the desired opacity based on whether the user is interacting with the joystick
		CurrentOpacity = FMath::Lerp(CurrentOpacity, GetBaseOpacity(), OPACITY_LERP_RATE * InDeltaTime);
	}

	// count how many controls are active
	int32 NumActiveControls = 0;

	// figure out how much to scale the control sizes
	float ScaleFactor = GetScaleFactor(AllottedGeometry);

	for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
	{
		FControlData& Control = Controls[ControlIndex];

		if (Control.bNeedUpdatedCenter)
		{
			Control.ElapsedTime += InDeltaTime;
			if (Control.ElapsedTime > ActivationDelay)
			{
				Control.bNeedUpdatedCenter = false;
				CurrentOpacity = ActiveOpacity;

				if (!bPreventReCenter)
				{
					Control.VisualCenter = Control.NextCenter;
				}

				HandleTouch(ControlIndex, Control.NextCenter, AllottedGeometry.GetLocalSize());
			}
		}

		// calculate absolute positions based on geometry
		// @todo: Need to manage geometry changing!
		if (!Control.bHasBeenPositioned || ScaleFactor != PreviousScalingFactor)
		{
			const FControlInfo& ControlInfo = Control.Info;
			
			// update all the sizes
			Control.CorrectedCenter = FVector2D(ResolveRelativePosition(ControlInfo.Center.X, AllottedGeometry.GetLocalSize().X, ScaleFactor), ResolveRelativePosition(ControlInfo.Center.Y, AllottedGeometry.GetLocalSize().Y, ScaleFactor));
			Control.VisualCenter = Control.CorrectedCenter;
			Control.CorrectedVisualSize = FVector2D(ResolveRelativePosition(ControlInfo.VisualSize.X, AllottedGeometry.GetLocalSize().X, ScaleFactor), ResolveRelativePosition(ControlInfo.VisualSize.Y, AllottedGeometry.GetLocalSize().Y, ScaleFactor));
			Control.CorrectedInteractionSize = FVector2D(ResolveRelativePosition(ControlInfo.InteractionSize.X, AllottedGeometry.GetLocalSize().X, ScaleFactor), ResolveRelativePosition(ControlInfo.InteractionSize.Y, AllottedGeometry.GetLocalSize().Y, ScaleFactor));
			Control.CorrectedThumbSize = FVector2D(ResolveRelativePosition(ControlInfo.ThumbSize.X, AllottedGeometry.GetLocalSize().X, ScaleFactor), ResolveRelativePosition(ControlInfo.ThumbSize.Y, AllottedGeometry.GetLocalSize().Y, ScaleFactor));
			Control.CorrectedInputScale = ControlInfo.InputScale; // *ScaleFactor;
			Control.bHasBeenPositioned = true;
		}

		if (Control.CapturedPointerIndex >= 0 || Control.bSendOneMoreEvent)
		{
			// cache the key released state so we can send input pressed/released events later
			bool bButtonPressed = !Control.bSendOneMoreEvent;

			Control.bSendOneMoreEvent = false;

			// Get the corrected thumb offset scale (now allows ellipse instead of assuming square)
			FVector2D ThumbScaledOffset = FVector2D(Control.ThumbPosition.X * 2.0f / Control.CorrectedVisualSize.X, Control.ThumbPosition.Y * 2.0f / Control.CorrectedVisualSize.Y);
			float ThumbSquareSum = ThumbScaledOffset.X * ThumbScaledOffset.X + ThumbScaledOffset.Y * ThumbScaledOffset.Y;
			float ThumbMagnitude = FMath::Sqrt(ThumbSquareSum);
			FVector2D ThumbNormalized = FVector2D(0.f, 0.f);
			if (ThumbSquareSum > SMALL_NUMBER)
			{
				const float Scale = 1.0f / ThumbMagnitude;
				ThumbNormalized = FVector2D(ThumbScaledOffset.X * Scale, ThumbScaledOffset.Y * Scale);
			}

			// Find the scale to apply to ThumbNormalized vector to project onto unit square
			float ToSquareScale = fabs(ThumbNormalized.Y) > fabs(ThumbNormalized.X) ? FMath::Sqrt((ThumbNormalized.X * ThumbNormalized.X) / (ThumbNormalized.Y * ThumbNormalized.Y) + 1.0f)
				: ThumbNormalized.X == 0.0f ? 1.0f : FMath::Sqrt((ThumbNormalized.Y * ThumbNormalized.Y) / (ThumbNormalized.X * ThumbNormalized.X) + 1.0f);

			// Apply proportional offset corrected for projection to unit square
			FVector2D NormalizedOffset = ThumbNormalized * Control.CorrectedInputScale * ThumbMagnitude * ToSquareScale;

			// now pass the fake joystick events to the game
			const FGamepadKeyNames::Type XAxis = (Control.Info.MainInputKey.IsValid() ? Control.Info.MainInputKey.GetFName() : (ControlIndex == 0 ? FGamepadKeyNames::LeftAnalogX : FGamepadKeyNames::RightAnalogX));
			const FGamepadKeyNames::Type YAxis = (Control.Info.AltInputKey.IsValid() ? Control.Info.AltInputKey.GetFName() : (ControlIndex == 0 ? FGamepadKeyNames::LeftAnalogY : FGamepadKeyNames::RightAnalogY));

			FSlateApplication::Get().SetAllUserFocusToGameViewport();
			
			FInputDeviceId PrimaryInputDevice = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(FSlateApplicationBase::SlateAppPrimaryPlatformUser);

			auto ApplyInput = [PrimaryInputDevice](const FGamepadKeyNames::Type KeyName, float Delta, bool bTreatAsButton, bool bPressed)
			{
				FKey Key(KeyName);
				if (Key.IsAnalog())
				{
					FSlateApplication::Get().OnControllerAnalog(KeyName, FSlateApplicationBase::SlateAppPrimaryPlatformUser, PrimaryInputDevice, Delta);
				}
				else if (bTreatAsButton)
				{
					if (bPressed)
					{
						FSlateApplication::Get().OnControllerButtonPressed(KeyName, FSlateApplicationBase::SlateAppPrimaryPlatformUser, PrimaryInputDevice, false);
					}
					else
					{
						FSlateApplication::Get().OnControllerButtonReleased(KeyName, FSlateApplicationBase::SlateAppPrimaryPlatformUser, PrimaryInputDevice, false);
					}
				}
				else if (Delta != 0.0f)
				{
					FSlateApplication::Get().OnControllerButtonPressed(KeyName, FSlateApplicationBase::SlateAppPrimaryPlatformUser, PrimaryInputDevice, false);
				}
				else
				{
					FSlateApplication::Get().OnControllerButtonReleased(KeyName, FSlateApplicationBase::SlateAppPrimaryPlatformUser, PrimaryInputDevice, false);
				}
			};

			ApplyInput(XAxis, NormalizedOffset.X, Control.Info.bTreatAsButton, bButtonPressed);

			// ignore the Y axis if this is a button
			if (!Control.Info.bTreatAsButton)
			{
				ApplyInput(YAxis, -NormalizedOffset.Y, false, bButtonPressed);
			}
		}
		

		// is this active?
		if (Control.CapturedPointerIndex != -1)
		{
			NumActiveControls++;
		}
	}

	// we need to store the computed scale factor so we can compare it with the value computed in the following frame and, if necessary, recompute widget position
	PreviousScalingFactor = ScaleFactor;

	// STATE MACHINE!
	if (NumActiveControls > 0 || bPreventReCenter)
	{
		// any active control snaps the state to active immediately
		State = State_Active;
	}
	else
	{
		switch (State)
		{
			case State_WaitForStart:
				{
					State = State_CountingDownToStart;
					Countdown = StartupDelay;
				}
				break;
			case State_CountingDownToStart:
				// update the countdown
				Countdown -= InDeltaTime;
				if (Countdown <= 0.0f)
				{
					State = State_Inactive;
				}
				break;
			case State_Active:
				if (NumActiveControls == 0)
				{
					// start going to inactive
					State = State_CountingDownToInactive;
					Countdown = TimeUntilDeactive;
				}
				break;

			case State_CountingDownToInactive:
				// update the countdown
				Countdown -= InDeltaTime;
				if (Countdown <= 0.0f)
				{
					// should we start counting down to a control reset?
					if (TimeUntilReset > 0.0f)
					{
						State = State_CountingDownToReset;
						Countdown = TimeUntilReset;
					}
					else
					{
						// if not, then just go inactive
						State = State_Inactive;
					}
				}
				break;

			case State_CountingDownToReset:
				Countdown -= InDeltaTime;
				if (Countdown <= 0.0f)
				{
					// reset all the controls
					for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
					{
						Controls[ControlIndex].Reset();
					}

					// finally, go inactive
					State = State_Inactive;
				}
				break;
		}
	}
}

void SVirtualJoystick::SetJoystickVisibility(const bool bInVisible, const bool bInFade)
{
	// if we aren't fading, then just set the current opacity to desired
	if (!bInFade)
	{
		if (bInVisible)
		{
			CurrentOpacity = GetBaseOpacity();
		}
		else
		{
			CurrentOpacity = 0.f;
		}
	}

	bVisible = bInVisible;

	//Update slate visibility - we still want to tick, so we use HitTestInvisible to avoid blocking other UI
	SetVisibility(bVisible ? EVisibility::Visible : EVisibility::HitTestInvisible);
}

void SVirtualJoystick::AddControl(const FControlInfo& Control)
{
	FControlData& ControlData = Controls.AddDefaulted_GetRef();
	ControlData.Info = Control;
}

void SVirtualJoystick::ClearControls()
{
	Controls.Empty();
}

void SVirtualJoystick::SetControls(const TArray<FControlInfo>& InControls)
{
	ClearControls();
	
	for (const FControlInfo& ControlInfo : InControls)
	{
		AddControl(ControlInfo);
	}
}

void SVirtualJoystick::AlignBoxIntoScreen(FVector2D& Position, const FVector2D& Size, const FVector2D& ScreenSize)
{
	if (Size.X > ScreenSize.X || Size.Y > ScreenSize.Y)
	{
		return;
	}

	// Align box to fit into screen
	if (Position.X - Size.X * 0.5f < 0.f)
	{
		Position.X = Size.X * 0.5f;
	}

	if (Position.X + Size.X * 0.5f > ScreenSize.X)
	{
		Position.X = ScreenSize.X - Size.X * 0.5f;
	}

	if (Position.Y - Size.Y * 0.5f < 0.f)
	{
		Position.Y = Size.Y * 0.5f;
	}

	if (Position.Y + Size.Y * 0.5f > ScreenSize.Y)
	{
		Position.Y = ScreenSize.Y - Size.Y * 0.5f;
	}
}
