// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SVirtualKeyboardEntry.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextEditHelper.h"

/** Constructor */
SVirtualKeyboardEntry::SVirtualKeyboardEntry()
	: ScrollHelper(),
	  bWasFocusedByLastMouseDown( false ),
	  bIsChangingText( false )
{ }


/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SVirtualKeyboardEntry::Construct( const FArguments& InArgs )
{
	Text = InArgs._Text;
	HintText = InArgs._HintText;

	Font = InArgs._Font;
	ColorAndOpacity = InArgs._ColorAndOpacity;
	IsReadOnly = InArgs._IsReadOnly;
	ClearKeyboardFocusOnCommit = InArgs._ClearKeyboardFocusOnCommit;
	OnTextChanged = InArgs._OnTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;
	MinDesiredWidth = InArgs._MinDesiredWidth;
	KeyboardType = InArgs._KeyboardType;
	VirtualKeyboardOptions = InArgs._VirtualKeyboardOptions;
	bEnableIntegratedKeyboard = InArgs._EnableIntegratedKeyboard;
}

void SVirtualKeyboardEntry::SetText(const TAttribute< FText >& InNewText)
{
	EditedText = InNewText.Get();

	// Don't set text if the text attribute has a 'getter' binding on it, otherwise we'd blow away
	// that binding.  If there is a getter binding, then we'll assume it will provide us with
	// updated text after we've fired our 'text changed' callbacks
	if (!Text.IsBound())
	{
		Text.Set(EditedText);
	}

	bNeedsUpdate = true;
}

void SVirtualKeyboardEntry::SetTextFromVirtualKeyboard(const FText& InNewText, ETextEntryType TextEntryType)
{
	check(IsInGameThread());

	// Only set the text if the text attribute doesn't have a getter binding (otherwise it would be blown away).
	// If it is bound, we'll assume that OnTextChanged will handle the update.
	if (!Text.IsBound())
	{
		Text.Set(InNewText);
	}

	if (!InNewText.EqualTo(EditedText))
	{
		EditedText = InNewText;

		// This method is called from the main thread (i.e. not the game thread) of the device with the virtual keyboard
		// This causes the app to crash on those devices, so we're using polling here to ensure delegates are
		// fired on the game thread in Tick.
		bNeedsUpdate = true;
	}
}

void SVirtualKeyboardEntry::SetSelectionFromVirtualKeyboard(int InSelStart, int InSelEnd)
{
	check(IsInGameThread());
	//Nothing to do for widgets without a cursor
}

/**
 * Restores the text to the original state
 */
void SVirtualKeyboardEntry::RestoreOriginalText()
{
	if( HasTextChangedFromOriginal() )
	{
		SetTextFromVirtualKeyboard(OriginalText, ETextEntryType::TextEntryCanceled);
	}
}

bool SVirtualKeyboardEntry::HasTextChangedFromOriginal() const
{
	return ( !IsReadOnly.Get() && !EditedText.EqualTo(OriginalText)  );
}

/**
 * Sets the font used to draw the text
 *
 * @param  InNewFont	The new font to use
 */
void SVirtualKeyboardEntry::SetFont( const TAttribute< FSlateFontInfo >& InNewFont )
{
	Font = InNewFont;
}

/**
 * Checks to see if this widget supports keyboard focus.  Override this in derived classes.
 *
 * @return  True if this widget can take keyboard focus
 */
bool SVirtualKeyboardEntry::SupportsKeyboardFocus() const
{
	return true;
}

bool SVirtualKeyboardEntry::GetIsReadOnly() const
{
	return IsReadOnly.Get();
}

/**
 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
 *
 * @param  AllottedGeometry Space allotted to this widget
 * @param  InCurrentTime  Current absolute real time
 * @param  InDeltaTime  Real time passed since last tick
 */
void SVirtualKeyboardEntry::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if(bNeedsUpdate)
	{
		// Let outsiders know that the text content has been changed
		OnTextChanged.ExecuteIfBound( HasKeyboardFocus() ? EditedText : Text.Get() );
		bNeedsUpdate = false;
	}


	// We'll draw with the 'focused' look if we're either focused or we have a context menu summoned
	const bool bShouldAppearFocused = HasKeyboardFocus();
}

/**
 * Computes the desired size of this widget (SWidget)
 *
 * @return  The widget's desired size
 */
FVector2D SVirtualKeyboardEntry::ComputeDesiredSize( float ) const
{
	const FSlateFontInfo& FontInfo = Font.Get();
	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const float FontMaxCharHeight = FTextEditHelper::GetFontHeight(FontInfo);

	FVector2D TextSize;

	const FString StringToRender = GetStringToRender();
	if( !StringToRender.IsEmpty() )
	{
		TextSize = FontMeasureService->Measure( StringToRender, FontInfo );
	}
	else
	{
		TextSize = FontMeasureService->Measure( HintText.Get().ToString(), FontInfo );
	}
	
	return FVector2D( FMath::Max(TextSize.X, MinDesiredWidth.Get()), FMath::Max(TextSize.Y, FontMaxCharHeight) );
}

int32 SVirtualKeyboardEntry::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// The text and some effects draws in front of the widget's background and selection.
	const int32 SelectionLayer = 0;
	const int32 TextLayer = 1;

	// See if a disabled effect should be used
	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	bool bIsReadonly = IsReadOnly.Get();
	ESlateDrawEffect DrawEffects = (bEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FSlateFontInfo& FontInfo = Font.Get();
	const FString VisibleText = GetStringToRender();
	const FLinearColor ThisColorAndOpacity = ColorAndOpacity.Get().GetColor(InWidgetStyle);
	const FLinearColor ColorAndOpacitySRGB = ThisColorAndOpacity * InWidgetStyle.GetColorAndOpacityTint();
	const float FontMaxCharHeight = FTextEditHelper::GetFontHeight(FontInfo);
	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	// We'll draw with the 'focused' look if we're either focused or we have a context menu summoned
	const bool bShouldAppearFocused = HasKeyboardFocus();

	const float DrawPositionY = ( AllottedGeometry.GetLocalSize().Y / 2 ) - ( FontMaxCharHeight / 2 );

	if (VisibleText.Len() == 0)
	{
		// Draw the hint text.
		const FLinearColor HintTextColor = FLinearColor(ColorAndOpacitySRGB.R, ColorAndOpacitySRGB.G, ColorAndOpacitySRGB.B, 0.35f);
		const FString ThisHintText = this->HintText.Get().ToString();
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId + TextLayer,
			AllottedGeometry.ToPaintGeometry( AllottedGeometry.GetLocalSize(), FSlateLayoutTransform(FVector2f( 0.0, DrawPositionY )) ),
			ThisHintText,          // Text
			FontInfo,              // Font information (font name, size)
			DrawEffects,           // Effects to use
			HintTextColor          // Color
		);
	}
	else
	{
		// Draw the text
#if 0
		// NOTE: this code is causing the text to not be visible at all once entered on device.
		// I've replaced it with the (working) hint-text version for now (without scrolling support)
		// until a real fix can be found. -dnikdel

		// Only draw the text that's potentially visible
		const float ScrollAreaLeft = ScrollHelper.ToScrollerSpace( FVector2D::ZeroVector ).X;
		const float ScrollAreaRight = ScrollHelper.ToScrollerSpace( AllottedGeometry.Size ).X;
		const int32 FirstVisibleCharacter = FontMeasureService->FindCharacterIndexAtOffset( VisibleText, FontInfo, ScrollAreaLeft );
		const int32 LastVisibleCharacter = FontMeasureService->FindCharacterIndexAtOffset( VisibleText, FontInfo, ScrollAreaRight );
		const FString PotentiallyVisibleText( VisibleText.Mid( FirstVisibleCharacter, ( LastVisibleCharacter - FirstVisibleCharacter ) + 1 ) );
		const float FirstVisibleCharacterPosition = FontMeasureService->Measure( VisibleText.Left( FirstVisibleCharacter ), FontInfo ).X;

		const float TextVertOffset = FontMaxCharHeight;
		const FVector2D DrawPosition = ScrollHelper.FromScrollerSpace( FVector2D( FirstVisibleCharacterPosition, DrawPositionY + TextVertOffset ) );
		const FVector2D DrawSize = ScrollHelper.SizeFromScrollerSpace( AllottedGeometry.Size );

		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId + TextLayer,
			AllottedGeometry.ToPaintGeometry( DrawPosition, DrawSize ),
			PotentiallyVisibleText,          // Text
			FontInfo,                        // Font information (font name, size)
			DrawEffects,                     // Effects to use
			ColorAndOpacitySRGB              // Color
		);
#else
		// Draw the whole text for now
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId + TextLayer,
			AllottedGeometry.ToPaintGeometry(AllottedGeometry.GetLocalSize(), FSlateLayoutTransform(FVector2f(0, DrawPositionY))),
			VisibleText,           // Text
			FontInfo,              // Font information (font name, size)
			DrawEffects,           // Effects to use
			ColorAndOpacitySRGB    // Color
			);
#endif
	}
	
	return LayerId + TextLayer;
}

FReply SVirtualKeyboardEntry::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// The user wants to edit text. Make a copy of the observed text for the user to edit.
	EditedText = Text.Get();

	int32 CaretPosition = EditedText.ToString().Len();
	FSlateApplication& CurrentApp = FSlateApplication::Get();
	CurrentApp.ShowVirtualKeyboard(true, InFocusEvent.GetUser(), SharedThis(this));

	return FReply::Handled();
}

/**
 * Called when this widget loses the keyboard focus.  This event does not bubble.
 *
 * @param  InFocusEvent  FocusEvent
 */
void SVirtualKeyboardEntry::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	// See if user explicitly tabbed away or moved focus
	ETextCommit::Type TextAction;
	switch ( InFocusEvent.GetCause() )
	{
		case EFocusCause::Navigation:
		case EFocusCause::Mouse:
			TextAction = ETextCommit::OnUserMovedFocus;
			break;

		case EFocusCause::Cleared:
			TextAction = ETextCommit::OnCleared;
			break;

		default:
			TextAction = ETextCommit::Default;
			break;
	}

	FSlateApplication& CurrentApp = FSlateApplication::Get();
	CurrentApp.ShowVirtualKeyboard(false, InFocusEvent.GetUser());

	OnTextCommitted.ExecuteIfBound( EditedText, TextAction );
}

/**
 * Gets the text that needs to be rendered
 *
 * @return  Text string
 */
FString SVirtualKeyboardEntry::GetStringToRender() const
{
	FString VisibleText;

	if(HasKeyboardFocus())
	{
		VisibleText = EditedText.ToString();
	}
	else
	{
		VisibleText = Text.Get().ToString();
	}

	// If this string is a password we need to replace all the characters in the string with something else
	if ( KeyboardType.Get() == EKeyboardType::Keyboard_Password )
	{
		const TCHAR Dot = 0x25CF;
		int32 VisibleTextLen = VisibleText.Len();
		VisibleText.Empty();
		while ( VisibleTextLen )
		{
			VisibleText += Dot;
			VisibleTextLen--;
		}
	}

	return VisibleText;
}
