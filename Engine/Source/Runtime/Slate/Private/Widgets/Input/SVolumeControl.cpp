// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SVolumeControl.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSlider.h"


/**
 * Construct the widget
 * 
 * @param InDeclaration   A declaration from which to construct the widget
 */
void SVolumeControl::Construct( const SVolumeControl::FArguments& InDeclaration )
{
	check(InDeclaration._Style);

	SpeakerIcons[ES_Full] = &InDeclaration._Style->HighVolumeImage;
	SpeakerIcons[ES_Mid] = &InDeclaration._Style->MidVolumeImage;
	SpeakerIcons[ES_Low] = &InDeclaration._Style->LowVolumeImage;
	SpeakerIcons[ES_Off] = &InDeclaration._Style->NoVolumeImage;
	SpeakerIcons[ES_Muted] = &InDeclaration._Style->MutedImage;

	MutedAttribute = InDeclaration._Muted;
	OnMutedChanged = InDeclaration._OnMuteChanged;
	VolumeAttribute = InDeclaration._Volume;
	OnVolumeChanged = InDeclaration._OnVolumeChanged;

	this->ChildSlot
	.Padding( FMargin( 2.0f, 1.0f) )
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding( FMargin(1.0f, 2.0f) )
		[
			SNew(SButton)
			.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
			.ContentPadding(FMargin(0))
			.OnClicked(this, &SVolumeControl::OnMuteClicked)
			[
				SNew(SImage)
				.Image(this, &SVolumeControl::GetSpeakerImage)
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding( FMargin(8.0f, 2.0f, 1.0f, 2.0f) )
		[
			SAssignNew(Slider, SSlider)
			.Style(&InDeclaration._Style->SliderStyle)
			.Value(this, &SVolumeControl::GetSliderPosition)
			.OnValueChanged(this, &SVolumeControl::OnWriteValue)
		]
 	];
}

const FSlateBrush* SVolumeControl::GetSpeakerImage() const
{
	float Value = Slider->GetValue();
	ESpeakerIcon Icon;

	if (MutedAttribute.Get())
	{
		Icon = ES_Muted;
	}
	else if (0.67f < Value)
	{
		Icon = ES_Full;
	}
	else if (0.33f < Value)
	{
		Icon = ES_Mid;
	}
	else if (0.01f < Value)
	{
		Icon = ES_Low;
	}
	else
	{
		Icon = ES_Off;
	}

	return SpeakerIcons[Icon];
}

FReply SVolumeControl::OnMuteClicked()
{
	if (!MutedAttribute.Get())
	{
		// set to MUTED
		if (!MutedAttribute.IsBound())
		{
			MutedAttribute.Set(true);
		}
		OnMutedChanged.ExecuteIfBound(true);		
	}
	else
	{
		// set to UNMUTED
		if (!MutedAttribute.IsBound())
		{
			MutedAttribute.Set(false);
		}
		OnMutedChanged.ExecuteIfBound(false);
	}

	return FReply::Handled();
}

void SVolumeControl::OnWriteValue(float NewValue)
{
	if (0.f < NewValue && MutedAttribute.Get())
	{
		// User moved the slider away from zero in mute mode - cancel mute
		if (!MutedAttribute.IsBound())
		{
			MutedAttribute.Set(false);
		}
		OnMutedChanged.ExecuteIfBound(false);
	}
	if (!VolumeAttribute.IsBound())
	{
		VolumeAttribute.Set(NewValue);
	}
	OnVolumeChanged.ExecuteIfBound(NewValue);
}
