// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/LayerManager/STooltipPresenter.h"
#include "Layout/LayoutUtils.h"
#include "Framework/Application/SlateApplication.h"

void STooltipPresenter::Construct(const FArguments& InArgs)
{
	this->ChildSlot.AttachWidget(InArgs._Content.Widget);
	SetCanTick(false);
}

void STooltipPresenter::SetContent(TSharedPtr<SWidget> InWidget)
{
	if (InWidget.IsValid())
	{
		ChildSlot.AttachWidget(InWidget.ToSharedRef());
	}
	else
	{
		ChildSlot.DetachWidget();
	}
	Invalidate(EInvalidateWidgetReason::Volatility);
}

bool STooltipPresenter::ComputeVolatility() const
{
	return SPanel::ComputeVolatility() || ChildSlot.Num() > 0;
}

void STooltipPresenter::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	static const FVector2D CursorSize = FVector2D(12, 12);

	if (ChildSlot.Num() > 0)
	{
		TSharedRef<SWidget> TooltipWidget = ChildSlot.GetWidget();

		// Cached geometry is in desktop space.  We need to convert from desktop space where the mouse is to local space so use CachedGeometry.
		const FVector2D LocalCursorPosition = GetCachedGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());

		const FSlateRect CursorAnchorRect(LocalCursorPosition, LocalCursorPosition + CursorSize);
		const FSlateRect TooltipPopup(LocalCursorPosition + CursorSize, LocalCursorPosition + CursorSize + TooltipWidget->GetDesiredSize());

		const FVector2D TooltipPosition = ComputePopupFitInRect(CursorAnchorRect, TooltipPopup, EOrientation::Orient_Vertical, FSlateRect(FVector2D::ZeroVector, AllottedGeometry.GetLocalSize()));

		// We round the final tooltip position so that our tooltip doesn't begin at a half pixel offset, which avoids the contents of the tooltip
		// jittering in relation to one another.
		const FVector2D TooltipPositionRounded = AllottedGeometry.LocalToRoundedLocal(TooltipPosition);

		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(
			ChildSlot.GetWidget(),
			ChildSlot.GetWidget()->GetDesiredSize(),
			FSlateLayoutTransform(TooltipPositionRounded)
		));
	}
}

FVector2D STooltipPresenter::ComputeDesiredSize( float ) const
{
	if (ChildSlot.Num() > 0)
	{
		return ChildSlot.GetWidget()->GetDesiredSize();
	}
	
	return FVector2D(0, 0);
}

FChildren* STooltipPresenter::GetChildren()
{
	return &ChildSlot;
}
