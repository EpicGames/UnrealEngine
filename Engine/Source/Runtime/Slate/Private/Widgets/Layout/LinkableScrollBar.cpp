// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/LinkableScrollBar.h"

#include "Widgets/Layout/SScrollBarTrack.h"
#include "Framework/FInvertiblePiecewiseLinearFunction.h"



enum class ELinkId : uint8
{
	LeftLink,
	RightLink,
	Disabled
};

void SLinkableScrollBar::SetState(float InOffsetFraction, float InThumbSizeFraction, bool bCallOnUserScrolled)
{
	const bool bLinkedRight = LinkedScrollBarRight.IsValid() && ScrollSyncRateRight.IsSet();
	const bool bLinkedLeft = LinkedScrollBarLeft.IsValid() && ScrollSyncRateLeft.IsSet();
	
	// fallback to default behavior if this isn't linked to another scrollbar
	if (!bLinkedRight && !bLinkedLeft)
	{
		SScrollBar::SetState(InOffsetFraction, InThumbSizeFraction, bCallOnUserScrolled);
		return;
	}

	const float PrevDistanceFromTop = FMath::Clamp(Track->DistanceFromTop(), 0.f, 1.f);
	const float PrevDistanceFromBottom = FMath::Clamp(Track->DistanceFromBottom(), 0.f, 1.f);
	
	SScrollBar::SetState(InOffsetFraction, InThumbSizeFraction, bCallOnUserScrolled);
	
	const float NewDistanceFromTop = FMath::Clamp(Track->DistanceFromTop(), 0.f, 1.f);
	const float NewDistanceFromBottom = FMath::Clamp(Track->DistanceFromBottom(), 0.f, 1.f);
	
	const bool bDirty = !FMath::IsNearlyEqual(NewDistanceFromTop, PrevDistanceFromTop, UE_KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(NewDistanceFromBottom, PrevDistanceFromBottom, UE_KINDA_SMALL_NUMBER);
	
	// if another scrollbar is linked to this one, Scroll it aswell
	if (bDirty)
	{
		auto UpdateLinkedScrollbar = [&](TSharedPtr<SScrollBar> OtherScrollbar, const FInvertiblePiecewiseLinearFunction ScrollRate, ELinkId LinkId)
		{
			
			float OtherScrollOffset = -1.f;
			switch(LinkId)
			{
			case ELinkId::LeftLink:
				OtherScrollOffset = ScrollRate.SolveForX(InOffsetFraction);
				break;
			case ELinkId::RightLink:
				OtherScrollOffset = ScrollRate.SolveForY(InOffsetFraction);
				break;
		
			case ELinkId::Disabled:;
			default: ;
			}

			if (NewDistanceFromTop > PrevDistanceFromTop) // scrolling down
				{
				if (FMath::IsNearlyZero(OtherScrollbar->DistanceFromBottom(), 0.01f))
				{
					return;
				}
				}
		
			if (OtherScrollOffset <= 1.f && OtherScrollOffset >= 0.f)
			{
				OtherScrollbar->SetState(OtherScrollOffset, OtherScrollbar->ThumbSizeFraction(), true);
			}
		};

		if (bLinkedLeft)
		{
			UpdateLinkedScrollbar(
				LinkedScrollBarLeft.Pin(),
				FInvertiblePiecewiseLinearFunction(ScrollSyncRateLeft.Get()),
				ELinkId::LeftLink
			);
		}

		if (bLinkedRight)
		{
			UpdateLinkedScrollbar(
				LinkedScrollBarRight.Pin(),
				FInvertiblePiecewiseLinearFunction(ScrollSyncRateRight.Get()),
				ELinkId::RightLink
			);
		}
		
	}
}

void SLinkableScrollBar::LinkScrollBars(TSharedRef<SLinkableScrollBar> Left, TSharedRef<SLinkableScrollBar> Right,
	TAttribute<TArray<FVector2f>> ScrollSyncRate)
{
	Left->LinkedScrollBarRight = Right.ToWeakPtr();
	Left->ScrollSyncRateRight = ScrollSyncRate;
	
	Right->LinkedScrollBarLeft = Left.ToWeakPtr();
	Right->ScrollSyncRateLeft = ScrollSyncRate;
}
