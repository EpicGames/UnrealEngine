// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/Layout/SBackgroundBlur.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/DrawElements.h"
#include "Styling/SlateBrush.h"

static int32 bAllowBackgroundBlur = 1;
static FAutoConsoleVariableRef CVarSlateAllowBackgroundBlurWidgets(TEXT("Slate.AllowBackgroundBlurWidgets"), bAllowBackgroundBlur, TEXT("If 0, no background blur widgets will be rendered"));

static int32 MaxKernelSize = 255;
static FAutoConsoleVariableRef CVarSlateMaxKernelSize(TEXT("Slate.BackgroundBlurMaxKernelSize"), MaxKernelSize, TEXT("The maximum allowed kernel size.  Note: Very large numbers can cause a huge decrease in performance"));

static int32 bDownsampleForBlur = 1;
static FAutoConsoleVariableRef CVarDownsampleForBlur(TEXT("Slate.BackgroundBlurDownsample"), bDownsampleForBlur, TEXT(""), ECVF_Cheat);

static int32 bForceLowQualityBrushFallback = 0;
static FAutoConsoleVariableRef CVarForceLowQualityBackgroundBlurOverride(TEXT("Slate.ForceBackgroundBlurLowQualityOverride"), bForceLowQualityBrushFallback, TEXT("Whether or not to force a slate brush to be used instead of actually blurring the background"), ECVF_Scalability);

SLATE_IMPLEMENT_WIDGET(SBackgroundBlur)
void SBackgroundBlur::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "BlurStrength", BlurStrengthAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "BlurRadius", BlurRadiusAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "CornerRadius", CornerRadiusAttribute, EInvalidateWidgetReason::Paint);
}

SBackgroundBlur::SBackgroundBlur()
	: BlurStrengthAttribute(*this, 0.f)
	, BlurRadiusAttribute(*this)
	, CornerRadiusAttribute(*this, FVector4(0.f, 0.f, 0.f, 0.f))
{
}

SBackgroundBlur::~SBackgroundBlur() = default;

void SBackgroundBlur::Construct(const FArguments& InArgs)
{
	bApplyAlphaToBlur = InArgs._bApplyAlphaToBlur;
	LowQualityFallbackBrush = InArgs._LowQualityFallbackBrush;
	SetBlurStrength(InArgs._BlurStrength);
	SetBlurRadius(InArgs._BlurRadius);
	SetCornerRadius(InArgs._CornerRadius);

	ChildSlot
		.HAlign(InArgs._HAlign)
		.VAlign(InArgs._VAlign)
		.Padding(InArgs._Padding)
	[
		InArgs._Content.Widget
	];

	SetCanTick(false);
}

void SBackgroundBlur::SetContent(const TSharedRef<SWidget>& InContent)
{
	ChildSlot.AttachWidget(InContent);
}

void SBackgroundBlur::SetApplyAlphaToBlur(bool bInApplyAlphaToBlur)
{
	if(bApplyAlphaToBlur != bInApplyAlphaToBlur)
	{
		bApplyAlphaToBlur = bInApplyAlphaToBlur;
		Invalidate(EInvalidateWidget::Paint);
	}
}

void SBackgroundBlur::SetBlurRadius(TAttribute<TOptional<int32>> InBlurRadius)
{
	BlurRadiusAttribute.Assign(*this, MoveTemp(InBlurRadius));
}

void SBackgroundBlur::SetBlurStrength(TAttribute<float> InStrength)
{
	BlurStrengthAttribute.Assign(*this, MoveTemp(InStrength));
}

void SBackgroundBlur::SetLowQualityBackgroundBrush(const FSlateBrush* InBrush)
{
	if(LowQualityFallbackBrush != InBrush)
	{
		LowQualityFallbackBrush = InBrush;
		Invalidate(EInvalidateWidget::Paint);
	}
}

void SBackgroundBlur::SetCornerRadius(TAttribute<FVector4> InCornerRadius)
{
	CornerRadiusAttribute.Assign(*this, MoveTemp(InCornerRadius));
}

void SBackgroundBlur::SetHAlign(EHorizontalAlignment HAlign)
{
	ChildSlot.SetHorizontalAlignment(HAlign);
}

void SBackgroundBlur::SetVAlign(EVerticalAlignment VAlign)
{
	ChildSlot.SetVerticalAlignment(VAlign);
}

void SBackgroundBlur::SetPadding(TAttribute<FMargin> InPadding)
{
	ChildSlot.Padding(MoveTemp(InPadding));
}

bool SBackgroundBlur::IsUsingLowQualityFallbackBrush() const
{
	return bForceLowQualityBrushFallback == 1 || !FPlatformMisc::SupportsBackbufferSampling();
}

int32 SBackgroundBlur::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bUsingLowQualityFallbackBrush = IsUsingLowQualityFallbackBrush();

	int32 PostFXLayerId = LayerId;
	if (bAllowBackgroundBlur && AllottedGeometry.GetLocalSize().GetMin() > 0)
	{
		if (!bUsingLowQualityFallbackBrush)
		{
			// Modulate blur strength by the widget alpha
			const float Strength = BlurStrengthAttribute.Get() * (bApplyAlphaToBlur ? InWidgetStyle.GetColorAndOpacityTint().A : 1.f);
			if (Strength > 0.f)
			{
				FSlateRect RenderBoundingRect = AllottedGeometry.GetRenderBoundingRect();
				FPaintGeometry PaintGeometry(RenderBoundingRect.GetTopLeft(), RenderBoundingRect.GetSize(), AllottedGeometry.GetAccumulatedLayoutTransform().GetScale());

				int32 RenderTargetWidth = FMath::RoundToInt(RenderBoundingRect.GetSize().X);
				int32 RenderTargetHeight = FMath::RoundToInt(RenderBoundingRect.GetSize().Y);

				int32 KernelSize = 0;
				int32 DownsampleAmount = 0;
				ComputeEffectiveKernelSize(Strength, KernelSize, DownsampleAmount);

				float ComputedStrength = FMath::Max(.5f, Strength);

				if (DownsampleAmount > 0)
				{
					RenderTargetWidth = FMath::DivideAndRoundUp(RenderTargetWidth, DownsampleAmount);
					RenderTargetHeight = FMath::DivideAndRoundUp(RenderTargetHeight, DownsampleAmount);
					ComputedStrength /= DownsampleAmount;
				}

				if (RenderTargetWidth > 0 && RenderTargetHeight > 0)
				{
					OutDrawElements.PushClip(FSlateClippingZone(AllottedGeometry));

					FSlateDrawElement::MakePostProcessBlur(OutDrawElements, LayerId, PaintGeometry, FVector4f((float)KernelSize, ComputedStrength, (float)RenderTargetWidth, (float)RenderTargetHeight), DownsampleAmount, FVector4f(CornerRadiusAttribute.Get() * AllottedGeometry.Scale));

					OutDrawElements.PopClip();
				}

				++PostFXLayerId;
			}
		}
		else if (bAllowBackgroundBlur && bUsingLowQualityFallbackBrush && LowQualityFallbackBrush && LowQualityFallbackBrush->DrawAs != ESlateBrushDrawType::NoDrawType)
		{
			const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
			const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * LowQualityFallbackBrush->GetTint(InWidgetStyle));

			FSlateDrawElement::MakeBox(OutDrawElements, PostFXLayerId, AllottedGeometry.ToPaintGeometry(), LowQualityFallbackBrush, DrawEffects, FinalColorAndOpacity);
			++PostFXLayerId;
		}
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, PostFXLayerId, InWidgetStyle, bParentEnabled);
}

void SBackgroundBlur::ComputeEffectiveKernelSize(float Strength, int32& OutKernelSize, int32& OutDownsampleAmount) const
{
	// If the radius isn't set, auto-compute it based on the strength
	OutKernelSize = BlurRadiusAttribute.Get().Get(FMath::RoundToInt(Strength * 3.f));

	// Downsample if needed
	if (bDownsampleForBlur && OutKernelSize > 9)
	{
		OutDownsampleAmount = OutKernelSize >= 64 ? OutKernelSize >= 96 ? 4 : 3 : 2;
		OutKernelSize /= OutDownsampleAmount;
	}

	// Kernel sizes must be odd
	if (OutKernelSize % 2 == 0)
	{
		++OutKernelSize;
	}

	OutKernelSize = FMath::Clamp(OutKernelSize, 3, MaxKernelSize);
}
