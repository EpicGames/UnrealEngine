// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SBorder.h"
#include "Rendering/DrawElements.h"

static FName SBorderTypeName("SBorder");

SLATE_IMPLEMENT_WIDGET(SBorder)
void SBorder::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "BorderImage", BorderImageAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "BorderBackgroundColor", BorderBackgroundColorAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "DesiredSizeScale", DesiredSizeScaleAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ShowDisabledEffect", ShowDisabledEffectAttribute, EInvalidateWidgetReason::Paint);
}

SBorder::SBorder()
	: BorderImageAttribute(*this, FCoreStyle::Get().GetBrush( "Border" ))
	, BorderBackgroundColorAttribute(*this, FLinearColor::White)
	, DesiredSizeScaleAttribute(*this, FVector2D(1,1))
	, ShowDisabledEffectAttribute(*this, true)
	, bFlipForRightToLeftFlowDirection(false)
{
}

SBorder::~SBorder() = default;

void SBorder::Construct( const SBorder::FArguments& InArgs )
{
	// Only do this if we're exactly an SBorder
	if ( GetType() == SBorderTypeName )
	{
		SetCanTick(false);
		bCanSupportFocus = false;
	}

	SetContentScale(InArgs._ContentScale);
	SetColorAndOpacity(InArgs._ColorAndOpacity);
	SetDesiredSizeScale(InArgs._DesiredSizeScale);
	SetShowEffectWhenDisabled(InArgs._ShowEffectWhenDisabled);

	bFlipForRightToLeftFlowDirection = InArgs._FlipForRightToLeftFlowDirection;

	SetBorderImage(InArgs._BorderImage);
	SetBorderBackgroundColor(InArgs._BorderBackgroundColor);
	SetForegroundColor(InArgs._ForegroundColor);

	if (InArgs._OnMouseButtonDown.IsBound())
	{
		SetOnMouseButtonDown(InArgs._OnMouseButtonDown);
	}

	if (InArgs._OnMouseButtonUp.IsBound())
	{
		SetOnMouseButtonUp(InArgs._OnMouseButtonUp);
	}

	if (InArgs._OnMouseMove.IsBound())
	{
		SetOnMouseMove(InArgs._OnMouseMove);
	}

	if (InArgs._OnMouseDoubleClick.IsBound())
	{
		SetOnMouseDoubleClick(InArgs._OnMouseDoubleClick);
	}

	ChildSlot
	.HAlign(InArgs._HAlign)
	.VAlign(InArgs._VAlign)
	.Padding(InArgs._Padding)
	[
		InArgs._Content.Widget
	];
}

void SBorder::SetContent( TSharedRef< SWidget > InContent )
{
	ChildSlot
	[
		InContent
	];
}

const TSharedRef< SWidget >& SBorder::GetContent() const
{
	return ChildSlot.GetWidget();
}

void SBorder::ClearContent()
{
	ChildSlot.DetachWidget();
}

int32 SBorder::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const FSlateBrush* BrushResource = BorderImageAttribute.Get();
		
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);

	if ( BrushResource && BrushResource->DrawAs != ESlateBrushDrawType::NoDrawType )
	{
		const bool bShowDisabledEffect = GetShowDisabledEffect();
		const ESlateDrawEffect DrawEffects = (bShowDisabledEffect && !bEnabled) ? ESlateDrawEffect::DisabledEffect : ESlateDrawEffect::None;

		if (bFlipForRightToLeftFlowDirection && GSlateFlowDirection == EFlowDirection::RightToLeft)
		{
			const FGeometry FlippedGeometry = AllottedGeometry.MakeChild(FSlateRenderTransform(FScale2D(-1, 1)));
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				FlippedGeometry.ToPaintGeometry(),
				BrushResource,
				DrawEffects,
				BrushResource->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint() * BorderBackgroundColorAttribute.Get().GetColor(InWidgetStyle)
			);
		}
		else
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				BrushResource,
				DrawEffects,
				BrushResource->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint() * BorderBackgroundColorAttribute.Get().GetColor(InWidgetStyle)
			);
		}
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled );
}

FVector2D SBorder::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return DesiredSizeScaleAttribute.Get() * Super::ComputeDesiredSize(LayoutScaleMultiplier);
}

void SBorder::SetBorderBackgroundColor(TAttribute<FSlateColor> InColorAndOpacity)
{
	BorderBackgroundColorAttribute.Assign(*this, InColorAndOpacity);
}

void SBorder::SetDesiredSizeScale(TAttribute<FVector2D> InDesiredSizeScale)
{
	DesiredSizeScaleAttribute.Assign(*this, InDesiredSizeScale);
}

void SBorder::SetHAlign(EHorizontalAlignment HAlign)
{
	ChildSlot.SetHorizontalAlignment(HAlign);
}

void SBorder::SetVAlign(EVerticalAlignment VAlign)
{
	ChildSlot.SetVerticalAlignment(VAlign);
}

void SBorder::SetPadding(TAttribute<FMargin> InPadding)
{
	ChildSlot.SetPadding(MoveTemp(InPadding));
}

void SBorder::SetShowEffectWhenDisabled(TAttribute<bool> InShowEffectWhenDisabled)
{
	ShowDisabledEffectAttribute.Assign(*this, InShowEffectWhenDisabled);
}

void SBorder::SetBorderImage(TAttribute<const FSlateBrush*> InBorderImage)
{
	BorderImageAttribute.Assign(*this, InBorderImage);
}
