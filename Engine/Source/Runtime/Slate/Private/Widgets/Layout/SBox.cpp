// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SBox.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"

SLATE_IMPLEMENT_WIDGET(SBox)
void SBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SlotPadding", ChildSlot.SlotPaddingAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, WidthOverride, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HeightOverride, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredWidth, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredHeight, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxDesiredWidth, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxDesiredHeight, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinAspectRatio, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxAspectRatio, EInvalidateWidgetReason::Paint);
}

SBox::SBox()
	: ChildSlot(this)
	, WidthOverride(*this)
	, HeightOverride(*this)
	, MinDesiredWidth(*this)
	, MinDesiredHeight(*this)
	, MaxDesiredWidth(*this)
	, MaxDesiredHeight(*this)
	, MinAspectRatio(*this)
	, MaxAspectRatio(*this)
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

SBox::~SBox() = default;

void SBox::Construct( const FArguments& InArgs )
{
	SetWidthOverride(InArgs._WidthOverride);
	SetHeightOverride(InArgs._HeightOverride);

	SetMinDesiredWidth(InArgs._MinDesiredWidth);
	SetMinDesiredHeight(InArgs._MinDesiredHeight);
	SetMaxDesiredWidth(InArgs._MaxDesiredWidth);
	SetMaxDesiredHeight(InArgs._MaxDesiredHeight);

	SetMinAspectRatio(InArgs._MinAspectRatio);
	SetMaxAspectRatio( InArgs._MaxAspectRatio);

	ChildSlot
		.HAlign( InArgs._HAlign )
		.VAlign( InArgs._VAlign )
		.Padding( InArgs._Padding )
		[
			InArgs._Content.Widget
		];
}

void SBox::SetContent(const TSharedRef< SWidget >& InContent)
{
	ChildSlot
	[
		InContent
	];
}

void SBox::SetHAlign(EHorizontalAlignment HAlign)
{
	ChildSlot.SetHorizontalAlignment(HAlign);
}

void SBox::SetVAlign(EVerticalAlignment VAlign)
{
	ChildSlot.SetVerticalAlignment(VAlign);
}

void SBox::SetPadding(TAttribute<FMargin> InPadding)
{
	ChildSlot.SetPadding(InPadding);
}

void SBox::SetWidthOverride(TAttribute<FOptionalSize> InWidthOverride)
{
	WidthOverride.Assign(*this, InWidthOverride);
}

void SBox::SetHeightOverride(TAttribute<FOptionalSize> InHeightOverride)
{
	HeightOverride.Assign(*this, InHeightOverride);
}

void SBox::SetMinDesiredWidth(TAttribute<FOptionalSize> InMinDesiredWidth)
{
	MinDesiredWidth.Assign(*this, InMinDesiredWidth);
}

void SBox::SetMinDesiredHeight(TAttribute<FOptionalSize> InMinDesiredHeight)
{
	MinDesiredHeight.Assign(*this, InMinDesiredHeight);
}

void SBox::SetMaxDesiredWidth(TAttribute<FOptionalSize> InMaxDesiredWidth)
{
	MaxDesiredWidth.Assign(*this, InMaxDesiredWidth);
}

void SBox::SetMaxDesiredHeight(TAttribute<FOptionalSize> InMaxDesiredHeight)
{
	MaxDesiredHeight.Assign(*this, InMaxDesiredHeight);
}

void SBox::SetMinAspectRatio(TAttribute<FOptionalSize> InMinAspectRatio)
{
	MinAspectRatio.Assign(*this, InMinAspectRatio);
}

void SBox::SetMaxAspectRatio(TAttribute<FOptionalSize> InMaxAspectRatio)
{
	MaxAspectRatio.Assign(*this, InMaxAspectRatio);
}

FVector2D SBox::ComputeDesiredSize( float ) const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();

	if ( ChildVisibility != EVisibility::Collapsed )
	{
		const FOptionalSize CurrentWidthOverride = WidthOverride.Get();
		const FOptionalSize CurrentHeightOverride = HeightOverride.Get();

		return FVector2D(
			( CurrentWidthOverride.IsSet() ) ? CurrentWidthOverride.Get() : ComputeDesiredWidth(),
			( CurrentHeightOverride.IsSet() ) ? CurrentHeightOverride.Get() : ComputeDesiredHeight()
		);
	}
	
	return FVector2D::ZeroVector;
}

float SBox::ComputeDesiredWidth() const
{
	// If the user specified a fixed width or height, those values override the Box's content.
	const FVector2D& UnmodifiedChildDesiredSize = ChildSlot.GetWidget()->GetDesiredSize() + ChildSlot.GetPadding().GetDesiredSize();
	const FOptionalSize CurrentMinDesiredWidth = MinDesiredWidth.Get();
	const FOptionalSize CurrentMaxDesiredWidth = MaxDesiredWidth.Get();

	float CurrentWidth = UnmodifiedChildDesiredSize.X;

	if (CurrentMinDesiredWidth.IsSet())
	{
		CurrentWidth = FMath::Max(CurrentWidth, CurrentMinDesiredWidth.Get());
	}

	if (CurrentMaxDesiredWidth.IsSet())
	{
		CurrentWidth = FMath::Min(CurrentWidth, CurrentMaxDesiredWidth.Get());
	}

	return CurrentWidth;
}

float SBox::ComputeDesiredHeight() const
{
	// If the user specified a fixed width or height, those values override the Box's content.
	const FVector2D& UnmodifiedChildDesiredSize = ChildSlot.GetWidget()->GetDesiredSize() + ChildSlot.GetPadding().GetDesiredSize();

	const FOptionalSize CurrentMinDesiredHeight = MinDesiredHeight.Get();
	const FOptionalSize CurrentMaxDesiredHeight = MaxDesiredHeight.Get();

	float CurrentHeight = UnmodifiedChildDesiredSize.Y;

	if (CurrentMinDesiredHeight.IsSet())
	{
		CurrentHeight = FMath::Max(CurrentHeight, CurrentMinDesiredHeight.Get());
	}

	if (CurrentMaxDesiredHeight.IsSet())
	{
		CurrentHeight = FMath::Min(CurrentHeight, CurrentMaxDesiredHeight.Get());
	}

	return CurrentHeight;
}

void SBox::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if ( ArrangedChildren.Accepts(ChildVisibility) )
	{
		const FOptionalSize CurrentMinAspectRatio = MinAspectRatio.Get();
		const FOptionalSize CurrentMaxAspectRatio = MaxAspectRatio.Get();
		const FMargin SlotPadding(ChildSlot.GetPadding());

		AlignmentArrangeResult XAlignmentResult = AlignChild<Orient_Horizontal>(AllottedGeometry.GetLocalSize().X, ChildSlot, SlotPadding);
		AlignmentArrangeResult YAlignmentResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, ChildSlot, SlotPadding);

		if (CurrentMaxAspectRatio.IsSet() || CurrentMinAspectRatio.IsSet())
		{
			const FVector2f CurrentSize = FVector2f(XAlignmentResult.Size, YAlignmentResult.Size);
			const float MinAspectRatioValue = CurrentMinAspectRatio.IsSet() ? CurrentMinAspectRatio.Get() : 0.0f;
			const float MaxAspectRatioValue = CurrentMaxAspectRatio.IsSet() ? CurrentMaxAspectRatio.Get() : 0.0f;
			if (CurrentSize.X > 0.0f && CurrentSize.Y > 0.0f)
			{
				const float CurrentRatio = CurrentSize.X / CurrentSize.Y;
				const bool bFitMaxRatio = (CurrentRatio > MaxAspectRatioValue && MaxAspectRatioValue != 0.0f);
				const bool bFitMinRatio = (CurrentRatio < MinAspectRatioValue && MinAspectRatioValue != 0.0f);
				if (bFitMaxRatio || bFitMinRatio)
				{

					FVector2f MaxSize = FVector2f(AllottedGeometry.Size.X - SlotPadding.GetTotalSpaceAlong<Orient_Horizontal>(), AllottedGeometry.Size.Y - SlotPadding.GetTotalSpaceAlong<Orient_Vertical>());

					const bool bXFillAlignment = ArrangeUtils::GetChildAlignment<Orient_Horizontal>::AsInt(EFlowDirection::LeftToRight, ChildSlot) == HAlign_Fill;
					const bool bYFillAlignment = ArrangeUtils::GetChildAlignment<Orient_Vertical>::AsInt(EFlowDirection::LeftToRight, ChildSlot) == HAlign_Fill;

					FVector2f NewSize = MaxSize;
					auto RatioPredicate = [&](float MinMaxAspectRatioValue)
					{
						const FVector2f AspectRatioValue = FVector2f(MinMaxAspectRatioValue, 1.0f / MinMaxAspectRatioValue);
						const bool bIsRatioXFirst = (bXFillAlignment && bYFillAlignment) || (!bXFillAlignment && !bYFillAlignment) ? MinMaxAspectRatioValue >= 1.0f : bXFillAlignment;
						if (bIsRatioXFirst)
						{
							NewSize.X = MinMaxAspectRatioValue >= 1.0f ? AspectRatioValue.X * CurrentSize.X : CurrentSize.X / AspectRatioValue.X;
							NewSize.Y = AspectRatioValue.Y * NewSize.X;
							MaxSize.X = FMath::Min(AspectRatioValue.X * MaxSize.Y, MaxSize.X);
							MaxSize.Y = AspectRatioValue.Y * MaxSize.X;
						}
						else
						{
							NewSize.Y = MinMaxAspectRatioValue >= 1.0f ? CurrentSize.Y / AspectRatioValue.Y : AspectRatioValue.Y * CurrentSize.Y;
							NewSize.X = AspectRatioValue.X * NewSize.Y;
							MaxSize.Y = FMath::Min(AspectRatioValue.Y * MaxSize.X, MaxSize.Y);
							MaxSize.X = AspectRatioValue.X * MaxSize.Y;
						}
					};
					if (bFitMaxRatio)
					{
						RatioPredicate(MaxAspectRatioValue);
					}
					else
					{
						RatioPredicate(MinAspectRatioValue);
					}

					// Make sure they are inside the max available size
					if (NewSize.X > MaxSize.X)
					{
						float Scale = NewSize.X != 0.0f ? MaxSize.X / NewSize.X : 0.0f;
						NewSize *= Scale;
					}

					if (NewSize.Y > MaxSize.Y)
					{
						float Scale = NewSize.Y != 0.0f ? MaxSize.Y / NewSize.Y : 0.0f;
						NewSize *= Scale;
					}

					// The size changed, realign them. If it's Fill, then center it if needed.
					if (!bXFillAlignment)
					{
						XAlignmentResult = AlignChild<Orient_Horizontal>(AllottedGeometry.GetLocalSize().X, NewSize.X, ChildSlot, SlotPadding);
					}
					else
					{
						XAlignmentResult = ArrangeUtils::AlignCenter<Orient_Horizontal>(AllottedGeometry.GetLocalSize().X, NewSize.X, SlotPadding);
					}
					if (!bYFillAlignment)
					{
						YAlignmentResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, NewSize.Y, ChildSlot, SlotPadding);
					}
					else
					{
						YAlignmentResult = ArrangeUtils::AlignCenter<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, NewSize.Y, SlotPadding);
					}
				}
			}
		}

		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(
				ChildSlot.GetWidget(),
				FVector2D(XAlignmentResult.Offset, YAlignmentResult.Offset),
				FVector2D(XAlignmentResult.Size, YAlignmentResult.Size)
			)
		);
	}
}

FChildren* SBox::GetChildren()
{
	return &ChildSlot;
}

int32 SBox::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// An SBox just draws its only child
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Maybe none of our children are visible
	if( ArrangedChildren.Num() > 0 )
	{
		check( ArrangedChildren.Num() == 1 );
		FArrangedWidget& TheChild = ArrangedChildren[0];

		return TheChild.Widget->Paint( Args.WithNewParent(this), TheChild.Geometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );
	}
	return LayerId;
}