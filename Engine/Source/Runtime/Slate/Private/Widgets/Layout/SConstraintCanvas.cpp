// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SConstraintCanvas.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "SlateSettings.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SConstraintCanvas::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
	if (InArgs._Offset.IsSet())
	{
		OffsetAttr = MoveTemp(InArgs._Offset);
	}
	if (InArgs._Anchors.IsSet())
	{
		AnchorsAttr = MoveTemp(InArgs._Anchors);
	}
	if (InArgs._Alignment.IsSet())
	{
		AlignmentAttr = MoveTemp(InArgs._Alignment);
	}
	if (InArgs._AutoSize.IsSet())
	{
		AutoSizeAttr = MoveTemp(InArgs._AutoSize);
	}
	ZOrder = InArgs._ZOrder.Get(ZOrder);
}

void SConstraintCanvas::FSlot::SetZOrder(float InZOrder)
{
	if (SWidget* OwnerWidget = FSlotBase::GetOwnerWidget())
	{
		if (ZOrder != InZOrder)
		{
			TPanelChildren<FSlot>& OwnerChildren = static_cast<SConstraintCanvas*>(OwnerWidget)->Children;

			int32 ChildIndexToMove = 0;
			{
				bool bFound = false;
				const int32 ChildrenNum = OwnerChildren.Num();
				for (; ChildIndexToMove < ChildrenNum; ++ChildIndexToMove)
				{
					const FSlot& CurSlot = OwnerChildren[ChildIndexToMove];
					if (&CurSlot == this)
					{
						bFound = true;
						break;
					}
				}
				// This slot has to be contained by the children's owner.
				check(OwnerChildren.IsValidIndex(ChildIndexToMove) && bFound);
			}

			int32 ChildIndexDestination = 0;
			{
				const int32 ChildrenNum = OwnerChildren.Num();
				for (; ChildIndexDestination < ChildrenNum; ++ChildIndexDestination)
				{
					const FSlot& CurSlot = OwnerChildren[ChildIndexDestination];
					if (InZOrder < CurSlot.GetZOrder() && &CurSlot != this)
					{
						// Insert before
						break;
					}
				}
				if (ChildIndexDestination >= ChildrenNum)
				{
					const FSlot& CurSlot = OwnerChildren[ChildrenNum - 1];
					if (&CurSlot == this)
					{
						// No need to move, it's already at the end.
						ChildIndexToMove = ChildIndexDestination;
					}
				}
			}

			ZOrder = InZOrder;

			if (ChildIndexToMove != ChildIndexDestination)
			{
				// TPanelChildren::Move does a remove before the insert, that may change the index location
				if (ChildIndexDestination > ChildIndexToMove)
				{
					--ChildIndexDestination;
				}
				OwnerChildren.Move(ChildIndexToMove, ChildIndexDestination);
			}
		}
	}
	else
	{
		ZOrder = InZOrder;
		ensureMsgf(false, TEXT("The order of the Slot could not be set because it's not added to an existing Widget."));
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/* SConstraintCanvas interface
 *****************************************************************************/

SConstraintCanvas::SConstraintCanvas()
	: Children(this)
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

SConstraintCanvas::~SConstraintCanvas() = default;

void SConstraintCanvas::Construct( const SConstraintCanvas::FArguments& InArgs )
{
	// Sort the children based on ZOrder.
	TArray<FSlot::FSlotArguments>& Slots = const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots);
	auto SortOperator = [](const FSlot::FSlotArguments& A, const FSlot::FSlotArguments& B)
	{
		int32 AZOrder = A._ZOrder.Get(0);
		int32 BZOrder = B._ZOrder.Get(0);
		return AZOrder == BZOrder ? reinterpret_cast<UPTRINT>(&A) < reinterpret_cast<UPTRINT>(&B) : AZOrder < BZOrder;
	};
	Slots.StableSort(SortOperator);

	Children.AddSlots(MoveTemp(Slots));
}

SConstraintCanvas::FSlot::FSlotArguments SConstraintCanvas::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

SConstraintCanvas::FScopedWidgetSlotArguments SConstraintCanvas::AddSlot()
{
	TWeakPtr<SConstraintCanvas> WeakWidget = SharedThis(this);
	return FScopedWidgetSlotArguments { MakeUnique<FSlot>(), this->Children, INDEX_NONE,
		[WeakWidget](const FSlot* InNewSlot, int32 InsertedLocation)
		{
			if (TSharedPtr<SConstraintCanvas> Pinned = WeakWidget.Pin())
			{
				int32 NewSlotZOrder = InNewSlot->GetZOrder();
				TPanelChildren<FSlot>& OwnerChildren = Pinned->Children;
				int32 ChildIndexDestination = 0;
				{
					const int32 ChildrenNum = OwnerChildren.Num();
					for (; ChildIndexDestination < ChildrenNum; ++ChildIndexDestination)
					{
						const FSlot& CurSlot = OwnerChildren[ChildIndexDestination];
						if (NewSlotZOrder < CurSlot.GetZOrder() && &CurSlot != InNewSlot)
						{
							// Insert before
							break;
						}
					}
					if (ChildIndexDestination >= ChildrenNum)
					{
						const FSlot& CurSlot = OwnerChildren[ChildrenNum - 1];
						if (&CurSlot == InNewSlot)
						{
							// No need to move, it's already at the end.
							ChildIndexDestination = INDEX_NONE;
						}
					}
				}

				// Move the slot to the correct location
				if (ChildIndexDestination != INDEX_NONE && InsertedLocation != ChildIndexDestination)
				{
					// TPanelChildren::Move does a remove before the insert, that may change the index location
					if (ChildIndexDestination > InsertedLocation)
					{
						--ChildIndexDestination;
						ensure(false); // we inserted at the end, that should not occurs.
					}
					OwnerChildren.Move(InsertedLocation, ChildIndexDestination);
				}
			}
		}};
}

void SConstraintCanvas::ClearChildren()
{
	Children.Empty();
}

int32 SConstraintCanvas::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	Invalidate(EInvalidateWidget::Layout);

	for (int32 SlotIdx = 0; SlotIdx < Children.Num(); ++SlotIdx)
	{
		if (SlotWidget == Children[SlotIdx].GetWidget())
		{
			Children.RemoveAt(SlotIdx);
			return SlotIdx;
		}
	}

	return -1;
}

/* SWidget overrides
 *****************************************************************************/

void SConstraintCanvas::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	FArrangedChildLayers ChildLayers;
	ArrangeLayeredChildren(AllottedGeometry, ArrangedChildren, ChildLayers);
}

void SConstraintCanvas::ArrangeLayeredChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, FArrangedChildLayers& ArrangedChildLayers) const
{
	if (Children.Num() > 0)
	{
		// Using a Project setting here to decide whether we automatically put children in front of all previous children
		// or allow the explicit ZOrder value to place children in the same layer. This will allow users to have non-touching
		// children render into the same layer and have a chance of being batched by the Slate renderer for better performance.
#if WITH_EDITOR
		const bool bExplicitChildZOrder = GetDefault<USlateSettings>()->bExplicitCanvasChildZOrder;
#else
		const static bool bExplicitChildZOrder = GetDefault<USlateSettings>()->bExplicitCanvasChildZOrder;
#endif

		float LastZOrder = -FLT_MAX;

		// Arrange the children now in their proper z-order.
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const SConstraintCanvas::FSlot& CurChild = Children[ChildIndex];
			const TSharedRef<SWidget>& CurWidget = CurChild.GetWidget();

			const EVisibility ChildVisibility = CurWidget->GetVisibility();
			if (ArrangedChildren.Accepts(ChildVisibility))
			{
				const FMargin Offset = CurChild.GetOffset();
				const FVector2D Alignment = CurChild.GetAlignment();
				const FAnchors Anchors = CurChild.GetAnchors();

				const bool AutoSize = CurChild.GetAutoSize();

				const FMargin AnchorPixels =
					FMargin(Anchors.Minimum.X * AllottedGeometry.GetLocalSize().X,
					Anchors.Minimum.Y * AllottedGeometry.GetLocalSize().Y,
					Anchors.Maximum.X * AllottedGeometry.GetLocalSize().X,
					Anchors.Maximum.Y * AllottedGeometry.GetLocalSize().Y);

				const bool bIsHorizontalStretch = Anchors.Minimum.X != Anchors.Maximum.X;
				const bool bIsVerticalStretch = Anchors.Minimum.Y != Anchors.Maximum.Y;

				const FVector2D SlotSize = FVector2D(Offset.Right, Offset.Bottom);

				const FVector2D Size = AutoSize ? CurWidget->GetDesiredSize() : SlotSize;

				// Calculate the offset based on the pivot position.
				FVector2D AlignmentOffset = Size * Alignment;

				// Calculate the local position based on the anchor and position offset.
				FVector2D LocalPosition, LocalSize;

				// Calculate the position and size based on the horizontal stretch or non-stretch
				if (bIsHorizontalStretch)
				{
					LocalPosition.X = AnchorPixels.Left + Offset.Left;
					LocalSize.X = AnchorPixels.Right - LocalPosition.X - Offset.Right;
				}
				else
				{
					LocalPosition.X = AnchorPixels.Left + Offset.Left - AlignmentOffset.X;
					LocalSize.X = Size.X;
				}

				// Calculate the position and size based on the vertical stretch or non-stretch
				if (bIsVerticalStretch)
				{
					LocalPosition.Y = AnchorPixels.Top + Offset.Top;
					LocalSize.Y = AnchorPixels.Bottom - LocalPosition.Y - Offset.Bottom;
				}
				else
				{
					LocalPosition.Y = AnchorPixels.Top + Offset.Top - AlignmentOffset.Y;
					LocalSize.Y = Size.Y;
				}

				// Add the information about this child to the output list (ArrangedChildren)
				ArrangedChildren.AddWidget(ChildVisibility, AllottedGeometry.MakeChild(
					// The child widget being arranged
					CurWidget,
					// Child's local position (i.e. position within parent)
					LocalPosition,
					// Child's size
					LocalSize
				));

				bool bNewLayer = true;
				if (bExplicitChildZOrder)
				{
					// Split children into discrete layers for the paint method
					bNewLayer = false;
					if (CurChild.GetZOrder() > LastZOrder + DELTA)
					{
						if (ArrangedChildLayers.Num() > 0)
						{
							bNewLayer = true;
						}
						LastZOrder = CurChild.GetZOrder();
					}

				}
				ArrangedChildLayers.Add(bNewLayer);
			}
		}
	}
}

int32 SConstraintCanvas::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	SCOPED_NAMED_EVENT_TEXT("SConstraintCanvas", FColor::Orange);

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	FArrangedChildLayers ChildLayers;
	ArrangeLayeredChildren(AllottedGeometry, ArrangedChildren, ChildLayers);

	const bool bForwardedEnabled = ShouldBeEnabled(bParentEnabled);

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;
	int32 ChildLayerId = LayerId;

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];

		if (!IsChildWidgetCulled(MyCullingRect, CurWidget))
		{
			// Bools in ChildLayers tell us whether to paint the next child in front of all previous
			if (ChildLayers[ChildIndex])
			{
				ChildLayerId = MaxLayerId + 1;
			}

			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, MyCullingRect, OutDrawElements, ChildLayerId, InWidgetStyle, bForwardedEnabled);

			MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
		}
		else
		{
			//SlateGI - RemoveContent
		}
	}

	return MaxLayerId;
}

FVector2D SConstraintCanvas::ComputeDesiredSize( float ) const
{
	FVector2D FinalDesiredSize(0,0);

	// Arrange the children now in their proper z-order.
	for ( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		const SConstraintCanvas::FSlot& CurChild = Children[ChildIndex];
		const TSharedRef<SWidget>& Widget = CurChild.GetWidget();
		const EVisibility ChildVisibilty = Widget->GetVisibility();

		// As long as the widgets are not collapsed, they should contribute to the desired size.
		if ( ChildVisibilty != EVisibility::Collapsed )
		{
			const FMargin Offset = CurChild.GetOffset();
			const FVector2D Alignment = CurChild.GetAlignment();
			const FAnchors Anchors = CurChild.GetAnchors();

			const FVector2D SlotSize = FVector2D(Offset.Right, Offset.Bottom);

			const bool AutoSize = CurChild.GetAutoSize();

			const FVector2D Size = AutoSize ? Widget->GetDesiredSize() : SlotSize;

			const bool bIsDockedHorizontally = ( Anchors.Minimum.X == Anchors.Maximum.X ) && ( Anchors.Minimum.X == 0 || Anchors.Minimum.X == 1 );
			const bool bIsDockedVertically = ( Anchors.Minimum.Y == Anchors.Maximum.Y ) && ( Anchors.Minimum.Y == 0 || Anchors.Minimum.Y == 1 );

			FinalDesiredSize.X = FMath::Max(FinalDesiredSize.X, Size.X + ( bIsDockedHorizontally ? FMath::Abs(Offset.Left) : 0.0f ));
			FinalDesiredSize.Y = FMath::Max(FinalDesiredSize.Y, Size.Y + ( bIsDockedVertically ? FMath::Abs(Offset.Top) : 0.0f ));
		}
	}

	return FinalDesiredSize;
}

FChildren* SConstraintCanvas::GetChildren()
{
	return &Children;
}
