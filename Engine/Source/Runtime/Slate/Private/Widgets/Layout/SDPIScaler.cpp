// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SDPIScaler.h"
#include "Layout/ArrangedChildren.h"

SLATE_IMPLEMENT_WIDGET(SDPIScaler)
void SDPIScaler::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SlotPadding", ChildSlot.SlotPaddingAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "DPIScale", DPIScaleAttribute, EInvalidateWidgetReason::Prepass);
}

SDPIScaler::SDPIScaler()
	: ChildSlot(this)
	, DPIScaleAttribute(*this, 1.f)
{
	SetCanTick(false);
	bCanSupportFocus = false;
	bHasRelativeLayoutScale = true;
}

SDPIScaler::~SDPIScaler() = default;

void SDPIScaler::Construct( const FArguments& InArgs )
{
	SetDPIScale(InArgs._DPIScale);

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

void SDPIScaler::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	const EVisibility MyVisibility = this->GetVisibility();
	if ( ArrangedChildren.Accepts( MyVisibility ) )
	{
		const float MyDPIScale = DPIScaleAttribute.Get();

		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild(
			this->ChildSlot.GetWidget(),
			FVector2D::ZeroVector,
			AllottedGeometry.GetLocalSize() / MyDPIScale,
			MyDPIScale
		));
	}
}
	
FVector2D SDPIScaler::ComputeDesiredSize( float ) const
{
	float DPIScaleValue = DPIScaleAttribute.Get();
	if (ensure(DPIScaleValue > 0.f))
	{
		return DPIScaleValue * ChildSlot.GetWidget()->GetDesiredSize();
	}
	return ChildSlot.GetWidget()->GetDesiredSize();
}

FChildren* SDPIScaler::GetChildren()
{
	return &ChildSlot;
}

void SDPIScaler::SetContent(TSharedRef<SWidget> InContent)
{
	ChildSlot
	[
		InContent
	];
}

void SDPIScaler::SetDPIScale(TAttribute<float> InDPIScale)
{
	DPIScaleAttribute.Assign(*this, MoveTemp(InDPIScale), 1.f);
	DPIScaleAttribute.UpdateNow(*this);
}

float SDPIScaler::GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const
{
	return DPIScaleAttribute.Get();
}
