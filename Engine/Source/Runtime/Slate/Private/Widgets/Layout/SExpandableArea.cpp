// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"


#define LOCTEXT_NAMESPACE "SExpandableArea"


void SExpandableArea::Construct( const FArguments& InArgs )
{
	check(InArgs._Style);

	bAreaCollapsed = InArgs._InitiallyCollapsed;
	MinWidth = InArgs._MinWidth;
	MaxHeight = InArgs._MaxHeight;
	OnAreaExpansionChanged = InArgs._OnAreaExpansionChanged;
	SetStyle(InArgs._Style);

	TSharedRef<SWidget> HeaderContent = InArgs._HeaderContent.Widget;
	if( HeaderContent == SNullWidget::NullWidget )
	{
		HeaderContent = 
			SNew(STextBlock)
			.Text(InArgs._AreaTitle)
			.Font(InArgs._AreaTitleFont);
	}

	// If the user wants the body of the expanded section to be different from the title area,
	// then we have to do two separate borders incase the body has any transparency.
	// Furthermore, we still need to fallback to just using one border if they do want them the
	// same, otherwise we could introduce curved edges between the upper and lower sections.
	const bool bBodyDiffers = InArgs._BodyBorderImage != nullptr || InArgs._BodyBorderBackgroundColor.IsSet();
	const TAttribute<const FSlateBrush*> FullBorderImage = bBodyDiffers ? FStyleDefaults::GetNoBrush() : InArgs._BorderImage;
	const TAttribute<FSlateColor> FullBorderBackgroundColor = bBodyDiffers ? FLinearColor::Transparent : InArgs._BorderBackgroundColor;
	const TAttribute<const FSlateBrush*> TitleBorderImage = !bBodyDiffers ? FStyleDefaults::GetNoBrush() : InArgs._BorderImage;
	const TAttribute<FSlateColor> TitleBorderBackgroundColor = !bBodyDiffers ? FLinearColor::Transparent : InArgs._BorderBackgroundColor;

	bAllowAnimatedTransition = InArgs._AllowAnimatedTransition;
	ChildSlot
	[
		SAssignNew(WidgetBorder, SBorder )
		.BorderImage( FullBorderImage )
		.BorderBackgroundColor( FullBorderBackgroundColor )
		.Padding(0.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew( TitleBorder, SBorder )
				.BorderImage(TitleBorderImage)
				.BorderBackgroundColor(TitleBorderBackgroundColor)
				.Padding(0.0f)
				[
					SAssignNew(ExpandingButton, SButton )
					.Cursor(InArgs._HeaderCursor.IsSet() ? InArgs._HeaderCursor : GetCursor())
					.ButtonStyle(FCoreStyle::Get(), "NoBorder")
					.ContentPadding(InArgs._HeaderPadding)
					.ForegroundColor(FSlateColor::UseForeground())
					.OnClicked( this, &SExpandableArea::OnHeaderClicked )
					[
						ConstructHeaderWidget( InArgs, HeaderContent )
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(BodyBorder, SBorder)
				.Visibility(this, &SExpandableArea::OnGetContentVisibility)
				.BorderImage(InArgs._BodyBorderImage)
				.BorderBackgroundColor(InArgs._BodyBorderBackgroundColor)
				.Padding(InArgs._Padding)
				.DesiredSizeScale(this, &SExpandableArea::GetSectionScale)
				[
					InArgs._BodyContent.Widget
				]
			]
		]
	];

	if (bBodyDiffers)
	{
		WidgetBorder = TitleBorder;
	}
}

void SExpandableArea::SetExpanded( bool bExpanded )
{
	// If setting to expanded and we are collapsed, change the state
	if ( bAreaCollapsed == bExpanded )
	{
		bAreaCollapsed = !bExpanded;

		if ( bExpanded )
		{
			RolloutCurve.JumpToEnd();
		}
		else
		{
			RolloutCurve.JumpToStart();
		}

		// Allow some section-specific code to be executed when the section becomes visible or collapsed
		OnAreaExpansionChanged.ExecuteIfBound( bExpanded );
	}
}

void SExpandableArea::SetExpanded_Animated(bool bExpanded)
{
	const bool bShouldBeCollapsed = !bExpanded;
	if (bAreaCollapsed != bShouldBeCollapsed)
	{
		bAreaCollapsed = bShouldBeCollapsed;
		if (!bAreaCollapsed)
		{
			RolloutCurve = FCurveSequence(0.0f, RolloutCurve.GetCurve(0).DurationSeconds, ECurveEaseFunction::CubicOut);
			RolloutCurve.Play(this->AsShared());
		}
		else
		{
			RolloutCurve = FCurveSequence(0.0f, RolloutCurve.GetCurve(0).DurationSeconds, ECurveEaseFunction::CubicIn);
			RolloutCurve.PlayReverse(this->AsShared());
		}

		// Allow some section-specific code to be executed when the section becomes visible or collapsed
		OnAreaExpansionChanged.ExecuteIfBound(!bAreaCollapsed);
	}
}

TSharedRef<SWidget> SExpandableArea::ConstructHeaderWidget(const FArguments& InArgs, TSharedRef<SWidget> HeaderContent)
{
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(InArgs._AreaTitlePadding)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(this, &SExpandableArea::OnGetCollapseImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			HeaderContent
		];
}

EVisibility SExpandableArea::OnGetContentVisibility() const
{
	float Scale = GetSectionScale().Y;
	// The section is visible if its scale in Y is greater than 0
	return Scale > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


FReply SExpandableArea::OnHeaderClicked()
{
	// Only expand/collapse if the mouse was pressed down and up inside the area
	OnToggleContentVisibility();

	return FReply::Handled();
}

void SExpandableArea::OnToggleContentVisibility()
{
	bAllowAnimatedTransition ? SetExpanded_Animated(!!bAreaCollapsed) : SetExpanded(!!bAreaCollapsed);
}

const FSlateBrush* SExpandableArea::OnGetCollapseImage() const
{
	return bAreaCollapsed ? CollapsedImage : ExpandedImage;
}

FVector2D SExpandableArea::GetSectionScale() const
{
	float Scale = RolloutCurve.GetLerp();
	return FVector2D( 1.0f, Scale );
}

FVector2D SExpandableArea::ComputeDesiredSize( float ) const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if ( ChildVisibility != EVisibility::Collapsed )
	{
		FVector2D SlotWidgetDesiredSize = ChildSlot.GetWidget()->GetDesiredSize() + ChildSlot.GetPadding().GetDesiredSize();
		
		// Only clamp if the user specified a min width
		if( MinWidth > 0.0f )
		{
			SlotWidgetDesiredSize.X = FMath::Max( MinWidth, SlotWidgetDesiredSize.X );
		}

		// Only clamp if the user specified a max height
		if( MaxHeight > 0.0f )
		{
			SlotWidgetDesiredSize.Y = FMath::Min( MaxHeight, SlotWidgetDesiredSize.Y );
		}

		return SlotWidgetDesiredSize;
	}

	return FVector2D::ZeroVector;
}

bool SExpandableArea::IsTitleHovered() const
{
	return TitleBorder.IsValid() && TitleBorder->IsHovered();
}


void SExpandableArea::SetMaxHeight(float InMaxHeight)
{
	if (MaxHeight != InMaxHeight)
	{
		MaxHeight = InMaxHeight;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SExpandableArea::SetStyle(const FExpandableAreaStyle* InStyle)
{
	check(InStyle);
	WidgetStyle = InStyle;
	CollapsedImage = &WidgetStyle->CollapsedImage;
	ExpandedImage = &WidgetStyle->ExpandedImage;
	RolloutCurve = FCurveSequence(0.0f, WidgetStyle->RolloutAnimationSeconds, ECurveEaseFunction::CubicOut);

	// If it should be visible, show it now
	if (!bAreaCollapsed)
	{
		RolloutCurve.JumpToEnd();
	}

	Invalidate(EInvalidateWidgetReason::Layout);
}

void SExpandableArea::SetBorderBrush(const FSlateBrush* InBorderBrush)
{
	check(WidgetBorder)
	WidgetBorder->SetBorderImage(InBorderBrush);
}

void SExpandableArea::InvalidateStyle()
{
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SExpandableArea::InvalidateBorderBrush()
{
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SExpandableArea::SetBorderBackgroundColor(const FSlateColor& InBorderColor)
{
	check(WidgetBorder)
	WidgetBorder->SetBorderBackgroundColor(InBorderColor);
}

void SExpandableArea::SetHeaderPadding(FMargin InHeaderPadding)
{
	check(ExpandingButton)
	ExpandingButton->SetContentPadding(InHeaderPadding);
}

void SExpandableArea::SetAreaPadding(FMargin InAreaPadding)
{
	check(BodyBorder)
	BodyBorder->SetPadding(InAreaPadding);
}

#undef LOCTEXT_NAMESPACE
