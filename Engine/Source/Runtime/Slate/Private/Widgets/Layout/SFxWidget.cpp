// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SFxWidget.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"


SFxWidget::SFxWidget()
	: RenderScale(*this, 1.f)
	, RenderScaleOrigin(*this, FVector2D::ZeroVector)
	, LayoutScale(*this, 1.f)
	, VisualOffset(*this, FVector2D::ZeroVector)
	, bIgnoreClipping(*this, true)
{
}

void SFxWidget::Construct( const FArguments& InArgs )
{
	RenderScale.Assign(*this, InArgs._RenderScale);
	RenderScaleOrigin.Assign(*this, InArgs._RenderScaleOrigin);
	LayoutScale.Assign(*this, InArgs._LayoutScale);
	VisualOffset.Assign(*this, InArgs._VisualOffset);
	bIgnoreClipping.Assign(*this, InArgs._IgnoreClipping);
	SetColorAndOpacity(InArgs._ColorAndOpacity);
	
	this->ChildSlot
	.HAlign(InArgs._HAlign)
	.VAlign(InArgs._VAlign)
	[
		InArgs._Content.Widget
	];
}

void SFxWidget::SetVisualOffset( TAttribute<FVector2D> InOffset )
{
	VisualOffset.Assign(*this, MoveTemp(InOffset));
}

void SFxWidget::SetVisualOffset( FVector InOffset )
{
	VisualOffset.Set(*this, FVector2D(InOffset.X, InOffset.Y));
}

void SFxWidget::SetRenderScale( TAttribute<float> InScale )
{
	RenderScale.Assign(*this, InScale);
}

void SFxWidget::SetRenderScale( float InScale )
{
	RenderScale.Set(*this, InScale);
}

/**
 * This widget was created before render transforms existed for each widget, and it chose to apply the render transform AFTER the layout transform.
 * This means leveraging the render transform of FGeometry would be expensive, as we would need to use Concat(LayoutTransform, RenderTransform, Inverse(LayoutTransform).
 * Instead, we maintain the old way of doing it by modifying the AllottedGeometry only during rendering to append the widget's implied RenderTransform to the existing LayoutTransform.
 */
int32 SFxWidget::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// Convert the 0..1 origin into local space extents.
	const FVector2D ScaleOrigin = RenderScaleOrigin.Get() * AllottedGeometry.GetLocalSize();
	const FVector2D Offset = VisualOffset.Get() * AllottedGeometry.GetLocalSize();
	// create the render transform as a scale around ScaleOrigin and offset it by Offset.
	const FSlateLayoutTransform SlateRenderTransform = Concatenate(FSlateLayoutTransform(Inverse(ScaleOrigin)), FSlateLayoutTransform(RenderScale.Get(), ScaleOrigin), FSlateLayoutTransform(Offset));
	// This will append the render transform to the layout transform, and we only use it for rendering.
	FGeometry ModifiedGeometry = AllottedGeometry.MakeChild(AllottedGeometry.GetLocalSize(), SlateRenderTransform);
	
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(ModifiedGeometry, ArrangedChildren);

	// There may be zero elements in this array if our child collapsed/hidden
	if( ArrangedChildren.Num() > 0 )
	{
		const bool bShouldBeEnabled = ShouldBeEnabled(bParentEnabled);

		// We can only have one direct descendant.
		check( ArrangedChildren.Num() == 1 );
		const FArrangedWidget& TheChild = ArrangedChildren[0];

		// SFxWidgets are able to ignore parent clipping.
		const FSlateRect ChildClippingRect = (bIgnoreClipping.Get())
			? ModifiedGeometry.GetLayoutBoundingRect()
			: MyCullingRect.IntersectionWith(ModifiedGeometry.GetLayoutBoundingRect());

		FWidgetStyle CompoundedWidgetStyle = FWidgetStyle(InWidgetStyle)
			.BlendColorAndOpacityTint(GetColorAndOpacity())
			.SetForegroundColor(bShouldBeEnabled ? GetForegroundColor() : GetDisabledForegroundColor());

		return TheChild.Widget->Paint( Args.WithNewParent(this), TheChild.Geometry, ChildClippingRect, OutDrawElements, LayerId + 1, CompoundedWidgetStyle, bShouldBeEnabled );
	}
	return LayerId;

}

FVector2D SFxWidget::ComputeDesiredSize( float ) const
{
	// Layout scale affects out desired size.
	return TransformVector(LayoutScale.Get(), ChildSlot.GetWidget()->GetDesiredSize());
}

void SFxWidget::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	const EVisibility MyVisibility = this->GetVisibility();
	if ( ArrangedChildren.Accepts( MyVisibility ) )
	{
		// Only layout scale affects the arranged geometry.
		const FSlateLayoutTransform LayoutTransform(LayoutScale.Get());

		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild(
			this->ChildSlot.GetWidget(),
			TransformVector(Inverse(LayoutTransform), AllottedGeometry.GetLocalSize()),
			LayoutTransform));
	}
}
