// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SGridPanel.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Layout/LayoutUtils.h"


SLATE_IMPLEMENT_WIDGET(SGridPanel)
void SGridPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Slots);
	FSlot::RegisterAttributes(Initializer);
}

void SGridPanel::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	TBasicLayoutWidgetSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
	if (InArgs._Column.IsSet())
	{
		ColumnParam = FMath::Max(0, InArgs._Column.GetValue());
	}
	if (InArgs._ColumnSpan.IsSet())
	{
		ColumnSpanParam = FMath::Max(1, InArgs._ColumnSpan.GetValue());
	}
	if (InArgs._Row.IsSet())
	{
		RowParam = FMath::Max(0, InArgs._Row.GetValue());
	}
	if (InArgs._RowSpan.IsSet())
	{
		RowSpanParam = FMath::Max(1, InArgs._RowSpan.GetValue());
	}
	if (InArgs._Layer.IsSet())
	{
		LayerParam = InArgs._Layer.GetValue();
	}
	if (InArgs._Nudge.IsSet())
	{
		NudgeParam = InArgs._Nudge.GetValue();
	}
}

SGridPanel::SGridPanel()
	: Slots(this, GET_MEMBER_NAME_CHECKED(SGridPanel, Slots))
{
	SetCanTick(false);
}

SGridPanel::~SGridPanel() = default;

SGridPanel::FSlot::FSlotArguments SGridPanel::Slot(int32 Column, int32 Row, SGridPanel::Layer InLayer)
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>(Column, Row, InLayer.TheLayer));
}

SGridPanel::FScopedWidgetSlotArguments SGridPanel::AddSlot( int32 Column, int32 Row, SGridPanel::Layer InLayer )
{
	TWeakPtr<SGridPanel> WeakPanel = SharedThis(this);

	TUniquePtr<FSlot> NewSlot = MakeUnique<FSlot>(Column, Row, InLayer.TheLayer);
	NewSlot->Panel = WeakPanel;

	int32 InsertLocation = FindInsertSlotLocation(NewSlot.Get());
	return FScopedWidgetSlotArguments{ MoveTemp(NewSlot), this->Slots, InsertLocation,
		[WeakPanel](const FSlot* InNewSlot, int32 InsertedLocation)
		{
			if (TSharedPtr<SGridPanel> Panel = WeakPanel.Pin())
			{
				Panel->NotifySlotChanged(InNewSlot);
			}
		}};
}

bool SGridPanel::RemoveSlot(const TSharedRef<SWidget>& SlotWidget)
{
	return Slots.Remove(SlotWidget) != INDEX_NONE;
}

void SGridPanel::ClearChildren()
{
	Columns.Empty();
	Rows.Empty();
	Slots.Empty();
}

void SGridPanel::Construct( const FArguments& InArgs )
{
	TotalDesiredSizes = FVector2D::ZeroVector;

	// Populate the slots such that they are sorted by Layer (order preserved within layers)
	// Also determine the grid size
	Slots.Reserve(InArgs._Slots.Num());
	for (int32 SlotIndex = 0; SlotIndex < InArgs._Slots.Num(); ++SlotIndex )
	{
		int32 SlotLocation = FindInsertSlotLocation(InArgs._Slots[SlotIndex].GetSlot());
		if (SlotLocation == INDEX_NONE)
		{
			SlotLocation = Slots.AddSlot(MoveTemp(const_cast<FSlot::FSlotArguments&>(InArgs._Slots[SlotIndex])));
		}
		else
		{
			Slots.InsertSlot(MoveTemp(const_cast<FSlot::FSlotArguments&>(InArgs._Slots[SlotIndex])), SlotLocation);
		}

		FSlot& NewSlot = Slots[SlotLocation];
		NewSlot.Panel = SharedThis(this);
		NotifySlotChanged(&NewSlot);
	}

	ColFillCoefficients = InArgs.ColFillCoefficients;
	RowFillCoefficients = InArgs.RowFillCoefficients;
}


int32 SGridPanel::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren(EVisibility::All);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;

	const FPaintArgs NewArgs = Args.WithNewParent(this);
	const bool bShouldBeEnabled = ShouldBeEnabled(bParentEnabled);

	// We need to iterate over slots, because slots know the GridLayers. This isn't available in the arranged children.
	// Some slots do not show up (they are hidden/collapsed). We need a 2nd index to skip over them.
	//
	// GridLayers must ensure that everything in LayerN is below LayerN+1. In other words,
	// every grid layer group must start at the current MaxLayerId (similar to how SOverlay works).
	if(ArrangedChildren.Num())
	{
		int32 LastGridLayer = 0;
		for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num() && ChildIndex < Slots.Num(); ++ChildIndex)
		{
			FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
			if (CurWidget.Widget->GetVisibility().IsVisible())
			{
				const FSlot& CurSlot = Slots[ChildIndex];

				if (!IsChildWidgetCulled(MyCullingRect, CurWidget))
				{
					if (LastGridLayer != CurSlot.GetLayer())
					{
						// We starting a new grid layer group?
						LastGridLayer = CurSlot.GetLayer();
						// Ensure that everything here is drawn on top of 
						// previously drawn grid content.
						LayerId = MaxLayerId + 1;
					}

					const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(
						NewArgs,
						CurWidget.Geometry,
						MyCullingRect,
						OutDrawElements,
						LayerId,
						InWidgetStyle,
						bShouldBeEnabled
					);

					MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
				}
				else
				{
					//SlateGI - RemoveContent
				}
			}
		}
	}

//#define LAYOUT_DEBUG

#ifdef LAYOUT_DEBUG
	LayerId = LayoutDebugPaint( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId );
#endif

	return MaxLayerId;
}

void CalculateStretchedCellSizes(TArray<float>& OutSizes, float AllotedSize, const TArray<float>& InDesiredSizes, const TArray<TAttribute<float>>& Coefficients)
{
	const int32 NumCoefficients = Coefficients.Num();
	float CoefficientTotal = 0.f;

	for(int32 Index=0; Index < InDesiredSizes.Num(); ++Index)
	{
		const float Coefficient = Index < NumCoefficients ? Coefficients[Index].Get(0) : 0;

		// Compute the total space available for stretchy columns.
		if (Coefficient == 0)
		{
			AllotedSize -= InDesiredSizes[Index];
		}
		else
		{
			// Compute the denominator for dividing up the stretchy column space
			CoefficientTotal += Coefficient;
		}
	}

	for(int32 Index=0; Index < InDesiredSizes.Num(); ++Index)
	{
		const float Coefficient = Index < NumCoefficients ? Coefficients[Index].Get(0) : 0;

		// Figure out how big each column needs to be
		OutSizes[Index] = Coefficient != 0
			? (Coefficient / CoefficientTotal * AllotedSize)
			: InDesiredSizes[Index];
	}
}

void SGridPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	// PREPARE PHASE
	// Prepare some data for arranging children.
	// FinalColumns will be populated with column sizes that include the stretched column sizes.
	// Then we will build partial sums so that we can easily handle column spans.
	// Repeat the same for rows.

	float ColumnCoeffTotal = 0.0f;
	TArray<float> FinalColumns;
	if ( Columns.Num() > 0 )
	{
		FinalColumns.AddUninitialized(Columns.Num());
		FinalColumns[FinalColumns.Num()-1] = 0.0f;
	}

	float RowCoeffTotal = 0.0f;
	TArray<float> FinalRows;
	if ( Rows.Num() > 0 )
	{
		FinalRows.AddUninitialized(Rows.Num());
		FinalRows[FinalRows.Num()-1] = 0.0f;
	}
	
	CalculateStretchedCellSizes(FinalColumns, AllottedGeometry.GetLocalSize().X, Columns, ColFillCoefficients);
	CalculateStretchedCellSizes(FinalRows, AllottedGeometry.GetLocalSize().Y, Rows, RowFillCoefficients);
	
	// Build up partial sums for row and column sizes so that we can handle column and row spans conveniently.
	ComputePartialSums(FinalColumns);
	ComputePartialSums(FinalRows);

	// ARRANGE PHASE
	for( int32 SlotIndex=0; SlotIndex < Slots.Num(); ++SlotIndex )
	{
		const FSlot& CurSlot = Slots[SlotIndex];
		const EVisibility ChildVisibility = CurSlot.GetWidget()->GetVisibility();
		if ( ArrangedChildren.Accepts(ChildVisibility) )
		{
			// Figure out the position of this cell.
			const FVector2D ThisCellOffset( FinalColumns[CurSlot.GetColumn()], FinalRows[CurSlot.GetRow()] );

			// Figure out the size of this slot; takes row span into account.
			// We use the properties of partial sums arrays to achieve this.
			const FVector2D CellSize(
				FinalColumns[CurSlot.GetColumn() + CurSlot.GetColumnSpan()] - ThisCellOffset.X,
				FinalRows[CurSlot.GetRow() + CurSlot.GetRowSpan()] - ThisCellOffset.Y);

			// Do the standard arrangement of elements within a slot
			// Takes care of alignment and padding.
			const FMargin SlotPadding(CurSlot.GetPadding());
			AlignmentArrangeResult XAxisResult = AlignChild<Orient_Horizontal>( CellSize.X, CurSlot, SlotPadding );
			AlignmentArrangeResult YAxisResult = AlignChild<Orient_Vertical>( CellSize.Y, CurSlot, SlotPadding );

			// Output the result
			ArrangedChildren.AddWidget( ChildVisibility, AllottedGeometry.MakeChild( 
				CurSlot.GetWidget(),
				ThisCellOffset + FVector2D( XAxisResult.Offset, YAxisResult.Offset ) + CurSlot.GetNudge(),
				FVector2D(XAxisResult.Size, YAxisResult.Size)
			));
		}
	}
}


void SGridPanel::CacheDesiredSize(float LayoutScaleMultiplier)
{
	// The desired size of the grid is the sum of the desires sizes for every row and column.
	ComputeDesiredCellSizes( Columns, Rows );

	TotalDesiredSizes = FVector2D::ZeroVector;
	for (int ColId=0; ColId < Columns.Num(); ++ColId)
	{
		TotalDesiredSizes.X += Columns[ColId];
	}

	for(int RowId=0; RowId < Rows.Num(); ++RowId)
	{
		TotalDesiredSizes.Y += Rows[RowId];
	}
	
	SPanel::CacheDesiredSize(LayoutScaleMultiplier);
}


FVector2D SGridPanel::ComputeDesiredSize( float ) const
{
	return TotalDesiredSizes;
}


FChildren* SGridPanel::GetChildren()
{
	return &Slots;
}


FVector2D SGridPanel::GetDesiredRegionSize( const FIntPoint& StartCell, int32 Width, int32 Height ) const
{
	if (Columns.Num() > 0 && Rows.Num() > 0)
	{
		const int32 FirstColumn = FMath::Clamp(StartCell.X, 0, Columns.Num()-1);
		const int32 LastColumn = FMath::Clamp(StartCell.X + Width, 0, Columns.Num()-1);

		const int32 FirstRow = FMath::Clamp(StartCell.Y, 0, Rows.Num()-1);
		const int32 LastRow = FMath::Clamp(StartCell.Y + Height, 0,  Rows.Num()-1);

		return FVector2D( Columns[LastColumn] - Columns[FirstColumn], Rows[LastRow] - Rows[FirstRow] );
	}
	else
	{
		return FVector2D::ZeroVector;
	}
}

void SGridPanel::SetColumnFill( int32 ColumnId, const TAttribute<float>& Coefficient )
{
	while (ColFillCoefficients.Num() <= ColumnId)
	{
		ColFillCoefficients.Emplace(0);
	}
	ColFillCoefficients[ColumnId] = Coefficient;

	Invalidate(EInvalidateWidgetReason::Layout);
}

void SGridPanel::SetRowFill( int32 RowId, const TAttribute<float>& Coefficient )
{
	while (RowFillCoefficients.Num() <= RowId)
	{
		RowFillCoefficients.Emplace(0);
	}
	RowFillCoefficients[RowId] = Coefficient;

	Invalidate(EInvalidateWidgetReason::Layout);
}

void SGridPanel::ClearFill()
{
	ColFillCoefficients.Reset();
	RowFillCoefficients.Reset();

	Invalidate(EInvalidateWidgetReason::Layout);
}

void SGridPanel::ComputePartialSums( TArray<float>& TurnMeIntoPartialSums )
{
	// We assume there is a 0-valued item already at the end of this array.
	// We need it so that we can  compute the original values
	// by doing Array[N] - Array[N-1];
	

	float LastValue = 0;
	float SumSoFar = 0;
	for(int32 Index=0; Index < TurnMeIntoPartialSums.Num(); ++Index)
	{
		LastValue = TurnMeIntoPartialSums[Index];
		TurnMeIntoPartialSums[Index] = SumSoFar;
		SumSoFar += LastValue;
	}
}


void SGridPanel::DistributeSizeContributions( float SizeContribution, TArray<float>& DistributeOverMe, int32 StartIndex, int32 UpperBound )
{
	for ( int32 Index = StartIndex; Index < UpperBound; ++Index )
	{
		// Each column or row only needs to get bigger if its current size does not already accommodate it.
		DistributeOverMe[Index] = FMath::Max( SizeContribution, DistributeOverMe[Index] );
	}
}


int32 SGridPanel::FindInsertSlotLocation( const FSlot* InSlot )
{
	// Insert the slot in the list such that slots are sorted by LayerOffset.
	for( int32 SlotIndex=0; SlotIndex < Slots.Num(); ++SlotIndex )
	{
		if ( InSlot->GetLayer() < this->Slots[SlotIndex].GetLayer() )
		{
			return SlotIndex;
		}
	}
	return INDEX_NONE;
}


void SGridPanel::NotifySlotChanged(const FSlot* InSlot, bool bSlotLayerChanged /*= false*/)
{
	// Keep the size of the grid up to date.
	// We need an extra cell at the end for easily figuring out the size across any number of cells
	// by doing Columns[End] - Columns[Start] or Rows[End] - Rows[Start].
	// The first Columns[]/Rows[] entry will be 0.

	const int32 NumColumnsRequiredForThisSlot = InSlot->GetColumn() + InSlot->GetColumnSpan() + 1;
	if (NumColumnsRequiredForThisSlot > Columns.Num())
	{
		Columns.AddZeroed(NumColumnsRequiredForThisSlot - Columns.Num());
	}

	const int32 NumRowsRequiredForThisSlot = InSlot->GetRow() + InSlot->GetRowSpan() + 1;
	if (NumRowsRequiredForThisSlot > Rows.Num())
	{
		Rows.AddZeroed(NumRowsRequiredForThisSlot - Rows.Num());
	}

	if (bSlotLayerChanged)
	{
		Slots.Sort([](const FSlot& LHS, const FSlot& RHS)
		{
			return LHS.GetLayer() < RHS.GetLayer();
		});
	}

	Invalidate(EInvalidateWidgetReason::Layout);
}

void SGridPanel::ComputeDesiredCellSizes( TArray<float>& OutColumns, TArray<float>& OutRows ) const
{
	FMemory::Memzero(OutColumns.GetData(), OutColumns.Num() * sizeof(float));
	FMemory::Memzero(OutRows.GetData(), OutRows.Num() * sizeof(float));

	for (int32 SlotIndex = 0; SlotIndex < Slots.Num(); ++SlotIndex)
	{
		const FSlot& CurSlot = Slots[SlotIndex];
		if (CurSlot.GetWidget()->GetVisibility() != EVisibility::Collapsed)
		{
			// The slots wants to be as big as its content along with the required padding.
			const FVector2D SlotDesiredSize = CurSlot.GetWidget()->GetDesiredSize() + CurSlot.GetPadding().GetDesiredSize();

			// If the slot has a (colspan, rowspan) of (1,1) it will only affect that slot.
			// For larger spans, the slot's size will be evenly distributed across all the affected slots.
			const FVector2D SizeContribution(SlotDesiredSize.X / CurSlot.GetColumnSpan(), SlotDesiredSize.Y / CurSlot.GetRowSpan());

			// Distribute the size contributions over all the columns and rows that this slot spans
			DistributeSizeContributions(SizeContribution.X, OutColumns, CurSlot.GetColumn(), CurSlot.GetColumn() + CurSlot.GetColumnSpan());
			DistributeSizeContributions(SizeContribution.Y, OutRows, CurSlot.GetRow(), CurSlot.GetRow() + CurSlot.GetRowSpan());
		}
	}
}


int32 SGridPanel::LayoutDebugPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId ) const
{
	float XOffset = 0;
	for (int32 Column=0; Column<Columns.Num(); ++Column)
	{
		float YOffset = 0;
		for (int32 Row=0; Row<Rows.Num(); ++Row)
		{
			FSlateDrawElement::MakeDebugQuad
			(
				OutDrawElements, 
				LayerId,
				AllottedGeometry.ToPaintGeometry( FVector2f( Columns[Column], Rows[Row] ), FSlateLayoutTransform(FVector2f(XOffset, YOffset)) )
			);

			YOffset += Rows[Row];
		}
		XOffset += Columns[Column];
	}

	return LayerId;
}

