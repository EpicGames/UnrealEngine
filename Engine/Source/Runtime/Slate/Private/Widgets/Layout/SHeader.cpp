// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SHeader.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SSeparator.h"


void SHeader::Construct( const FArguments& InArgs )
{
	SHorizontalBox::FSlot* FirstSlot = nullptr;
	SHorizontalBox::FSlot* LastSlot = nullptr;

	SHorizontalBox::Construct( SHorizontalBox::FArguments()

		+ SHorizontalBox::Slot()
		.Expose( FirstSlot )
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
				.SeparatorImage(FCoreStyle::Get().GetBrush("Header.Pre"))
				.Orientation(Orient_Horizontal)
		]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5, 0)
			[
				InArgs._Content.Widget
			]

		+ SHorizontalBox::Slot()
			.Expose( LastSlot )
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SSeparator)
					.SeparatorImage(FCoreStyle::Get().GetBrush("Header.Post"))
					.Orientation(Orient_Horizontal)
			]	
	);

	// Either the left or right side of the header needs to be auto-sized based on the alignment of the content.
	// @see the HAlign argument
	switch( InArgs._HAlign )
	{
		default:
		case HAlign_Center:
		case HAlign_Fill:
			break;

		case HAlign_Left:
			FirstSlot->SetAutoWidth();
			break;

		case HAlign_Right:
			LastSlot->SetAutoWidth();
			break;
	};
}
