// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SMenuOwner.h"

void SMenuOwner::Construct(const FArguments& InArgs)
{
	this->ChildSlot
		[
			InArgs._Content.Widget
		];
}

void SMenuOwner::SetSummonedMenu( TSharedRef< SMenuAnchor > InMenuAnchor )
{
	SummonedMenuAnchor = InMenuAnchor;
}
	
TSharedPtr< const SMenuAnchor > SMenuOwner::GetOpenMenu() const
{
	if( SummonedMenuAnchor.IsValid() && SummonedMenuAnchor.Pin()->IsOpen() )
	{
		return SummonedMenuAnchor.Pin();
	}

	// No open menus
	return TSharedPtr< SMenuAnchor >();
}

void SMenuOwner::CloseSummonedMenus()
{
	if( GetOpenMenu().IsValid() )
	{
		SummonedMenuAnchor.Pin()->SetIsOpen( false );

		// Menu was closed, so we no longer need a weak reference to it
		SummonedMenuAnchor = nullptr;
	}
}
