// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SMissingWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SMissingWidget"


TSharedRef<class SWidget> SMissingWidget::MakeMissingWidget( )
{
	return SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("MissingWidget", "Missing Widget"))
				.TextStyle(FCoreStyle::Get(), "EmbossedText")
		];
}


#undef LOCTEXT_NAMESPACE
