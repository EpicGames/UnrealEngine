// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SPopup.h"
#include "Rendering/DrawElements.h"
#include "Layout/ArrangedChildren.h"

void SPopup::Construct(const FArguments& InArgs)
{
	this->ChildSlot
	[
		InArgs._Content.Widget
	];
}

int32 SPopup::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// There may be zero elements in this array if our child collapsed/hidden
	if (ArrangedChildren.Num() > 0)
	{
		const bool bShouldBeEnabled = ShouldBeEnabled(bParentEnabled);

		check(ArrangedChildren.Num() == 1);
		FArrangedWidget& TheChild = ArrangedChildren[0];

		FWidgetStyle CompoundedWidgetStyle = FWidgetStyle(InWidgetStyle)
			.BlendColorAndOpacityTint(GetColorAndOpacity())
			.SetForegroundColor(bShouldBeEnabled ? GetForegroundColor() : GetDisabledForegroundColor());

		// An SPopup just queues up its children to be painted after everything in this window is done painting.
		OutDrawElements.QueueDeferredPainting(
			FSlateWindowElementList::FDeferredPaint(TheChild.Widget, Args.WithNewParent(this), TheChild.Geometry, CompoundedWidgetStyle, bShouldBeEnabled)
		);
	}
	return LayerId;
}
