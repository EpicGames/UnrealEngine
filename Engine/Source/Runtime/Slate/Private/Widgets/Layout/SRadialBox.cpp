// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SRadialBox.h"
#include "Layout/LayoutUtils.h"

SRadialBox::SRadialBox()
	: Slots(this)
	, PreferredWidth(*this, 100.f)
{
}

SRadialBox::FSlot::FSlotArguments SRadialBox::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

SRadialBox::FScopedWidgetSlotArguments SRadialBox::AddSlot()
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Slots, INDEX_NONE };
}

int32 SRadialBox::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	return Slots.Remove(SlotWidget);
}

void SRadialBox::Construct( const FArguments& InArgs )
{
	PreferredWidth.Assign(*this, InArgs._PreferredWidth);
	bUseAllottedWidth = InArgs._UseAllottedWidth;
	StartingAngle = InArgs._StartingAngle;
	bDistributeItemsEvenly = InArgs._bDistributeItemsEvenly;
	bClockwiseOrder = InArgs._bClockwiseOrder;
	SectorCentralAngle = InternalNormalizeAngle(InArgs._SectorCentralAngle);
	AngleBetweenItems = InArgs._AngleBetweenItems;
	MarginSize = InArgs._MarginSize;

	Slots.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));
}

void SRadialBox::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bUseAllottedWidth)
	{
		PreferredWidth.Set(*this, AllottedGeometry.GetLocalSize().X);
	}
}

/*
 * Simple class for handling the circular arrangement of elements
 */
class SRadialBox::FChildArranger
{
public:
	struct FArrangementData
	{
		FVector2D SlotOffset;
		FVector2D SlotSize;
	};

	typedef TFunctionRef<void(const FSlot& Slot, const FArrangementData& ArrangementData)> FOnSlotArranged;

	static void Arrange(const SRadialBox& RadialBox, const FOnSlotArranged& OnSlotArranged);

private:
	FChildArranger(const SRadialBox& RadialBox, const FOnSlotArranged& OnSlotArranged);
	void Arrange();

	const SRadialBox& RadialBox;
	const FOnSlotArranged& OnSlotArranged;
	TMap<int32, FArrangementData> OngoingArrangementDataMap;
};

SRadialBox::FChildArranger::FChildArranger(const SRadialBox& InRadialBox, const FOnSlotArranged& InOnSlotArranged)
	: RadialBox(InRadialBox)
	, OnSlotArranged(InOnSlotArranged)
{
	OngoingArrangementDataMap.Reserve(RadialBox.Slots.Num());
}

void SRadialBox::FChildArranger::Arrange()
{
	const int32 NumItems = RadialBox.Slots.Num();
	const float Radius = (RadialBox.PreferredWidth.Get() / 2.f) - RadialBox.MarginSize;

	int32 TargetNumItems = FMath::Max(1, NumItems);
	const bool bIsFullCircumference = RadialBox.SectorCentralAngle >= 360.f;
	const float DegreeIncrements = RadialBox.bDistributeItemsEvenly? RadialBox.SectorCentralAngle / TargetNumItems : RadialBox.AngleBetweenItems;
	const float EqualDistributionOffset = RadialBox.bDistributeItemsEvenly && !bIsFullCircumference ? DegreeIncrements / 2.f : 0.f;
	float DegreeOffset = (RadialBox.StartingAngle + EqualDistributionOffset) * (-1.f);

	//Offset to create the elements based on the middle of the widget as starting point
	const float MiddlePointOffset = RadialBox.PreferredWidth.Get() / 2.f;

	for (int32 ChildIndex = 0; ChildIndex < NumItems; ++ChildIndex)
	{
		const FSlot& Slot = RadialBox.Slots[ChildIndex];
		const TSharedRef<SWidget>& Widget = Slot.GetWidget();

		// Skip collapsed widgets.
		if (Widget->GetVisibility() == EVisibility::Collapsed)
		{
			continue;
		}

		FArrangementData& ArrangementData = OngoingArrangementDataMap.Add(ChildIndex, FArrangementData());

		const FVector2D DesiredSizeOfSlot = Widget->GetDesiredSize();

		float SmallestSide = FMath::Min(DesiredSizeOfSlot.X / 2.f, DesiredSizeOfSlot.Y / 2.f);
		ArrangementData.SlotOffset.X = (Radius - SmallestSide) * FMath::Cos(FMath::DegreesToRadians(DegreeOffset)) + MiddlePointOffset - DesiredSizeOfSlot.X / 2.f;
		ArrangementData.SlotOffset.Y = (Radius - SmallestSide)* FMath::Sin(FMath::DegreesToRadians(DegreeOffset)) + MiddlePointOffset - DesiredSizeOfSlot.Y / 2.f;
		ArrangementData.SlotSize.X = DesiredSizeOfSlot.X;
		ArrangementData.SlotSize.Y = DesiredSizeOfSlot.Y;

		DegreeOffset += RadialBox.bClockwiseOrder ? DegreeIncrements : (-1 * DegreeIncrements);


		OnSlotArranged(Slot, ArrangementData);
	}
}

void SRadialBox::FChildArranger::Arrange(const SRadialBox& RadialBox, const FOnSlotArranged& OnSlotArranged)
{
	FChildArranger(RadialBox, OnSlotArranged).Arrange();
}

void SRadialBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	FChildArranger::Arrange(*this, [&](const FSlot& Slot, const FChildArranger::FArrangementData& ArrangementData)
	{
		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(Slot.GetWidget(), ArrangementData.SlotOffset, ArrangementData.SlotSize));
	});
}

void SRadialBox::ClearChildren()
{
	Slots.Empty();
}

FVector2D SRadialBox::ComputeDesiredSize( float ) const
{
	FVector2D MyDesiredSize = FVector2D::ZeroVector;

	FChildArranger::Arrange(*this, [&](const FSlot& Slot, const FChildArranger::FArrangementData& ArrangementData)
	{
		// Increase desired size to the maximum X and Y positions of any child widget.
		MyDesiredSize.X = FMath::Max(MyDesiredSize.X, ArrangementData.SlotOffset.X + ArrangementData.SlotSize.X);
		MyDesiredSize.Y = FMath::Max(MyDesiredSize.Y, ArrangementData.SlotOffset.Y + ArrangementData.SlotSize.Y);
	});

	return MyDesiredSize;
}

FChildren* SRadialBox::GetChildren()
{
	return &Slots;	
}

void SRadialBox::SetSectorCentralAngle(float InSectorCentralAngle)
{
	SectorCentralAngle = InternalNormalizeAngle(InSectorCentralAngle);
}

void SRadialBox::SetUseAllottedWidth(bool bInUseAllottedWidth)
{
	bUseAllottedWidth = bInUseAllottedWidth;
}

float SRadialBox::InternalNormalizeAngle(float Angle) const
{
	if (FMath::IsNearlyZero(Angle))
	{
		return 0.f;
	}
	float NormalizedAngle = FMath::Fmod(Angle, 360.f);
	return NormalizedAngle < UE_SMALL_NUMBER ? NormalizedAngle + 360.f : NormalizedAngle;
}
