// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/Layout/SResponsiveGridPanel.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"

SLATE_IMPLEMENT_WIDGET(SResponsiveGridPanel)
void SResponsiveGridPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Slots);
	FSlot::RegisterAttributes(Initializer);
}

/**
 *  !!!!!!!!!!!!!!!!!   EXPERIMENTAL  !!!!!!!!!!!!!!!!!   
 * The SResponsiveGridPanel is still in development and the API may change drastically in the future
 * or maybe removed entirely.
 */
SResponsiveGridPanel::SResponsiveGridPanel()
: Slots(this, GET_MEMBER_NAME_CHECKED(SResponsiveGridPanel, Slots))
{}

SResponsiveGridPanel::~SResponsiveGridPanel() = default;

SResponsiveGridPanel::FScopedWidgetSlotArguments SResponsiveGridPanel::AddSlot(int32 Row)
{
	TWeakPtr<SResponsiveGridPanel> WeakPanel = SharedThis(this);

	TUniquePtr<FSlot> NewSlot = MakeUnique<FSlot>(Row);
	NewSlot->Panel = WeakPanel;

	int32 InsertLocation = FindInsertSlotLocation(NewSlot.Get());
	return FScopedWidgetSlotArguments{ MoveTemp(NewSlot), this->Slots, InsertLocation};
}

bool SResponsiveGridPanel::RemoveSlot(const TSharedRef<SWidget>& SlotWidget)
{
	return Slots.Remove(SlotWidget) != INDEX_NONE;
}

void SResponsiveGridPanel::ClearChildren()
{
	Slots.Empty();
}

void SResponsiveGridPanel::Construct(const FArguments& InArgs, int32 InTotalColumns)
{
	TotalColumns = InTotalColumns;
	ColumnGutter = InArgs._ColumnGutter;
	RowGutter = InArgs._RowGutter;

	PreviousWidth = 0;

	RowFillCoefficients = InArgs.RowFillCoefficients;

	// Populate the slots such that they are sorted by Layer (order preserved within layers)

	TWeakPtr<SResponsiveGridPanel> WeakPanel = SharedThis(this);
	Slots.Reserve(InArgs._Slots.Num());
	for (int32 SlotIndex = 0; SlotIndex < InArgs._Slots.Num(); ++SlotIndex)
	{
		int32 SlotLocation = FindInsertSlotLocation(InArgs._Slots[SlotIndex].GetSlot());
		if (SlotLocation == INDEX_NONE)
		{
			SlotLocation = Slots.AddSlot(MoveTemp(const_cast<FSlot::FSlotArguments&>(InArgs._Slots[SlotIndex])));
		}
		else
		{
			Slots.InsertSlot(MoveTemp(const_cast<FSlot::FSlotArguments&>(InArgs._Slots[SlotIndex])), SlotLocation);
		}

		FSlot& NewSlot = Slots[SlotLocation];
		NewSlot.Panel = WeakPanel;
	}
}

int32 SResponsiveGridPanel::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren(EVisibility::All);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	// We need to iterate over slots, because slots know the GridLayers. This isn't available in the arranged children.
	// Some slots do not show up (they are hidden/collapsed). We need a 2nd index to skip over them.
	 
	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		if (CurWidget.Widget->GetVisibility().IsVisible())
		{
			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(
				NewArgs,
				CurWidget.Geometry,
				MyCullingRect,
				OutDrawElements,
				LayerId,
				InWidgetStyle,
				ShouldBeEnabled( bParentEnabled )
			);
			
			MaxLayerId = FMath::Max( MaxLayerId, CurWidgetsMaxLayerId );
		}
	}

#ifdef LAYOUT_DEBUG
	LayerId = LayoutDebugPaint( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId );
#endif

	return MaxLayerId;
}

void SResponsiveGridPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	// Don't attempt to array anything if we don't have any slots allocated.
	if ( Slots.Num() == 0 )
	{
		return;
	}

	// PREPARE PHASE
	// Prepare some data for arranging children.
	// FinalColumns will be populated with column sizes that include the stretched column sizes.
	// Then we will build partial sums so that we can easily handle column spans.

	const FVector2D LocalSize = AllottedGeometry.GetLocalSize();
	FVector2D FlexSpace = LocalSize;

	PreviousWidth = LocalSize.X;
	const float FullColumnGutter = (ColumnGutter * 2);
	const float FullRowGutter = (RowGutter * 2);

	TArray<float> Rows;
	TArray<float> RowToSlot;
	TArray<float> Columns;
	ComputeDesiredCellSizes(LocalSize.X, Columns, Rows, RowToSlot);
	check(Rows.Num() == RowToSlot.Num());

	TArray<float> FinalColumns;
	if (Columns.Num() > 0)
	{
		FinalColumns.AddUninitialized(Columns.Num());
		FinalColumns[FinalColumns.Num() - 1] = 0.0f;

		// We need an extra cell at the end for easily figuring out the size across any number of cells
		FinalColumns.AddZeroed((TotalColumns + 1) - Columns.Num());
	}

	// You can't remove the gutter space from the flexspace for the columns because you don't know how 
	// many columns there actually are in a single row at this point

	float ColumnCoeffTotal = TotalColumns;
	for (int32 ColIndex = 0; ColIndex < Columns.Num(); ++ColIndex)
	{
		// Figure out how big each column needs to be
		FinalColumns[ColIndex] = (1.0f / ColumnCoeffTotal * FlexSpace.X);
	}

	// FinalColumns will be populated with column sizes that include the stretched column sizes.
	// Then we will build partial sums so that we can easily handle column spans.
	float RowCoeffTotal = 0.0f;
	TArray<float> FinalRows;
	if (Rows.Num() > 0)
	{
		FinalRows.AddUninitialized(Rows.Num());
		FinalRows[FinalRows.Num() - 1] = 0.0f;

		// We need an extra cell at the end for easily figuring out the size across any number of cells
		FinalRows.AddZeroed(1);
	}

	FlexSpace.Y -= (RowGutter * 2)  * (Slots[Slots.Num() - 1].RowParam);

	const int32 RowFillCoeffsLength = RowFillCoefficients.Num();
	for (int32 RowIndex = 0; RowIndex < Rows.Num(); ++RowIndex)
	{
		// Compute the total space available for stretchy rows.
		if (RowToSlot[RowIndex] >= RowFillCoeffsLength || RowFillCoefficients[RowToSlot[RowIndex]] == 0)
		{
			FlexSpace.Y -= Rows[RowIndex];
		}
		else //(RowIndex < RowFillCoeffsLength)
		{
			// Compute the denominator for dividing up the stretchy row space
			RowCoeffTotal += RowFillCoefficients[RowToSlot[RowIndex]];
		}
	}

	for (int32 RowIndex = 0; RowIndex < Rows.Num(); ++RowIndex)
	{
		// Compute how big each row needs to be
		FinalRows[RowIndex] = (RowToSlot[RowIndex] < RowFillCoeffsLength && RowFillCoefficients[RowToSlot[RowIndex]] != 0)
			? (RowFillCoefficients[RowToSlot[RowIndex]] / RowCoeffTotal * FlexSpace.Y)
			: Rows[RowIndex];
	}

	// Build up partial sums for row and column sizes so that we can handle column and row spans conveniently.
	ComputePartialSums(FinalColumns);
	ComputePartialSums(FinalRows);

	// ARRANGE PHASE
	int32 ColumnsSoFar = 0;
	int32 CurrentRow = INDEX_NONE;
	int32 LastRowParam = INDEX_NONE;
	float RowGuttersSoFar = 0;
	for (int32 SlotIndex = 0; SlotIndex < Slots.Num(); ++SlotIndex)
	{
		const FSlot& CurSlot = Slots[SlotIndex];

		const EVisibility ChildVisibility = CurSlot.GetWidget()->GetVisibility();
		if (ChildVisibility != EVisibility::Collapsed)
		{
			// Find the appropriate column layout for the slot
			FSlot::FColumnLayout ColumnLayout;
			ColumnLayout.Span = TotalColumns;
			ColumnLayout.Offset = 0;
			for (int32 Index = CurSlot.ColumnLayouts.Num() - 1; Index >= 0; Index--)
			{
				if (CurSlot.ColumnLayouts[Index].LayoutSize < LocalSize.X)
				{
					ColumnLayout = CurSlot.ColumnLayouts[Index];
					break;
				}
			}
		
			if (ColumnLayout.Span == 0)
			{
				continue;
			}

			if (CurSlot.RowParam != LastRowParam)
			{
				ColumnsSoFar = 0;
				LastRowParam = CurSlot.RowParam;
				++CurrentRow;

				if (LastRowParam > 0)
				{
					RowGuttersSoFar += FullRowGutter;
				}
			}

			// Figure out the position of this cell.
			int32 StartColumn = ColumnsSoFar + ColumnLayout.Offset;
			int32 EndColumn = StartColumn + ColumnLayout.Span;
			ColumnsSoFar = FMath::Max(EndColumn, ColumnsSoFar);

			if (ColumnsSoFar > TotalColumns)
			{
				StartColumn = 0;
				EndColumn = StartColumn + ColumnLayout.Span;
				ColumnsSoFar = EndColumn - StartColumn;
				++CurrentRow;
			}

			FVector2D ThisCellOffset(FinalColumns[StartColumn], FinalRows[CurrentRow]);

			// Account for the gutters applied to columns before the starting column of this cell
			if (StartColumn > 0)
			{
				ThisCellOffset.X += FullColumnGutter;
			}

			// Figure out the size of this slot; takes row span into account.
			// We use the properties of partial sums arrays to achieve this.
			FVector2D CellSize(
				FinalColumns[EndColumn] - ThisCellOffset.X,
				FinalRows[CurrentRow + 1] - ThisCellOffset.Y);


			// Do the standard arrangement of elements within a slot
			// Takes care of alignment and padding.
			FMargin SlotPadding(CurSlot.GetPadding());

			AlignmentArrangeResult XAxisResult = AlignChild<Orient_Horizontal>(CellSize.X, CurSlot, SlotPadding);
			AlignmentArrangeResult YAxisResult = AlignChild<Orient_Vertical>(CellSize.Y, CurSlot, SlotPadding);

			// The row gutters have already been accounted for in the cell size by removing them from the flexspace, 
			// so we just need to offset the cells appropriately
			ThisCellOffset.Y += RowGuttersSoFar;

			// Output the result
			ArrangedChildren.AddWidget(ChildVisibility, AllottedGeometry.MakeChild(
				CurSlot.GetWidget(),
				ThisCellOffset + FVector2D(XAxisResult.Offset, YAxisResult.Offset),
				FVector2D(XAxisResult.Size, YAxisResult.Size)
				));
		}
	}
}

void SResponsiveGridPanel::CacheDesiredSize(float LayoutScaleMultiplier)
{
	// The desired size of the grid is the sum of the desires sizes for every row and column.
	TArray<float> Columns;
	TArray<float> Rows;
	TArray<float> RowToSlot;
	ComputeDesiredCellSizes(PreviousWidth, Columns, Rows, RowToSlot);

	TotalDesiredSizes = FVector2D::ZeroVector;

	if ( Slots.Num() > 0 )
	{
		for ( int ColId = 0; ColId < Columns.Num(); ++ColId )
		{
			TotalDesiredSizes.X += Columns[ColId];
		}
		TotalDesiredSizes.X += ( ColumnGutter * 2 ) * ( TotalColumns - 1 );

		for ( int RowId = 0; RowId < Rows.Num(); ++RowId )
		{
			TotalDesiredSizes.Y += Rows[RowId];
		}
		TotalDesiredSizes.Y += ( RowGutter * 2 )  * ( Slots[Slots.Num() - 1].RowParam );
	}

	SPanel::CacheDesiredSize(LayoutScaleMultiplier);
}

FVector2D SResponsiveGridPanel::ComputeDesiredSize( float ) const
{
	return TotalDesiredSizes;
}

void SResponsiveGridPanel::ComputeDesiredCellSizes(float AvailableWidth, TArray<float>& OutColumns, TArray<float>& OutRows, TArray<float>& OutRowToSlot) const
{
	FMemory::Memzero(OutColumns.GetData(), OutColumns.Num() * sizeof(float));
	FMemory::Memzero(OutRows.GetData(), OutRows.Num() * sizeof(float));

	int32 ColumnsSoFar = 0;
	int32 CurrentRow = INDEX_NONE;
	int32 LastRowParam = INDEX_NONE;
	for (int32 SlotIndex = 0; SlotIndex < Slots.Num(); ++SlotIndex)
	{
		const FSlot& CurSlot = Slots[SlotIndex];
		if (CurSlot.GetWidget()->GetVisibility() != EVisibility::Collapsed)
		{
			// Find the appropriate column layout for the slot
			FSlot::FColumnLayout ColumnLayout;
			ColumnLayout.Span = TotalColumns;
			ColumnLayout.Offset = 0;

			for (int32 Index = CurSlot.ColumnLayouts.Num() - 1; Index >= 0; Index--)
			{
				if (CurSlot.ColumnLayouts[Index].LayoutSize < AvailableWidth)
				{
					ColumnLayout = CurSlot.ColumnLayouts[Index];
					break;
				}
			}

			if (ColumnLayout.Span == 0)
			{
				continue;
			}

			if (CurSlot.RowParam != LastRowParam)
			{
				ColumnsSoFar = 0;
				LastRowParam = CurSlot.RowParam;
				++CurrentRow;

				OutRowToSlot.AddZeroed((CurrentRow + 1) - OutRowToSlot.Num());
				OutRowToSlot[CurrentRow] = CurSlot.RowParam;
			}

			// The slots want to be as big as its content along with the required padding.
			const FVector2D SlotDesiredSize = CurSlot.GetWidget()->GetDesiredSize() + CurSlot.GetPadding().GetDesiredSize();

			// If the slot has a (colspan,rowspan) of (1,1) it will only affect that cell.
			// For larger spans, the slots size will be evenly distributed across all the affected cells.
			const FVector2D SizeContribution(SlotDesiredSize.X / ColumnLayout.Span, SlotDesiredSize.Y);

			int32 StartColumnIndex = ColumnsSoFar + ColumnLayout.Offset;
			int32 EndColumnIndex = StartColumnIndex + ColumnLayout.Span;
			ColumnsSoFar = FMath::Max(EndColumnIndex, ColumnsSoFar);

			if (ColumnsSoFar > TotalColumns)
			{
				StartColumnIndex = 0;
				EndColumnIndex = StartColumnIndex + ColumnLayout.Span;
				ColumnsSoFar = EndColumnIndex - StartColumnIndex;
				++CurrentRow;

				OutRowToSlot.AddZeroed((CurrentRow + 1) - OutRowToSlot.Num());
				OutRowToSlot[CurrentRow] = CurSlot.RowParam;
			}

			OutColumns.AddZeroed(FMath::Max(0, ColumnsSoFar - OutColumns.Num()));
			OutRows.AddZeroed((CurrentRow + 1) - OutRows.Num());

			// Distribute the size contributions over all the columns and rows that this slot spans
			DistributeSizeContributions(SizeContribution.X, OutColumns, StartColumnIndex, EndColumnIndex);
			DistributeSizeContributions(SizeContribution.Y, OutRows, CurrentRow, CurrentRow + 1);
		}
	}
}

void SResponsiveGridPanel::DistributeSizeContributions(float SizeContribution, TArray<float>& DistributeOverMe, int32 StartIndex, int32 UpperBound)
{
	for (int32 Index = StartIndex; Index < UpperBound; ++Index)
	{
		// Each column or row only needs to get bigger if its current size does not already accommodate it.
		DistributeOverMe[Index] = FMath::Max(SizeContribution, DistributeOverMe[Index]);
	}
}

FChildren* SResponsiveGridPanel::GetChildren()
{
	return &Slots;
}

void SResponsiveGridPanel::ComputePartialSums( TArray<float>& TurnMeIntoPartialSums )
{
	// We assume there is a 0-valued item already at the end of this array.
	// We need it so that we can  compute the original values
	// by doing Array[N] - Array[N-1];
	
	float LastValue = 0;
	float SumSoFar = 0;
	for (int32 Index = 0; Index < TurnMeIntoPartialSums.Num(); ++Index)
	{
		LastValue = TurnMeIntoPartialSums[Index];
		TurnMeIntoPartialSums[Index] = SumSoFar;
		SumSoFar += LastValue;
	}
}

void SResponsiveGridPanel::SetRowFill(int32 RowId, float Coefficient)
{
	if (RowFillCoefficients.Num() <= RowId)
	{
		RowFillCoefficients.AddZeroed(RowId - RowFillCoefficients.Num() + 1);
	}

	RowFillCoefficients[RowId] = Coefficient;
}

int32 SResponsiveGridPanel::FindInsertSlotLocation(SResponsiveGridPanel::FSlot* InSlot) const
{
	// Insert the slot in the list such that slots are sorted by LayerOffset.
	for (int32 SlotIndex = 0; SlotIndex < Slots.Num(); ++SlotIndex)
	{
		if (InSlot->RowParam < this->Slots[SlotIndex].RowParam)
		{
			return SlotIndex;
		}
	}

	return INDEX_NONE;
}

int32 SResponsiveGridPanel::LayoutDebugPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId ) const
{
	//float XOffset = 0;
	//for (int32 Column=0; Column<Columns.Num(); ++Column)
	//{
	//	float YOffset = 0;
	//	for (int32 Row=0; Row<Rows.Num(); ++Row)
	//	{
	//		FSlateDrawElement::MakeDebugQuad
	//		(
	//			OutDrawElements, 
	//			LayerId,
	//			AllottedGeometry.ToPaintGeometry( FVector2D(XOffset, YOffset), FVector2D( Columns[Column], Rows[Row] ) ),
	//			MyCullingRect
	//		);

	//		YOffset += Rows[Row];
	//	}
	//	XOffset += Columns[Column];
	//}

	return LayerId;
}

