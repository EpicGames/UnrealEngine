// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SSafeZone.h"
#include "Layout/LayoutUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/CoreDelegates.h"
#include "Widgets/SViewport.h"

float GSafeZoneScale = 1.0f;
static FAutoConsoleVariableRef CVarSafeZoneScale(
	TEXT("SafeZone.Scale"),
	GSafeZoneScale,
	TEXT("The safezone scale."),
	ECVF_Default
);

bool bEnableSafeZoneScale = false;
static FAutoConsoleVariableRef CVarEnableSafeZoneScale(
	TEXT("SafeZone.EnableScale"),
	bEnableSafeZoneScale,
	TEXT("IS the safe zone scale enabled?"),
	ECVF_Default
);

SSafeZone::SSafeZone()
	: Padding(*this, 0.f)
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

void SSafeZone::SetGlobalSafeZoneScale(TOptional<float> InScale)
{
	float NewScale = InScale.Get(1.f);
	if (NewScale < 0)
	{
		NewScale = 1.f;
	}
	bEnableSafeZoneScale = InScale.IsSet();
	GSafeZoneScale = NewScale;

	FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();
}

TOptional<float> SSafeZone::GetGlobalSafeZoneScale()
{
	if (bEnableSafeZoneScale)
	{
		return GSafeZoneScale;
	}
	return TOptional<float>();
}

void SSafeZone::Construct( const FArguments& InArgs )
{
	SBox::Construct(SBox::FArguments()
		.HAlign(InArgs._HAlign)
		.VAlign(InArgs._VAlign)
		[
			InArgs._Content.Widget
		]
	);

	Padding.Assign(*this, InArgs._Padding);
	SafeAreaScale = InArgs._SafeAreaScale;
	bIsTitleSafe = InArgs._IsTitleSafe;
	bPadLeft = InArgs._PadLeft;
	bPadRight = InArgs._PadRight;
	bPadTop = InArgs._PadTop;
	bPadBottom = InArgs._PadBottom;
	bSafeMarginNeedsUpdate = true;

#if WITH_EDITOR
	OverrideScreenSize = InArgs._OverrideScreenSize;
	OverrideDpiScale = InArgs._OverrideDpiScale;
	FSlateApplication::Get().OnDebugSafeZoneChanged.AddSP(this, &SSafeZone::DebugSafeAreaUpdated);
#endif

	SetTitleSafe(bIsTitleSafe);

	OnSafeFrameChangedHandle = FCoreDelegates::OnSafeFrameChangedEvent.AddSP(this, &SSafeZone::UpdateSafeMargin);
}

SSafeZone::~SSafeZone()
{
	FCoreDelegates::OnSafeFrameChangedEvent.Remove(OnSafeFrameChangedHandle);
}

void SSafeZone::SetTitleSafe( bool InIsTitleSafe )
{
	UpdateSafeMargin();
}

void SSafeZone::UpdateSafeMargin()
{
	bSafeMarginNeedsUpdate = true;

#if WITH_EDITOR
	if (OverrideScreenSize.IsSet() && !OverrideScreenSize.GetValue().IsZero())
	{
		FSlateApplication::Get().GetSafeZoneSize(SafeMargin, OverrideScreenSize.GetValue());
	}
	else
	{
		// Need to get owning viewport not display 
		// use pixel values (same as custom safe zone above)
		TSharedPtr<SViewport> GameViewport = FSlateApplication::Get().GetGameViewport();
		if (GameViewport.IsValid())
		{
			TSharedPtr<ISlateViewport> ViewportInterface = GameViewport->GetViewportInterface().Pin();
			if (ViewportInterface.IsValid())
			{
				const FIntPoint ViewportSize = ViewportInterface->GetSize();
				FSlateApplication::Get().GetSafeZoneSize(SafeMargin, ViewportSize);
			}
			else
			{
				return;
			}
		}
		else
		{
			return;
		}
	}
#else
	// The second parameter of GetSafeZoneSize is only used #if WITH_EDITOR
	FSlateApplication::Get().GetSafeZoneSize(SafeMargin, FDeprecateSlateVector2D());
#endif

	TOptional<float> SafeZoneScale = GetGlobalSafeZoneScale();
	if (SafeZoneScale.IsSet())
	{
		SafeMargin = SafeMargin * GSafeZoneScale;
	}

	SafeMargin = FMargin(bPadLeft ? SafeMargin.Left : 0.0f, bPadTop ? SafeMargin.Top : 0.0f, bPadRight ? SafeMargin.Right : 0.0f, bPadBottom ? SafeMargin.Bottom : 0.0f);

	Invalidate(EInvalidateWidgetReason::Layout);
	bSafeMarginNeedsUpdate = false;
}

void SSafeZone::SetSidesToPad(bool InPadLeft, bool InPadRight, bool InPadTop, bool InPadBottom)
{
	bPadLeft = InPadLeft;
	bPadRight = InPadRight;
	bPadTop = InPadTop;
	bPadBottom = InPadBottom;

	SetTitleSafe(bIsTitleSafe);
}

#if WITH_EDITOR

void SSafeZone::SetOverrideScreenInformation(TOptional<FVector2D> InScreenSize, TOptional<float> InOverrideDpiScale)
{
	OverrideScreenSize = InScreenSize;
	OverrideDpiScale = InOverrideDpiScale;
	SetTitleSafe(bIsTitleSafe);
}

void SSafeZone::DebugSafeAreaUpdated(const FMargin& NewSafeZone, bool bShouldRecacheMetrics)
{
	UpdateSafeMargin();
}

#endif

FMargin SSafeZone::GetSafeMargin(float InLayoutScale) const
{
	if (bSafeMarginNeedsUpdate)
	{
		const_cast<SSafeZone*>(this)->UpdateSafeMargin();
	}

	const FMargin SlotPadding = Padding.Get() + (ComputeScaledSafeMargin(InLayoutScale) * SafeAreaScale);
	return SlotPadding;
}

void SSafeZone::SetSafeAreaScale(FMargin InSafeAreaScale)
{
	SafeAreaScale = InSafeAreaScale;
}

FMargin SSafeZone::ComputeScaledSafeMargin(float Scale) const
{
#if WITH_EDITOR
	const float InvScale = OverrideDpiScale.IsSet() ? 1.0f / OverrideDpiScale.GetValue() : 1.0f / Scale;
#else
	const float InvScale = 1.0f / Scale;
#endif

	const FMargin ScaledSafeMargin(
		FMath::RoundToFloat(SafeMargin.Left * InvScale),
		FMath::RoundToFloat(SafeMargin.Top * InvScale),
		FMath::RoundToFloat(SafeMargin.Right * InvScale),
		FMath::RoundToFloat(SafeMargin.Bottom * InvScale));
	return ScaledSafeMargin;
}

void SSafeZone::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	const EVisibility& MyCurrentVisibility = this->GetVisibility();
	if ( ArrangedChildren.Accepts( MyCurrentVisibility ) )
	{
		const FMargin SlotPadding               = GetSafeMargin(AllottedGeometry.Scale);
		AlignmentArrangeResult XAlignmentResult = AlignChild<Orient_Horizontal>( AllottedGeometry.GetLocalSize().X, ChildSlot, SlotPadding );
		AlignmentArrangeResult YAlignmentResult = AlignChild<Orient_Vertical>( AllottedGeometry.GetLocalSize().Y, ChildSlot, SlotPadding );

		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(
			ChildSlot.GetWidget(),
			FVector2D( XAlignmentResult.Offset, YAlignmentResult.Offset ),
			FVector2D( XAlignmentResult.Size, YAlignmentResult.Size )
			)
		);
	}
}

FVector2D SSafeZone::ComputeDesiredSize(float LayoutScale) const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();

	if ( ChildVisibility != EVisibility::Collapsed )
	{
		const FMargin SlotPadding = GetSafeMargin(LayoutScale);
		FVector2D BaseDesiredSize = SBox::ComputeDesiredSize(LayoutScale);

		return BaseDesiredSize + SlotPadding.GetDesiredSize();
	}

	return FVector2D(0, 0);
}
