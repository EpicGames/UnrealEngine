// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SScaleBox.h"
#include "Layout/LayoutUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"
#include "Misc/CoreDelegates.h"
#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SScaleBox)


/* SScaleBox interface
 *****************************************************************************/
SLATE_IMPLEMENT_WIDGET(SScaleBox)
void SScaleBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "StretchDirection", StretchDirectionAttribute, EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Prepass);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "UserSpecifiedScale", UserSpecifiedScaleAttribute, EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Prepass);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "IgnoreInheritedScale", IgnoreInheritedScaleAttribute, EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Prepass);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Stretch", StretchAttribute, EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Prepass)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				static_cast<SScaleBox&>(Widget).RefreshSafeZoneScale();
			}));
}

SScaleBox::SScaleBox()
	: StretchDirectionAttribute(*this, EStretchDirection::Both)
	, StretchAttribute(*this, EStretch::None)
	, UserSpecifiedScaleAttribute(*this, 1.0f)
	, IgnoreInheritedScaleAttribute(*this, false)
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

void SScaleBox::Construct(const SScaleBox::FArguments& InArgs)
{
	bHasCustomPrepass = true;
	bHasRelativeLayoutScale = true;

	StretchAttribute.Assign(*this, InArgs._Stretch);
	StretchDirectionAttribute.Assign(*this, InArgs._StretchDirection);
	UserSpecifiedScaleAttribute.Assign(*this, InArgs._UserSpecifiedScale, 1.0f);
	IgnoreInheritedScaleAttribute.Assign(*this, InArgs._IgnoreInheritedScale, false);

	LastFinalOffset = FVector2D(0, 0);

	ChildSlot
	.HAlign(InArgs._HAlign)
	.VAlign(InArgs._VAlign)
	[
		InArgs._Content.Widget
	];

#if WITH_EDITOR
	OverrideScreenSize = InArgs._OverrideScreenSize;
	FSlateApplication::Get().OnDebugSafeZoneChanged.AddSP(this, &SScaleBox::DebugSafeAreaUpdated);
#endif

	RefreshSafeZoneScale();
	OnSafeFrameChangedHandle = FCoreDelegates::OnSafeFrameChangedEvent.AddSP(this, &SScaleBox::HandleSafeFrameChangedEvent);
}

SScaleBox::~SScaleBox()
{
	FCoreDelegates::OnSafeFrameChangedEvent.Remove(OnSafeFrameChangedHandle);
}

bool SScaleBox::CustomPrepass(float LayoutScaleMultiplier)
{
	SWidget& ChildSlotWidget = ChildSlot.GetWidget().Get();

	// If the child is collapsed don't bother calculating any scale for it.
	const EVisibility ChildVisibility = ChildSlotWidget.GetVisibility();
	if (ChildVisibility == EVisibility::Collapsed)
	{
		return false;
	}

	const bool bNeedsNormalizingPrepassOrLocalGeometry = DoesScaleRequireNormalizingPrepassOrLocalGeometry();

	// If we need a normalizing prepass, or we've yet to give the child a chance to generate a desired
	// size, do that now.
	if (bNeedsNormalizingPrepassOrLocalGeometry || !LastAllocatedArea.IsSet())
	{
		ChildSlotWidget.MarkPrepassAsDirty();
		ChildSlotWidget.SlatePrepass(LayoutScaleMultiplier);

		NormalizedContentDesiredSize = ChildSlotWidget.GetDesiredSize();
	}
	else
	{
		NormalizedContentDesiredSize.Reset();
	}

	TOptional<float> NewComputedContentScale;

	if (bNeedsNormalizingPrepassOrLocalGeometry)
	{
		if (LastAllocatedArea.IsSet())
		{
			NewComputedContentScale = ComputeContentScale(LastPaintGeometry.GetValue());
		}
	}
	else
	{
		// If we don't need the area, send a false geometry.
		static const FGeometry NullGeometry;
		NewComputedContentScale = ComputeContentScale(NullGeometry);
	}

	if (bNeedsNormalizingPrepassOrLocalGeometry)
	{
		ChildSlotWidget.Invalidate(EInvalidateWidgetReason::Prepass);
	}

	// Extract the incoming scale out of the layout scale if 
	if (NewComputedContentScale.IsSet())
	{
		if (IgnoreInheritedScaleAttribute.Get() && LayoutScaleMultiplier != 0)
		{
			NewComputedContentScale = NewComputedContentScale.GetValue() / LayoutScaleMultiplier;
		}
	}

	ComputedContentScale = NewComputedContentScale;

	return true;
}

bool SScaleBox::DoesScaleRequireNormalizingPrepassOrLocalGeometry() const
{
	const EStretch::Type CurrentStretch = StretchAttribute.Get();
	switch (CurrentStretch)
	{
	case EStretch::None:
	case EStretch::Fill:
	case EStretch::ScaleBySafeZone:
	case EStretch::UserSpecified:
	case EStretch::UserSpecifiedWithClipping:
		return false;
	default:
		return true;
	}
}

bool SScaleBox::IsDesiredSizeDependentOnAreaAndScale() const
{
	const EStretch::Type CurrentStretch = StretchAttribute.Get();
	switch (CurrentStretch)
	{
	case EStretch::ScaleToFitX:
	case EStretch::ScaleToFitY:
		return true;
	default:
		return false;
	}
}

float SScaleBox::ComputeContentScale(const FGeometry& PaintGeometry) const
{
	const EStretch::Type CurrentStretch = StretchAttribute.Get();
	const EStretchDirection::Type CurrentStretchDirection = StretchDirectionAttribute.Get();

	switch (CurrentStretch)
	{
	case EStretch::ScaleBySafeZone:
		return SafeZoneScale;
	case EStretch::UserSpecified:
	case EStretch::UserSpecifiedWithClipping:
		return UserSpecifiedScaleAttribute.Get();
	}

	float FinalScale = 1;

	const FVector2D ChildDesiredSize = ChildSlot.GetWidget()->GetDesiredSize();

	if (ChildDesiredSize.X != 0 && ChildDesiredSize.Y != 0)
	{
		switch (CurrentStretch)
		{
			case EStretch::ScaleToFit:
			{
				FinalScale = FMath::Min(PaintGeometry.GetLocalSize().X / ChildDesiredSize.X, PaintGeometry.GetLocalSize().Y / ChildDesiredSize.Y);
				break;
			}
			case EStretch::ScaleToFitX:
			{
				FinalScale = PaintGeometry.GetLocalSize().X / ChildDesiredSize.X;
				break;
			}
			case EStretch::ScaleToFitY:
			{
				FinalScale = PaintGeometry.GetLocalSize().Y / ChildDesiredSize.Y;
				break;
			}
			case EStretch::Fill:
			{
				break;
			}
			case EStretch::ScaleToFill:
			{
				FinalScale = FMath::Max(PaintGeometry.GetLocalSize().X / ChildDesiredSize.X, PaintGeometry.GetLocalSize().Y / ChildDesiredSize.Y);
				break;
			}
		}

		switch (CurrentStretchDirection)
		{
		case EStretchDirection::DownOnly:
			FinalScale = FMath::Min(FinalScale, 1.0f);
			break;
		case EStretchDirection::UpOnly:
			FinalScale = FMath::Max(FinalScale, 1.0f);
			break;
		case EStretchDirection::Both:
			break;
		}
	}

	return FinalScale;
}

void SScaleBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if (ArrangedChildren.Accepts(ChildVisibility))
	{
		const FVector2D AreaSize = AllottedGeometry.GetLocalSize();

		const FVector2D CurrentWidgetDesiredSize = ChildSlot.GetWidget()->GetDesiredSize();
		FVector2D SlotWidgetDesiredSize = CurrentWidgetDesiredSize;

		const EStretch::Type CurrentStretch = StretchAttribute.Get();
		const EStretchDirection::Type CurrentStretchDirection = StretchDirectionAttribute.Get();

		if (CurrentStretch == EStretch::Fill)
		{
			SlotWidgetDesiredSize = AreaSize;
		}

		// This scale may not look right, the item being 
		// shown may need 2 frames to truly be drawn correctly,
		// but rather than have a blank frame, it's better for us to try
		// and fit the contents to our known geometry.
		const float TempComputedContentScale = ComputedContentScale.IsSet() ? ComputedContentScale.GetValue() : ComputeContentScale(AllottedGeometry);

		LastFinalOffset = FVector2D(0, 0);
		float FinalScale = TempComputedContentScale;

		if (FMath::IsNearlyZero(FinalScale))
		{
			return;
		}

		// If we're just filling, there's no scale applied, we're just filling the area.
		if (CurrentStretch != EStretch::Fill)
		{
			const FMargin SlotPadding(ChildSlot.GetPadding());
			AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(AreaSize.X, ChildSlot, SlotPadding, FinalScale, false);
			AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AreaSize.Y, ChildSlot, SlotPadding, FinalScale, false);

			LastFinalOffset = FVector2D(XResult.Offset, YResult.Offset) / FinalScale;

			// If the layout horizontally is fill, then we need the desired size to be the whole size of the widget, 
			// but scale the inverse of the scale we're applying.
			if (ChildSlot.GetHorizontalAlignment() == HAlign_Fill)
			{
				SlotWidgetDesiredSize.X = AreaSize.X / FinalScale;
			}

			// If the layout vertically is fill, then we need the desired size to be the whole size of the widget, 
			// but scale the inverse of the scale we're applying.
			if (ChildSlot.GetVerticalAlignment() == VAlign_Fill)
			{
				SlotWidgetDesiredSize.Y = AreaSize.Y / FinalScale;
			}
		}

		ArrangedChildren.AddWidget(ChildVisibility, AllottedGeometry.MakeChild(
			ChildSlot.GetWidget(),
			LastFinalOffset,
			SlotWidgetDesiredSize,
			FinalScale
		));
	}
}

int32 SScaleBox::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// We need another layout pass if the incoming allocated geometry is different from last frames.
	if (!LastAllocatedArea.IsSet() || !AllottedGeometry.GetLocalSize().Equals(LastAllocatedArea.GetValue()))
	{
		LastAllocatedArea = AllottedGeometry.GetLocalSize();
		LastPaintGeometry = AllottedGeometry;

		if (DoesScaleRequireNormalizingPrepassOrLocalGeometry())
		{
			const_cast<SScaleBox*>(this)->Invalidate(EInvalidateWidgetReason::Prepass);
		}
	}

	bool bClippingNeeded = false;

	if (GetClipping() == EWidgetClipping::Inherit)
	{
		const EStretch::Type CurrentStretch = StretchAttribute.Get();

		// There are a few stretch modes that require we clip, even if the user didn't set the property.
		switch (CurrentStretch)
		{
		case EStretch::ScaleToFitX:
		case EStretch::ScaleToFitY:
		case EStretch::ScaleToFill:
		case EStretch::UserSpecifiedWithClipping:
			bClippingNeeded = true;
			break;
		}
	}

	if (bClippingNeeded)
	{
		OutDrawElements.PushClip(FSlateClippingZone(AllottedGeometry));
		FGeometry HitTestGeometry = AllottedGeometry;
		HitTestGeometry.AppendTransform(FSlateLayoutTransform(Args.GetWindowToDesktopTransform()));
	}

	int32 MaxLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	if (bClippingNeeded)
	{
		OutDrawElements.PopClip();
	}

	return MaxLayerId;
}

void SScaleBox::SetContent(TSharedRef<SWidget> InContent)
{
	ChildSlot
	[
		InContent
	];
}

void SScaleBox::SetHAlign(EHorizontalAlignment HAlign)
{
	ChildSlot.SetHorizontalAlignment(HAlign);
}

void SScaleBox::SetVAlign(EVerticalAlignment VAlign)
{
	ChildSlot.SetVerticalAlignment(VAlign);
}

void SScaleBox::SetStretchDirection(EStretchDirection::Type InStretchDirection)
{
	StretchDirectionAttribute.Set(*this, InStretchDirection);
}

void SScaleBox::SetStretch(EStretch::Type InStretch)
{
	StretchAttribute.Set(*this, InStretch);
}

void SScaleBox::SetUserSpecifiedScale(float InUserSpecifiedScale)
{
	UserSpecifiedScaleAttribute.Set(*this, InUserSpecifiedScale);
}

void SScaleBox::SetIgnoreInheritedScale(bool InIgnoreInheritedScale)
{
	IgnoreInheritedScaleAttribute.Set(*this, InIgnoreInheritedScale);
}

FVector2D SScaleBox::ComputeDesiredSize(float InScale) const
{
	if (DoesScaleRequireNormalizingPrepassOrLocalGeometry())
	{
		if (NormalizedContentDesiredSize.IsSet())
		{
			FVector2D ContentDesiredSizeValue = NormalizedContentDesiredSize.GetValue();

			if (IsDesiredSizeDependentOnAreaAndScale())
			{
				// SUPER SPECIAL CASE - 
				// In the special case that we're only fitting one dimension, we can have the opposite dimension desire the growth of the
				// expected scale, if we can get that extra space, awesome.
				if (ComputedContentScale.IsSet() && ComputedContentScale.GetValue() != 0)
				{
					const EStretch::Type CurrentStretch = StretchAttribute.Get();

					switch (CurrentStretch)
					{
					case EStretch::ScaleToFitX:
						ContentDesiredSizeValue.Y = ContentDesiredSizeValue.Y * ComputedContentScale.GetValue();
						break;
					case EStretch::ScaleToFitY:
						ContentDesiredSizeValue.X = ContentDesiredSizeValue.X * ComputedContentScale.GetValue();
						break;
					}
				}
			}

			// If we require a normalizing pre-pass, we can never allow the scaled content's desired size to affect
			// the area we return that we need, otherwise, we'll be introducing hysteresis.
			return ContentDesiredSizeValue;
		}
	}
	// If we don't need a normalizing prepass, then we can safely just multiply
	// the desired size of the children by the computed content scale, so that we request the now larger or smaller
	// area that we need - this area is a constant scale, either by safezone or user scale.
	else if (ComputedContentScale.IsSet())
	{
		return SCompoundWidget::ComputeDesiredSize(InScale) * ComputedContentScale.GetValue();
	}
	
	return SCompoundWidget::ComputeDesiredSize(InScale);
}

float SScaleBox::GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const
{
	return ComputedContentScale.IsSet() ? ComputedContentScale.GetValue() : 1.0f;
}

void SScaleBox::RefreshSafeZoneScale()
{
	float ScaleDownBy = 0.f;
	FMargin SafeMargin(0, 0, 0, 0);
	FVector2D ScaleBy(1, 1);

#if WITH_EDITOR
	if (OverrideScreenSize.IsSet() && !OverrideScreenSize.GetValue().IsZero())
	{
		FSlateApplication::Get().GetSafeZoneSize(SafeMargin, OverrideScreenSize.GetValue());
		ScaleBy = OverrideScreenSize.GetValue();
	}
	else
#endif
	{
		if (StretchAttribute.Get() == EStretch::ScaleBySafeZone)
		{
			TSharedPtr<SViewport> GameViewport = FSlateApplication::Get().GetGameViewport();
			if (GameViewport.IsValid())
			{
				TSharedPtr<ISlateViewport> ViewportInterface = GameViewport->GetViewportInterface().Pin();
				if (ViewportInterface.IsValid())
				{
					const FIntPoint ViewportSize = ViewportInterface->GetSize();

					FSlateApplication::Get().GetSafeZoneSize(SafeMargin, ViewportSize);
					ScaleBy = ViewportSize;
				}
			}
		}
	}

	const float SafeZoneScaleX = FMath::Max(SafeMargin.Left, SafeMargin.Right)/ (float)ScaleBy.X;
	const float SafeZoneScaleY = FMath::Max(SafeMargin.Top, SafeMargin.Bottom) / (float)ScaleBy.Y;

	// In order to deal with non-uniform safe-zones we take the largest scale as the amount to scale down by.
	ScaleDownBy = FMath::Max(SafeZoneScaleX, SafeZoneScaleY);

	SafeZoneScale = 1.f - ScaleDownBy;
}

void SScaleBox::HandleSafeFrameChangedEvent()
{
	RefreshSafeZoneScale();
	Invalidate(EInvalidateWidgetReason::Prepass);
}

#if WITH_EDITOR

void SScaleBox::DebugSafeAreaUpdated(const FMargin& NewSafeZone, bool bShouldRecacheMetrics)
{
	HandleSafeFrameChangedEvent();
}

void SScaleBox::SetOverrideScreenInformation(TOptional<FVector2D> InScreenSize)
{
	OverrideScreenSize = InScreenSize;
	HandleSafeFrameChangedEvent();
}

#endif
