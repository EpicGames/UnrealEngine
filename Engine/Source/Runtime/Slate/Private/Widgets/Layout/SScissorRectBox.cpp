// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SScissorRectBox.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/RenderingCommon.h"
#include "HAL/IConsoleManager.h"

namespace Slate
{
	static FAutoConsoleVariable OverrideScissorRect( TEXT( "Slate.OverrideScissorRect" ), 1, TEXT( "Whether to allow Slate to apply a scissor clip to UI elements to prevent certain artifacts.  You might need to disable this if you are drawing UI over a mirror backbuffer while in VR." ) );
}

SScissorRectBox::SScissorRectBox()
	:ChildSlot(this)
{

}

void SScissorRectBox::Construct(const FArguments& InArgs)
{
	SetCanTick(false);

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

FVector2D SScissorRectBox::ComputeDesiredSize( float ) const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();

	if (ChildVisibility != EVisibility::Collapsed)
	{
		return ChildSlot.GetWidget()->GetDesiredSize();
	}
	
	return FVector2D::ZeroVector;
}

void SScissorRectBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// the child takes up the full space of this panel, always.
	ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ChildSlot.GetWidget(), AllottedGeometry.GetLocalSize(), FSlateLayoutTransform()));
}

FChildren* SScissorRectBox::GetChildren()
{
	return &ChildSlot;
}

int32 SScissorRectBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// just draw the only child
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Maybe none of our children are visible
	if( ArrangedChildren.Num() > 0 )
	{
		check( ArrangedChildren.Num() == 1 );
		FArrangedWidget& TheChild = ArrangedChildren[0];

		int32 Result = TheChild.Widget->Paint( Args.WithNewParent(this), TheChild.Geometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );
		
		return Result;
	}

	return LayerId;
}

void SScissorRectBox::SetContent(const TSharedRef< SWidget >& InContent)
{
	ChildSlot
	[
		InContent
	];
}
