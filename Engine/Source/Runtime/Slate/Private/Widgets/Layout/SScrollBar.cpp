// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBarTrack.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Application/SlateApplication.h"

SLATE_IMPLEMENT_WIDGET(SScrollBar)
void SScrollBar::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, UserVisibilityAttribute, EInvalidateWidgetReason::Visibility)
		.AffectVisibility();
}

void SScrollBar::Construct(const FArguments& InArgs)
{
	SetCanTick(false);
	
	OnUserScrolled = InArgs._OnUserScrolled;
	OnScrollBarVisibilityChanged = InArgs._OnScrollBarVisibilityChanged;
	Orientation = InArgs._Orientation;
	DragFocusCause = InArgs._DragFocusCause;
	SetUserVisibility(InArgs._Visibility);

	check(InArgs._Style);

	bIsThicknessAttrSet = InArgs._Thickness.IsSet();
	const TAttribute<FVector2D> Thickness = InArgs._Thickness.IsSet() ? InArgs._Thickness : FVector2D(InArgs._Style->Thickness);

	EHorizontalAlignment HorizontalAlignment = Orientation == Orient_Vertical ? HAlign_Center : HAlign_Fill;
	EVerticalAlignment VerticalAlignment = Orientation == Orient_Vertical ? VAlign_Fill : VAlign_Center;

	bHideWhenNotInUse = InArgs._HideWhenNotInUse;
	bPreventThrottling = InArgs._PreventThrottling;
	ScrollbarDisabledVisibility = InArgs._ScrollbarDisabledVisibility;
	bIsScrolling = false;
	LastInteractionTime = 0;

	SBorder::Construct( SBorder::FArguments()
		.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
		.Padding(InArgs._Padding)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				SAssignNew(BackgroundBorder, SBorder)
				.HAlign(HorizontalAlignment)
				.VAlign(VerticalAlignment)
				.Padding(0.f)
				[
					SAssignNew(Track, SScrollBarTrack)
					.Orientation(InArgs._Orientation)
					.TopSlot()
					[
						SNew(SBox)
						.HAlign(HorizontalAlignment)
						.VAlign(VerticalAlignment)
						[
							SAssignNew(TopImage, SImage)
							.ColorAndOpacity(TAttribute<FSlateColor>(SharedThis(this), &SScrollBar::GetTrackOpacity))
						]
					]
					.ThumbSlot()
					[
						SAssignNew(DragThumb, SBorder)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.BorderImage(TAttribute<const FSlateBrush*>(SharedThis(this), &SScrollBar::GetDragThumbImage))
						.BorderBackgroundColor_Lambda([WeakThis = AsWeak()]()
						{
							if (const TSharedPtr<SWidget> This = WeakThis.Pin())
							{
								return FSlateColor(StaticCastSharedPtr<SScrollBar>(This)->GetThumbOpacity());
							}
							return FSlateColor();
						})
						.Padding(0.f)
						[
							SAssignNew(ThicknessSpacer, SSpacer)
							.Size(Thickness)
						]
					]
					.BottomSlot()
					[
						SNew(SBox)
						.HAlign(HorizontalAlignment)
						.VAlign(VerticalAlignment)
						[
							SAssignNew(BottomImage, SImage)
							.ColorAndOpacity(TAttribute<FSlateColor>(SharedThis(this), &SScrollBar::GetTrackOpacity))
						]
					]
				]
			]
		]
	);

	SetStyle(InArgs._Style);

	SetEnabled(TAttribute<bool>( Track.ToSharedRef(), &SScrollBarTrack::IsNeeded ));
	SetVisibility(TAttribute<EVisibility>(SharedThis(this), &SScrollBar::ShouldBeVisible));
	SetScrollBarAlwaysVisible(InArgs._AlwaysShowScrollbar);
	bAlwaysShowScrollbarTrack = InArgs._AlwaysShowScrollbarTrack;
}

void SScrollBar::SetOnUserScrolled( const FOnUserScrolled& InHandler )
{
	OnUserScrolled = InHandler;
}

void SScrollBar::SetOnScrollBarVisibilityChanged( const FOnScrollBarVisibilityChanged& InHandler )
{
	OnScrollBarVisibilityChanged = InHandler;
}

void SScrollBar::SetState( float InOffsetFraction, float InThumbSizeFraction, bool bCallOnUserScrolled )
{
	if ( Track->DistanceFromTop() != InOffsetFraction || Track->GetThumbSizeFraction() != InThumbSizeFraction )
	{
		// Note that the maximum offset depends on how many items fit per screen
		// It is 1.0f-InThumbSizeFraction.
		Track->SetSizes(InOffsetFraction, InThumbSizeFraction);

		LastInteractionTime = FSlateApplication::Get().GetCurrentTime();
		if (bCallOnUserScrolled)
		{
			OnUserScrolled.ExecuteIfBound(InOffsetFraction);
		}
	}
}

FReply SScrollBar::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && IsNeeded())
	{
		FGeometry ThumbGeometry = FindChildGeometry(MyGeometry, DragThumb.ToSharedRef());

		if (DragThumb->IsDirectlyHovered())
		{
			// Clicking on the scrollbar drag thumb
			if( Orientation == Orient_Horizontal )
			{
				DragGrabOffset = ThumbGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).X;
			}
			else
			{
				DragGrabOffset = ThumbGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).Y;
			}

			bDraggingThumb = true;
		}
		else if (OnUserScrolled.IsBound())
		{
			// Clicking in the non drag thumb area of the scrollbar
			DragGrabOffset = Orientation == Orient_Horizontal ? (ThumbGeometry.GetLocalSize().X * 0.5f) : (ThumbGeometry.GetLocalSize().Y * 0.5f);
			bDraggingThumb = true;

			ExecuteOnUserScrolled( MyGeometry, MouseEvent );
		}
	}

	if( bDraggingThumb )
	{
		FReply ReturnReply = FReply::Handled().CaptureMouse(AsShared()).SetUserFocus(AsShared(), DragFocusCause);

		if (bPreventThrottling)
		{
			ReturnReply.PreventThrottling();
		}

		return ReturnReply;
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SScrollBar::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bDraggingThumb = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SScrollBar::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( this->HasMouseCapture() )
	{
		if (!MouseEvent.GetCursorDelta().IsZero())
		{
			if (OnUserScrolled.IsBound())
			{
				ExecuteOnUserScrolled(MyGeometry, MouseEvent);
			}
			return FReply::Handled();
		}
	}
	
	return FReply::Unhandled();
}

void SScrollBar::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SBorder::OnMouseEnter(MyGeometry, MouseEvent);
	LastInteractionTime = FSlateApplication::Get().GetCurrentTime();
}

void SScrollBar::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SBorder::OnMouseLeave(MouseEvent);
	LastInteractionTime = FSlateApplication::Get().GetCurrentTime();
}

void SScrollBar::ExecuteOnUserScrolled( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const int32 AxisId = (Orientation == Orient_Horizontal) ? 0 : 1;
	const FGeometry TrackGeometry = FindChildGeometry( MyGeometry, Track.ToSharedRef() );
	const float UnclampedOffsetInTrack = TrackGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() )[ AxisId ] - DragGrabOffset;
	const float TrackSizeBiasedByMinThumbSize = TrackGeometry.GetLocalSize()[ AxisId ] - Track->GetMinThumbSize();
	const float ThumbOffsetInTrack = FMath::Clamp( UnclampedOffsetInTrack, 0.0f, TrackSizeBiasedByMinThumbSize );
	const float ThumbOffset = ThumbOffsetInTrack / TrackSizeBiasedByMinThumbSize;
	OnUserScrolled.ExecuteIfBound( ThumbOffset );
}

bool SScrollBar::IsNeeded() const
{
	return Track->IsNeeded();
}

float SScrollBar::DistanceFromTop() const
{
	return Track->DistanceFromTop();
}

float SScrollBar::DistanceFromBottom() const
{
	return Track->DistanceFromBottom();
}

float SScrollBar::ThumbSizeFraction() const
{
	return Track->GetThumbSizeFraction();
}

SScrollBar::SScrollBar()
	: bDraggingThumb(false)
	, DragGrabOffset( 0.0f )
	, UserVisibilityAttribute(*this)
{
}

SScrollBar::~SScrollBar() = default;

FSlateColor SScrollBar::GetTrackOpacity() const
{
	if (bDraggingThumb || IsHovered())
	{
		return FLinearColor(1,1,1,1);
	}
	else if ((bAlwaysShowScrollbarTrack  && Track->GetThumbSizeFraction() > KINDA_SMALL_NUMBER) || AlwaysShowScrollbar())
	{
		return FLinearColor(1, 1, 1, 0.5f);
	}
	else
	{
		return FLinearColor(1,1,1,0);
	}
}

FLinearColor SScrollBar::GetThumbOpacity() const
{
	if (Track->GetThumbSizeFraction() <= 0.0f)
	{
		return FLinearColor(1, 1, 1, 0);
	}
	else if ( bDraggingThumb || IsHovered() )
	{
		return FLinearColor(1,1,1,1);
	}
	else
	{
		if ( bHideWhenNotInUse && !bAlwaysShowScrollbar )
		{
			const double LastInteractionDelta = bIsScrolling ? 0 : ( FSlateApplication::Get().GetCurrentTime() - LastInteractionTime );

			float ThumbOpacity = FMath::Lerp(1.0f, 0.0f, FMath::Clamp((float)( ( LastInteractionDelta - 0.2 ) / 0.2 ), 0.0f, 1.0f));
			return FLinearColor(1, 1, 1, ThumbOpacity);
		}
		else 
		{
			return FLinearColor(1, 1, 1, 0.75f);
		}
	}
}

void SScrollBar::BeginScrolling()
{
	bIsScrolling = true;
}

void SScrollBar::EndScrolling()
{
	bIsScrolling = false;
	LastInteractionTime = FSlateApplication::Get().GetCurrentTime();
}

const FSlateBrush* SScrollBar::GetDragThumbImage() const
{
	if ( bDraggingThumb )
	{
		return DraggedThumbImage;
	}
	else if (IsHovered())
	{
		return HoveredThumbImage;
	}
	else
	{
		return NormalThumbImage;
	}
}

EVisibility SScrollBar::ShouldBeVisible() const
{
	const EVisibility CurrentVisibility = GetVisibility();
    EVisibility NewVisibility = ScrollbarDisabledVisibility;
    
	if ( bAlwaysShowScrollbar || this->HasMouseCapture() )
	{
		NewVisibility = EVisibility::Visible;
	}
	else if( Track->IsNeeded() )
	{
		NewVisibility = UserVisibilityAttribute.Get();
	}

	if (NewVisibility != CurrentVisibility)
	{
		OnScrollBarVisibilityChanged.ExecuteIfBound(NewVisibility);
	}
	
	return NewVisibility;
}

bool SScrollBar::IsScrolling() const
{
	return bDraggingThumb;
}

EOrientation SScrollBar::GetOrientation() const
{
	return Orientation;
}

void SScrollBar::SetStyle(const FScrollBarStyle* InStyle)
{
	Style = InStyle;

	if (Style == nullptr)
	{
		FArguments Defaults;
		Style = Defaults._Style;
	}

	check(Style);

	NormalThumbImage = &Style->NormalThumbImage;
	HoveredThumbImage = &Style->HoveredThumbImage;
	DraggedThumbImage = &Style->DraggedThumbImage;

	if (Orientation == Orient_Vertical)
	{
		BackgroundBrush = &Style->VerticalBackgroundImage;
		TopBrush = &Style->VerticalTopSlotImage;
		BottomBrush = &Style->VerticalBottomSlotImage;
	}
	else
	{
		BackgroundBrush = &Style->HorizontalBackgroundImage;
		TopBrush = &Style->HorizontalTopSlotImage;
		BottomBrush = &Style->HorizontalBottomSlotImage;
	}
	BackgroundBorder->SetBorderImage(BackgroundBrush);
	TopImage->SetImage(TopBrush);
	BottomImage->SetImage(BottomBrush);

	if (!bIsThicknessAttrSet)
	{
		ThicknessSpacer->SetSize(FVector2D(Style->Thickness));
	}

	InvalidateStyle();
}

void SScrollBar::InvalidateStyle()
{
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SScrollBar::SetUserVisibility(TAttribute<EVisibility> InUserVisibility)
{
	UserVisibilityAttribute.Assign(*this, MoveTemp(InUserVisibility));
}

void SScrollBar::SetDragFocusCause(EFocusCause InDragFocusCause)
{
	DragFocusCause = InDragFocusCause;
}

void SScrollBar::SetThickness(TAttribute<FVector2D> InThickness)
{
	if (InThickness.IsSet())
	{
		bIsThicknessAttrSet = true;
		ThicknessSpacer->SetSize( MoveTemp(InThickness) );
	}
	else
	{
		bIsThicknessAttrSet = false;
		ThicknessSpacer->SetSize(FVector2D(Style->Thickness));
	}
}

void SScrollBar::SetScrollBarAlwaysVisible(bool InAlwaysVisible)
{
	bAlwaysShowScrollbar = InAlwaysVisible;

	Track->SetIsAlwaysVisible(InAlwaysVisible);
}

void SScrollBar::SetScrollBarTrackAlwaysVisible(bool InAlwaysVisible)
{
	bAlwaysShowScrollbarTrack = InAlwaysVisible;
}

void SScrollBar::SetScrollbarDisabledVisibility(EVisibility InVisibility)
{
	ScrollbarDisabledVisibility = InVisibility;
}

bool SScrollBar::AlwaysShowScrollbar() const
{
	return bAlwaysShowScrollbar;
}
