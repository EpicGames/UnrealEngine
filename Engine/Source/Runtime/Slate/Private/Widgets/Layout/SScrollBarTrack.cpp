// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SScrollBarTrack.h"
#include "Layout/ArrangedChildren.h"
#include "Widgets/SOverlay.h"

static const int32 NUM_SCROLLBAR_SLOTS = 3;

void SScrollBarTrack::Construct(const FArguments& InArgs)
{
	OffsetFraction = 0;
	ThumbSizeFraction = 1.0; // default to zero offset from the top with a full track thumb (scrolling not needed)
	MinThumbSize = 35;
	Orientation = InArgs._Orientation;

	
	static_assert(NUM_SCROLLBAR_SLOTS == 3, "SScrollBarTrack::FSlot numbers changed");

	static_assert(TOP_SLOT_INDEX == 0, "SScrollBarTrack::FSlot order has changed");
	Children.AddSlot(MoveTemp(FSlot::FSlotArguments(MakeUnique<FSlot>())[InArgs._TopSlot.Widget]));

	static_assert(BOTTOM_SLOT_INDEX == 1, "SScrollBarTrack::FSlot order has changed");
	Children.AddSlot(MoveTemp(FSlot::FSlotArguments(MakeUnique<FSlot>())
	[
		SNew(SOverlay) // We need a parent widget to control the visibility
		+ SOverlay::Slot()
		[
			InArgs._BottomSlot.Widget
		]
	]));

	static_assert(THUMB_SLOT_INDEX == 2, "SScrollBarTrack::FSlot order has changed");
	Children.AddSlot(MoveTemp(FSlot::FSlotArguments(MakeUnique<FSlot>())
	[
		SNew(SOverlay) // We need a parent widget to control the visibility
		+ SOverlay::Slot()
		[
			InArgs._ThumbSlot.Widget
		]
	]));

	// Ensure the visibilities are set as expected
	InvalidateIsNeeded(!IsNeeded());
}

SScrollBarTrack::FTrackSizeInfo SScrollBarTrack::GetTrackSizeInfo(const FGeometry& InTrackGeometry) const
{
	const float CurrentMinThumbSize = ThumbSizeFraction <= 0.0f ? 0.0f : MinThumbSize;
	return FTrackSizeInfo(InTrackGeometry, Orientation, CurrentMinThumbSize, this->ThumbSizeFraction, this->OffsetFraction);
}

void SScrollBarTrack::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const float Width = AllottedGeometry.Size.X;
	const float Height = AllottedGeometry.Size.Y;

	// We only need to show all three children when the thumb is visible, otherwise we only need to show the track
	if (IsNeeded())
	{
		FTrackSizeInfo TrackSizeInfo = this->GetTrackSizeInfo(AllottedGeometry);

		// Arrange top half of the track
		FVector2D ChildSize = (Orientation == Orient_Horizontal)
			? FVector2D(TrackSizeInfo.ThumbStart, Height)
			: FVector2D(Width, TrackSizeInfo.ThumbStart);

		FVector2D ChildPos(0, 0);
		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(Children[TOP_SLOT_INDEX].GetWidget(), ChildPos, ChildSize)
			);

		// Arrange bottom half of the track
		ChildPos = (Orientation == Orient_Horizontal)
			? FVector2D(TrackSizeInfo.GetThumbEnd(), 0)
			: FVector2D(0, TrackSizeInfo.GetThumbEnd());

		ChildSize = (Orientation == Orient_Horizontal)
			? FVector2D(AllottedGeometry.GetLocalSize().X - TrackSizeInfo.GetThumbEnd(), Height)
			: FVector2D(Width, AllottedGeometry.GetLocalSize().Y - TrackSizeInfo.GetThumbEnd());

		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(Children[BOTTOM_SLOT_INDEX].GetWidget(), ChildPos, ChildSize)
			);

		// Arrange the thumb
		ChildPos = (Orientation == Orient_Horizontal)
			? FVector2D(TrackSizeInfo.ThumbStart, 0)
			: FVector2D(0, TrackSizeInfo.ThumbStart);

		ChildSize = (Orientation == Orient_Horizontal)
			? FVector2D(TrackSizeInfo.ThumbSize, Height)
			: FVector2D(Width, TrackSizeInfo.ThumbSize);

		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(Children[THUMB_SLOT_INDEX].GetWidget(), ChildPos, ChildSize)
			);
	}
	else
	{
		// No thumb is visible, so just show the top half of the track at the current width/height
		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(Children[TOP_SLOT_INDEX].GetWidget(), FVector2D(0, 0), FVector2D(Width, Height))
			);
	}
}

FVector2D SScrollBarTrack::ComputeDesiredSize(float) const
{
	if (Orientation == Orient_Horizontal)
	{
		const float DesiredHeight = FMath::Max3(Children[0].GetWidget()->GetDesiredSize().Y, Children[1].GetWidget()->GetDesiredSize().Y, Children[2].GetWidget()->GetDesiredSize().Y);
		return FVector2D(MinThumbSize, DesiredHeight);
	}
	else
	{
		const float DesiredWidth = FMath::Max3(Children[0].GetWidget()->GetDesiredSize().X, Children[1].GetWidget()->GetDesiredSize().X, Children[2].GetWidget()->GetDesiredSize().X);
		return FVector2D(DesiredWidth, MinThumbSize);
	}
}

FChildren* SScrollBarTrack::GetChildren()
{
	return &Children;
}

void SScrollBarTrack::SetSizes(float InThumbOffsetFraction, float InThumbSizeFraction)
{
	// If you have no thumb, then it's effectively the size of the whole track
	if (InThumbSizeFraction == 0.0f && !bIsAlwaysVisible)
	{
		InThumbSizeFraction = 1.0f;
	}
	else if (InThumbSizeFraction > 1.0f && bIsAlwaysVisible)
	{
		InThumbSizeFraction = 0.0f;
	}

	if (OffsetFraction != InThumbOffsetFraction || ThumbSizeFraction != InThumbSizeFraction)
	{
		const bool bWasNeeded = IsNeeded();
		OffsetFraction = InThumbOffsetFraction;
		ThumbSizeFraction = InThumbSizeFraction;

		InvalidateIsNeeded(bWasNeeded);
	}
}

bool SScrollBarTrack::IsNeeded() const
{
	// We use a small epsilon here to avoid the scroll bar showing up when all of the content is already in view, due to
	// floating point precision when the scroll bar state is set
	return ThumbSizeFraction < (1.0f - KINDA_SMALL_NUMBER) || bIsAlwaysVisible;
}

float SScrollBarTrack::DistanceFromTop() const
{
	return OffsetFraction;
}

float SScrollBarTrack::DistanceFromBottom() const
{
	return 1.0f - (OffsetFraction + ThumbSizeFraction);
}

float SScrollBarTrack::GetMinThumbSize() const
{
	return MinThumbSize;
}

float SScrollBarTrack::GetThumbSizeFraction() const
{
	return ThumbSizeFraction;
}

void SScrollBarTrack::SetIsAlwaysVisible(bool InIsAlwaysVisible)
{
	if (bIsAlwaysVisible != InIsAlwaysVisible)
	{
		const bool bWasNeeded = IsNeeded();
		bIsAlwaysVisible = InIsAlwaysVisible;

		// SetSizes is called because it can adjust the ThumbSizeFraction value
		SetSizes(OffsetFraction, ThumbSizeFraction);
		
		InvalidateIsNeeded(bWasNeeded);
	}
}

void SScrollBarTrack::InvalidateIsNeeded(bool bWasNeeded)
{
	Invalidate(EInvalidateWidget::Layout);
	if (IsNeeded() != bWasNeeded)
	{
		const EVisibility NewVisibility = IsNeeded() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;
		Children[BOTTOM_SLOT_INDEX].GetWidget()->SetVisibility(NewVisibility);
		Children[THUMB_SLOT_INDEX].GetWidget()->SetVisibility(NewVisibility);
	}
}
