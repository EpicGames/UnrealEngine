// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Layout/IScrollableWidget.h"


SLATE_IMPLEMENT_WIDGET(SScrollBorder)
void SScrollBorder::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BorderFadeDistanceAttribute, EInvalidateWidgetReason::Paint);
}


SScrollBorder::SScrollBorder()
	: BorderFadeDistanceAttribute(*this, FVector2D(0.01f, 0.01f))
{
}

SScrollBorder::~SScrollBorder()
{ }


void SScrollBorder::Construct(const FArguments& InArgs, TSharedRef<IScrollableWidget> InScrollableWidget)
{
	check( InArgs._Style );

	SetBorderFadeDistance(InArgs._BorderFadeDistance);
	ScrollableWidget = InScrollableWidget;

	TSharedRef<SOverlay> Overlay = SNew(SOverlay)
		+ SOverlay::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		[
			InArgs._Content.Widget
		];

	Overlay->AddSlot()
	.HAlign( HAlign_Fill )
	.VAlign( VAlign_Top )
	[
		// Shadow: Hint to scroll up
		SNew( SImage )
		.Visibility( EVisibility::HitTestInvisible )
		.ColorAndOpacity( this, &SScrollBorder::GetTopBorderOpacity )
		.Image( &InArgs._Style->TopShadowBrush )
	];

	Overlay->AddSlot()
	.HAlign( HAlign_Fill )
	.VAlign( VAlign_Bottom )
	[
		// Shadow: a hint to scroll down
		SNew( SImage )
		.Visibility( EVisibility::HitTestInvisible )
		.ColorAndOpacity( this, &SScrollBorder::GetBottomBorderOpacity )
		.Image( &InArgs._Style->BottomShadowBrush )
	];

	ChildSlot
	[
		Overlay
	];
}

void SScrollBorder::SetBorderFadeDistance(TAttribute<FVector2D> InBorderFadeDistance)
{
	BorderFadeDistanceAttribute.Assign(*this, MoveTemp(InBorderFadeDistance));
}

FSlateColor SScrollBorder::GetTopBorderOpacity() const
{
	float ShadowOpacity = 0;

	TSharedPtr< IScrollableWidget > Scrollable = ScrollableWidget.Get().Pin();
	if ( Scrollable.IsValid() )
	{
		// The shadow should only be visible when the user needs a hint that they can scroll up.
		const float FadeDistance = FMath::Max(0.01f, BorderFadeDistanceAttribute.Get().Y);
		ShadowOpacity = FMath::Clamp( Scrollable->GetScrollDistance().Y / FadeDistance, 0.0f, 1.0f );
	}

	return FLinearColor( 1.0f, 1.0f, 1.0f, ShadowOpacity );
}

FSlateColor SScrollBorder::GetBottomBorderOpacity() const
{
	float ShadowOpacity = 0;

	TSharedPtr< IScrollableWidget > Scrollable = ScrollableWidget.Get().Pin();
	if (Scrollable.IsValid())
	{
		// The shadow should only be visible when the user needs a hint that they can scroll down.
		const float FadeDistance = FMath::Max(0.01f, BorderFadeDistanceAttribute.Get().Y);
		ShadowOpacity = FMath::Clamp( Scrollable->GetScrollDistanceRemaining().Y / FadeDistance, 0.0f, 1.0f );
	}

	return FLinearColor( 1.0f, 1.0f, 1.0f, ShadowOpacity );
}
