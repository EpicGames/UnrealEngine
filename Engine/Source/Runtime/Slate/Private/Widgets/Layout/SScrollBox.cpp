// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SScrollBox.h"
#include "Rendering/DrawElements.h"
#include "Types/NavigationMetaData.h"
#include "Types/SlateConstants.h"
#include "Layout/LayoutUtils.h"
#include "Widgets/SOverlay.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SScrollBox)

namespace ScrollBoxFocus
{
static void EmplaceArrangedWidget(const FGeometry& MyGeometry, TSharedRef<SWidget> InParentWidget, TSharedRef<SWidget> InChildWidget, TMap<TSharedRef<SWidget>, FArrangedWidget>& OutWidgetsToArranged)
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	InParentWidget->ArrangeChildren(MyGeometry, ArrangedChildren);
	if (FArrangedWidget* FoundSelfArranged = ArrangedChildren.GetInternalArray().FindByPredicate([InChildWidget](const FArrangedWidget& WidgetIt) { return InChildWidget == WidgetIt.Widget; }))
	{
		OutWidgetsToArranged.Emplace(FoundSelfArranged->Widget, *FoundSelfArranged);
	}
}

static void AppendFocusableChildren(
	const FGeometry& MyGeometry,
	TSharedRef<SWidget> InParentWidget,
	TSharedRef<SWidget> InRuleWidget,
	const uint32 InUserIndex,
	const bool bIsRootCall,
	TSharedPtr<SWidget>& OutUserFocusedWidget,
	TMap<TSharedRef<SWidget>, FArrangedWidget>& OutWidgetsToArranged,
	FHittestGrid& OutHitTestGrid)
{
	FChildren* InWidgetChildren = InParentWidget->GetChildren();
	if (InWidgetChildren)
	{
		for (int32 SlotIndex = 0; SlotIndex < InWidgetChildren->Num(); ++SlotIndex)
		{
			TSharedRef<SWidget> Child = InWidgetChildren->GetChildAt(SlotIndex);

			if (Child->GetVisibility().IsVisible())
			{
				const bool bIsRuleWidget = Child == InRuleWidget;
				const bool bSupportsKeyboardFocus = Child->SupportsKeyboardFocus();

				if (bIsRuleWidget)
				{
					EmplaceArrangedWidget(MyGeometry, InParentWidget, Child, OutWidgetsToArranged);
				}
				else if (bSupportsKeyboardFocus)
				{
					OutHitTestGrid.AddWidget(&Child.Get(), 0, 0, FSlateInvalidationWidgetSortOrder());
					const bool bHasUserFocus = Child->HasUserFocus(InUserIndex).IsSet() || Child->HasUserFocusedDescendants(InUserIndex);
					if (bHasUserFocus)
					{
						ensureMsgf(!OutUserFocusedWidget.IsValid(), TEXT("This should only be set once"));
						OutUserFocusedWidget = Child.ToSharedPtr();

						EmplaceArrangedWidget(MyGeometry, InParentWidget, Child, OutWidgetsToArranged);
					}
				}

				// We only want to add Focusable Children that are children of the Rule Widget, i.e. the ScrollBox.
				// We do need to start one widget higher in the hiearchy to arrange the ScrollBox/Rule widget itself
				// If we added this child widget as keyboard focus already, we do NOT need to search it's children
				const bool bIsChildThatDoesNotSupportKeyBoardFocus = !bIsRootCall && !bSupportsKeyboardFocus;
				if (bIsRuleWidget || bIsChildThatDoesNotSupportKeyBoardFocus)
				{
					AppendFocusableChildren(MyGeometry, Child, InRuleWidget, InUserIndex, false, OutUserFocusedWidget, OutWidgetsToArranged, OutHitTestGrid);
				}
			}
		}
	}
}

static TSharedPtr<SWidget> GetNextWidget(
	const FGeometry& MyGeometry,
	const UE::Slate::FDeprecateVector2DResult& InHitTestGridSize,
	TSharedRef<SWidget> InRuleWidget,
	const uint32 InUserIndex,
	const EUINavigation InNavigation,
	FNavigationMethod& InNavigationMethod)
{
	TSharedPtr<SWidget> ReturnWidget;

	TSharedPtr<SWidget> ParentWidget = InRuleWidget->GetParentWidget();
	if (ParentWidget.IsValid())
	{
		TSharedPtr<SWidget> StartingWidget; // This is the current user focused widget
		TMap<TSharedRef<SWidget>, FArrangedWidget> WidgetsToArranged;
		FHittestGrid HitTestGrid;
		{
			UE::Slate::FDeprecateVector2DResult RectTopLeft = InRuleWidget->GetTickSpaceGeometry().GetRenderBoundingRect().GetTopLeft();		
			HitTestGrid.SetHittestArea(RectTopLeft, InHitTestGridSize);
		}

		constexpr bool bIsRootCall = true;
		AppendFocusableChildren(
			MyGeometry,
			ParentWidget.ToSharedRef(),
			InRuleWidget,
			InUserIndex,
			bIsRootCall,
			StartingWidget,
			WidgetsToArranged,
			HitTestGrid);

		if (StartingWidget.IsValid())
		{
			const TSharedPtr<FNavigationMetaData> FocusWidgetNavMetaData = StartingWidget->GetMetaData<FNavigationMetaData>();
			const TSharedPtr<SWidget> ExplicitNextWidget = FocusWidgetNavMetaData.IsValid() ? FocusWidgetNavMetaData->GetFocusRecipient(InNavigation).Pin() : nullptr;
			if (ExplicitNextWidget.IsValid())
			{
				return ExplicitNextWidget;
			}
		}

		const bool bRuleWidgetArranged = WidgetsToArranged.Contains(InRuleWidget);
		const bool bUserFocusedWidgetArranged = StartingWidget.IsValid() && WidgetsToArranged.Contains(StartingWidget.ToSharedRef());
		const bool bFoundArrangedWidgets = bRuleWidgetArranged && bUserFocusedWidgetArranged;
		if (bFoundArrangedWidgets)
		{
			InNavigationMethod.Initialize(&HitTestGrid, nullptr);
			
			const FNavigationReply FocusedNavigationReply = StartingWidget->OnNavigation(MyGeometry, InNavigation);
			const FArrangedWidget& ArrangedStartingWidget = WidgetsToArranged[StartingWidget.ToSharedRef()];
			const FArrangedWidget& ArrangedRuleWidget = WidgetsToArranged[InRuleWidget];

			ReturnWidget = InNavigationMethod.FindNextFocusableWidget(ArrangedStartingWidget, InNavigation, FocusedNavigationReply, ArrangedRuleWidget, InUserIndex);
		}
	}

	return ReturnWidget;
}
} //namespace ScrollBoxFocus

void SScrollBox::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	TBasicLayoutWidgetSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
	TResizingWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
}

void SScrollBox::FSlot::RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
{
	TBasicLayoutWidgetSlot<FSlot>::RegisterAttributes(AttributeInitializer);
	TResizingWidgetSlotMixin<FSlot>::RegisterAttributesMixin(AttributeInitializer);
}

SScrollBox::FSlot::FSlotArguments SScrollBox::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

SLATE_IMPLEMENT_WIDGET(SScrollPanel)
void SScrollPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	SScrollBox::FSlot::RegisterAttributes(Initializer);
}

void SScrollPanel::Construct(const FArguments& InArgs, TArray<SScrollBox::FSlot::FSlotArguments> InSlots, TSharedRef<SScrollBox> InScrollBox)
{
	OwningScrollBox = InScrollBox;
	PhysicalOffset = 0;
	Children.AddSlots(MoveTemp(InSlots));
	Orientation = InArgs._Orientation;
	BackPadScrolling = InArgs._BackPadScrolling;
	FrontPadScrolling = InArgs._FrontPadScrolling;
	bAllowContentToShrink = InArgs._AllowContentToShrink;
}

void SScrollPanel::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const float ScrollPadding = Orientation == Orient_Vertical ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X;
	const float ChildrenOffset = -PhysicalOffset + (BackPadScrolling ? ScrollPadding : 0);

	if (Orientation == EOrientation::Orient_Horizontal)
	{
		ArrangeChildrenInStack<EOrientation::Orient_Horizontal, TPanelChildren<SScrollBox::FSlot>, SScrollBox::FSlot>(GSlateFlowDirection, this->Children, AllottedGeometry, ArrangedChildren, ChildrenOffset, bAllowContentToShrink, ArrangedSize);
	}
	else
	{
		ArrangeChildrenInStack<EOrientation::Orient_Vertical, TPanelChildren<SScrollBox::FSlot>, SScrollBox::FSlot>(GSlateFlowDirection, this->Children, AllottedGeometry, ArrangedChildren, ChildrenOffset, bAllowContentToShrink, ArrangedSize);
	}
}

int32 SScrollPanel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bHasNavigationMethod = OwningScrollBox.IsValid() && OwningScrollBox.Pin()->HasNavigationMetaData();
	if (bHasNavigationMethod)
	{
		// Use the content size of the scrollbox to navigate to offscreen rendered widgets
		const FVector2D ScrollPanelSize = FVector2D::Max(AllottedGeometry.Size, GetArrangedSize());
		const FGeometry ScrollBoxGeometry = AllottedGeometry.MakeChild(ScrollPanelSize, FSlateLayoutTransform());
		const FSlateRect BoundingRect = OwningScrollBox.IsValid() ? OwningScrollBox.Pin()->GetHitTestBoundingRect() : GetHitTestBoundingRect();

		return SPanel::OnPaint(Args, ScrollBoxGeometry, BoundingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
	else
	{
		return SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
}

FVector2D SScrollPanel::ComputeDesiredSize(float) const
{
	FVector2D ThisDesiredSize = FVector2D::ZeroVector;
	for (int32 SlotIndex = 0; SlotIndex < Children.Num(); ++SlotIndex)
	{
		const SScrollBox::FSlot& ThisSlot = Children[SlotIndex];
		if (ThisSlot.GetWidget()->GetVisibility() == EVisibility::Collapsed)
		{
			continue;
		}

		const FVector2D ChildDesiredSize = ThisSlot.GetWidget()->GetDesiredSize();
		if (Orientation == Orient_Vertical)
		{
			ThisDesiredSize.X = FMath::Max(ChildDesiredSize.X + ThisSlot.GetPadding().GetTotalSpaceAlong<Orient_Horizontal>(), ThisDesiredSize.X);
			ThisDesiredSize.Y += ChildDesiredSize.Y + ThisSlot.GetPadding().GetTotalSpaceAlong<Orient_Vertical>();
		}
		else
		{
			ThisDesiredSize.X += ChildDesiredSize.X + ThisSlot.GetPadding().GetTotalSpaceAlong<Orient_Horizontal>();
			ThisDesiredSize.Y = FMath::Max(ChildDesiredSize.Y + ThisSlot.GetPadding().GetTotalSpaceAlong<Orient_Vertical>(), ThisDesiredSize.Y);
		}
	}

	FVector2D::FReal ScrollPadding = Orientation == Orient_Vertical ? GetTickSpaceGeometry().GetLocalSize().Y : GetTickSpaceGeometry().GetLocalSize().X;
	FVector2D::FReal& SizeSideToPad = Orientation == Orient_Vertical ? ThisDesiredSize.Y : ThisDesiredSize.X;
	SizeSideToPad += BackPadScrolling ? ScrollPadding : 0;
	SizeSideToPad += FrontPadScrolling ? ScrollPadding : 0;

	return ThisDesiredSize;
}

SScrollBox::SScrollBox()
{
	VerticalScrollBarSlot = nullptr;
	bClippingProxy = true;
}

SScrollBox::~SScrollBox() = default;

void SScrollBox::Construct( const FArguments& InArgs )
{
	check(InArgs._Style);

	Style = InArgs._Style;
	ScrollBarStyle = InArgs._ScrollBarStyle;
	DesiredScrollOffset = 0;
	bIsScrolling = false;
	bAnimateScroll = false;
	ScrollingAnimationInterpolationSpeed = InArgs._ScrollAnimationInterpSpeed;
	AmountScrolledWhileRightMouseDown = 0;
	PendingScrollTriggerAmount = 0;
	bShowSoftwareCursor = false;
	SoftwareCursorPosition = FVector2f::ZeroVector;
	OnUserScrolled = InArgs._OnUserScrolled;
	OnScrollBarVisibilityChanged = InArgs._OnScrollBarVisibilityChanged;
	Orientation = InArgs._Orientation;
	bScrollToEnd = false;
	bIsScrollingActiveTimerRegistered = false;
	bAllowsRightClickDragScrolling = InArgs._ScrollBarRightClickDragAllowed;
	ConsumeMouseWheel = InArgs._ConsumeMouseWheel;
	TickScrollDelta = 0;
	AllowOverscroll = InArgs._AllowOverscroll;
	BackPadScrolling = InArgs._BackPadScrolling;
	FrontPadScrolling = InArgs._FrontPadScrolling;
	bAnimateWheelScrolling = InArgs._AnimateWheelScrolling;
	WheelScrollMultiplier = InArgs._WheelScrollMultiplier;
	bEnableTouchScrolling = InArgs._EnableTouchScrolling;
	bConsumePointerInput = InArgs._ConsumePointerInput;
	NavigationScrollPadding = InArgs._NavigationScrollPadding;
	NavigationDestination = InArgs._NavigationDestination;
	ScrollWhenFocusChanges = InArgs._ScrollWhenFocusChanges;
	OnScrollBoxFocusReceived = InArgs._OnFocusReceived;
	OnScrollBoxFocusLost = InArgs._OnFocusLost;
	OnScrollBoxFocusChanging = InArgs._OnFocusChanging;
	bTouchPanningCapture = false;
	bIsFocusable = false;
	bVolatilityAlwaysInvalidatesPrepass = true;
	bAllowContentToShrink = InArgs._AllowContentToShrink;

	if (InArgs._ExternalScrollbar.IsValid())
	{
		// An external scroll bar was specified by the user
		ScrollBar = InArgs._ExternalScrollbar;
		ScrollBar->SetOnUserScrolled(FOnUserScrolled::CreateSP(this, &SScrollBox::ScrollBar_OnUserScrolled));
		ScrollBar->SetOnScrollBarVisibilityChanged(FOnScrollBarVisibilityChanged::CreateSP(this, &SScrollBox::ScrollBar_OnScrollBarVisibilityChanged));
		bScrollBarIsExternal = true;
	}
	else
	{
		// Make a scroll bar 
		ScrollBar = ConstructScrollBar();
		ScrollBar->SetDragFocusCause(InArgs._ScrollBarDragFocusCause);
		ScrollBar->SetThickness(InArgs._ScrollBarThickness);
		ScrollBar->SetUserVisibility(InArgs._ScrollBarVisibility);
		ScrollBar->SetScrollBarAlwaysVisible(InArgs._ScrollBarAlwaysVisible);
		ScrollBar->SetOnScrollBarVisibilityChanged(FOnScrollBarVisibilityChanged::CreateSP(this, &SScrollBox::ScrollBar_OnScrollBarVisibilityChanged));
		ScrollBarSlotPadding = InArgs._ScrollBarPadding;

		bScrollBarIsExternal = false;
	}

	SAssignNew(ScrollPanel, SScrollPanel, MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)), StaticCastSharedRef<SScrollBox>(AsShared()))
		.Clipping(InArgs._Clipping)
		.Orientation(Orientation)
		.BackPadScrolling(BackPadScrolling)
		.FrontPadScrolling(FrontPadScrolling);

	if (Orientation == Orient_Vertical)
	{
		ConstructVerticalLayout();
	}
	else
	{
		ConstructHorizontalLayout();
	}

	ScrollBar->SetState( 0.0f, 1.0f );
}

void SScrollBox::OnClippingChanged()
{
	ScrollPanel->SetClipping(Clipping);
}

TSharedPtr<SScrollBar> SScrollBox::ConstructScrollBar()
{
	return TSharedPtr<SScrollBar>(SNew(SScrollBar)
		.Style(ScrollBarStyle)
		.Orientation(Orientation)
		.Padding(0.0f)
		.OnUserScrolled(this, &SScrollBox::ScrollBar_OnUserScrolled));
}

void SScrollBox::ConstructVerticalLayout()
{
	TSharedPtr<SHorizontalBox> PanelAndScrollbar;
	this->ChildSlot
	[
		SAssignNew(PanelAndScrollbar, SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.Padding(Style->VerticalScrolledContentPadding)
			[
				// Scroll panel that presents the scrolled content
				ScrollPanel.ToSharedRef()
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				// Shadow: Hint to scroll up
				SAssignNew(TopLeftShadowHint, SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(this, &SScrollBox::GetStartShadowOpacity)
				.Image(&Style->TopShadowBrush)
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			[
				// Shadow: a hint to scroll down
				SAssignNew(BottomRightShadowHint, SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(this, &SScrollBox::GetEndShadowOpacity)
				.Image(&Style->BottomShadowBrush)
			]
		]
	];

	VerticalScrollBarSlot = nullptr;
	if (!bScrollBarIsExternal)
	{
		PanelAndScrollbar->AddSlot()
		.Padding(ScrollBarSlotPadding)
		.AutoWidth()
		.Expose(VerticalScrollBarSlot)
		[
			ScrollBar.ToSharedRef()
		];
	}
}

void SScrollBox::ConstructHorizontalLayout()
{
	TSharedPtr<SVerticalBox> PanelAndScrollbar;
	this->ChildSlot
	[
		SAssignNew(PanelAndScrollbar, SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.Padding(Style->HorizontalScrolledContentPadding)
			[
				// Scroll panel that presents the scrolled content
				ScrollPanel.ToSharedRef()
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			[
				// Shadow: Hint to left
				SAssignNew(TopLeftShadowHint, SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(this, &SScrollBox::GetStartShadowOpacity)
				.Image(&Style->LeftShadowBrush)
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Fill)
			[
				// Shadow: a hint to scroll right
				SAssignNew(BottomRightShadowHint, SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(this, &SScrollBox::GetEndShadowOpacity)
				.Image(&Style->RightShadowBrush)
			]
		]
	];

	HorizontalScrollBarSlot = nullptr;
	if (!bScrollBarIsExternal)
	{
		PanelAndScrollbar->AddSlot()
		.Padding(ScrollBarSlotPadding)
		.AutoHeight()
		.Expose(HorizontalScrollBarSlot)
		[
			ScrollBar.ToSharedRef()
		];
	}
}

/** Adds a slot to SScrollBox */
SScrollBox::FScopedWidgetSlotArguments SScrollBox::AddSlot()
{
	return InsertSlot(INDEX_NONE);
}

SScrollBox::FScopedWidgetSlotArguments SScrollBox::InsertSlot(int32 Index)
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), ScrollPanel->Children, Index };
}

const SScrollBox::FSlot& SScrollBox::GetSlot(int32 SlotIndex) const
{
	check(ScrollPanel->Children.IsValidIndex(SlotIndex));
	const FSlotBase& BaseSlot = static_cast<const FSlotBase&>(ScrollPanel->Children[SlotIndex]);
	return static_cast<const FSlot&>(BaseSlot);
}

SScrollBox::FSlot& SScrollBox::GetSlot(int32 SlotIndex)
{
	return const_cast<FSlot&>(const_cast<const SScrollBox*>(this)->GetSlot(SlotIndex));
}

void SScrollBox::RemoveSlot( const TSharedRef<SWidget>& WidgetToRemove )
{
	ScrollPanel->Children.Remove(WidgetToRemove);
}

int32 SScrollBox::NumSlots() const
{
	return ScrollPanel->Children.Num();
}

void SScrollBox::ClearChildren()
{
	ScrollPanel->Children.Empty();
}

bool SScrollBox::IsRightClickScrolling() const
{
	return FSlateApplication::IsInitialized() && AmountScrolledWhileRightMouseDown >= FSlateApplication::Get().GetDragTriggerDistance() && this->ScrollBar->IsNeeded();
}

float SScrollBox::GetScrollOffset() const
{
	return DesiredScrollOffset;
}

UE::Slate::FDeprecateVector2DResult SScrollBox::GetScrollPanelContentSize() const
{
	UE::Slate::FDeprecateVector2DResult Size;
	Size = ScrollPanel->GetArrangedSize();
	if (!bAllowContentToShrink || Size.IsNearlyZero())
	{
		Size = ScrollPanel->GetDesiredSize();
	}

	return Size;
}

float SScrollBox::GetOverscrollOffset() const
{
	if (AllowOverscroll != EAllowOverscroll::Yes)
	{
		return 0.0f;
	}

	const float ScrollOffset = ScrollPanel->PhysicalOffset;
	if (ScrollOffset < 0.0f)
	{
		// Negative overscroll (before the start of the ScrollBox)
		return ScrollOffset;
	}

	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());
	const float ViewSize = GetScrollComponentFromVector(CachedGeometry.GetLocalSize());
	if (ScrollOffset > ContentSize - ViewSize)
	{
		// Positive overscroll (past the end of the ScrollBox)
		return ScrollOffset - (ContentSize - ViewSize);
	}

	// No overscroll
	return 0.0f;
}

float SScrollBox::GetOverscrollPercentage() const
{
	if (AllowOverscroll != EAllowOverscroll::Yes)
	{
		return 0.0f;
	}

	const float ViewSize = GetScrollComponentFromVector(CachedGeometry.GetLocalSize());
	if (ViewSize <= 0.0f)
	{
		// Avoid division by zero
		return 0.0f;
	}
	
	const float ScrollOffset = ScrollPanel->PhysicalOffset;
	if (ScrollOffset < 0.0f)
	{
		// Negative overscroll (before the start of the ScrollBox)
		return (ScrollOffset / ViewSize) * 100.0f;
	}

	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());
	if (ScrollOffset > ContentSize - ViewSize)
	{
		// Positive overscroll (past the end of the ScrollBox)
		const float OverscrollAmount = ScrollOffset - (ContentSize - ViewSize);
		return (OverscrollAmount / ViewSize) * 100.0f;
	}

	// No overscroll
	return 0.0f;
}

float SScrollBox::GetScrollOffsetOfEnd() const
{
	const FGeometry ScrollPanelGeometry = FindChildGeometry(CachedGeometry, ScrollPanel.ToSharedRef());
	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());
	return FMath::Max(ContentSize - GetScrollComponentFromVector(ScrollPanelGeometry.Size), 0.0f);
}

float SScrollBox::GetViewFraction() const
{
	const FGeometry ScrollPanelGeometry = FindChildGeometry(CachedGeometry, ScrollPanel.ToSharedRef());
	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());

	return FMath::Clamp<float>(GetScrollComponentFromVector(CachedGeometry.GetLocalSize()) > 0 ? GetScrollComponentFromVector(ScrollPanelGeometry.Size) / ContentSize : 1, 0.0f, 1.0f);
}

float SScrollBox::GetViewOffsetFraction() const
{
	const FGeometry ScrollPanelGeometry = FindChildGeometry(CachedGeometry, ScrollPanel.ToSharedRef());
	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());

	const float ViewFraction = GetViewFraction();
	return FMath::Clamp( DesiredScrollOffset/ContentSize, 0.f, 1.f - ViewFraction );
}

void SScrollBox::SetScrollOffset( float NewScrollOffset )
{
	DesiredScrollOffset = NewScrollOffset;
	bScrollToEnd = false;

	Invalidate(EInvalidateWidget::Layout);
}

void SScrollBox::ScrollToStart()
{
	SetScrollOffset(0);
}

void SScrollBox::ScrollToEnd()
{
	bScrollToEnd = true;

	Invalidate(EInvalidateWidget::Layout);
}

void SScrollBox::ScrollDescendantIntoView(const TSharedPtr<SWidget>& WidgetToScrollIntoView, bool InAnimateScroll, EDescendantScrollDestination InDestination, float InScrollPadding)
{
	TWeakPtr<SWidget> WidgetWeak = WidgetToScrollIntoView;

	ScrollIntoViewRequest = [this, WidgetWeak, InAnimateScroll, InDestination, InScrollPadding](FGeometry AllottedGeometry)
	{
		if (WidgetWeak.IsValid())
		{
			InternalScrollDescendantIntoView(AllottedGeometry, WidgetWeak.Pin(), InAnimateScroll, InDestination, InScrollPadding);
		}
	};

	BeginInertialScrolling();
}

bool SScrollBox::InternalScrollDescendantIntoView(const FGeometry& MyGeometry, const TSharedPtr<SWidget>& WidgetToFind, bool InAnimateScroll, EDescendantScrollDestination InDestination, float InScrollPadding)
{
	// We need to safely find the one WidgetToFind among our descendants.
	TSet< TSharedRef<SWidget> > WidgetsToFind;
	{
		if (WidgetToFind.IsValid())
		{
			WidgetsToFind.Add(WidgetToFind.ToSharedRef());
		}
	}
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;

	FindChildGeometries( MyGeometry, WidgetsToFind, Result );

	if (WidgetToFind.IsValid())
	{
		FArrangedWidget* WidgetGeometry = Result.Find(WidgetToFind.ToSharedRef());
		if (!WidgetGeometry)
		{
			UE_LOGF(LogSlate, Warning, "Unable to scroll to descendant as it's not a child of the scrollbox");
		}
		else
		{
			float ScrollOffset = 0.0f;
			if (InDestination == EDescendantScrollDestination::TopOrLeft)
			{
				// Calculate how much we would need to scroll to bring this to the top/left of the scroll box
				const float WidgetPosition = GetScrollComponentFromVector(MyGeometry.AbsoluteToLocal(WidgetGeometry->Geometry.GetAbsolutePosition()));
				const float MyPosition = InScrollPadding;
				ScrollOffset = WidgetPosition - MyPosition;
			}
			else if (InDestination == EDescendantScrollDestination::BottomOrRight)
			{
				// Calculate how much we would need to scroll to bring this to the bottom/right of the scroll box
				const float WidgetPosition = GetScrollComponentFromVector(MyGeometry.AbsoluteToLocal(WidgetGeometry->Geometry.GetAbsolutePosition() + WidgetGeometry->Geometry.GetAbsoluteSize()) - MyGeometry.GetLocalSize());
				const float MyPosition = InScrollPadding;
				ScrollOffset = WidgetPosition - MyPosition;
			}
			else if (InDestination == EDescendantScrollDestination::Center)
			{
				// Calculate how much we would need to scroll to bring this to the top/left of the scroll box
				const float WidgetPosition = GetScrollComponentFromVector(MyGeometry.AbsoluteToLocal(WidgetGeometry->Geometry.GetAbsolutePosition()) + (WidgetGeometry->Geometry.GetLocalSize() / 2));
				const float MyPosition = GetScrollComponentFromVector(MyGeometry.GetLocalSize() * FVector2f(0.5f, 0.5f));
				ScrollOffset = WidgetPosition - MyPosition;
			}
			else
			{
				const float WidgetStartPosition = GetScrollComponentFromVector(MyGeometry.AbsoluteToLocal(WidgetGeometry->Geometry.GetAbsolutePosition()));
				const float WidgetEndPosition = WidgetStartPosition + GetScrollComponentFromVector(WidgetGeometry->Geometry.GetLocalSize());
				const float ViewStartPosition = InScrollPadding;
				const float ViewEndPosition = GetScrollComponentFromVector(MyGeometry.GetLocalSize() - InScrollPadding);

				const float ViewDelta = (ViewEndPosition - ViewStartPosition);
				const float WidgetDelta = (WidgetEndPosition - WidgetStartPosition);

				if (WidgetStartPosition < ViewStartPosition)
				{
					ScrollOffset = WidgetStartPosition - ViewStartPosition;
				}
				else if (WidgetEndPosition > ViewEndPosition)
				{
					ScrollOffset = (WidgetEndPosition - ViewDelta) - ViewStartPosition;
				}
			}

			if (ScrollOffset != 0.0f)
			{
				DesiredScrollOffset = ScrollPanel->PhysicalOffset;
				ScrollBy(MyGeometry, ScrollOffset, EAllowOverscroll::No, InAnimateScroll);
			}

			return true;
		}
	}

	return false;
}

void SScrollBox::SetStyle(const FScrollBoxStyle* InStyle)
{
	check(InStyle);
	if (Style != InStyle)
	{
		Style = InStyle;
		TopLeftShadowHint->SetImage(Orientation == Orient_Vertical ? &Style->TopShadowBrush : &Style->LeftShadowBrush);
		BottomRightShadowHint->SetImage(Orientation == Orient_Vertical ? &Style->BottomShadowBrush : &Style->RightShadowBrush);
		InvalidateStyle();
	}
}

void SScrollBox::SetScrollBarStyle(const FScrollBarStyle* InBarStyle)
{
	if (InBarStyle != ScrollBarStyle)
	{
		ScrollBarStyle = InBarStyle;
		if (!bScrollBarIsExternal && ScrollBar.IsValid())
		{
			ScrollBar->SetStyle(ScrollBarStyle);
		}
	}
}

void SScrollBox::InvalidateStyle()
{
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SScrollBox::InvalidateScrollBarStyle()
{
	if (ScrollBar.IsValid())
	{
		ScrollBar->InvalidateStyle();
	}
}

EOrientation SScrollBox::GetOrientation()
{
	return Orientation;
}

void SScrollBox::SetNavigationDestination(const EDescendantScrollDestination NewNavigationDestination)
{
	NavigationDestination = NewNavigationDestination;
}

void SScrollBox::SetConsumeMouseWheel(EConsumeMouseWheel NewConsumeMouseWheel)
{
	ConsumeMouseWheel = NewConsumeMouseWheel;
}

void SScrollBox::SetAnalogMouseWheelKey(FKey InAnalogKey)
{
	ensure(!InAnalogKey.IsValid() || InAnalogKey.IsAnalog());
	AnalogMouseWheelKey = InAnalogKey;
}

void SScrollBox::SetIsFocusable(bool bInIsFocusable)
{
	bIsFocusable = bInIsFocusable;
}

void SScrollBox::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		if (!bScrollBarIsExternal)
		{
			ScrollBar = ConstructScrollBar();
		}
		ScrollPanel->SetOrientation(Orientation);
		if (Orientation == Orient_Vertical)
		{
			ConstructVerticalLayout();
		}
		else
		{
			ConstructHorizontalLayout();
		}
	}
}

void SScrollBox::SetScrollBarVisibility(EVisibility InVisibility)
{
	ScrollBar->SetUserVisibility(InVisibility);
}

void SScrollBox::SetScrollBarAlwaysVisible(bool InAlwaysVisible)
{
	ScrollBar->SetScrollBarAlwaysVisible(InAlwaysVisible);
}

void SScrollBox::SetScrollBarTrackAlwaysVisible(bool InAlwaysVisible)
{
	ScrollBar->SetScrollBarTrackAlwaysVisible(InAlwaysVisible);
}

void SScrollBox::SetScrollBarThickness(UE::Slate::FDeprecateVector2DParameter InThickness)
{
	ScrollBar->SetThickness(InThickness);
}

void SScrollBox::SetScrollBarPadding(const FMargin& InPadding)
{
	ScrollBarSlotPadding = InPadding;

	if (Orientation == Orient_Vertical)
	{
		if (VerticalScrollBarSlot)
		{
			VerticalScrollBarSlot->SetPadding(ScrollBarSlotPadding);
		}
	}
	else
	{
		if (HorizontalScrollBarSlot)
		{
			HorizontalScrollBarSlot->SetPadding(ScrollBarSlotPadding);
		}
	}
}

void SScrollBox::SetScrollBarRightClickDragAllowed(bool bIsAllowed)
{
	bAllowsRightClickDragScrolling = bIsAllowed;
}

EActiveTimerReturnType SScrollBox::UpdateInertialScroll(double InCurrentTime, float InDeltaTime)
{
	bool bKeepTicking = bIsScrolling;

	if ( bIsScrolling )
	{
		InertialScrollManager.UpdateScrollVelocity(InDeltaTime);
		const float ScrollVelocityLocal = InertialScrollManager.GetScrollVelocity() / CachedGeometry.Scale;

		if (ScrollVelocityLocal != 0.f )
		{
			if ( CanUseInertialScroll(ScrollVelocityLocal) )
			{
				bKeepTicking = true;
				ScrollBy(CachedGeometry, ScrollVelocityLocal * InDeltaTime, AllowOverscroll, false);
			}
			else
			{
				InertialScrollManager.ClearScrollVelocity();
			}
		}
	}

	if ( AllowOverscroll == EAllowOverscroll::Yes )
	{
		// If we are currently in overscroll, the list will need refreshing.
		// Do this before UpdateOverscroll, as that could cause GetOverscroll() to be 0
		if ( Overscroll.GetOverscroll(CachedGeometry) != 0.0f )
		{
			bKeepTicking = true;
		}

		Overscroll.UpdateOverscroll(InDeltaTime);
	}

	TickScrollDelta = 0.f;

	if ( !bKeepTicking )
	{
		bIsScrolling = false;
		bIsScrollingActiveTimerRegistered = false;
		Invalidate(EInvalidateWidget::LayoutAndVolatility);
		UpdateInertialScrollHandle.Reset();
	}

	return bKeepTicking ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}

void SScrollBox::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (!bAllowContentToShrink)
	{
		CachedGeometry = AllottedGeometry;
	}

	if ( bTouchPanningCapture && (FSlateApplication::Get().GetCurrentTime() - LastScrollTime) > 0.10 )
	{
		InertialScrollManager.ClearScrollVelocity();
	}

	// If we needed a widget to be scrolled into view, make that happen.
	if ( ScrollIntoViewRequest )
	{
		ScrollIntoViewRequest(CachedGeometry);
		ScrollIntoViewRequest = nullptr;
	}
	
	const FGeometry ScrollPanelGeometry = FindChildGeometry(CachedGeometry, ScrollPanel.ToSharedRef() );
	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());

	if ( bScrollToEnd )
	{
		DesiredScrollOffset = FMath::Max(ContentSize - GetScrollComponentFromVector(ScrollPanelGeometry.GetLocalSize()), 0.0f);
		bScrollToEnd = false;
	}
	
	// If this scroll box has no size, do not compute a view fraction because it will be wrong and causes pop in when the size is available
	const float ViewFraction = GetViewFraction();
	const float TargetViewOffset = GetViewOffsetFraction();
	
	const float CurrentViewOffset = bAnimateScroll ? FMath::FInterpTo(ScrollBar->DistanceFromTop(), TargetViewOffset, InDeltaTime, ScrollingAnimationInterpolationSpeed) : TargetViewOffset;

	// Update the scrollbar with the clamped version of the offset
	float NewPhysicalOffset = GetScrollComponentFromVector(CurrentViewOffset * GetScrollPanelContentSize());
	if ( AllowOverscroll == EAllowOverscroll::Yes )
	{
		NewPhysicalOffset += Overscroll.GetOverscroll(CachedGeometry);
	}

	const bool bWasScrolling = bIsScrolling;
	bIsScrolling = !FMath::IsNearlyEqual(NewPhysicalOffset, ScrollPanel->PhysicalOffset, 0.001f);

	ScrollPanel->PhysicalOffset = NewPhysicalOffset;
	
	if (bWasScrolling && !bIsScrolling)
	{
		Invalidate(EInvalidateWidget::Layout);
	}
	if (bWasScrolling != bIsScrolling)
	{
		Invalidate(EInvalidateWidget::Volatility);
	}
	
	ScrollBar->SetState(CurrentViewOffset, ViewFraction);
	if (!ScrollBar->IsNeeded())
	{
		// We cannot scroll, so ensure that there is no offset.
		ScrollPanel->PhysicalOffset = 0.0f;
	}

	if (bAllowContentToShrink)
	{
		CachedGeometry = AllottedGeometry;
	}
}

bool SScrollBox::ComputeVolatility() const
{
	return bIsScrolling || IsRightClickScrolling();
}

FReply SScrollBox::OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bEnableTouchScrolling && MouseEvent.IsTouchEvent() && !bFingerOwningTouchInteraction.IsSet())
	{
		// Clear any inertia 
		InertialScrollManager.ClearScrollVelocity();
		// We have started a new interaction; track how far the user has moved since they put their finger down.
		AmountScrolledWhileRightMouseDown = 0;
		PendingScrollTriggerAmount = 0;
		// Someone put their finger down in this list, so they probably want to drag the list.
		bFingerOwningTouchInteraction = MouseEvent.GetPointerIndex();

		Invalidate(EInvalidateWidget::Layout);
	}
	return FReply::Unhandled();
}

FReply SScrollBox::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( !bFingerOwningTouchInteraction.IsSet() )
	{
		EndInertialScrolling();
	}

	if ( MouseEvent.IsTouchEvent() )
	{
		return bConsumePointerInput ? FReply::Handled() : SWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	}
	else
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && ScrollBar->IsNeeded()  && bAllowsRightClickDragScrolling)
		{
			AmountScrolledWhileRightMouseDown = 0;

			Invalidate(EInvalidateWidget::Layout);

			return bConsumePointerInput ? FReply::Handled() : SWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
		}
	}

	return FReply::Unhandled();	
}

FReply SScrollBox::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && bAllowsRightClickDragScrolling)
	{
		if ( !bIsScrollingActiveTimerRegistered && IsRightClickScrolling() )
		{
			// Register the active timer to handle the inertial scrolling
			CachedGeometry = MyGeometry;
			BeginInertialScrolling();
		}

		AmountScrolledWhileRightMouseDown = 0;

		Invalidate(EInvalidateWidget::Layout);

		FReply Reply = bConsumePointerInput ? FReply::Handled().ReleaseMouseCapture() : SWidget::OnMouseButtonUp(MyGeometry, MouseEvent).ReleaseMouseCapture();
		bShowSoftwareCursor = false;

		// If we have mouse capture, snap the mouse back to the closest location that is within the panel's bounds
		if ( HasMouseCapture() )
		{
			FSlateRect PanelScreenSpaceRect = MyGeometry.GetLayoutBoundingRect();
			FVector2f CursorPosition = MyGeometry.LocalToAbsolute( SoftwareCursorPosition );

			FIntPoint BestPositionInPanel(
				FMath::RoundToInt( FMath::Clamp( CursorPosition.X, PanelScreenSpaceRect.Left, PanelScreenSpaceRect.Right ) ),
				FMath::RoundToInt( FMath::Clamp( CursorPosition.Y, PanelScreenSpaceRect.Top, PanelScreenSpaceRect.Bottom ) )
				);

			Reply.SetMousePos(BestPositionInPanel);
		}
		
		return Reply;
	}

	return FReply::Unhandled();
}

FReply SScrollBox::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const float ScrollByAmountScreen = GetScrollComponentFromVector(MouseEvent.GetCursorDelta());
	const float ScrollByAmountLocal = ScrollByAmountScreen / MyGeometry.Scale;

	if ( MouseEvent.IsTouchEvent() )
	{
		FReply Reply = FReply::Unhandled();

		if ( !bTouchPanningCapture )
		{
			if ( bFingerOwningTouchInteraction.IsSet() && MouseEvent.IsTouchEvent() && !HasMouseCapture() )
			{
				PendingScrollTriggerAmount += ScrollByAmountScreen;

				if ( FMath::Abs(PendingScrollTriggerAmount) > FSlateApplication::Get().GetDragTriggerDistance() )
				{
					bTouchPanningCapture = true;
					ScrollBar->BeginScrolling();

					// The user has moved the list some amount; they are probably
					// trying to scroll. From now on, the list assumes the user is scrolling
					// until they lift their finger.
					Reply = bConsumePointerInput ? FReply::Handled().CaptureMouse(AsShared()) : SWidget::OnMouseMove(MyGeometry, MouseEvent).CaptureMouse(AsShared()) ;
				}
				else
				{
					Reply = bConsumePointerInput ? FReply::Handled() : SWidget::OnMouseMove(MyGeometry, MouseEvent);
				}
			}
		}
		else
		{
			if ( bFingerOwningTouchInteraction.IsSet() && HasMouseCaptureByUser(MouseEvent.GetUserIndex(), MouseEvent.GetPointerIndex()) )
			{
				LastScrollTime = FSlateApplication::Get().GetCurrentTime();
				InertialScrollManager.AddScrollSample(-ScrollByAmountScreen, FSlateApplication::Get().GetCurrentTime());
				ScrollBy(MyGeometry, -ScrollByAmountLocal, EAllowOverscroll::Yes, false);

				Reply = bConsumePointerInput ? FReply::Handled() : SWidget::OnMouseMove(MyGeometry, MouseEvent);
			}
		}

		return Reply;
	}
	else
	{
		if ( MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton)  && bAllowsRightClickDragScrolling)
		{
			// If scrolling with the right mouse button, we need to remember how much we scrolled.
			// If we did not scroll at all, we will bring up the context menu when the mouse is released.
			AmountScrolledWhileRightMouseDown += FMath::Abs(ScrollByAmountScreen);

			// Has the mouse moved far enough with the right mouse button held down to start capturing
			// the mouse and dragging the view?
			if ( IsRightClickScrolling() )
			{
				InertialScrollManager.AddScrollSample(-ScrollByAmountScreen, FPlatformTime::Seconds());
				const bool bDidScroll = ScrollBy(MyGeometry, -ScrollByAmountLocal, AllowOverscroll, false);

				FReply Reply = bConsumePointerInput ? FReply::Handled() : SWidget::OnMouseMove(MyGeometry, MouseEvent);

				// Capture the mouse if we need to
				if ( HasMouseCapture() == false )
				{
					Reply.CaptureMouse(AsShared()).UseHighPrecisionMouseMovement(AsShared());
					SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
					bShowSoftwareCursor = true;
				}

				// Check if the mouse has moved.
				if ( bDidScroll )
				{
					SetScrollComponentOnVector(SoftwareCursorPosition, GetScrollComponentFromVector(SoftwareCursorPosition) + ScrollByAmountLocal);
				}

				return Reply;
			}
		}
	}

	return FReply::Unhandled();
}

void SScrollBox::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bEnableTouchScrolling && MouseEvent.IsTouchEvent())
	{
		if ( !bFingerOwningTouchInteraction.IsSet() )
		{
			// If we currently do not have touch capture, allow this widget to begin scrolling on pointer enter events
			// if it comes from a child widget
			if ( MyGeometry.IsUnderLocation(MouseEvent.GetLastScreenSpacePosition()) )
			{
				bFingerOwningTouchInteraction = MouseEvent.GetPointerIndex();
			}
		}
	}
}

void SScrollBox::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	if ( HasMouseCapture() == false )
	{
		// No longer scrolling (unless we have mouse capture)
		if ( AmountScrolledWhileRightMouseDown != 0 )
		{
			AmountScrolledWhileRightMouseDown = 0;
			Invalidate(EInvalidateWidget::Layout);
		}

		if ( MouseEvent.IsTouchEvent() )
		{
			bFingerOwningTouchInteraction.Reset();
		}
	}
}

FReply SScrollBox::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ((ScrollBar->IsNeeded() && ConsumeMouseWheel != EConsumeMouseWheel::Never) || ConsumeMouseWheel == EConsumeMouseWheel::Always)
	{
		// Make sure scroll velocity is cleared so it doesn't fight with the mouse wheel input
		InertialScrollManager.ClearScrollVelocity();

		const bool bScrollWasHandled = ScrollBy(MyGeometry, -MouseEvent.GetWheelDelta() * GetGlobalScrollAmount() * WheelScrollMultiplier, EAllowOverscroll::No, bAnimateWheelScrolling);

		if ( bScrollWasHandled && !bIsScrollingActiveTimerRegistered )
		{
			// Register the active timer to handle the inertial scrolling
			CachedGeometry = MyGeometry;
			BeginInertialScrolling();
		}

		return bScrollWasHandled ? FReply::Handled() : FReply::Unhandled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SScrollBox::OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent)
{
	const float AnalogInputThreshold = 0.2f;
	if ((ScrollBar->IsNeeded() && ConsumeMouseWheel != EConsumeMouseWheel::Never) || ConsumeMouseWheel == EConsumeMouseWheel::Always)
	{
		const FKey& AnalogKey   = InAnalogInputEvent.GetKey();
		const float AnalogValue = InAnalogInputEvent.GetAnalogValue();

		// If not the correct input, or not within input thresholds, ignore.
		if (AnalogKey != AnalogMouseWheelKey || (AnalogValue < AnalogInputThreshold && AnalogValue > -AnalogInputThreshold))
		{
			return FReply::Unhandled();
		}
		
		// Make sure scroll velocity is cleared so it doesn't fight with the mouse wheel input
		InertialScrollManager.ClearScrollVelocity();

		const float ScrollAmount = GetGlobalScrollAmount() * GetGlobalGamepadScrollMultiplier() * WheelScrollMultiplier;
		const bool bScrollWasHandled = ScrollBy(MyGeometry, -AnalogValue * ScrollAmount, EAllowOverscroll::No, bAnimateWheelScrolling);

		if ( bScrollWasHandled && !bIsScrollingActiveTimerRegistered )
		{
			// Register the active timer to handle the inertial scrolling
			CachedGeometry = MyGeometry;
			BeginInertialScrolling();
		}

		return bScrollWasHandled ? FReply::Handled() : FReply::Unhandled();
	}
	return FReply::Unhandled();
}

bool SScrollBox::ScrollBy(const FGeometry& AllottedGeometry, float LocalScrollAmount, EAllowOverscroll Overscrolling, bool InAnimateScroll)
{
	Invalidate(EInvalidateWidget::LayoutAndVolatility);

	bAnimateScroll = InAnimateScroll;

	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());
	const FGeometry ScrollPanelGeometry = FindChildGeometry( AllottedGeometry, ScrollPanel.ToSharedRef() );

	const float PreviousScrollOffset = DesiredScrollOffset;

	if (LocalScrollAmount != 0 )
	{
		const float ScrollMin = 0.0f;
		const float ScrollMax = FMath::Max(ContentSize - GetScrollComponentFromVector(ScrollPanelGeometry.GetLocalSize()), 0.0f);

		if ( AllowOverscroll == EAllowOverscroll::Yes && Overscrolling == EAllowOverscroll::Yes && Overscroll.ShouldApplyOverscroll(DesiredScrollOffset == 0, DesiredScrollOffset == ScrollMax, LocalScrollAmount) )
		{
			Overscroll.ScrollBy(AllottedGeometry, LocalScrollAmount);
		}
		else
		{
			DesiredScrollOffset = FMath::Clamp(DesiredScrollOffset + LocalScrollAmount, ScrollMin, ScrollMax);
		}
	}

	OnUserScrolled.ExecuteIfBound(DesiredScrollOffset);

	return ConsumeMouseWheel == EConsumeMouseWheel::Always || DesiredScrollOffset != PreviousScrollOffset;
}

FCursorReply SScrollBox::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if ( IsRightClickScrolling() )
	{
		// We hide the native cursor as we'll be drawing the software EMouseCursor::GrabHandClosed cursor
		return FCursorReply::Cursor( EMouseCursor::None );
	}
	else
	{
		return FCursorReply::Unhandled();
	}
}

FReply SScrollBox::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	CachedGeometry = MyGeometry;

	if ( HasMouseCaptureByUser(InTouchEvent.GetUserIndex(), InTouchEvent.GetPointerIndex()) )
	{
		ScrollBar->EndScrolling();
		Invalidate(EInvalidateWidget::Layout);
		
		BeginInertialScrolling();

		return bConsumePointerInput ? FReply::Handled().ReleaseMouseCapture() : SWidget::OnTouchEnded(MyGeometry, InTouchEvent).ReleaseMouseCapture() ;
	}

	return FReply::Unhandled();
}

void SScrollBox::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SCompoundWidget::OnMouseCaptureLost(CaptureLostEvent);
	AmountScrolledWhileRightMouseDown = 0;
	PendingScrollTriggerAmount = 0;
	bFingerOwningTouchInteraction.Reset();
	bTouchPanningCapture = false;

}

FNavigationReply SScrollBox::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	TSharedPtr<FNavigationMetaData> NavMetaData = GetMetaData<FNavigationMetaData>();
	if (NavMetaData.IsValid())
	{
		TInstancedStruct<FNavigationMethod>& NavigationMethodInstance = NavMetaData->GetMutableNavigationMethodStruct();
		FNavigationMethod* NavigationMethod = NavigationMethodInstance.IsValid() ? NavigationMethodInstance.GetMutablePtr() : nullptr;
		if (NavigationMethod)
		{
			TSharedPtr<SWidget> NextFocusableChild = ScrollBoxFocus::GetNextWidget(MyGeometry, GetScrollPanelContentSize(), AsShared(), InNavigationEvent.GetUserIndex(), InNavigationEvent.GetNavigationType(), *NavigationMethod);
			if (NextFocusableChild.IsValid())
			{
				InternalScrollDescendantIntoView(MyGeometry, NextFocusableChild, false, NavigationDestination, NavigationScrollPadding);
				return FNavigationReply::Explicit(NextFocusableChild);
			}
		}
	}

	return OnNavigation_Default(MyGeometry, InNavigationEvent);
}

FNavigationReply SScrollBox::OnNavigation_Default(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	TSharedPtr<SWidget> FocusedChild;
	int32 FocusedChildIndex = -1;
	int32 FocusedChildDirection = 0;

	// Find the child with focus currently so that we can find the next logical child we're going to move to.
	TPanelChildren<SScrollBox::FSlot>& Children = ScrollPanel->Children;
	for ( int32 SlotIndex=0; SlotIndex < Children.Num(); ++SlotIndex )
	{
		if ( Children[SlotIndex].GetWidget()->HasUserFocus(InNavigationEvent.GetUserIndex()).IsSet() ||
			 Children[SlotIndex].GetWidget()->HasUserFocusedDescendants(InNavigationEvent.GetUserIndex()) )
		{
			FocusedChild = Children[SlotIndex].GetWidget();
			FocusedChildIndex = SlotIndex;
			break;
		}
	}

	if ( FocusedChild.IsValid() )
	{
		if ( Orientation == Orient_Vertical )
		{
			switch ( InNavigationEvent.GetNavigationType() )
			{
			case EUINavigation::Up:
				FocusedChildDirection = -1;
				break;
			case EUINavigation::Down:
				FocusedChildDirection = 1;
				break;
			default:
				// If we don't handle this direction in our current orientation we can 
				// just allow the behavior of the boundary rule take over.
				return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
			}
		}
		else // Orient_Horizontal
		{
			switch ( InNavigationEvent.GetNavigationType() )
			{
			case EUINavigation::Left:
				FocusedChildDirection = -1;
				break;
			case EUINavigation::Right:
				FocusedChildDirection = 1;
				break;
			default:
				// If we don't handle this direction in our current orientation we can 
				// just allow the behavior of the boundary rule take over.
				return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
			}
		}

		// If the focused child index is in a valid range we know we can successfully focus
		// the new child we're moving focus to.
		if ( FocusedChildDirection != 0 )
		{
			TSharedPtr<SWidget> NextFocusableChild;

			// Search in the direction we need to move for the next focusable child of the scrollbox.
			for ( int32 ChildIndex = FocusedChildIndex + FocusedChildDirection; ChildIndex >= 0 && ChildIndex < Children.Num(); ChildIndex += FocusedChildDirection )
			{
				TSharedPtr<SWidget> PossiblyFocusableChild = GetKeyboardFocusableWidget(Children[ChildIndex].GetWidget());
				if ( PossiblyFocusableChild.IsValid() )
				{
					NextFocusableChild = PossiblyFocusableChild;
					break;
				}
			}

			// If we found a focusable child, scroll to it, and shift focus.
			if ( NextFocusableChild.IsValid() )
			{
				InternalScrollDescendantIntoView(MyGeometry, NextFocusableChild, false, NavigationDestination, NavigationScrollPadding);
				return FNavigationReply::Explicit(NextFocusableChild);
			}
		}
	}

	return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
}

FSlateRect SScrollBox::GetHitTestBoundingRect() const
{
	const FGeometry WidgetGeo = GetPaintSpaceGeometry();
	FSlateRect RenderRect = WidgetGeo.GetRenderBoundingRect();

	const FVector2f ContentSize = GetScrollPanelContentSize();
	const FVector2f TransformedContentSize = WidgetGeo.GetAccumulatedLayoutTransform().TransformVector(ContentSize);

	RenderRect.Bottom = RenderRect.Top + TransformedContentSize.Y;
	RenderRect.Right = RenderRect.Left + TransformedContentSize.X;

	if (Orientation == EOrientation::Orient_Vertical)
	{
		RenderRect.Top -= DesiredScrollOffset;
	}
	else
	{
		RenderRect.Left -= DesiredScrollOffset;
	}

	return RenderRect;
}

void SScrollBox::OnAddedMetadata(const TSharedRef<ISlateMetaData>& AddedMetaData)
{
	if (AddedMetaData->IsOfType<FNavigationMetaData>())
	{
		bNavigationDataExists = true;
	}
}

void SScrollBox::OnRemovedMetadata(const TSharedRef<ISlateMetaData>& RemovedMetaData)
{
	if (RemovedMetaData->IsOfType<FNavigationMetaData>())
	{
		bNavigationDataExists = false;
	}
}

FReply SScrollBox::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if (OnScrollBoxFocusReceived.IsBound())
	{
		OnScrollBoxFocusReceived.ExecuteIfBound();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SScrollBox::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	OnScrollBoxFocusLost.ExecuteIfBound();
}

void SScrollBox::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	if (ScrollWhenFocusChanges != EScrollWhenFocusChanges::NoScroll)
	{
		if (NewWidgetPath.IsValid() && NewWidgetPath.ContainsWidget(this))
		{
			ScrollDescendantIntoView(NewWidgetPath.GetLastWidget(), ScrollWhenFocusChanges == EScrollWhenFocusChanges::AnimatedScroll ? true : false, NavigationDestination, NavigationScrollPadding);
		}
	}
	OnScrollBoxFocusChanging.ExecuteIfBound(PreviousFocusPath, NewWidgetPath);
}

TSharedPtr<SWidget> SScrollBox::GetKeyboardFocusableWidget(TSharedPtr<SWidget> InWidget)
{
	if (EVisibility::DoesVisibilityPassFilter(InWidget->GetVisibility(), EVisibility::Visible))
	{
		if (InWidget->SupportsKeyboardFocus())
		{
			return InWidget;
		}
		else
		{
			FChildren* Children = InWidget->GetChildren();
			for (int32 i = 0; i < Children->Num(); ++i)
			{
				TSharedPtr<SWidget> ChildWidget = Children->GetChildAt(i);
				TSharedPtr<SWidget> FoucusableWidget = GetKeyboardFocusableWidget(ChildWidget);
				if (FoucusableWidget.IsValid() && EVisibility::DoesVisibilityPassFilter(FoucusableWidget->GetVisibility(), EVisibility::Visible))
				{
					return FoucusableWidget;
				}
			}
		}
	}
	return nullptr;
}

int32 SScrollBox::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayerId = SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	if( !bShowSoftwareCursor )
	{
		return NewLayerId;
	}

	const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_Grab"));
	const FVector2f CursorSize = Brush->ImageSize / AllottedGeometry.Scale;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		++NewLayerId,
		AllottedGeometry.ToPaintGeometry( CursorSize, FSlateLayoutTransform(SoftwareCursorPosition - (CursorSize *.5f )) ),
		Brush
	);

	return NewLayerId;
}

void SScrollBox::ScrollBar_OnUserScrolled( float InScrollOffsetFraction )
{
	bAnimateScroll = false;

	const float ContentSize = GetScrollComponentFromVector(GetScrollPanelContentSize());
	const FGeometry ScrollPanelGeometry = FindChildGeometry(CachedGeometry, ScrollPanel.ToSharedRef());

	// Clamp to max scroll offset
	DesiredScrollOffset = FMath::Min(InScrollOffsetFraction * ContentSize, ContentSize - GetScrollComponentFromVector(ScrollPanelGeometry.GetLocalSize()));
	OnUserScrolled.ExecuteIfBound(DesiredScrollOffset);

	Invalidate(EInvalidateWidget::Layout);
}

void SScrollBox::ScrollBar_OnScrollBarVisibilityChanged( EVisibility NewVisibility )
{
	OnScrollBarVisibilityChanged.ExecuteIfBound(NewVisibility);
}

const float ShadowFadeDistance = 32.0f;

FSlateColor SScrollBox::GetStartShadowOpacity() const
{
	// The shadow should only be visible when the user needs a hint that they can scroll up.
	const float ShadowOpacity = FMath::Clamp( ScrollPanel->PhysicalOffset/ShadowFadeDistance, 0.0f, 1.0f);
	
	return FLinearColor(1.0f, 1.0f, 1.0f, ShadowOpacity);
}

FSlateColor SScrollBox::GetEndShadowOpacity() const
{
	// The shadow should only be visible when the user needs a hint that they can scroll down.
	const float ShadowOpacity = (ScrollBar->DistanceFromBottom() * GetScrollComponentFromVector(GetScrollPanelContentSize()) / ShadowFadeDistance);
	
	return FLinearColor(1.0f, 1.0f, 1.0f, ShadowOpacity);
}

bool SScrollBox::CanUseInertialScroll(float ScrollAmount) const
{
	const auto CurrentOverscroll = Overscroll.GetOverscroll(CachedGeometry);

	// We allow sampling for the inertial scroll if we are not in the overscroll region,
	// Or if we are scrolling outwards of the overscroll region
	return CurrentOverscroll == 0.f || FMath::Sign(CurrentOverscroll) != FMath::Sign(ScrollAmount);
}

EAllowOverscroll SScrollBox::GetAllowOverscroll() const
{
	return AllowOverscroll;
}

void SScrollBox::SetAllowOverscroll(EAllowOverscroll NewAllowOverscroll)
{
	AllowOverscroll = NewAllowOverscroll;

	if (AllowOverscroll == EAllowOverscroll::No)
	{
		Overscroll.ResetOverscroll();
	}
}

void SScrollBox::SetAnimateWheelScrolling(bool bInAnimateWheelScrolling)
{
	bAnimateWheelScrolling = bInAnimateWheelScrolling;
}

void SScrollBox::SetScrollingAnimationInterpolationSpeed(float NewScrollingAnimationInterpolationSpeed)
{
	ScrollingAnimationInterpolationSpeed = NewScrollingAnimationInterpolationSpeed;
}

void SScrollBox::SetWheelScrollMultiplier(float NewWheelScrollMultiplier)
{
	WheelScrollMultiplier = NewWheelScrollMultiplier;
}

void SScrollBox::SetIsTouchScrollingEnabled(const bool bInEnableTouchScrolling)
{
	bEnableTouchScrolling = bInEnableTouchScrolling;
	ensureMsgf(!bFingerOwningTouchInteraction.IsSet(), TEXT("TouchScrollingEnabled flag should not be changed while scrolling."));
}

void SScrollBox::SetScrollWhenFocusChanges(EScrollWhenFocusChanges NewScrollWhenFocusChanges)
{
	ScrollWhenFocusChanges = NewScrollWhenFocusChanges;
}

void SScrollBox::BeginInertialScrolling()
{
	if ( !UpdateInertialScrollHandle.IsValid() )
	{
		bIsScrolling = true;
		bIsScrollingActiveTimerRegistered = true;
		UpdateInertialScrollHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SScrollBox::UpdateInertialScroll));
		Invalidate(EInvalidateWidget::LayoutAndVolatility);
	}
}

void SScrollBox::EndInertialScrolling()
{
	bIsScrolling = false;
	bIsScrollingActiveTimerRegistered = false;
	Invalidate(EInvalidateWidget::LayoutAndVolatility);
	if ( UpdateInertialScrollHandle.IsValid() )
	{
		UnRegisterActiveTimer(UpdateInertialScrollHandle.ToSharedRef());
		UpdateInertialScrollHandle.Reset();
	}

	// Zero the scroll velocity so the panel stops immediately on mouse down, even if the user does not drag
	InertialScrollManager.ClearScrollVelocity();

	// Sets the desired scroll offset to the current one so the panel does not continue moving
	DesiredScrollOffset = ScrollPanel->PhysicalOffset;
}
