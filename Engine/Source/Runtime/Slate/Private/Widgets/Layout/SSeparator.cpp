// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SSeparator.h"

/////////////////////////////////////////////////////
// SSeparator

// Construct this widget
void SSeparator::Construct( const FArguments& InArgs )
{
	SBorder::Construct(
		SBorder::FArguments()
		.BorderImage(InArgs._SeparatorImage)
		.Padding(0.0f)
		.BorderBackgroundColor(InArgs._ColorAndOpacity)
	);

	Orientation = InArgs._Orientation;
	Thickness = InArgs._Thickness;
}

FVector2D SSeparator::ComputeDesiredSize( float ) const
{
	const float Length = 16.0f;
	return (Orientation == Orient_Horizontal) ? FVector2D(Length, Thickness) : FVector2D(Thickness, Length);
}
