// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SSpacer.h"

SLATE_IMPLEMENT_WIDGET(SSpacer)
void SSpacer::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, SpacerSize, EInvalidateWidgetReason::Layout);
}

SSpacer::SSpacer()
	: SpacerSize(*this, FVector2D::ZeroVector)
	, bIsSpacerSizeBound(false)
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

void SSpacer::Construct( const FArguments& InArgs )
{
	SetSize(InArgs._Size);
}


int32 SSpacer::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// We did not paint anything. The parent's current LayerId is probably the max we were looking for.
	return LayerId;
}

FVector2D SSpacer::ComputeDesiredSize( float ) const
{
	return SpacerSize.Get();
}
