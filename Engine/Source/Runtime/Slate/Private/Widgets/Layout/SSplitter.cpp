// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SSplitter.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"


SLATE_IMPLEMENT_WIDGET(SSplitter)
void SSplitter::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FSlot::RegisterAttributes(Initializer);
}

SSplitter::FSlot::FSlot()
 : TWidgetSlotWithAttributeSupport<FSlot>()
 , SizingRule(*this, FractionOfParent)
 , SizeValue(*this, 1.f)
 , MinSizeValue(*this, 0.f)
{
}

void SSplitter::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	TWidgetSlotWithAttributeSupport<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));

	SetSizingRule( MoveTemp(InArgs._SizeRule) );
	SetSizeValue( MoveTemp(InArgs._Value) );
	SetMinSize( MoveTemp(InArgs._MinSize) );

	if (InArgs._Resizable.IsSet())
	{
		bIsResizable = InArgs._Resizable.GetValue();
	}

	if (InArgs._OnSlotResized.IsBound())
	{
		OnSlotResized_Handler = MoveTemp(InArgs._OnSlotResized);
	}
}

void SSplitter::FSlot::RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
{
	TWidgetSlotWithAttributeSupport<FSlot>::RegisterAttributes(AttributeInitializer);

	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION(FSlot, AttributeInitializer, SizingRule, EInvalidateWidgetReason::Layout);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION(FSlot, AttributeInitializer, SizeValue, EInvalidateWidgetReason::Layout);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION(FSlot, AttributeInitializer, MinSizeValue, EInvalidateWidgetReason::Layout);
}

void SSplitter::FSlot::SetSizeValue( TAttribute<float> InValue )
{
	SizeValue.Assign(*this, MoveTemp(InValue));
}

void SSplitter::FSlot::SetSizingRule( TAttribute<ESizeRule> InSizeRule ) 
{
	SizingRule.Assign(*this, MoveTemp(InSizeRule));
}

void SSplitter::FSlot::SetMinSize(TAttribute<float> InMinSize)
{
	MinSizeValue.Assign(*this, MoveTemp(InMinSize));
}

bool SSplitter::FSlot::CanBeResized() const
{
	const ESizeRule CurrentSizingRule = SizingRule.Get();
	if (CurrentSizingRule == SSplitter::ESizeRule::FractionOfParent)
	{
		return !bIsResizable.IsSet() || bIsResizable.GetValue();
	}
	else
	{
		return bIsResizable.IsSet() && bIsResizable.GetValue() && OnSlotResized_Handler.IsBound();
	}
}

SSplitter::FSlot::FSlotArguments SSplitter::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

SSplitter::FScopedWidgetSlotArguments SSplitter::AddSlot( int32 AtIndex )
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Children, AtIndex };
}

SSplitter::FSlot& SSplitter::SlotAt( int32 SlotIndex )
{
	return Children[SlotIndex];
}

int32 SSplitter::RemoveSlot(const TSharedRef<SWidget>& SlotWidget)
{
	return Children.Remove(SlotWidget);
}

void SSplitter::RemoveAt( int32 IndexToRemove )
{
	Children.RemoveAt( IndexToRemove );
}

void SSplitter::ClearChildren()
{
	Children.Empty();
}

int32 SSplitter::NumSlots() const
{
	return Children.Num();
}

bool SSplitter::IsValidSlotIndex(int32 Index) const
{
	return Children.IsValidIndex(Index);
}

/**
* Construct this widget
*
* @param	InArgs	The declaration data for this widget
*/
void SSplitter::Construct( const SSplitter::FArguments& InArgs )
{
	check(InArgs._Style);

	OnHandleHovered = InArgs._OnHandleHovered;
	OnSplitterFinishedResizing = InArgs._OnSplitterFinishedResizing;
	ResizeMode = InArgs._ResizeMode;
	PhysicalSplitterHandleSize = InArgs._PhysicalSplitterHandleSize;
	HitDetectionSplitterHandleSize = InArgs._HitDetectionSplitterHandleSize;
	Orientation = InArgs._Orientation;
	HoveredHandleIndex = INDEX_NONE;
	HighlightedHandleIndex.Assign(*this, InArgs._HighlightedHandleIndex);
	bIsResizing = false;
	Style = InArgs._Style;
	OnGetMaxSlotSize = InArgs._OnGetMaxSlotSize;
	MinSplitterChildLength = InArgs._MinimumSlotHeight;
	Children.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));
}

TArray<FLayoutGeometry> SSplitter::ArrangeChildrenForLayout(const FGeometry& AllottedGeometry) const
{
	TArray<FLayoutGeometry> ResultGeometries;
	ResultGeometries.Empty(Children.Num());

	const int32 AxisIndex = (Orientation == Orient_Horizontal) ? 0 : 1;

	// Splitters divide the space between their children proportionately based on size coefficients.
	// The size coefficients are usually determined by a user, who grabs the handled between the child elements
	// and moves them to resize the space available to the children.
	// Some children are sized automatically based on their content; those children cannot be resized.
	//
	// e.g.   _____________________________________ Children
	//       /              /                  /
	//      v              v                  v
	//   + - - - - - + + - - - + + - - - - - - - - - - - - - - +
	//   |           | |       | |                             |
	//   | Child 0   | |Child1 | |  Child2                     |
	//   + - - - - - + + - - - + + - - - - - - - - - - - - - - +
	//                ^         ^
	//                 \_________\___________ Resize handles.

	int32 NumNonCollapsedChildren = 0;
	int32 NumResizeableChildren = 0;
	float CoefficientTotal = 0;
	// Some space is claimed by non-resizable elements (auto-sized elements)
	float NonResizableSpace = 0;
	float MinResizableSpace = 0;

	auto IsSlotCollapsed = [](const FSlot& Slot)
	{
		return Slot.GetWidget()->GetVisibility() == EVisibility::Collapsed;
	};

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FSlot& CurSlot = Children[ChildIndex];
		if (IsSlotCollapsed(CurSlot))
		{
			continue;
		}

		++NumNonCollapsedChildren;

		if (CurSlot.GetSizingRule() == SSplitter::SizeToContent)
		{
			NonResizableSpace += CurSlot.GetWidget()->GetDesiredSize()[AxisIndex];
		}
		else // SizingRule == SSplitter::FractionOfParent
		{
			MinResizableSpace += FMath::Max(MinSplitterChildLength, CurSlot.GetMinSize());
			CoefficientTotal += CurSlot.GetSizeValue();
		}
	}

	// The user-sizable children must make room for the resize handles and for auto-sized children.
	const float SpaceNeededForHandles = FMath::Max(0, NumNonCollapsedChildren - 1) * PhysicalSplitterHandleSize;
	const float ResizableSpace = AllottedGeometry.Size.Component(AxisIndex) - SpaceNeededForHandles - NonResizableSpace;

	TArray<float, FConcurrentLinearArrayAllocator> SlotSizes;
	SlotSizes.Empty(Children.Num());

	float ExtraRequiredSpace = 0;

	// calculate slot sizes
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FSlot& CurSlot = Children[ChildIndex];

		// slot is collapsed, size is automatically 0
		if (IsSlotCollapsed(CurSlot))
		{
			SlotSizes.Add(0);
			continue;
		}

		float ChildSpace;
		
		if (CurSlot.GetSizingRule() == SSplitter::SizeToContent)
		{
			ChildSpace = CurSlot.GetWidget()->GetDesiredSize()[AxisIndex];
		}
		else
		{
			ChildSpace = ResizableSpace * CurSlot.GetSizeValue() / CoefficientTotal  - ExtraRequiredSpace;
			ExtraRequiredSpace = 0;
		}


		// we have enough space to clamp this and steal space from earlier slots
		if (ResizableSpace >= MinResizableSpace)
		{
			const float ClampedSpace = ClampChild(CurSlot, ChildSpace);
			float CurrentRequiredSpace = ClampedSpace - ChildSpace;

			for (int32 PrevIndex = ChildIndex - 1; PrevIndex >= 0 && CurrentRequiredSpace > 0; --PrevIndex)
			{
				if (Children[PrevIndex].GetSizingRule() == SSplitter::FractionOfParent && !IsSlotCollapsed(CurSlot))
				{
					const float MinChildSize = FMath::Max(MinSplitterChildLength, Children[PrevIndex].GetMinSize());
					const float AvailableSpace = SlotSizes[PrevIndex] - MinChildSize;
					if (AvailableSpace > 0)
					{
						const float SpaceToSteal = FMath::Min(AvailableSpace, CurrentRequiredSpace);
						SlotSizes[PrevIndex] -= SpaceToSteal;
						CurrentRequiredSpace -= SpaceToSteal;
					}
				}
			}
			
			if (CurrentRequiredSpace > 0)
			{
				// we weren't able to steal enough space from previous children, store this value to try and take it from following children
				ExtraRequiredSpace = CurrentRequiredSpace;
			}

			ChildSpace = ClampedSpace;
		}

		SlotSizes.Add(ChildSpace);
	}

	// create geometries and calculate offsets
	float Offset = 0;
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FVector2D ChildOffset = Orientation == Orient_Horizontal ? FVector2D(Offset, 0) : FVector2D(0, Offset);
		const FVector2D ChildSize = Orientation == Orient_Horizontal ? FVector2D(SlotSizes[ChildIndex], AllottedGeometry.Size.Y) : FVector2D(AllottedGeometry.Size.X, SlotSizes[ChildIndex]);
		ResultGeometries.Emplace(FSlateLayoutTransform(ChildOffset), ChildSize);

		// Advance to the next slot. If the child is collapsed, it takes up no room and does not need a splitter
		if (!IsSlotCollapsed(Children[ChildIndex]))
		{
			Offset += FMath::RoundToInt(SlotSizes[ChildIndex] + PhysicalSplitterHandleSize);
		}
	}

	return ResultGeometries;
}

/**
* Panels arrange their children in a space described by the AllottedGeometry parameter. The results of the arrangement
* should be returned by appending a FArrangedWidget pair for every child widget. See StackPanel for an example
*/
void SSplitter::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(AllottedGeometry);

	// Arrange the children horizontally or vertically.
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild( Children[ChildIndex].GetWidget(), LayoutChildren[ChildIndex] ) );
	}
}


int32 SSplitter::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren( EVisibility::Visible );
	ArrangeChildren( AllottedGeometry, ArrangedChildren );

	int32 MaxLayerId = PaintArrangedChildren( Args, ArrangedChildren, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	const FSlateBrush* NormalHandleBrush = &Style->HandleNormalBrush;

	// Draw the splitter above any children
	MaxLayerId += 1;

	// HandleIndex is an index into our Children array, but some of those may be hidden and missing from ArrangedChildren so we need to update the index for the loop below
	auto FindArrangedHandleIndex = [this, &ArrangedChildren](int32 HandleIndex)
	{
		if (HandleIndex == INDEX_NONE || ArrangedChildren.Num() == Children.Num())
		{
			return HandleIndex;
		}

		const TSharedRef<SWidget>& HoveredChildToFind = Children[HandleIndex].GetWidget();
		return ArrangedChildren.IndexOfByPredicate([&HoveredChildToFind](const FArrangedWidget& InArrangedWidget)
		{
			return InArrangedWidget.Widget == HoveredChildToFind;
		});
	};

	int32 ArrangedHoveredHandleIndex = FindArrangedHandleIndex(HoveredHandleIndex);
	int32 ArrangedHighlightedHandleIndex = FindArrangedHandleIndex(HighlightedHandleIndex.Get());

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num() - 1; ++ChildIndex)
	{
		const FGeometry& GeometryAfterSplitter = ArrangedChildren[ FMath::Clamp(ChildIndex + 1, 0, ArrangedChildren.Num()-1) ].Geometry;

		const float HalfHitDetectionSplitterHandleSize = FMath::RoundToFloat(HitDetectionSplitterHandleSize / 2.0f);
		const float HalfPhysicalSplitterHandleSize = PhysicalSplitterHandleSize;// FMath::RoundToInt(PhysicalSplitterHandleSize / 2.0f);

		FVector2f HandleSize;
		FVector2f HandlePosition;
		if (Orientation == Orient_Horizontal)
		{
			HandleSize.Set( PhysicalSplitterHandleSize, GeometryAfterSplitter.Size.Y );
			HandlePosition.Set( -PhysicalSplitterHandleSize/*-(HalfHitDetectionSplitterHandleSize + HalfPhysicalSplitterHandleSize)*/, 0 );
		}
		else
		{
			HandleSize.Set( GeometryAfterSplitter.Size.X, PhysicalSplitterHandleSize );
			HandlePosition.Set( 0, -PhysicalSplitterHandleSize/*-(HalfHitDetectionSplitterHandleSize + HalfPhysicalSplitterHandleSize) */);
		}

		if (ArrangedHoveredHandleIndex == ChildIndex || ArrangedHighlightedHandleIndex == ChildIndex)
		{
			// handle is hovered
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				MaxLayerId,
				GeometryAfterSplitter.ToPaintGeometry(HandleSize, FSlateLayoutTransform(1.0f, TransformPoint(1.0f, HandlePosition))),
				NormalHandleBrush,
				ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				InWidgetStyle.GetColorAndOpacityTint() * Style->HandleHighlightBrush.TintColor.GetSpecifiedColor()
			);
		}
		else
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				MaxLayerId,
				GeometryAfterSplitter.ToPaintGeometry(HandleSize, FSlateLayoutTransform(1.0f, TransformPoint(1.0f, HandlePosition))),
				&Style->HandleHighlightBrush,
				ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				InWidgetStyle.GetColorAndOpacityTint() * NormalHandleBrush->TintColor.GetSpecifiedColor()
			);	
		}
	}

	return MaxLayerId;
}

template<EOrientation Orientation>
static FVector2D ComputeDesiredSizeForSplitter( const float PhysicalSplitterHandleSize, const TPanelChildren<SSplitter::FSlot>& Children )
{
	FVector2D MyDesiredSize(0, 0);

	int32 NumNonCollapsed = 0;
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const SSplitter::FSlot& CurSlot = Children[ChildIndex];
		const EVisibility ChildVisibility = CurSlot.GetWidget()->GetVisibility();
		if ( ChildVisibility != EVisibility::Collapsed )
		{
			++NumNonCollapsed;

			FVector2D ChildDesiredSize( CurSlot.GetWidget()->GetDesiredSize() );
			if ( Orientation == Orient_Horizontal )
			{
				MyDesiredSize.X += ChildDesiredSize.X;
				MyDesiredSize.Y = FMath::Max(ChildDesiredSize.Y, MyDesiredSize.Y);
			}
			else
			{
				MyDesiredSize.X = FMath::Max(ChildDesiredSize.X, MyDesiredSize.X);				
				MyDesiredSize.Y += ChildDesiredSize.Y;
			}
		}
	}

	float SpaceNeededForHandles = FMath::Max(0, NumNonCollapsed-1) * PhysicalSplitterHandleSize;
	if (Orientation == Orient_Horizontal)
	{
		MyDesiredSize.X += SpaceNeededForHandles;
	}
	else
	{
		MyDesiredSize.Y += SpaceNeededForHandles;
	}

	return MyDesiredSize;
}

/**
* A Panel's desired size in the space required to arrange of its children on the screen while respecting all of
* the children's desired sizes and any layout-related options specified by the user. See StackPanel for an example.
*/
FVector2D SSplitter::ComputeDesiredSize( float ) const
{
	FVector2D MyDesiredSize = (Orientation == Orient_Horizontal)
		? ComputeDesiredSizeForSplitter<Orient_Horizontal>( PhysicalSplitterHandleSize, Children )
		: ComputeDesiredSizeForSplitter<Orient_Vertical>( PhysicalSplitterHandleSize, Children );

	return MyDesiredSize;
}

/**
* All widgets must provide a way to access their children in a layout-agnostic way.
* Panels store their children in Slots, which creates a dilemma. Most panels
* can store their children in a TPanelChildren<Slot>, where the Slot class
* provides layout information about the child it stores. In that case
* GetChildren should simply return the TPanelChildren<Slot>. See StackPanel for an example.
*/
FChildren* SSplitter::GetChildren()
{
	return &Children;
}

/**
* The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
*
* @param MyGeometry The Geometry of the widget receiving the event
* @param MouseEvent Information about the input event
*
* @return Whether the event was handled along with possible requests for the system to take action.
*/
FReply SSplitter::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HoveredHandleIndex != INDEX_NONE )
	{
		bIsResizing = true;
		return FReply::Handled().CaptureMouse( SharedThis(this) );
	}
	else
	{
		return FReply::Unhandled();
	}
}

/**
* The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
*
* @param MyGeometry The Geometry of the widget receiving the event
* @param MouseEvent Information about the input event
*
* @return Whether the event was handled along with possible requests for the system to take action.
*/
FReply SSplitter::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsResizing == true )
	{
		OnSplitterFinishedResizing.ExecuteIfBound();

		bIsResizing = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

/**
* The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
*
* @param MyGeometry The Geometry of the widget receiving the event
* @param MouseEvent Information about the input event
*
* @return Whether the event was handled along with possible requests for the system to take action.
*/
FReply SSplitter::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const FVector2D LocalMousePosition = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );

	TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(MyGeometry);

	if (bIsResizing)
	{
		if (!MouseEvent.GetCursorDelta().IsZero())
		{
			HandleResizingByMousePosition(Orientation, PhysicalSplitterHandleSize, ResizeMode, HoveredHandleIndex, LocalMousePosition, Children, LayoutChildren );
		}

		return FReply::Handled();
	}
	else
	{
		// Hit test which handle we are hovering over.
		int32 PreviousHoveredHandleIndex = HoveredHandleIndex;

		HoveredHandleIndex = (Orientation == Orient_Horizontal)
			? GetHandleBeingResizedFromMousePosition<Orient_Horizontal>( PhysicalSplitterHandleSize, HitDetectionSplitterHandleSize, LocalMousePosition, LayoutChildren )
			: GetHandleBeingResizedFromMousePosition<Orient_Vertical>( PhysicalSplitterHandleSize, HitDetectionSplitterHandleSize, LocalMousePosition, LayoutChildren );

		if (HoveredHandleIndex != INDEX_NONE)
		{
			if (FindResizeableSlotBeforeHandle(HoveredHandleIndex, Children) <= INDEX_NONE || FindResizeableSlotAfterHandle(HoveredHandleIndex, Children) >= Children.Num())
			{
				HoveredHandleIndex = INDEX_NONE;
			}
		}

		if (HoveredHandleIndex != PreviousHoveredHandleIndex)
		{
			OnHandleHovered.ExecuteIfBound(HoveredHandleIndex);
		}

		return FReply::Unhandled();
	}

}

FReply SSplitter::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (OnGetMaxSlotSize.IsBound())
	{
		FVector2D MaxSlotSize = OnGetMaxSlotSize.Execute(HoveredHandleIndex);

		if (!MaxSlotSize.IsZero())
		{
			TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(InMyGeometry);

			HandleResizingBySize(Orientation, PhysicalSplitterHandleSize, ResizeMode, HoveredHandleIndex, MaxSlotSize, Children, LayoutChildren);

			return FReply::Handled();
		}
	}

	return SWidget::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

void SSplitter::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	int32 PreviousHoveredHandleIndex = HoveredHandleIndex;

	if (!bIsResizing)
	{
		HoveredHandleIndex = INDEX_NONE;
	}

	if (HoveredHandleIndex != PreviousHoveredHandleIndex)
	{
		OnHandleHovered.ExecuteIfBound(HoveredHandleIndex);
	}
}

/**
* The system asks each widget under the mouse to provide a cursor. This event is bubbled.
* 
* @return FCursorReply::Unhandled() if the event is not handled; return FCursorReply::Cursor() otherwise.
*/
FCursorReply SSplitter::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (bIsResizing)
	{
		return Orientation == Orient_Horizontal
			? FCursorReply::Cursor( EMouseCursor::ResizeLeftRight )
			: FCursorReply::Cursor( EMouseCursor::ResizeUpDown );
	}

	const FVector2D LocalMousePosition = MyGeometry.AbsoluteToLocal( CursorEvent.GetScreenSpacePosition() );

	TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(MyGeometry);

	// Hit test which handle we are hovering over.		
	const int32 CurrentHoveredHandleIndex = (Orientation == Orient_Horizontal)
		? GetHandleBeingResizedFromMousePosition<Orient_Horizontal>( PhysicalSplitterHandleSize, HitDetectionSplitterHandleSize, LocalMousePosition, LayoutChildren )
		: GetHandleBeingResizedFromMousePosition<Orient_Vertical>( PhysicalSplitterHandleSize, HitDetectionSplitterHandleSize, LocalMousePosition, LayoutChildren );

	if (CurrentHoveredHandleIndex != INDEX_NONE &&
		Children[CurrentHoveredHandleIndex].CanBeResized())
	{
		return Orientation == Orient_Horizontal
			? FCursorReply::Cursor( EMouseCursor::ResizeLeftRight )
			: FCursorReply::Cursor( EMouseCursor::ResizeUpDown );
	}
	else
	{
		return FCursorReply::Unhandled();
	}
}

/**
* Change the orientation of the splitter
*
* @param NewOrientation  Should the splitter be horizontal or vertical
*/
void SSplitter::SetOrientation( EOrientation NewOrientation )
{
	if (Orientation != NewOrientation)
	{
		Orientation = NewOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

/**
* @return the current orientation of the splitter.
*/
EOrientation SSplitter::GetOrientation() const
{
	return Orientation;
}


SSplitter::SSplitter()
	: Children(this)
	, HoveredHandleIndex(INDEX_NONE)
	, HighlightedHandleIndex(*this, INDEX_NONE)
	, bIsResizing( false )
	, Orientation( Orient_Horizontal )
	, Style( nullptr )
{
}


int32 SSplitter::FindResizeableSlotBeforeHandle( int32 DraggedHandle, const TPanelChildren<FSlot>& Children )
{
	// Resizing collapsed or autosizing slots does not make sense (their size is predetermined).
	// Search out from the DraggedHandle to find the first non-collapsed, non-autosizing slot we can resize.
	int32 SlotBeforeDragHandle = DraggedHandle;
	while (SlotBeforeDragHandle >= 0
		&& (Children[SlotBeforeDragHandle].GetWidget()->GetVisibility() == EVisibility::Collapsed || !Children[SlotBeforeDragHandle].CanBeResized()))
	{
		--SlotBeforeDragHandle;
	}

	return SlotBeforeDragHandle;
}


int32 SSplitter::FindResizeableSlotAfterHandle( int32 DraggedHandle, const TPanelChildren<FSlot>& Children )
{
	const int32 NumChildren = Children.Num();

	// But the slots list does contain collapsed children! Make sure that we are not resizing a collapsed slot.
	// We also cannot resize auto-sizing slots.
	int32 SlotAfterDragHandle = DraggedHandle+1;
	while (SlotAfterDragHandle < NumChildren
		&& (Children[SlotAfterDragHandle].GetWidget()->GetVisibility() == EVisibility::Collapsed || !Children[SlotAfterDragHandle].CanBeResized()))
	{
		++SlotAfterDragHandle;
	}

	return SlotAfterDragHandle;
}


void SSplitter::FindAllResizeableSlotsAfterHandle( int32 DraggedHandle, const TPanelChildren<FSlot>& Children, TArray< int32, FConcurrentLinearArrayAllocator >& OutSlotIndicies )
{
	const int32 NumChildren = Children.Num();

	for (int SlotIndex = DraggedHandle+1; SlotIndex < NumChildren; SlotIndex++)
	{
		if (Children[ SlotIndex ].GetWidget()->GetVisibility() == EVisibility::Collapsed || !Children[ SlotIndex ].CanBeResized())
		{
			continue;
		}

		OutSlotIndicies.Add( SlotIndex );
	}
};

void SSplitter::HandleResizingDelta(EOrientation SplitterOrientation, const float InPhysicalSplitterHandleSize, const ESplitterResizeMode::Type InResizeMode, int32 DraggedHandle, float Delta, TPanelChildren<FSlot>& InChildren, const TArray<FLayoutGeometry>& ChildGeometries)
{
	const int32 NumChildren = InChildren.Num();
	const int32 AxisIndex = (SplitterOrientation == Orient_Horizontal) ? 0 : 1;

	// Note:
	//  - Prev vs. Next refers to the widgets in the order they are laid out (left->right, top->bottom).
	//  - New vs. Old refers to the Old values for width/height vs. the post-resize values.

	TArray<int32, FConcurrentLinearArrayAllocator> SlotsAfterDragHandleIndices;
	SlotsAfterDragHandleIndices.Reserve(NumChildren);
	if (InResizeMode == ESplitterResizeMode::FixedPosition)
	{
		int32 SlotAfterDragHandle = FindResizeableSlotAfterHandle(DraggedHandle, Children);

		if (InChildren.IsValidIndex(SlotAfterDragHandle))
		{
			SlotsAfterDragHandleIndices.Add(SlotAfterDragHandle);
		}
	}
	else if (InResizeMode == ESplitterResizeMode::Fill || InResizeMode == ESplitterResizeMode::FixedSize)
	{
		FindAllResizeableSlotsAfterHandle( DraggedHandle, InChildren, /*OUT*/ SlotsAfterDragHandleIndices );
	}

	const int32 NumSlotsAfterDragHandle = SlotsAfterDragHandleIndices.Num();
	if (NumSlotsAfterDragHandle > 0)
	{
		struct FSlotInfo 
		{
			FSlot* Slot;
			const FLayoutGeometry* Geometry;
			float NewSize;
		};

		TArray<FSlotInfo, FConcurrentLinearArrayAllocator> SlotsAfterDragHandle;
		SlotsAfterDragHandle.Reserve(NumSlotsAfterDragHandle);
		for (int32 SlotIndex = 0; SlotIndex < NumSlotsAfterDragHandle; SlotIndex++)
		{
			FSlotInfo SlotInfo;

			SlotInfo.Slot = &InChildren[ SlotsAfterDragHandleIndices[ SlotIndex ] ];
			SlotInfo.Geometry = &ChildGeometries[ SlotsAfterDragHandleIndices[ SlotIndex ] ];
			SlotInfo.NewSize = ClampChild(*SlotInfo.Slot, SlotInfo.Geometry->GetSizeInParentSpace().Component( AxisIndex ));

			SlotsAfterDragHandle.Add( SlotInfo );
		}

		// Get references the prev and next children and their layout settings so that we can modify them.
		const int32 SlotBeforeDragHandle = FindResizeableSlotBeforeHandle(DraggedHandle, InChildren);
		check(InChildren.IsValidIndex(SlotBeforeDragHandle));
		FSlot& PrevChild = InChildren[SlotBeforeDragHandle];
		const FLayoutGeometry& PrevChildGeom = ChildGeometries[SlotBeforeDragHandle];

		// Compute the new sizes of the children
		const float PrevChildLength = PrevChildGeom.GetSizeInParentSpace().Component(AxisIndex);
		float NewPrevChildLength = ClampChild(PrevChild, PrevChildLength + Delta);
		Delta = NewPrevChildLength - PrevChildLength;

		// Distribute the Delta across the affected slots after the drag handle
		float UnusedDelta = Delta;

		for (int32 DistributionCount = 0; DistributionCount < NumSlotsAfterDragHandle && UnusedDelta != 0; DistributionCount++)
		{
			float DividedDelta = InResizeMode != ESplitterResizeMode::FixedSize ? UnusedDelta / NumSlotsAfterDragHandle : UnusedDelta;

			// Reset the unused delta to keep track of any leftover space to distribute later through stretching.
			UnusedDelta = 0;

			int32 SlotIndex = 0;
			// Resize only the last handle in the case of fixed size
			if (InResizeMode == ESplitterResizeMode::FixedSize)
			{
				SlotIndex = NumSlotsAfterDragHandle - 1;
			}

			for (/*SlotIndex*/; SlotIndex < NumSlotsAfterDragHandle; SlotIndex++)
			{
				FSlotInfo& SlotInfo = SlotsAfterDragHandle[SlotIndex];
				float CurrentSize = SlotInfo.NewSize;
				SlotInfo.NewSize = ClampChild(*SlotInfo.Slot, CurrentSize - DividedDelta);

				// If one of the slots couldn't be fully adjusted by the delta due to min/max constraints then
				// the leftover delta needs to be evenly distributed to all of the other slots
				UnusedDelta += SlotInfo.NewSize - (CurrentSize - DividedDelta);
			}
		}

		Delta = Delta - UnusedDelta;

		// PrevChildLength needs to be updated: it's value has to take into account the next child's min/max restrictions
		NewPrevChildLength = ClampChild(PrevChild, PrevChildLength + Delta);

		// Cells being resized are both stretch values -> redistribute the stretch coefficients proportionately
		// to match the new child sizes on the screen.
		{
			float TotalStretchLength = 0.f;
			float TotalStretchCoefficients = 0.f;
			
			if (PrevChild.GetSizingRule() == ESizeRule::FractionOfParent)
			{
				TotalStretchLength = NewPrevChildLength;
				TotalStretchCoefficients = PrevChild.GetSizeValue();
			}

			for (int32 SlotIndex = 0; SlotIndex < NumSlotsAfterDragHandle; SlotIndex++)
			{
				const FSlotInfo& SlotInfo = SlotsAfterDragHandle[SlotIndex];

				if (SlotInfo.Slot->GetSizingRule() == ESizeRule::FractionOfParent)
				{
					TotalStretchLength += SlotInfo.NewSize;
					TotalStretchCoefficients += SlotInfo.Slot->GetSizeValue();
				}
			}

			auto SetSize = [](FSlot& Slot, float TotalStretchCoefficients, float NewLength, float TotalLength)
			{
				if (Slot.GetSizingRule() == ESizeRule::FractionOfParent)
				{
					float NewFillSize = TotalLength > 0.f ? (TotalStretchCoefficients * (NewLength / TotalLength)) : TotalStretchCoefficients;
					if (Slot.OnSlotResized().IsBound())
					{
						Slot.OnSlotResized().Execute(NewFillSize);
					}
					else
					{
						Slot.SetSizeValue(NewFillSize);
					}
				}
				else
				{
					ensure(Slot.OnSlotResized().IsBound());
					Slot.OnSlotResized().ExecuteIfBound(NewLength);
				}
			};

			SetSize(PrevChild, TotalStretchCoefficients, NewPrevChildLength, TotalStretchLength);
			for (int32 SlotIndex = 0; SlotIndex < NumSlotsAfterDragHandle; SlotIndex++)
			{
				const FSlotInfo& SlotInfo = SlotsAfterDragHandle[ SlotIndex ];
				SetSize(*(SlotInfo.Slot), TotalStretchCoefficients, SlotInfo.NewSize, TotalStretchLength);
			}
		}
	}
}

void SSplitter::HandleResizingBySize(EOrientation SplitterOrientation, const float InPhysicalSplitterHandleSize, const ESplitterResizeMode::Type InResizeMode, int32 DraggedHandle, const FVector2D& InDesiredSize, TPanelChildren<FSlot>& InChildren, const TArray<FLayoutGeometry>& ChildGeometries)
{
	const int32 AxisIndex = (SplitterOrientation == Orient_Horizontal) ? 0 : 1;

	float CurrentSlotSize = ChildGeometries[DraggedHandle].GetSizeInParentSpace().Component(AxisIndex);
	float Delta = InDesiredSize.Component(AxisIndex) - CurrentSlotSize;

	HandleResizingDelta(SplitterOrientation, InPhysicalSplitterHandleSize, InResizeMode, DraggedHandle, Delta, InChildren, ChildGeometries);
}

void SSplitter::HandleResizingByMousePosition(EOrientation SplitterOrientation, const float InPhysicalSplitterHandleSize, const ESplitterResizeMode::Type InResizeMode, int32 DraggedHandle, const FVector2D& LocalMousePos, TPanelChildren<FSlot>& InChildren, const TArray<FLayoutGeometry>& ChildGeometries )
{
	const int32 AxisIndex = (SplitterOrientation == Orient_Horizontal) ? 0 : 1;

	const float HandlePos = ChildGeometries[DraggedHandle+1].GetLocalToParentTransform().GetTranslation().Component(AxisIndex) - InPhysicalSplitterHandleSize / 2;
	float Delta = LocalMousePos.Component(AxisIndex) - HandlePos;

	HandleResizingDelta(SplitterOrientation, InPhysicalSplitterHandleSize, InResizeMode, DraggedHandle, Delta, InChildren, ChildGeometries);
}

/**
* @param ProposedSize  A size that a child would like to be
*
* @return A size that is clamped against the minimum size allowed for children.
*/
float SSplitter::ClampChild(const FSlot& ChildSlot, float ProposedSize) const
{
	float ClampedSize = FMath::Max(MinSplitterChildLength, ProposedSize);
	ClampedSize = FMath::Max(ChildSlot.GetMinSize(), ProposedSize);
	return ClampedSize;
}


template<EOrientation SplitterOrientation>
int32 SSplitter::GetHandleBeingResizedFromMousePosition( float InPhysicalSplitterHandleSize, float HitDetectionSplitterHandleSize, FVector2D LocalMousePos, const TArray<FLayoutGeometry>& ChildGeometries )
{
	const int32 AxisIndex = (SplitterOrientation == Orient_Horizontal) ? 0 : 1;
	const float HalfHitDetectionSplitterHandleSize = ( HitDetectionSplitterHandleSize / 2 );
	const float HalfPhysicalSplitterHandleSize = ( InPhysicalSplitterHandleSize / 2 );

	// Search for the two widgets between which the cursor currently resides.
	for ( int32 ChildIndex = 1; ChildIndex < ChildGeometries.Num(); ++ChildIndex )
	{
		FSlateRect PrevChildRect = ChildGeometries[ChildIndex - 1].GetRectInParentSpace();
		FVector2D NextChildOffset = ChildGeometries[ChildIndex].GetOffsetInParentSpace();
		float PrevBound = PrevChildRect.GetTopLeft().Component(AxisIndex) + PrevChildRect.GetSize().Component(AxisIndex) - HalfHitDetectionSplitterHandleSize + HalfPhysicalSplitterHandleSize;
		float NextBound = NextChildOffset.Component(AxisIndex) + HalfHitDetectionSplitterHandleSize - HalfPhysicalSplitterHandleSize;

		if ( LocalMousePos.Component(AxisIndex) > PrevBound && LocalMousePos.Component(AxisIndex) < NextBound )
		{
			return ChildIndex - 1;
		}
	}

	return INDEX_NONE;
}

/********************************************************************
* SSplitter2x2														*
* A splitter which has exactly 4 children and allows simultaneous	*		
* of all children along an axis as well as resizing all children	* 
* by dragging the center of the splitter.							*
********************************************************************/

SSplitter2x2::FSlot::FSlot(const TSharedRef<SWidget>& InWidget)
	: TSlotBase<FSlot>(InWidget)
	, PercentageAttribute(FVector2D(0.5, 0.5))
{

}

void SSplitter2x2::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArg)
{
	TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArg));
	if (InArg._Percentage.IsSet())
	{
		PercentageAttribute = MoveTemp(InArg._Percentage);
	}
}

SSplitter2x2::SSplitter2x2()
	: Children(this)
{
}

void SSplitter2x2::Construct( const FArguments& InArgs )
{
	Children.AddSlot(FSlot::FSlotArguments(MakeUnique<FSlot>(InArgs._TopLeft.Widget)));
	Children.AddSlot(FSlot::FSlotArguments(MakeUnique<FSlot>(InArgs._BottomLeft.Widget)));
	Children.AddSlot(FSlot::FSlotArguments(MakeUnique<FSlot>(InArgs._TopRight.Widget)));
	Children.AddSlot(FSlot::FSlotArguments(MakeUnique<FSlot>(InArgs._BottomRight.Widget)));

	Style = InArgs._Style;

	SplitterHandleSize = 5.0f;
	MinSplitterChildLength = 20.0f;
	bIsResizing = false;
	ResizingAxisMask = EResizingAxis::None;
}

TArray<FLayoutGeometry> SSplitter2x2::ArrangeChildrenForLayout( const FGeometry& AllottedGeometry ) const
{
	check( Children.Num() == 4 );

	TArray<FLayoutGeometry> Result;
	Result.Empty(Children.Num());

	int32 NumNonCollapsedChildren = 0;
	FVector2D CoefficientTotal(0,0);

	// The allotted space for our children is our geometry minus a little space to show splitter handles
	const FVector2D SpaceAllottedForChildren = AllottedGeometry.GetLocalSize() - FVector2D(SplitterHandleSize, SplitterHandleSize);

	// The current offset that the next child should be positioned at.
	FVector2D Offset(0,0);

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FSlot& CurSlot = Children[ChildIndex];

		// Calculate the amount of space that this child should take up.  
		// It is based on the current percentage of space it should take up which is defined by a user moving the splitters
		const FVector2D ChildSpace = SpaceAllottedForChildren * CurSlot.GetPercentage();

		// put them in their spot
		Result.Emplace(FSlateLayoutTransform(Offset), ChildSpace);

		// Advance to the next slot. If the child is collapsed, it takes up no room and does not need a splitter
		if( ChildIndex == 1 )
		{
			// ChildIndex of 1 means we are starting the next column so reset the Y offset.
			Offset.Y = 0.0f;
			Offset += FVector2D( ChildSpace.X + SplitterHandleSize, 0);
		}
		else
		{
			Offset += FVector2D( 0, ChildSpace.Y + SplitterHandleSize );
		}
	}
	return Result;
}

void SSplitter2x2::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(AllottedGeometry);

	for (int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FSlot& CurSlot = Children[ChildIndex];

		// put them in their spot
		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild( Children[ChildIndex].GetWidget(), LayoutChildren[ChildIndex] ) );
	}
}

int32 SSplitter2x2::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	int32 MaxLayerId = PaintArrangedChildren(Args, ArrangedChildren, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const FVector2D LocalMousePos = GetTickSpaceGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());

	EResizingAxis HoveredAxis = CalculateResizingAxis(GetTickSpaceGeometry(), LocalMousePos);

	// First two children are top left and bottom left. Use that to calculate horizontal separator

	const FVector2D HorizontalSplitterPos(0, ArrangedChildren[0].Geometry.GetLocalSize().Y);
	const FVector2D HorizontalSplitterSize(AllottedGeometry.GetLocalSize().X, ArrangedChildren[1].Geometry.Position.Y - ArrangedChildren[0].Geometry.GetLocalSize().Y);

	const FSlateBrush* HorizontalBrush = EnumHasAnyFlags(HoveredAxis, EResizingAxis::UpDownMask) || EnumHasAnyFlags(ResizingAxisMask, EResizingAxis::UpDownMask) ? &Style->HandleHighlightBrush : &Style->HandleNormalBrush;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		MaxLayerId,
		AllottedGeometry.ToPaintGeometry(HorizontalSplitterSize, FSlateLayoutTransform(HorizontalSplitterPos)),
		HorizontalBrush,
		ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		InWidgetStyle.GetColorAndOpacityTint() * HorizontalBrush->TintColor.GetSpecifiedColor());

	const FVector2D VerticalSplitterPos(ArrangedChildren[0].Geometry.GetLocalSize().X,0);
	const FVector2D VerticalSplitterSize(ArrangedChildren[2].Geometry.Position.X - ArrangedChildren[0].Geometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y);

	const FSlateBrush* VerticalBrush = EnumHasAnyFlags(HoveredAxis, EResizingAxis::LeftRightMask) || EnumHasAnyFlags(ResizingAxisMask, EResizingAxis::LeftRightMask) ? &Style->HandleHighlightBrush : &Style->HandleNormalBrush;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		MaxLayerId,
		AllottedGeometry.ToPaintGeometry(VerticalSplitterSize, FSlateLayoutTransform(VerticalSplitterPos)),
		VerticalBrush,
		ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		InWidgetStyle.GetColorAndOpacityTint() * VerticalBrush->TintColor.GetSpecifiedColor());

	return MaxLayerId;
}

FVector2D SSplitter2x2::ComputeDesiredSize( float ) const
{
	return FVector2D(100,100);
}


FChildren* SSplitter2x2::GetChildren()
{
	return &Children;
}


FReply SSplitter2x2::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		const FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
		ResizingAxisMask = CalculateResizingAxis( MyGeometry, LocalMousePos );
		if (ResizingAxisMask != EResizingAxis::None )
		{
			bIsResizing = true;
			Reply = FReply::Handled().CaptureMouse( SharedThis(this) );
		}
	}

	return Reply;
}


FReply SSplitter2x2::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsResizing == true )
	{
		bIsResizing = false;
		ResizingAxisMask = EResizingAxis::None;
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}


FReply SSplitter2x2::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );

	if ( bIsResizing && this->HasMouseCapture() )
	{	
		TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(MyGeometry);
		ResizeChildren( MyGeometry, LayoutChildren, LocalMousePos );
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FCursorReply SSplitter2x2::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const 
{
	const FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition());
	EResizingAxis HoveredAxis = CalculateResizingAxis(GetTickSpaceGeometry(), LocalMousePos);

	if (EnumHasAnyFlags(HoveredAxis, EResizingAxis::CrossMask) || EnumHasAnyFlags(ResizingAxisMask, EResizingAxis::CrossMask))
	{
		return FCursorReply::Cursor(EMouseCursor::CardinalCross);
	}
	else if(EnumHasAnyFlags(HoveredAxis,EResizingAxis::LeftRightMask) || EnumHasAnyFlags(ResizingAxisMask, EResizingAxis::LeftRightMask) )
	{
		return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight );
	}
	else if(EnumHasAnyFlags(HoveredAxis, EResizingAxis::UpDownMask) || EnumHasAnyFlags(ResizingAxisMask, EResizingAxis::UpDownMask))
	{
		return FCursorReply::Cursor( EMouseCursor::ResizeUpDown );
	}


	return FCursorReply::Unhandled();
}

void SSplitter2x2::ResizeChildren( const FGeometry& MyGeometry, const TArray<FLayoutGeometry>& ArrangedChildren, const FVector2D& LocalMousePos )
{
	// Compute the handle position.  The last child is used because it is always the furthest away from the origin
	const FVector2D HandlePos = ArrangedChildren[3].GetOffsetInParentSpace() - (FVector2D( SplitterHandleSize , SplitterHandleSize ) * .5f);
	FVector2D Delta = LocalMousePos - HandlePos;

	FVector2D TopLeftSize = ArrangedChildren[0].GetSizeInParentSpace();
	FVector2D BotLeftSize = ArrangedChildren[1].GetSizeInParentSpace();
	FVector2D TopRightSize = ArrangedChildren[2].GetSizeInParentSpace();
	FVector2D BotRightSize = ArrangedChildren[3].GetSizeInParentSpace();

	FSlot& TopLeft = Children[0];
	FSlot& BotLeft = Children[1];
	FSlot& TopRight = Children[2];
	FSlot& BotRight = Children[3];

	if(ResizingAxisMask == EResizingAxis::LeftRightMask )
	{
		// Ensure deltas along the Y axis are not taken into account
		Delta.Y = 0;
	}
	else if(ResizingAxisMask == EResizingAxis::UpDownMask )
	{
		// Ensure deltas along the X axis are not taken into account
		Delta.X = 0;
	}

	// The new size of each child
	FVector2D NewSizeTL;
	FVector2D NewSizeBL;
	FVector2D NewSizeTR;
	FVector2D NewSizeBR;

	if( ResizingAxisMask == EResizingAxis::LeftRightMask)
	{
		NewSizeTL = TopLeftSize + Delta;
		NewSizeBL = BotLeftSize + Delta;
		NewSizeTR = TopRightSize - Delta;
		NewSizeBR = BotRightSize - Delta;
	}
	else if( ResizingAxisMask == EResizingAxis::UpDownMask)
	{
		NewSizeTL = TopLeftSize + Delta;
		NewSizeBL = BotLeftSize - Delta;
		NewSizeTR = TopRightSize + Delta;
		NewSizeBR = BotRightSize - Delta;
	}
	else
	{
		// Resize X and Y independently as they have different rules for X and Y
		NewSizeTL.X = TopLeftSize.X + Delta.X;
		NewSizeBL.X = BotLeftSize.X + Delta.X;

		// @todo: revisit workaround for compiler crash on clang version > 9
#if PLATFORM_ANDROID_X64
		volatile float workaround = 0.0f;
		NewSizeTR.X = TopRightSize.X - Delta.X + workaround;
#else
		NewSizeTR.X = TopRightSize.X - Delta.X;
#endif

		NewSizeBR.X = BotRightSize.X - Delta.X;


		NewSizeTL.Y = TopLeftSize.Y + Delta.Y;
		NewSizeBL.Y = BotLeftSize.Y - Delta.Y;
		NewSizeTR.Y = TopRightSize.Y + Delta.Y;
		NewSizeBR.Y = BotRightSize.Y - Delta.Y;
	}

	// Clamp all values so they cant be too small
	// Must be done independently on each axis because of how FVector2D handles greater than
	NewSizeTL.X = FMath::Max(MinSplitterChildLength, NewSizeTL.X);
	NewSizeBL.X = FMath::Max(MinSplitterChildLength, NewSizeBL.X);
	NewSizeTR.X = FMath::Max(MinSplitterChildLength, NewSizeTR.X);
	NewSizeBR.X = FMath::Max(MinSplitterChildLength, NewSizeBR.X);

	NewSizeTL.Y = FMath::Max(MinSplitterChildLength, NewSizeTL.Y);
	NewSizeBL.Y = FMath::Max(MinSplitterChildLength, NewSizeBL.Y);
	NewSizeTR.Y = FMath::Max(MinSplitterChildLength, NewSizeTR.Y);
	NewSizeBR.Y = FMath::Max(MinSplitterChildLength, NewSizeBR.Y);

	// Set the percentage space within the allotted area that each child should take up
	FVector2D TotalLength = NewSizeTL+NewSizeBR;
	TopLeft.SetPercentage( NewSizeTL / TotalLength );
	TopRight.SetPercentage( NewSizeTR / TotalLength );
	BotLeft.SetPercentage( NewSizeBL / TotalLength );
	BotRight.SetPercentage( NewSizeBR / TotalLength );

}

EResizingAxis SSplitter2x2::CalculateResizingAxis( const FGeometry& MyGeometry, const FVector2D& LocalMousePos ) const
{
	EResizingAxis Axis = EResizingAxis::None;

	TArray<FLayoutGeometry> LayoutChildren = ArrangeChildrenForLayout(MyGeometry);

	// The axis is in the center if it passes all hit tests
	bool bInCenter = true;
	// Search for the two widgets between which the cursor currently resides.
	for ( int32 ChildIndex = 1; ChildIndex < LayoutChildren.Num(); ++ChildIndex )
	{
		FLayoutGeometry& PrevChild = LayoutChildren[ChildIndex - 1];
		FLayoutGeometry& NextChild = LayoutChildren[ChildIndex];
		FVector2D PrevBound = PrevChild.GetOffsetInParentSpace() + PrevChild.GetSizeInParentSpace();
		FVector2D NextBound = NextChild.GetOffsetInParentSpace();

		if( LocalMousePos.X > PrevBound.X && LocalMousePos.X < NextBound.X )
		{
			// The mouse is in between two viewports vertically
			// Resizing axis is the X axis
			Axis = EResizingAxis::LeftRightMask;
		}
		else if( LocalMousePos.Y > PrevBound.Y && LocalMousePos.Y < NextBound.Y )
		{
			// The mouse is in between two viewports horizontally
			// Resizing axis is the Y axis
			Axis = EResizingAxis::UpDownMask;
		}
		else
		{
			// Failed a hit test
			bInCenter = false;
		}
	}

	if( bInCenter )
	{
		Axis = EResizingAxis::CrossMask;
	}

	return Axis;
}

TSharedRef< SWidget > SSplitter2x2::GetTopLeftContent()
{
	return Children[ 0 ].GetWidget();
}


TSharedRef< SWidget > SSplitter2x2::GetBottomLeftContent()
{
	return Children[ 1 ].GetWidget();
}


TSharedRef< SWidget > SSplitter2x2::GetTopRightContent()
{
	return Children[ 2 ].GetWidget();
}


TSharedRef< SWidget > SSplitter2x2::GetBottomRightContent()
{
	return Children[ 3 ].GetWidget();
}


void SSplitter2x2::SetTopLeftContent( TSharedRef< SWidget > TopLeftContent )
{
	Children[ 0 ][ TopLeftContent ];
}


void SSplitter2x2::SetBottomLeftContent( TSharedRef< SWidget > BottomLeftContent )
{
	Children[ 1 ][ BottomLeftContent ];
}


void SSplitter2x2::SetTopRightContent( TSharedRef< SWidget > TopRightContent )
{
	Children[ 2 ][ TopRightContent ];
}


void SSplitter2x2::SetBottomRightContent( TSharedRef< SWidget > BottomRightContent )
{
	Children[ 3 ][ BottomRightContent ];
}

void SSplitter2x2::GetSplitterPercentages( TArray< FVector2D >& OutPercentages ) const
{
	OutPercentages.Reset(4);
	for (int32 i = 0; i < 4; ++i)
	{
		OutPercentages.Add(Children[i].GetPercentage());
	}
}

void SSplitter2x2::SetSplitterPercentages( TArrayView< FVector2D > InPercentages )
{
	int32 MaxSize = FMath::Min(InPercentages.Num(), 4);
	for (int32 i = 0; i < MaxSize; ++i)
	{
		Children[i].SetPercentage(InPercentages[i]);
	}
}
