// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SUniformGridPanel.h"
#include "Layout/LayoutUtils.h"

namespace UE::Slate::Private
{

// Get the column/row index minus the count of preceding collapsed columns/rows
int32 GetAdjustedIndex(const int32 OriginalIndex, const TSet<int32>& CollapsedSet)
{
	int32 ResultIndex = OriginalIndex;
	for (int32 CheckIndex = 0; CheckIndex < OriginalIndex; CheckIndex++)
	{
		if (CollapsedSet.Contains(CheckIndex))
		{
			--ResultIndex;
		}
	}

	return ResultIndex;
}

}

SUniformGridPanel::SUniformGridPanel()
	: Children(this)
	, SlotPadding(*this, FMargin(0.0f))
	, MinDesiredSlotWidth(*this, 0.f)
	, MinDesiredSlotHeight(*this, 0.f)
{
}

SUniformGridPanel::~SUniformGridPanel() = default;

SUniformGridPanel::FSlot::FSlotArguments SUniformGridPanel::Slot(int32 Column, int32 Row)
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>(Column, Row));
}

void SUniformGridPanel::Construct( const FArguments& InArgs )
{
	SlotPadding.Assign(*this, InArgs._SlotPadding);
	NumColumns = 0;
	NumRows = 0;
	MinDesiredSlotWidth.Assign(*this, InArgs._MinDesiredSlotWidth);
	MinDesiredSlotHeight.Assign(*this, InArgs._MinDesiredSlotHeight);

	Children.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));
}

void SUniformGridPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	if ( Children.Num() > 0 )
	{
		const FVector2D CellSize(AllottedGeometry.GetLocalSize().X / NumColumns, AllottedGeometry.GetLocalSize().Y / NumRows);
		const FMargin& CurrentSlotPadding(SlotPadding.Get());
		for ( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
		{
			const FSlot& Child = Children[ChildIndex];
			const EVisibility ChildVisibility = Child.GetWidget()->GetVisibility();
			if ( ArrangedChildren.Accepts(ChildVisibility) )
			{
				// Do the standard arrangement of elements within a slot
				// Takes care of alignment and padding.
				AlignmentArrangeResult XAxisResult = AlignChild<Orient_Horizontal>(CellSize.X, Child, CurrentSlotPadding);
				AlignmentArrangeResult YAxisResult = AlignChild<Orient_Vertical>(CellSize.Y, Child, CurrentSlotPadding);

				ArrangedChildren.AddWidget(ChildVisibility,
					AllottedGeometry.MakeChild(Child.GetWidget(),
					FVector2D(CellSize.X * UE::Slate::Private::GetAdjustedIndex(Child.GetColumn(), CollapsedColumns) + XAxisResult.Offset,
					          CellSize.Y * UE::Slate::Private::GetAdjustedIndex(Child.GetRow(), CollapsedRows) + YAxisResult.Offset),
					FVector2D(XAxisResult.Size, YAxisResult.Size)
					));
			}

		}
	}
}

FVector2D SUniformGridPanel::ComputeDesiredSize( float ) const
{
	FVector2D MaxChildDesiredSize = FVector2D::ZeroVector;
	const FVector2D SlotPaddingDesiredSize = SlotPadding.Get().GetDesiredSize();
	
	const float CachedMinDesiredSlotWidth = MinDesiredSlotWidth.Get();
	const float CachedMinDesiredSlotHeight = MinDesiredSlotHeight.Get();
	
	NumColumns = 0;
	NumRows = 0;

	CollapsedColumns.Reset();
	CollapsedRows.Reset();

	// Track rows/columns that have been verified as containing visible widgets
	TSet<int32> VisibleColumns;
	TSet<int32> VisibleRows;

	for (int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FSlot& Child = Children[ChildIndex];

		// A single cell at (N,M) means our grid size is (N+1, M+1)
		NumColumns = FMath::Max(Child.GetColumn() + 1, NumColumns);
		NumRows = FMath::Max(Child.GetRow() + 1, NumRows);

		// If collapsed, we may want to collapse the entire row/column
		if (Child.GetWidget()->GetVisibility() == EVisibility::Collapsed)
		{
			if (!VisibleColumns.Contains(Child.GetColumn()))
			{
				CollapsedColumns.Add(Child.GetColumn());
			}

			if (!VisibleRows.Contains(Child.GetRow()))
			{
				CollapsedRows.Add(Child.GetRow());
			}

			continue;
		}

		// Verify the row & column as visible
		CollapsedColumns.Remove(Child.GetColumn());
		CollapsedRows.Remove(Child.GetRow());
		VisibleColumns.Add(Child.GetColumn());
		VisibleRows.Add(Child.GetRow());

		FVector2D ChildDesiredSize = Child.GetWidget()->GetDesiredSize() + SlotPaddingDesiredSize;

		ChildDesiredSize.X = FMath::Max(ChildDesiredSize.X, CachedMinDesiredSlotWidth);
		ChildDesiredSize.Y = FMath::Max(ChildDesiredSize.Y, CachedMinDesiredSlotHeight);

		MaxChildDesiredSize.X = FMath::Max(MaxChildDesiredSize.X, ChildDesiredSize.X);
		MaxChildDesiredSize.Y = FMath::Max(MaxChildDesiredSize.Y, ChildDesiredSize.Y);
	}

	// Final row & column count shouldn't consider collapsed rows & columns
	NumColumns -= CollapsedColumns.Num();
	NumRows -= CollapsedRows.Num();

	return FVector2D( NumColumns*MaxChildDesiredSize.X, NumRows*MaxChildDesiredSize.Y );
}

FChildren* SUniformGridPanel::GetChildren()
{
	return &Children;
}

void SUniformGridPanel::SetSlotPadding(TAttribute<FMargin> InSlotPadding)
{
	SlotPadding.Assign(*this, MoveTemp(InSlotPadding));
}

void SUniformGridPanel::SetMinDesiredSlotWidth(TAttribute<float> InMinDesiredSlotWidth)
{
	MinDesiredSlotWidth.Assign(*this, MoveTemp(InMinDesiredSlotWidth));
}

void SUniformGridPanel::SetMinDesiredSlotHeight(TAttribute<float> InMinDesiredSlotHeight)
{
	MinDesiredSlotHeight.Assign(*this, MoveTemp(InMinDesiredSlotHeight));
}

SUniformGridPanel::FScopedWidgetSlotArguments SUniformGridPanel::AddSlot( int32 Column, int32 Row )
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(Column, Row), Children, INDEX_NONE };
}

bool SUniformGridPanel::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	return Children.Remove(SlotWidget) != INDEX_NONE;
}

void SUniformGridPanel::ClearChildren()
{
	NumColumns = 0;
	NumRows = 0;
	Children.Empty();
}
