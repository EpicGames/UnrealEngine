// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Layout/LayoutUtils.h"

SLATE_IMPLEMENT_WIDGET(SUniformWrapPanel)
void SUniformWrapPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FSlot::RegisterAttributes(Initializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, SlotPadding, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredSlotWidth, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredSlotHeight, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxDesiredSlotWidth, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, NumColumnsOverride, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxDesiredSlotHeight, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HAlign, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, EvenRowDistribution, EInvalidateWidgetReason::Paint);
}

SUniformWrapPanel::SUniformWrapPanel()
	: Children(this)
	, SlotPadding(*this, FMargin(0.0f))
	, NumColumnsOverride(*this, 0)
	, MinDesiredSlotWidth(*this, 0.f)
	, MinDesiredSlotHeight(*this, 0.f)
	, MaxDesiredSlotWidth(*this, FLT_MAX)
	, MaxDesiredSlotHeight(*this, FLT_MAX)
	, HAlign(*this, EHorizontalAlignment::HAlign_Left)
	, EvenRowDistribution(*this, false)
{
}

SUniformWrapPanel::~SUniformWrapPanel() = default;

void SUniformWrapPanel::Construct( const FArguments& InArgs )
{
	SlotPadding.Assign(*this, InArgs._SlotPadding);
	NumColumns = 0;
	NumRows = 0;
	NumVisibleChildren = 0;
	MinDesiredSlotWidth.Assign(*this, InArgs._MinDesiredSlotWidth);
	MinDesiredSlotHeight.Assign(*this, InArgs._MinDesiredSlotHeight);
	MaxDesiredSlotWidth.Assign(*this, InArgs._MaxDesiredSlotWidth);
	NumColumnsOverride.Assign(*this, InArgs._NumColumnsOverride);
	MaxDesiredSlotHeight.Assign(*this, InArgs._MaxDesiredSlotHeight);
	HAlign.Assign(*this, InArgs._HAlign);
	EvenRowDistribution.Assign(*this, InArgs._EvenRowDistribution);

	Children.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));
}

void SUniformWrapPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	if ( Children.Num() > 0)
	{
		FVector2D CellSize = ComputeUniformCellSize();
		NumColumns = NumColumnsOverride.Get() != 0 ?
				NumColumnsOverride.Get() :
		FMath::Max(1, FMath::Min(NumVisibleChildren, FMath::FloorToInt( AllottedGeometry.GetLocalSize().X / CellSize.X )));
		NumRows = FMath::CeilToInt ( (float) NumVisibleChildren / (float) NumColumns );

		// If we have to have N rows, try to distribute the items across the rows evenly
		int32 AdjNumColumns = EvenRowDistribution.Get() ? FMath::CeilToInt( (float) NumVisibleChildren / (float) NumRows ) : NumColumns;

		float LeftSlop = 0.0f;

		switch (HAlign.Get())
		{
			case HAlign_Fill:
			{
				// CellSize = FVector2D(AllottedGeometry.GetLocalSize().X / NumColumns, AllottedGeometry.GetLocalSize().Y / NumRows);
				CellSize = FVector2D (AllottedGeometry.GetLocalSize().X / AdjNumColumns, CellSize.Y);
				break;
			}
			case HAlign_Center:
			{

				LeftSlop = FMath::FloorToFloat((AllottedGeometry.GetLocalSize().X - (CellSize.X * AdjNumColumns)) / 2.0f);
				break;
			}
			case HAlign_Right:
			{
				LeftSlop = 	FMath::FloorToFloat(AllottedGeometry.GetLocalSize().X - (CellSize.X * AdjNumColumns));
				break;
			}
		};


		const FMargin& CurrentSlotPadding(SlotPadding.Get());
		int32 VisibleChildIndex = 0;
		for ( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
		{
			const FSlot& Child = Children[ChildIndex];
			const EVisibility ChildVisibility = Child.GetWidget()->GetVisibility();
			if ( ArrangedChildren.Accepts(ChildVisibility)  && !Child.GetWidget()->GetDesiredSize().IsZero() )
			{

				// Do the standard arrangement of elements within a slot
				// Takes care of alignment and padding.
				AlignmentArrangeResult XAxisResult = AlignChild<Orient_Horizontal>(CellSize.X, Child, CurrentSlotPadding);
				AlignmentArrangeResult YAxisResult = AlignChild<Orient_Vertical>(CellSize.Y, Child, CurrentSlotPadding);

				int32 col = VisibleChildIndex % AdjNumColumns;
				int32 row = VisibleChildIndex / AdjNumColumns;

				if (row == (NumRows - 1))
				{
					int32 NumLastRowColumns = (NumVisibleChildren % AdjNumColumns != 0) ? NumVisibleChildren % AdjNumColumns : AdjNumColumns;
					if (HAlign.Get() == HAlign_Right)
					{
						LeftSlop = FMath::FloorToFloat(AllottedGeometry.GetLocalSize().X - (CellSize.X * NumLastRowColumns));
					}
					else if (HAlign.Get() == HAlign_Center)
					{
						LeftSlop = FMath::FloorToFloat((AllottedGeometry.GetLocalSize().X - (CellSize.X * NumLastRowColumns)) / 2.0f);
					}
				}

				ArrangedChildren.AddWidget(ChildVisibility,
					AllottedGeometry.MakeChild(Child.GetWidget(),
					FVector2D(CellSize.X*col + XAxisResult.Offset + LeftSlop, CellSize.Y*row + YAxisResult.Offset),
					FVector2D(XAxisResult.Size, YAxisResult.Size)
					));

				VisibleChildIndex++;
			}
		}
	}
}

FVector2D SUniformWrapPanel::ComputeUniformCellSize() const
{
	FVector2D MaxChildDesiredSize = FVector2D::ZeroVector;
	const FVector2D SlotPaddingDesiredSize = SlotPadding.Get().GetDesiredSize();
	
	const float CachedMinDesiredSlotWidth = MinDesiredSlotWidth.Get();
	const float CachedMinDesiredSlotHeight = MinDesiredSlotHeight.Get();

	const float CachedMaxDesiredSlotWidth = MaxDesiredSlotWidth.Get();
	const float CachedMaxDesiredSlotHeight = MaxDesiredSlotHeight.Get();
	
	NumColumns = 0;
	NumRows = 0;

	NumVisibleChildren = 0;
	for ( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		const FSlot& Child = Children[ ChildIndex ];
		if (Child.GetWidget()->GetVisibility() != EVisibility::Collapsed)
		{
			FVector2D ChildDesiredSize = Child.GetWidget()->GetDesiredSize();
			if (!ChildDesiredSize.IsZero())
			{
				NumVisibleChildren++;
				ChildDesiredSize += SlotPaddingDesiredSize;

				ChildDesiredSize.X = FMath::Max( ChildDesiredSize.X, CachedMinDesiredSlotWidth);
				ChildDesiredSize.Y = FMath::Max( ChildDesiredSize.Y, CachedMinDesiredSlotHeight);

				MaxChildDesiredSize.X = FMath::Max( MaxChildDesiredSize.X, ChildDesiredSize.X );
				MaxChildDesiredSize.Y = FMath::Max( MaxChildDesiredSize.Y, ChildDesiredSize.Y );

				MaxChildDesiredSize.X = FMath::Min( MaxChildDesiredSize.X, CachedMaxDesiredSlotWidth);
				MaxChildDesiredSize.Y = FMath::Min( MaxChildDesiredSize.Y, CachedMaxDesiredSlotHeight);
			}
		}
	}

	return MaxChildDesiredSize;
}

FVector2D SUniformWrapPanel::ComputeDesiredSize( float ) const
{
	FVector2D MaxChildDesiredSize = ComputeUniformCellSize();

	if (NumVisibleChildren > 0)
	{
		// Try to use the current geometry. If the preferred width or geometry isn't available
		// then try to make a square.
		const FVector2D& LocalSize = GetTickSpaceGeometry().GetLocalSize();
		if (!LocalSize.IsZero()) 
		{
			NumColumns = NumColumnsOverride.Get() != 0 ? NumColumnsOverride.Get() : 
				FMath::FloorToInt(LocalSize.X / MaxChildDesiredSize.X);
		}
		else 
		{
			NumColumns = NumColumnsOverride.Get() != 0 ? NumColumnsOverride.Get() :
			    FMath::CeilToInt(FMath::Sqrt((float)NumVisibleChildren));
		}

		if (NumColumns > 0)
		{
			NumRows = FMath::CeilToInt((float)NumVisibleChildren / (float)NumColumns);
			return FVector2D(NumColumns * MaxChildDesiredSize.X, NumRows * MaxChildDesiredSize.Y);
		}
		
		/* If we have gotten this far we may be small enough that the best fit formulas fail yet we still have visible children to show ~
		 * default to one column the width of the child desired size and the number of children rows times their height */
		NumRows = NumVisibleChildren;
		NumColumns = 1;
		return FVector2D(MaxChildDesiredSize.X, NumRows * MaxChildDesiredSize.Y);
	}

	return FVector2D::ZeroVector;
}

FChildren* SUniformWrapPanel::GetChildren()
{
	return &Children;
}

void SUniformWrapPanel::SetSlotPadding(TAttribute<FMargin> InSlotPadding)
{
	SlotPadding.Assign(*this, MoveTemp(InSlotPadding));
}

void SUniformWrapPanel::SetMinDesiredSlotWidth(TAttribute<float> InMinDesiredSlotWidth)
{
	MinDesiredSlotWidth.Assign(*this, MoveTemp(InMinDesiredSlotWidth));
}

void SUniformWrapPanel::SetMinDesiredSlotHeight(TAttribute<float> InMinDesiredSlotHeight)
{
	MinDesiredSlotHeight.Assign(*this, MoveTemp(InMinDesiredSlotHeight));
}

void SUniformWrapPanel::SetMaxDesiredSlotWidth(TAttribute<float> InMaxDesiredSlotWidth)
{
	MaxDesiredSlotWidth.Assign(*this, MoveTemp(InMaxDesiredSlotWidth));
}

void SUniformWrapPanel::SetNumColumnsOverride(TAttribute<int32> InNumColumnsOverride)
{
	NumColumnsOverride.Assign(*this, MoveTemp(InNumColumnsOverride));
}

void SUniformWrapPanel::SetMaxDesiredSlotHeight(TAttribute<float> InMaxDesiredSlotHeight)
{
	MaxDesiredSlotHeight.Assign(*this, MoveTemp(InMaxDesiredSlotHeight));
}

void SUniformWrapPanel::SetHorizontalAlignment(TAttribute<EHorizontalAlignment> InHAlignment)
{
	HAlign.Assign(*this, MoveTemp(InHAlignment));
}

void SUniformWrapPanel::SetEvenRowDistribution(TAttribute<bool> InEvenRowDistribution)
{
	EvenRowDistribution.Assign(*this, MoveTemp(InEvenRowDistribution));
}

SUniformWrapPanel::FScopedWidgetSlotArguments SUniformWrapPanel::AddSlot()
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), this->Children, INDEX_NONE };
}

bool SUniformWrapPanel::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	return Children.Remove(SlotWidget) != INDEX_NONE;
}

void SUniformWrapPanel::ClearChildren()
{
	NumColumns = 0;
	NumRows = 0;
	Children.Empty();
}
