// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Layout/LayoutUtils.h"
#include "Types/ReflectionMetadata.h"
#include "FastUpdate/SlateInvalidationRoot.h"

#if WITH_ACCESSIBILITY
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

SLATE_IMPLEMENT_WIDGET(SWidgetSwitcher)
void SWidgetSwitcher::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	// This attribute should invalidate ChildOrder but we cannot invalidate ChildOrder from an attribute.
	// The ChildOrder invalidation will be handled in CustomPrepass if it cannot be processed now
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, WidgetIndex, EInvalidateWidgetReason::Prepass)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			const bool bIsProcessingAttributeUpdate = Widget.GetProxyHandle().IsValid(Widget) &&
				Widget.GetProxyHandle().GetInvalidationRootHandle().GetInvalidationRoot() &&
				Widget.GetProxyHandle().GetInvalidationRootHandle().GetInvalidationRoot()->IsProcessingAttributeUpdate();

			// If we are not currently updating attributes, we can safely process the widget index change
			if (!bIsProcessingAttributeUpdate)
			{
				static_cast<SWidgetSwitcher&>(Widget).SetActiveWidgetIndexInternal();
			}
		}));
}

SWidgetSwitcher::SWidgetSwitcher()
	: WidgetIndex(*this, 0)
	, AllChildren(this)
	, OneDynamicChild(this, &AllChildren, &WidgetIndex)
{ 
	SetCanTick(false);
}

SWidgetSwitcher::~SWidgetSwitcher() = default;

SWidgetSwitcher::FSlot::FSlotArguments SWidgetSwitcher::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

bool SWidgetSwitcher::CustomPrepass(float LayoutScaleMultiplier)
{
	SetActiveWidgetIndexInternal();
	return true;
}

SWidgetSwitcher::FScopedWidgetSlotArguments SWidgetSwitcher::AddSlot(int32 SlotIndex)
{
	TWeakPtr<SWidgetSwitcher> WeakSwitcher = SharedThis(this);
	if (!AllChildren.IsValidIndex(SlotIndex))
	{
		// Insert at the end
		return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), AllChildren, INDEX_NONE, [WeakSwitcher](const FSlot*, int32 SlotIndex)
			{
				if (TSharedPtr<SWidgetSwitcher> Switcher = WeakSwitcher.Pin())
				{
					Switcher->OnSlotAdded(SlotIndex);
				}
			} };
	}
	else
	{
		return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), AllChildren, SlotIndex, [WeakSwitcher](const FSlot*, int32 SlotIndex)
			{
				if (TSharedPtr<SWidgetSwitcher> Switcher = WeakSwitcher.Pin())
				{
					// adjust the active WidgetIndex based on this slot change
					if (!Switcher->WidgetIndex.IsBound(*Switcher.Get()) && Switcher->WidgetIndex.Get() >= SlotIndex)
					{
						Switcher->SetActiveWidgetIndex(Switcher->WidgetIndex.Get() + 1);
					}
					Switcher->OnSlotAdded(SlotIndex);
				}
			}};
	}
}

void SWidgetSwitcher::Construct( const FArguments& InArgs )
{
	AllChildren.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));

	PreviousWidgetIndex = INDEX_NONE;
	WidgetIndex.Assign(*this, InArgs._WidgetIndex);
	SetCanTick(false);
	bHasCustomPrepass = true;
}

TSharedPtr<SWidget> SWidgetSwitcher::GetActiveWidget( ) const
{
	const FSlot* ActiveSlot = GetActiveSlot();

	if (ActiveSlot != nullptr)
	{
		return ActiveSlot->GetWidget();
	}

	return nullptr;
}

TSharedPtr<SWidget> SWidgetSwitcher::GetWidget( int32 SlotIndex ) const
{
	if (AllChildren.IsValidIndex(SlotIndex))
	{
		return AllChildren[SlotIndex].GetWidget();
	}

	return nullptr;
}

int32 SWidgetSwitcher::GetWidgetIndex( TSharedRef<SWidget> Widget ) const
{
	for (int32 Index = 0; Index < AllChildren.Num(); ++Index)
	{
		const FSlot& Slot = AllChildren[Index];

		if (Slot.GetWidget() == Widget)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

int32 SWidgetSwitcher::RemoveSlot( TSharedRef<SWidget> WidgetToRemove )
{
	for (int32 SlotIndex = 0; SlotIndex < AllChildren.Num(); ++SlotIndex)
	{
		if (AllChildren[SlotIndex].GetWidget() == WidgetToRemove)
		{
			// adjust the active WidgetIndex based on this slot change
			const int32 ActiveWidgetIndex = WidgetIndex.Get();
			const bool bWasActiveSlot = (SlotIndex == ActiveWidgetIndex);

			if (!WidgetIndex.IsBound(*this))
			{
				if (ActiveWidgetIndex > 0 && ActiveWidgetIndex >= SlotIndex)
				{
					WidgetIndex.Assign(*this, ActiveWidgetIndex - 1);
				}
			}
			else
			{
				// We want to give a chance for the attribute to update its index now that we have removed a child
				WidgetIndex.UpdateNow(*this);
			}

			AllChildren.RemoveAt(SlotIndex);
			OnSlotRemoved(SlotIndex, WidgetToRemove, bWasActiveSlot);
			return SlotIndex;
		}
	}

	return -1;
}

void SWidgetSwitcher::SetActiveWidgetIndex( int32 Index )
{
	WidgetIndex.Assign(*this, Index);
}

bool SWidgetSwitcher::ValidatePathToChild(SWidget* InChild)
{
	return InChild == GetActiveWidget().Get();
}

void SWidgetSwitcher::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const TAttribute<FVector2D> ContentScale = FVector2D::UnitVector;

	if (const FSlot* ActiveSlotPtr = GetActiveSlot())
	{
		ArrangeSingleChild(GSlateFlowDirection, AllottedGeometry, ArrangedChildren, *ActiveSlotPtr, ContentScale);

#if WITH_ACCESSIBILITY
		// This would go in SetActiveWidgetIndex were this not a TAttribute. WidgetIndex changing does not affect
		// parent or visibility assignments, so we need a special custom notification to tell the accessibility
		// tree to update itself.
		if (LastActiveWidget != ActiveSlotPtr->GetWidget())
		{
			const_cast<SWidgetSwitcher*>(this)->LastActiveWidget = ActiveSlotPtr->GetWidget();
			FSlateApplication::Get().GetAccessibleMessageHandler()->MarkDirty();
		}
#endif
	}
}
	
FVector2D SWidgetSwitcher::ComputeDesiredSize(float) const
{
	if (const FSlot* ActiveSlotPtr = GetActiveSlot())
	{
		const TSharedRef<SWidget>& Widget = ActiveSlotPtr->GetWidget();
		const EVisibility ChildVisibility = Widget->GetVisibility();
		if (ChildVisibility != EVisibility::Collapsed)
		{
			return Widget->GetDesiredSize() + ActiveSlotPtr->GetPadding().GetDesiredSize();
		}
	}

	return FVector2D::ZeroVector;
}

FChildren* SWidgetSwitcher::GetChildren()
{
	return &OneDynamicChild;
}

const SWidgetSwitcher::FSlot* SWidgetSwitcher::GetActiveSlot() const
{
	const int32 ActiveWidgetIndex = WidgetIndex.Get();
	if (ActiveWidgetIndex >= 0 && ActiveWidgetIndex < AllChildren.Num())
	{
		return &AllChildren[ActiveWidgetIndex];
	}

	return nullptr;
}

void SWidgetSwitcher::SetActiveWidgetIndexInternal()
{
	const int NewIndex = WidgetIndex.Get();

	if (NewIndex == PreviousWidgetIndex)
	{
		return;
	}

	Invalidate(EInvalidateWidgetReason::ChildOrder);

	const FSlot* ActiveSlot = (PreviousWidgetIndex >= 0 && PreviousWidgetIndex < AllChildren.Num()) ? &AllChildren[PreviousWidgetIndex] : nullptr;

	// Active slot can be null if the widget switcher was initialized to an invalid index.
	if (ActiveSlot)
	{
		SWidget& OldActiveWidget = ActiveSlot->GetWidget().Get();
		InvalidateChildRemovedFromTree(OldActiveWidget);

#if WITH_SLATE_DEBUGGING
		UE_LOGF(LogSlate, Verbose, "WidgetSwitcher ('%ls') Active Slot Changed: %d(%ls) -FROM- %d(%ls)", *FReflectionMetaData::GetWidgetDebugInfo(this), 
			NewIndex, *FReflectionMetaData::GetWidgetDebugInfo(GetWidget(NewIndex).Get()),
			PreviousWidgetIndex, *FReflectionMetaData::GetWidgetDebugInfo(OldActiveWidget));
#endif
	}
	else
	{
#if WITH_SLATE_DEBUGGING
		UE_LOGF(LogSlate, Verbose, "WidgetSwitcher ('%ls') Active Slot Changed: %d(%ls)", *FReflectionMetaData::GetWidgetDebugInfo(this), NewIndex, *FReflectionMetaData::GetWidgetDebugInfo(GetWidget(NewIndex).Get()));
#endif
	}

	PreviousWidgetIndex = NewIndex;
}
