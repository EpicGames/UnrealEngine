// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SWindowTitleBarArea.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"
#include "Widgets/SWindow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"

#define WINDOW_IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush(FPaths::EngineContentDir() / TEXT("Slate") / RelativePath + TEXT(".png"), __VA_ARGS__)
#define WINDOW_BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush(FPaths::EngineContentDir() / TEXT("Slate") / RelativePath + TEXT(".png"), __VA_ARGS__)

FSimpleDelegate SWindowTitleBarArea::OnCloseButtonClicked;
bool SWindowTitleBarArea::bIsCloseButtonActive = true;

FButtonStyle SWindowTitleBarArea::MinimizeButtonStyle;
FButtonStyle SWindowTitleBarArea::MaximizeButtonStyle;
FButtonStyle SWindowTitleBarArea::RestoreButtonStyle;
FButtonStyle SWindowTitleBarArea::CloseButtonStyle;

SWindowTitleBarArea::SWindowTitleBarArea()
	: ChildSlot(this)
	, bIsMinimizeButtonEnabled(true)
	, bIsMaximizeRestoreButtonEnabled(true)
	, bIsCloseButtonEnabled(true)
{
	SetCanTick(false);
	bCanSupportFocus = false;

	static bool bButtonsStyleInitialized = false;
	if (!bButtonsStyleInitialized)
	{
		const FButtonStyle ButtonStyle = FButtonStyle()
			.SetNormal(WINDOW_BOX_BRUSH("Common/Button", FVector2D(32, 32), 8.0f / 32.0f))
			.SetHovered(WINDOW_BOX_BRUSH("Common/Button_Hovered", FVector2D(32, 32), 8.0f / 32.0f))
			.SetPressed(WINDOW_BOX_BRUSH("Common/Button_Pressed", FVector2D(32, 32), 8.0f / 32.0f))
			.SetNormalPadding(FMargin(2, 2, 2, 2))
			.SetPressedPadding(FMargin(2, 3, 2, 1));

		MinimizeButtonStyle = FButtonStyle(ButtonStyle)
			.SetNormal(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Normal", FVector2D(27.0f, 18.0f)))
			.SetHovered(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Hovered", FVector2D(27.0f, 18.0f)))
			.SetPressed(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Pressed", FVector2D(27.0f, 18.0f)))
			.SetDisabled(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Disabled", FVector2D(27.0f, 18.0f)));

		MaximizeButtonStyle = FButtonStyle(ButtonStyle)
			.SetNormal(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Normal", FVector2D(23.0f, 18.0f)))
			.SetHovered(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Hovered", FVector2D(23.0f, 18.0f)))
			.SetPressed(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Pressed", FVector2D(23.0f, 18.0f)))
			.SetDisabled(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Disabled", FVector2D(27.0f, 18.0f)));

		RestoreButtonStyle = FButtonStyle(ButtonStyle)
			.SetNormal(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Restore_Normal", FVector2D(23.0f, 18)))
			.SetHovered(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Restore_Hovered", FVector2D(23.0f, 18)))
			.SetPressed(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Restore_Pressed", FVector2D(23.0f, 18)))
			.SetDisabled(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Disabled", FVector2D(27.0f, 18.0f)));

		CloseButtonStyle = FButtonStyle(ButtonStyle)
			.SetNormal(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Close_Normal", FVector2D(44.0f, 18.0f)))
			.SetHovered(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Close_Hovered", FVector2D(44.0f, 18.0f)))
			.SetPressed(WINDOW_IMAGE_BRUSH("Common/Window/WindowButton_Close_Pressed", FVector2D(44.0f, 18.0f)));

		bButtonsStyleInitialized = true;
	}
}

SWindowTitleBarArea::~SWindowTitleBarArea() = default;

void SWindowTitleBarArea::Construct( const FArguments& InArgs )
{
	MinimizeButton = SNew(SButton)
		.IsFocusable(false)
		.IsEnabled(bIsMinimizeButtonEnabled)
		.ContentPadding(0.f)
		.OnClicked(this, &SWindowTitleBarArea::MinimizeButton_OnClicked)
		.Cursor(EMouseCursor::Default)
		.ButtonStyle(FCoreStyle::Get(), "NoBorder")
		[
			SNew(SImage)
			.Image(this, &SWindowTitleBarArea::GetMinimizeImage)
			.AccessibleText(NSLOCTEXT("WindowTitleBar", "Minimize", "Minimize"))
		];

	MaximizeRestoreButton = SNew(SButton)
		.IsFocusable(false)
		.IsEnabled(bIsMaximizeRestoreButtonEnabled)
		.ContentPadding(0.f)
		.OnClicked(this, &SWindowTitleBarArea::MaximizeRestoreButton_OnClicked)
		.Cursor(EMouseCursor::Default)
		.ButtonStyle(FCoreStyle::Get(), "NoBorder")
		[
			SNew(SImage)
			.Image(this, &SWindowTitleBarArea::GetMaximizeRestoreImage)
			.AccessibleText(NSLOCTEXT("WindowTitleBar", "Maximize", "Maximize"))
		];

	CloseButton = SNew(SButton)
		.IsFocusable(false)
		.IsEnabled(bIsCloseButtonEnabled)
		.ContentPadding(0.0f)
		.OnClicked(this, &SWindowTitleBarArea::CloseButton_OnClicked)
		.Cursor(EMouseCursor::Default)
		.ButtonStyle(FCoreStyle::Get(), "NoBorder")
		[
			SNew(SImage)
			.Image(this, &SWindowTitleBarArea::GetCloseImage)
			.AccessibleText(NSLOCTEXT("WindowTitleBar", "Close", "Close"))
		];

	ChildSlot
		.HAlign( InArgs._HAlign )
		.VAlign( InArgs._VAlign )
		.Padding( InArgs._Padding )
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			InArgs._Content.Widget
		]

		+ SOverlay::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		[
			SAssignNew(WindowButtonsBox, SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SBox)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.Padding(FMargin(0))
					[
						// Minimize
						SNew(SHorizontalBox)
						.Visibility(EVisibility::SelfHitTestInvisible)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							MinimizeButton.ToSharedRef()
						]

						// Maximize/Restore
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							MaximizeRestoreButton.ToSharedRef()
						]

						// Close button
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							CloseButton.ToSharedRef()
						]
					]
				]
			]
		]
	];

	SetWindowButtonsVisibility(false);

	RequestToggleFullscreen = InArgs._RequestToggleFullscreen;
}

void SWindowTitleBarArea::SetContent(const TSharedRef< SWidget >& InContent)
{
	ChildSlot
	[
		InContent
	];
}

void SWindowTitleBarArea::SetHAlign(EHorizontalAlignment HAlign)
{
	ChildSlot.SetHorizontalAlignment(HAlign);
}

void SWindowTitleBarArea::SetVAlign(EVerticalAlignment VAlign)
{
	ChildSlot.SetVerticalAlignment(VAlign);
}

void SWindowTitleBarArea::SetPadding(TAttribute<FMargin> InPadding)
{
	ChildSlot.SetPadding(MoveTemp(InPadding));
}

FVector2D SWindowTitleBarArea::ComputeDesiredSize( float ) const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();

	if ( ChildVisibility != EVisibility::Collapsed )
	{
		return ChildSlot.GetWidget()->GetDesiredSize() + ChildSlot.GetPadding().GetDesiredSize();
	}
	
	return FVector2D::ZeroVector;
}

void SWindowTitleBarArea::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	const EVisibility& MyCurrentVisibility = this->GetVisibility();
	if ( ArrangedChildren.Accepts( MyCurrentVisibility ) )
	{
		const FMargin SlotPadding(ChildSlot.GetPadding());
		AlignmentArrangeResult XAlignmentResult = AlignChild<Orient_Horizontal>( AllottedGeometry.GetLocalSize().X, ChildSlot, SlotPadding );
		AlignmentArrangeResult YAlignmentResult = AlignChild<Orient_Vertical>( AllottedGeometry.GetLocalSize().Y, ChildSlot, SlotPadding );

		ArrangedChildren.AddWidget(
			AllottedGeometry.MakeChild(
				ChildSlot.GetWidget(),
				FVector2D(XAlignmentResult.Offset, YAlignmentResult.Offset),
				FVector2D(XAlignmentResult.Size, YAlignmentResult.Size)
			)
		);
	}
}

FChildren* SWindowTitleBarArea::GetChildren()
{
	return &ChildSlot;
}

int32 SWindowTitleBarArea::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// An SWindowTitleBarArea just draws its only child
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Maybe none of our children are visible
	if( ArrangedChildren.Num() > 0 )
	{
		check( ArrangedChildren.Num() == 1 );
		FArrangedWidget& TheChild = ArrangedChildren[0];

		return TheChild.Widget->Paint( Args.WithNewParent(this), TheChild.Geometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );
	}

	return LayerId;
}

FReply SWindowTitleBarArea::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Handle double click here in fullscreen mode only. In windowed mode double click is handled via window actions.
	if (GameWindow.IsValid() && GameWindow->GetWindowMode() != EWindowMode::Windowed)
	{
		if (RequestToggleFullscreen.IsBound())
		{
			RequestToggleFullscreen.Execute();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

EWindowZone::Type SWindowTitleBarArea::GetWindowZoneOverride() const
{
	EWindowZone::Type Zone = EWindowZone::TitleBar;
	if (GameWindow.IsValid() && GameWindow->GetWindowMode() != EWindowMode::Windowed)
	{
		// In fullscreen, return ClientArea to prevent window from being moved
		Zone = EWindowZone::ClientArea;
	}
	return Zone;
}

FReply SWindowTitleBarArea::MinimizeButton_OnClicked()
{
	if (GameWindow.IsValid())
	{
		TSharedPtr<FGenericWindow> NativeWindow = GameWindow->GetNativeWindow();

		if (NativeWindow.IsValid())
		{
			NativeWindow->Minimize();
		}
	}

	return FReply::Handled();
}

FReply SWindowTitleBarArea::MaximizeRestoreButton_OnClicked()
{
	if (GameWindow.IsValid())
	{
		TSharedPtr<FGenericWindow> NativeWindow = GameWindow->GetNativeWindow();

		if (NativeWindow.IsValid())
		{
			if (NativeWindow->GetWindowMode() != EWindowMode::Windowed && RequestToggleFullscreen.IsBound())
			{
				RequestToggleFullscreen.Execute();
			}
			else if (NativeWindow->IsMaximized())
			{
				NativeWindow->Restore();
			}
			else
			{
				NativeWindow->Maximize();
			}
		}
	}

	return FReply::Handled();
}

FReply SWindowTitleBarArea::CloseButton_OnClicked()
{
	if (bIsCloseButtonActive)
	{
		if (OnCloseButtonClicked.IsBound())
		{
			OnCloseButtonClicked.Execute();
		}
		else if (GameWindow.IsValid())
		{
			GameWindow->RequestDestroyWindow();
		}
	}

	return FReply::Handled();
}

const FSlateBrush* SWindowTitleBarArea::GetMinimizeImage() const
{
	if (!GameWindow.IsValid())
	{
		return &MinimizeButtonStyle.Normal;
	}

	TSharedPtr<FGenericWindow> NativeWindow = GameWindow->GetNativeWindow();

	if (!bIsMinimizeButtonEnabled || !GameWindow->HasMinimizeBox())
	{
		return &MinimizeButtonStyle.Disabled;
	}
	else if (MinimizeButton->IsPressed())
	{
		return &MinimizeButtonStyle.Pressed;
	}
	else if (MinimizeButton->IsHovered())
	{
		return &MinimizeButtonStyle.Hovered;
	}
	else
	{
		return &MinimizeButtonStyle.Normal;
	}
}

const FSlateBrush* SWindowTitleBarArea::GetMaximizeRestoreImage() const
{
	if (!GameWindow.IsValid())
	{
		return &MaximizeButtonStyle.Normal;
	}

	TSharedPtr<FGenericWindow> NativeWindow = GameWindow->GetNativeWindow();

	if (NativeWindow.IsValid() && (NativeWindow->IsMaximized() || NativeWindow->GetWindowMode() != EWindowMode::Windowed))
	{
		if (!bIsMaximizeRestoreButtonEnabled || !GameWindow->HasMaximizeBox())
		{
			return &MaximizeButtonStyle.Disabled;
		}
		else if (MaximizeRestoreButton->IsPressed())
		{
			return &RestoreButtonStyle.Pressed;
		}
		else if (MaximizeRestoreButton->IsHovered())
		{
			return &RestoreButtonStyle.Hovered;
		}
		else
		{
			return &RestoreButtonStyle.Normal;
		}
	}
	else
	{
		if (!bIsMaximizeRestoreButtonEnabled || !GameWindow->HasMaximizeBox())
		{
			return &MaximizeButtonStyle.Disabled;
		}
		else if (MaximizeRestoreButton->IsPressed())
		{
			return &MaximizeButtonStyle.Pressed;
		}
		else if (MaximizeRestoreButton->IsHovered())
		{
			return &MaximizeButtonStyle.Hovered;
		}
		else
		{
			return &MaximizeButtonStyle.Normal;
		}
	}
}

const FSlateBrush* SWindowTitleBarArea::GetCloseImage() const
{
	if (!GameWindow.IsValid())
	{
		return &CloseButtonStyle.Normal;
	}

	TSharedPtr<FGenericWindow> NativeWindow = GameWindow->GetNativeWindow();

	if (!bIsCloseButtonEnabled)
	{
		return &CloseButtonStyle.Disabled;
	}
	else if (CloseButton->IsPressed())
	{
		return &CloseButtonStyle.Pressed;
	}
	else if (CloseButton->IsHovered())
	{
		return &CloseButtonStyle.Hovered;
	}
	else
	{
		return &CloseButtonStyle.Normal;
	}
}
