// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SWrapBox.h"
#include "Layout/LayoutUtils.h"

SLATE_IMPLEMENT_WIDGET(SWrapBox)
void SWrapBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer SlotInitializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Slots);
	FSlot::RegisterAttributes(SlotInitializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, PreferredSize, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HAlign, EInvalidateWidgetReason::Layout);
}

SWrapBox::SWrapBox()
	: PreferredSize(*this, 100.f)
	, HAlign(*this, EHorizontalAlignment::HAlign_Left)
	, Slots(this, GET_MEMBER_NAME_CHECKED(SWrapBox, Slots))
{
}

SWrapBox::~SWrapBox() = default;

SWrapBox::FScopedWidgetSlotArguments SWrapBox::AddSlot()
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Slots, INDEX_NONE };
}

int32 SWrapBox::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	return Slots.Remove(SlotWidget);
}

void SWrapBox::Construct( const FArguments& InArgs )
{
	PreferredSize.Assign(*this, InArgs._PreferredSize);

	InnerSlotPadding = InArgs._InnerSlotPadding;
	bUseAllottedSize = InArgs._UseAllottedSize;
	Orientation = InArgs._Orientation;
	HAlign.Assign(*this, InArgs._HAlign);

	// Build the children from the declaration to the widget
	Slots.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));

	SetCanTick(bUseAllottedSize);
}

void SWrapBox::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bUseAllottedSize)
	{
		PreferredSize.Set(*this, Orientation == EOrientation::Orient_Vertical ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X);
	}
}

/*
 * Simple class for handling the somewhat complex state tracking for wrapping based on otherwise simple rules.
 * Singular static method in public interface simplifies use to a single function call by encapsulating and hiding
 * awkward inline helper object instantiation and method calls from user code. 
 */
class SWrapBox::FChildArranger
{
public:
	struct FArrangementData
	{
		FVector2D SlotOffset;
		FVector2D SlotSize;
		float WidthOfCurrentLine;
		float TotalLinePadding;
		int ChildIndexRelativeToLine;
	};

	typedef TFunctionRef<void(const FSlot& Slot, const FArrangementData& ArrangementData)> FOnSlotArranged;

	static void Arrange(const SWrapBox& WrapBox, const FOnSlotArranged& OnSlotArranged);

private:
	FChildArranger(const SWrapBox& WrapBox, const FOnSlotArranged& OnSlotArranged);
	void Arrange();
	void FinalizeLine(int32 IndexOfLastChildInCurrentLine);

	const SWrapBox& WrapBox;
	const FOnSlotArranged& OnSlotArranged;
	FVector2D Offset;
	float MaximumSizeInCurrentLine;

	// This is the total width of the line, including padding.
	float WidthOfCurrentLine;

	// This is the total width of just the padding in the current line. Not the other content.
	float TotalLinePadding;
	int32 IndexOfFirstChildInCurrentLine;
	TMap<int32, FArrangementData> OngoingArrangementDataMap;
};


SWrapBox::FChildArranger::FChildArranger(const SWrapBox& InWrapBox, const FOnSlotArranged& InOnSlotArranged)
	: WrapBox(InWrapBox)
	, OnSlotArranged(InOnSlotArranged)
	, Offset(FVector2D::ZeroVector)
	, MaximumSizeInCurrentLine(0.0f)
	, WidthOfCurrentLine(0.0f)
	, TotalLinePadding(0.0f)
	, IndexOfFirstChildInCurrentLine(INDEX_NONE)
{
	OngoingArrangementDataMap.Reserve(WrapBox.Slots.Num());
}

void SWrapBox::FChildArranger::Arrange()
{
	int32 ChildIndex;
	for (ChildIndex = 0; ChildIndex < WrapBox.Slots.Num(); ++ChildIndex)
	{
		const FSlot& Slot = WrapBox.Slots[ChildIndex];
		const TSharedRef<SWidget>& Widget = Slot.GetWidget();

		/*
		* Simple utility lambda for determining if the current child index is the first child of the current line.
		*/
		const auto& IsFirstChildInCurrentLine = [&]() -> bool
		{
			return ChildIndex == IndexOfFirstChildInCurrentLine;
		};

		// Skip collapsed widgets.
		if (Widget->GetVisibility() == EVisibility::Collapsed)
		{
			continue;
		}

		FArrangementData& ArrangementData = OngoingArrangementDataMap.Add(ChildIndex, FArrangementData());

		// If there is no first child in the current line, we must be the first child.
		if (IndexOfFirstChildInCurrentLine == INDEX_NONE)
		{
			IndexOfFirstChildInCurrentLine = ChildIndex;
		}

		/*
		* Simple utility lambda for beginning a new line with the current child, updating it's offset for the new line.
		*/
		const auto& BeginNewLine = [&]()
		{
			FinalizeLine(ChildIndex - 1);

			// Starting a new line.
			IndexOfFirstChildInCurrentLine = ChildIndex;

			// Update child's offset to new X and Y values for new line.
			ArrangementData.SlotOffset.X = Offset.X;
			ArrangementData.SlotOffset.Y = Offset.Y;
		};

		// Rule: If this child is not the first child in the line, "inner slot padding" needs to be injected left or top of it, dependently of the orientation.
		if (!IsFirstChildInCurrentLine())
		{
			if (WrapBox.Orientation == EOrientation::Orient_Horizontal)
			{
				Offset.X += WrapBox.InnerSlotPadding.X;
			}
			else
			{
				Offset.Y += WrapBox.InnerSlotPadding.Y;
			}
		}

		const FVector2D DesiredSizeOfSlot = Slot.GetPadding().GetDesiredSize() + Widget->GetDesiredSize();

		// Populate arrangement data with default size and offset at right end of current line.
		ArrangementData.SlotOffset.X = Offset.X;
		ArrangementData.SlotOffset.Y = Offset.Y;
		ArrangementData.SlotSize.X = DesiredSizeOfSlot.X;
		ArrangementData.SlotSize.Y = DesiredSizeOfSlot.Y;

		if (WrapBox.Orientation == EOrientation::Orient_Vertical)
		{
			const float BottomBoundOfChild = ArrangementData.SlotOffset.Y + ArrangementData.SlotSize.Y;

			if (Slot.GetForceNewLine())
			{
				// Begin a new line if the current one isn't empty, because the slot demanded a new line.
				if (!IsFirstChildInCurrentLine())
				{
					BeginNewLine();
				}
			}
			// Rule: If required due to a wrapping height under specified threshold, start a new line and allocate all of it to this child.
			else if (Slot.GetFillLineWhenSizeLessThan().IsSet() && WrapBox.PreferredSize.Get() < Slot.GetFillLineWhenSizeLessThan().GetValue())
			{
				// Begin a new line if the current one isn't empty, because we demand a whole line to ourselves.
				if (!IsFirstChildInCurrentLine())
				{
					BeginNewLine();
				}

				// Fill height of rest of wrap box.
				ArrangementData.SlotSize.Y = WrapBox.PreferredSize.Get() - Offset.Y;
			}
			// Rule: If the end of a child would go beyond the width to wrap at, it should move to a new line.
			else if (BottomBoundOfChild > WrapBox.PreferredSize.Get())
			{
				// Begin a new line if the current one isn't empty, because we demand a new line.
				if (!IsFirstChildInCurrentLine())
				{
					BeginNewLine();
				}
			}

			// Update current line maximum size.
			MaximumSizeInCurrentLine = FMath::Max(MaximumSizeInCurrentLine, ArrangementData.SlotSize.X);

			// Update offset to bottom bound of child.
			Offset.Y = ArrangementData.SlotOffset.Y + ArrangementData.SlotSize.Y;
		}
		else
		{
			const float RightBoundOfChild = ArrangementData.SlotOffset.X + ArrangementData.SlotSize.X;

			if (Slot.GetForceNewLine())
			{
				// Begin a new line if the current one isn't empty, because the slot demanded a new line.
				if (!IsFirstChildInCurrentLine())
				{
					BeginNewLine();
				}
			}
			// Rule: If required due to a wrapping width under specified threshold, start a new line and allocate all of it to this child.
			else if (Slot.GetFillLineWhenSizeLessThan().IsSet() && WrapBox.PreferredSize.Get() < Slot.GetFillLineWhenSizeLessThan().GetValue())
			{
				// Begin a new line if the current one isn't empty, because we demand a whole line to ourselves.
				if (!IsFirstChildInCurrentLine())
				{
					BeginNewLine();
				}

				// Fill width of rest of wrap box.
				ArrangementData.SlotSize.X = WrapBox.PreferredSize.Get() - Offset.X;
			}
			// Rule: If the end of a child would go beyond the width to wrap at, it should move to a new line.
			else if (RightBoundOfChild > WrapBox.PreferredSize.Get())
			{
				// Begin a new line if the current one isn't empty, because we demand a new line.
				if (!IsFirstChildInCurrentLine())
				{
					BeginNewLine();
				}
			}

			// Update current line maximum size.
			MaximumSizeInCurrentLine = FMath::Max(MaximumSizeInCurrentLine, ArrangementData.SlotSize.Y);
			WidthOfCurrentLine += ArrangementData.SlotSize.X;

			// At this point, we know that this child is going on this line. If it is not the first child on this line, then we must include the inner slot padding in this line's width.
			if (!IsFirstChildInCurrentLine())
			{
				WidthOfCurrentLine += WrapBox.InnerSlotPadding.X;
				TotalLinePadding += WrapBox.InnerSlotPadding.X;
			}

			// Update offset to right bound of child.
			Offset.X = ArrangementData.SlotOffset.X + ArrangementData.SlotSize.X;
		}
	}

	// Attempt to finalize the final line if there are any children in it.
	if (IndexOfFirstChildInCurrentLine != INDEX_NONE)
	{
		FinalizeLine(ChildIndex - 1);
	}
}

void SWrapBox::FChildArranger::FinalizeLine(int32 IndexOfLastChildInCurrentLine)
{
	// Iterate backwards through children in this line. Iterate backwards because the last uncollapsed child may wish to fill the remaining empty space of the line.
	for (; IndexOfLastChildInCurrentLine >= IndexOfFirstChildInCurrentLine; --IndexOfLastChildInCurrentLine)
	{
		if (WrapBox.Slots[IndexOfLastChildInCurrentLine].GetWidget()->GetVisibility() != EVisibility::Collapsed)
		{
			break;
		}
	}

	// If we're doing this on a line where we fill the empty space,
	// the widget of the current line will just be the whole line.
	if (WrapBox.Slots.IsValidIndex(IndexOfLastChildInCurrentLine) && WrapBox.Slots[IndexOfLastChildInCurrentLine].GetFillEmptySpace())
	{
		WidthOfCurrentLine = WrapBox.PreferredSize.Get();
	}

	int32 RelativeChildIndex = 0;
	// Now iterate forward so tab navigation works properly
	for (int32 ChildIndex = IndexOfFirstChildInCurrentLine; ChildIndex <= IndexOfLastChildInCurrentLine; ++ChildIndex)
	{
		const FSlot& Slot = WrapBox.Slots[ChildIndex];
		const TSharedRef<SWidget>& Widget = Slot.GetWidget();

		// Skip collapsed widgets.
		if (Widget->GetVisibility() == EVisibility::Collapsed)
		{
			continue;
		}

		FArrangementData& ArrangementData = OngoingArrangementDataMap[ChildIndex];
		ArrangementData.ChildIndexRelativeToLine = RelativeChildIndex;
		ArrangementData.WidthOfCurrentLine = WidthOfCurrentLine;
		ArrangementData.TotalLinePadding = TotalLinePadding;

		RelativeChildIndex++;

		// Rule: The last uncollapsed child in a line may request to fill the remaining empty space in the line.
		if (ChildIndex == IndexOfLastChildInCurrentLine && Slot.GetFillEmptySpace())
		{
			if (WrapBox.Orientation == EOrientation::Orient_Vertical)
			{
				ArrangementData.SlotSize.Y = WrapBox.PreferredSize.Get() - ArrangementData.SlotOffset.Y;
			}
			else
			{
				ArrangementData.SlotSize.X = WrapBox.PreferredSize.Get() - ArrangementData.SlotOffset.X;
			}
		}
		
		// All slots on this line should now match to the tallest element's height, which they can then use to do their alignment in OnSlotArranged below (eg. center within that)
		// If we left their height as is, then their slots would just be whatever their child's desired height was, and so a vertical alignment of "center" would actually 
		// leave the widget at the top of the line, since you couldn't calculate how much to offset by to actually reach the center of the "container"
		if (WrapBox.Orientation == EOrientation::Orient_Vertical)
		{
			ArrangementData.SlotSize.X = MaximumSizeInCurrentLine;
		}
		else
		{
			ArrangementData.SlotSize.Y = MaximumSizeInCurrentLine;
		}
		
		// Flip horizontally if flow direction is set to RightToLeft
		if (GSlateFlowDirection == EFlowDirection::RightToLeft)
		{
			ArrangementData.SlotOffset.X = WrapBox.PreferredSize.Get() - ArrangementData.SlotOffset.X - ArrangementData.SlotSize.X;
		}
				
		OnSlotArranged(Slot, ArrangementData);
	}

	if (WrapBox.Orientation == EOrientation::Orient_Vertical)
	{
		// Set initial state for new vertical line.
		Offset.Y = 0.0f;

		// Since this is the initial state for a new vertical line, this only happens after the first line, so the inner slot horizontal padding should always be added.
		Offset.X += MaximumSizeInCurrentLine + WrapBox.InnerSlotPadding.X;
	}
	else
	{
		// Set initial state for horizontal new line.
		Offset.X = 0.0f;

		// Since this is the initial state for a new horizontal line, this only happens after the first line, so the inner slot vertical padding should always be added.
		Offset.Y += MaximumSizeInCurrentLine + WrapBox.InnerSlotPadding.Y;
	}

	WidthOfCurrentLine = 0.0f;
	TotalLinePadding = 0.0f;
	MaximumSizeInCurrentLine = 0.0f;
	IndexOfFirstChildInCurrentLine = INDEX_NONE;
}

void SWrapBox::FChildArranger::Arrange(const SWrapBox& WrapBox, const FOnSlotArranged& OnSlotArranged)
{
	FChildArranger(WrapBox, OnSlotArranged).Arrange();
}

void SWrapBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const EHorizontalAlignment HAlignment = HAlign.Get();
	
	float LeftStop = 0;
	FChildArranger::Arrange(*this, [&](const FSlot& Slot, const FChildArranger::FArrangementData& ArrangementData)
	{
		FVector2D AllottedSlotSize = ArrangementData.SlotSize;
		float SlotSizeAdjustment = 0;
		if (Orientation == EOrientation::Orient_Horizontal)
		{
			// Reset leftstop when the line changes.
			if (ArrangementData.ChildIndexRelativeToLine == 0)
			{
				LeftStop = 0;
			}

			switch (HAlignment)
			{
			case HAlign_Right:
				LeftStop = FMath::FloorToFloat(AllottedGeometry.GetLocalSize().X - ArrangementData.WidthOfCurrentLine);
				break;
			case HAlign_Center:
				LeftStop = FMath::FloorToFloat((AllottedGeometry.GetLocalSize().X - ArrangementData.WidthOfCurrentLine) / 2.0f);
				break;
			case HAlign_Fill:
				// We must make sure to not include the size of the line's padding when calculating how much to stretch the slot by.
				// We should be careful to not have any division by zero issues here.
				float NewSlotSize = 0.0f;
				if (!FMath::IsNearlyZero(ArrangementData.WidthOfCurrentLine - ArrangementData.TotalLinePadding))
				{
					NewSlotSize = AllottedSlotSize.X / (ArrangementData.WidthOfCurrentLine - ArrangementData.TotalLinePadding) * (AllottedGeometry.GetLocalSize().X - ArrangementData.TotalLinePadding);
				}
				SlotSizeAdjustment = NewSlotSize - AllottedSlotSize.X;
				AllottedSlotSize.X = NewSlotSize;
				break;
			}
		}

		// Calculate offset and size in slot using alignment.
		const AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(AllottedSlotSize.X, Slot, Slot.GetPadding());
		const AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedSlotSize.Y, Slot, Slot.GetPadding());

		// Note: Alignment offset is relative to slot offset.
		const FVector2D PostAlignmentOffset = ArrangementData.SlotOffset + FVector2D(XResult.Offset, YResult.Offset) + FVector2D(LeftStop, 0);
		const FVector2D PostAlignmentSize = FVector2D(XResult.Size, YResult.Size);

		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(Slot.GetWidget(), PostAlignmentOffset, PostAlignmentSize));

		LeftStop += SlotSizeAdjustment;
	});
}

void SWrapBox::ClearChildren()
{
	Slots.Empty();
}

FVector2D SWrapBox::ComputeDesiredSize( float ) const
{
	FVector2D MyDesiredSize = FVector2D::ZeroVector;

	FChildArranger::Arrange(*this, [&](const FSlot& Slot, const FChildArranger::FArrangementData& ArrangementData)
	{
		// Increase desired size to the maximum X and Y positions of any child widget.
		MyDesiredSize.X = FMath::Max(MyDesiredSize.X, ArrangementData.SlotOffset.X + ArrangementData.SlotSize.X);
		MyDesiredSize.Y = FMath::Max(MyDesiredSize.Y, ArrangementData.SlotOffset.Y + ArrangementData.SlotSize.Y);
	});

	return MyDesiredSize;
}

FChildren* SWrapBox::GetChildren()
{
	return &Slots;
}

void SWrapBox::SetInnerSlotPadding(FVector2D InInnerSlotPadding)
{
	if (InnerSlotPadding != InInnerSlotPadding)
	{
		InnerSlotPadding = InInnerSlotPadding;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SWrapBox::SetWrapSize(TAttribute<float> InWrapSize)
{
	PreferredSize.Assign(*this, MoveTemp(InWrapSize));
}

void SWrapBox::SetUseAllottedSize(bool bInUseAllottedSize)
{
	if (bUseAllottedSize != bInUseAllottedSize)
	{
		bUseAllottedSize = bInUseAllottedSize;
		SetCanTick(bUseAllottedSize);
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SWrapBox::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SWrapBox::SetHorizontalAlignment(TAttribute<EHorizontalAlignment> InHAlignment)
{
	HAlign.Assign(*this, MoveTemp(InHAlignment));
}