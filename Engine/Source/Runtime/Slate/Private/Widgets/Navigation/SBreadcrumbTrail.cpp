// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Navigation/SBreadcrumbTrail.h"

void SBreadcrumbTrailBase::ScrollToStart()
{
    CrumbBox->ScrollToStart();
}

void SBreadcrumbTrailBase::ScrollToEnd()
{
    CrumbBox->ScrollToEnd();
}