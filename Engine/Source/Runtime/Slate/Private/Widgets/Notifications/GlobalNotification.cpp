// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/GlobalNotification.h"
#include "Layout/Visibility.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

TSharedPtr<SNotificationItem> FGlobalNotification::BeginNotification()
{
	TSharedPtr<SNotificationItem> NotificationItem = NotificationPtr.Pin();

	if (NotificationItem.IsValid())
	{
		NotificationItem->ExpireAndFadeout();
		NotificationPtr.Reset();
	}

	FNotificationInfo Info(FText::GetEmpty());
	Info.bFireAndForget = false;

	// Setting fade out and expire time to 0 as the expire message is currently very obnoxious
	Info.FadeOutDuration = 0.0f;
	Info.ExpireDuration = 0.0f;

	NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
	NotificationPtr = NotificationItem;

	if (NotificationItem.IsValid())
	{
		NotificationItem->SetCompletionState(SNotificationItem::CS_Pending);
		NotificationItem->SetVisibility(EVisibility::HitTestInvisible);
	}

	return NotificationItem;
}

void FGlobalNotification::EndNotification()
{
	TSharedPtr<SNotificationItem> NotificationItem = NotificationPtr.Pin();

	if (NotificationItem.IsValid())
	{
		NotificationItem->SetText(FText::GetEmpty());
		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();

		NotificationPtr.Reset();
	}
}

void FGlobalNotification::TickNotification(float DeltaTime)
{
	TSharedPtr<SNotificationItem> NotificationItem = NotificationPtr.Pin();
	const bool bIsNotificationActive = NotificationItem.IsValid() && NotificationItem->GetCompletionState() == SNotificationItem::CS_Pending;
	const bool bShouldShowNotification = ShouldShowNotification(bIsNotificationActive);

	// Trigger a new notification if we should be active and we haven't displayed the notification recently
	const double TimeNowInSeconds = FPlatformTime::Seconds();
	if (bShouldShowNotification && !bIsNotificationActive)
	{
		if (NextEnableTimeInSeconds == 0.0)
		{
			NextEnableTimeInSeconds = TimeNowInSeconds + EnableDelayInSeconds;
		}

		if (TimeNowInSeconds >= NextEnableTimeInSeconds)
		{
			NotificationItem = BeginNotification();
		}
	}
	// Disable the notification when we are no longer streaming textures
	else if (!bShouldShowNotification)
	{
		if (bIsNotificationActive)
		{
			EndNotification();
		}

		NextEnableTimeInSeconds = 0.0;
	}

	if (bShouldShowNotification && NotificationItem.IsValid())
	{
		SetNotificationText(NotificationItem);
	}
}
