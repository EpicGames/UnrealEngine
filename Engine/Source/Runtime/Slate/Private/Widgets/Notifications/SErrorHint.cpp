// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SErrorHint.h"
#include "Widgets/Images/SImage.h"


void SErrorHint::Construct(const FArguments& InArgs)
{
	SetContentScale(MakeAttributeSP(this, &SErrorHint::GetDesiredSizeScale));

	ExpandAnimation = FCurveSequence(0.0f, 0.15f);

	CustomVisibility = GetVisibilityAttribute().ToAttribute();
	SetVisibility(MakeAttributeSP(this, &SErrorHint::MyVisibility));

	ChildSlot
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Padding( FMargin(3,0) )
	[
		SAssignNew( ImageWidget, SImage )
		.Image( FCoreStyle::Get().GetBrush("Icons.Error") )
		.ToolTipText( this, &SErrorHint::GetErrorText )
	];

	SetError( InArgs._ErrorText );
}

void SErrorHint::SetError( const FText& InErrorText )
{
	if ( ErrorText.IsEmpty() && !InErrorText.IsEmpty() )
	{
		ExpandAnimation.Play( this->AsShared() );
	}

	ErrorText = InErrorText;
	SetToolTipText( ErrorText );
}

void SErrorHint::SetError( const FString& InErrorText )
{
	SetError( FText::FromString(InErrorText) );
}

bool SErrorHint::HasError() const
{
	return !ErrorText.IsEmpty();
}

TSharedRef<SWidget> SErrorHint::AsWidget()
{
	return SharedThis(this);
}

EVisibility SErrorHint::MyVisibility() const
{
	return HasError()
		? CustomVisibility.Get(EVisibility::Visible)
		: EVisibility::Hidden;
}

FVector2D SErrorHint::GetDesiredSizeScale() const
{
	const float AnimAmount = ExpandAnimation.GetLerp();
	return FVector2D( 1.0f, AnimAmount );
}

FText SErrorHint::GetErrorText() const
{
	return ErrorText;
}
