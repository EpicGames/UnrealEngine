// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SErrorText.h"


void SErrorText::Construct(const FArguments& InArgs)
{
	ExpandAnimation = FCurveSequence(0.0f, 0.15f);

	CustomVisibility = GetVisibilityAttribute().ToAttribute();
	SetVisibility(MakeAttributeSP(this, &SErrorText::MyVisibility));

	SBorder::Construct( SBorder::FArguments()
		.BorderBackgroundColor( InArgs._BackgroundColor )
		.BorderImage( FCoreStyle::Get().GetBrush("ErrorReporting.Box") )
		.ContentScale( this, &SErrorText::GetDesiredSizeScale )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding( FMargin(3,0) )
		[
			SAssignNew( TextBlock, STextBlock )
			.ColorAndOpacity( FCoreStyle::Get().GetColor("ErrorReporting.ForegroundColor") )
			.Font(InArgs._Font)
			.AutoWrapText(InArgs._AutoWrapText)
		]
	);

	SetError( InArgs._ErrorText );
}

void SErrorText::SetError( const FText& InErrorText )
{
	if ( TextBlock->GetText().IsEmpty() && !InErrorText.IsEmpty() )
	{
		ExpandAnimation.Play( this->AsShared() );
	}

	TextBlock->SetText( InErrorText );
}

void SErrorText::SetError( const FString& InErrorText )
{
	SetError( FText::FromString(InErrorText) );
}

bool SErrorText::HasError() const
{
	return ! TextBlock->GetText().IsEmpty();
}

TSharedRef<SWidget> SErrorText::AsWidget()
{
	return SharedThis(this);
}

EVisibility SErrorText::MyVisibility() const
{
	return (!TextBlock->GetText().IsEmpty())
		? CustomVisibility.Get()
		: EVisibility::Collapsed;
}

FVector2D SErrorText::GetDesiredSizeScale() const
{
	const float AnimAmount = ExpandAnimation.GetLerp();
	return FVector2D( 1.0f, AnimAmount );
}
