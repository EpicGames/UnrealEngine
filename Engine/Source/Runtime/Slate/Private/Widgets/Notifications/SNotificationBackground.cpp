// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SNotificationBackground.h"
#include "Styling/StyleColors.h"

void SNotificationBackground::Construct(const FArguments& InArgs)
{
	WatermarkBrush = FAppStyle::Get().GetBrush("NotificationList.Watermark");
	BorderBrush = FAppStyle::Get().GetBrush("NotificationList.ItemBackground_Border");

	SetClipping(EWidgetClipping::ClipToBounds);

	WatermarkTint = FStyleColors::Notifications.GetSpecifiedColor() * FLinearColor(1.15f, 1.15f, 1.15f, 1.0f);

	SBorder::Construct(
		SBorder::FArguments()
		.BorderImage(FStyleDefaults::GetNoBrush())
		.Padding(InArgs._Padding)
		.BorderBackgroundColor(InArgs._BorderBackgroundColor)
		.ColorAndOpacity(InArgs._ColorAndOpacity)
		.DesiredSizeScale(InArgs._DesiredSizeScale)
		[
			InArgs._Content.Widget
		]
	);
}

int32 SNotificationBackground::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (WatermarkBrush && WatermarkBrush->DrawAs != ESlateBrushDrawType::NoDrawType)
	{
		// The watermark should be 250% bigger than the size of the notification to get the desired effect
		const float SizeY = AllottedGeometry.GetLocalSize().Y;
		const float WatermarkSize = FMath::Clamp(SizeY * 2.50f, 130.0f, 220.0f);


		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			BorderBrush,
			ESlateDrawEffect::None,
			BorderBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint());

		OutDrawElements.PushClip(FSlateClippingZone(MyCullingRect.InsetBy(FMargin(2))));

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(WatermarkSize, WatermarkSize), FSlateLayoutTransform(FVector2f(4.0f, 4.0f))),
			WatermarkBrush,
			ESlateDrawEffect::None,
			WatermarkTint * InWidgetStyle.GetColorAndOpacityTint() * GetBorderBackgroundColor().GetColor(InWidgetStyle));

		OutDrawElements.PopClip();
	}


	int32 OutLayerId = SBorder::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId+2, InWidgetStyle, bParentEnabled);

	return OutLayerId;
}

