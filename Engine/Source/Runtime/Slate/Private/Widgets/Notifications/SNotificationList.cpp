// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Application/ThrottleManager.h"
#include "Containers/Ticker.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Styling/StyleColors.h"
#include "Widgets/Notifications/SNotificationBackground.h"

#define LOCTEXT_NAMESPACE "SNotificationList"

/////////////////////////////////////////////////
// SNotificationExtendable

/** Contains the standard functionality for a notification to inherit from*/
class SNotificationExtendable : public SNotificationItem
{
public:
	SNotificationExtendable()
	{
		SetCanTick(false);

		TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &SNotificationExtendable::Update));
	}

	virtual ~SNotificationExtendable()
	{
		// Just in case, make sure we have left responsive mode when getting cleaned up
		if ( ThrottleHandle.IsValid() )
		{
			FSlateThrottleManager::Get().LeaveResponsiveMode( ThrottleHandle );
		}

		FTSTicker::RemoveTicker(TickDelegateHandle);
	}
	
	
	/** Sets the text for message element */
	virtual void SetText( const TAttribute<FText>& InText ) override
	{
		Text = InText;
		MyTextBlock->SetText( Text );
	}

	/** Sets the text for message element */
	virtual void SetSubText(const TAttribute<FText>& InSubText) override
	{
		SubText = InSubText;
		MySubTextBlock->SetText(SubText);
	}

	virtual ECompletionState GetCompletionState() const override
	{
		return CompletionState;
	}

	virtual void SetCompletionState(ECompletionState State) override
	{
		CompletionState = State;

		if (State == CS_Success || State == CS_Fail)
		{
			CompletionStateAnimation = FCurveSequence();
			GlowCurve = CompletionStateAnimation.AddCurve(0.f, 0.75f);
			CompletionStateAnimation.Play( this->AsShared() );
		}
	}

	virtual void ExpireAndFadeout() override
	{
		bAutoExpire = true;
	}

	/** Begins the fadein of this message */
	void Fadein( const bool bAllowThrottleWhenFrameRateIsLow )
	{
		// Make visible
		SetVisibility(EVisibility::Visible);

		TSharedRef<SWidget> Self = AsShared();

		// Play Fadein animation
		FadeAnimation = FCurveSequence();
		FadeCurve = FadeAnimation.AddCurve(0.f, FadeInDuration.Get());
		FadeAnimation.Play(Self);

		ThrobberAnimation = FCurveSequence(0.0f, 1.0f);
		ThrobberAnimation.Play(Self, true);

		// Scale up/flash animation
		IntroAnimation = FCurveSequence();
		ScaleCurveX = IntroAnimation.AddCurve(0.2f, 0.3f, ECurveEaseFunction::QuadOut);
		ScaleCurveY = IntroAnimation.AddCurve(0.f, 0.2f);
		GlowCurve = IntroAnimation.AddCurve(0.5f, 0.55f, ECurveEaseFunction::QuadOut);
		IntroAnimation.Play(Self);

		// When a fade in occurs, we need a high framerate for the animation to look good
		if( FadeInDuration.Get() > KINDA_SMALL_NUMBER && bAllowThrottleWhenFrameRateIsLow && !ThrottleHandle.IsValid() )
		{
			if( !FSlateApplication::Get().IsRunningAtTargetFrameRate() )
			{
				ThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();
			}
		}
	}

	/** Begins the fadeout of this message */
	virtual void Fadeout() override
	{
		// Start fade animation
		FadeAnimation = FCurveSequence();
		FadeCurve = FadeAnimation.AddCurve(0.f, FadeOutDuration.Get());
		FadeAnimation.PlayReverse(this->AsShared());
	}

	virtual void Pulse(const FLinearColor& GlowColor)
	{
		DefaultGlowColor = GlowColor;
		CompletionStateAnimation = FCurveSequence();
		GlowCurve = CompletionStateAnimation.AddCurve(0.f, 0.75f);
		CompletionStateAnimation.Play(this->AsShared());
	}

	/** Sets the ExpireDuration */
	virtual void SetExpireDuration(float Duration) override
	{
		ExpireDuration = Duration;
	}

	/** Sets the FadeInDuration */
	virtual void SetFadeInDuration(float Duration) override
	{
		FadeInDuration = Duration;
	}

	/** Sets the FadeOutDuration */
	virtual void SetFadeOutDuration(float Duration) override
	{
		FadeOutDuration = Duration;
	}

	bool Update(float InDeltaTime)
	{
		bool bIsFadingOut = FadeAnimation.IsInReverse();
		if (bAutoExpire)
		{
			InternalTime += InDeltaTime;

			if (!bIsFadingOut && 
				InternalTime > (FadeInDuration.Get() + ExpireDuration.Get()))
			{
				// if we've faded in and shown the notification for the expire duration, start fading out
				Fadeout();
				bIsFadingOut = true;
			}
		}

		const bool bIsCurrentlyPlaying = FadeAnimation.IsPlaying();
		if (bIsFadingOut && !bIsCurrentlyPlaying)
		{
			// Reset the Animation
			FadeoutComplete();
		}

		const bool bIsIntroPlaying = IntroAnimation.IsPlaying();
		if (!bIsIntroPlaying && ThrottleHandle.IsValid())
		{
			// Leave responsive mode once the intro finishes playing
			FSlateThrottleManager::Get().LeaveResponsiveMode( ThrottleHandle );
		}

		return true;
	}

protected:
	/** A fadeout has completed */
	void FadeoutComplete()
	{
		// Make sure we are no longer fading
		FadeAnimation = FCurveSequence();
		FadeCurve = FCurveHandle();

		// Clear the complete state to hide all the images/throbber
		SetCompletionState(CS_None);

		// Make sure we have left responsive mode
		if ( ThrottleHandle.IsValid() )
		{
			FSlateThrottleManager::Get().LeaveResponsiveMode( ThrottleHandle );
		}

		// Clear reference
		if(TSharedPtr< SNotificationList> MyListPinned = MyList.Pin())
		{
			MyListPinned->NotificationItemFadedOut(SharedThis(this));
		}
	}

	/** Gets the current color along the fadeout curve */
	FSlateColor GetContentColor() const
	{
		return GetContentColorRaw();
	}

	/** Gets the current color along the fadeout curve */
	FLinearColor GetContentColorRaw() const
	{
		// if we have a parent window, we need to make that transparent, rather than
		// this widget
		if(TSharedPtr<SNotificationList> MyListPinned = MyList.Pin())
		{
			if (TSharedPtr<SWindow> MyWindow = MyListPinned->ParentWindowPtr.Pin())
			{
				MyWindow->SetOpacity(FadeCurve.GetLerp());
			}
			return FLinearColor(1,1,1,1);
		}
		else
		{
			return FMath::Lerp( FLinearColor(1,1,1,0), FLinearColor(1,1,1,1), FadeCurve.GetLerp() );
		}
	}

	/** Gets the color of the glow effect */
	FSlateColor GetGlowColor() const
	{
		float GlowAlpha = 1.0f - GlowCurve.GetLerp();
		
		if (GlowAlpha == 1.0f)
		{
			GlowAlpha = 0.0f;
		}

		switch (CompletionState)
		{
		case CS_Success: return FLinearColor(0,1,0,GlowAlpha);
		case CS_Fail: return FLinearColor(1,0,0,GlowAlpha);
		default: return DefaultGlowColor.CopyWithNewOpacity(GlowAlpha);
		}
	}

	/** Gets the scale for the entire item */
	FVector2D GetItemScale() const
	{
		return FVector2D(  ScaleCurveX.GetLerp(), ScaleCurveY.GetLerp() );
	}

	FMargin GetIconPadding() const
	{
		if (GetThrobberVisibility().IsVisible())
		{
			// Account for the throbber taking up space
			return FMargin(-6.0f, 0.0f, 0.0f, 0.0f);
		}

		return FMargin(0);
	}

	/** Gets the visibility for the throbber */
	EVisibility GetThrobberVisibility() const
	{
		return CompletionState == CS_Pending ? EVisibility::Visible : EVisibility::Collapsed;
	}

	TOptional<FSlateRenderTransform> GetThrobberTransform() const
	{
		const float DeltaAngle = ThrobberAnimation.GetLerp()*2*PI;
		return FSlateRenderTransform(FQuat2D(DeltaAngle));
	}

	EVisibility GetSuccessFailImageVisibility() const
	{
		return (CompletionState == CS_Success || CompletionState == CS_Fail) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	const FSlateBrush* GetNotificationIcon() const
	{
		switch (CompletionState)
		{
		case ECompletionState::CS_Success:
			return FAppStyle::Get().GetBrush("Icons.SuccessWithColor.Large");
		case ECompletionState::CS_Fail:
			return FAppStyle::Get().GetBrush("Icons.ErrorWithColor.Large");
		default:
			return BaseIcon.Get();
		}
	}

public:
	/** The SNotificationList that is displaying this Item */
	TWeakPtr<SNotificationList> MyList;

protected:

	/** The main text displayed in this text block */
	TAttribute<FText> Text;

	/** The sub text displayed in this text block. Subtext uses a smaller font and */
	TAttribute<FText> SubText;

	/** The fade in duration for this element */
	TAttribute<float> FadeInDuration;

	/** The fade out duration for this element */
	TAttribute<float> FadeOutDuration;

	/** The duration before a fadeout for this element */
	TAttribute<float> ExpireDuration;

	TAttribute<const FSlateBrush*> BaseIcon;

	/** The default glow color for pulse animation. */
	FLinearColor DefaultGlowColor = FLinearColor(1.0f, 1.0f, 1.0f);

	/** The text displayed in this element */
	TSharedPtr<STextBlock> MyTextBlock;

	TSharedPtr<SRichTextBlock> MySubTextBlock;

	/** The completion state of this message */	
	ECompletionState CompletionState;

	/** The fading animation */
	FCurveSequence FadeAnimation;
	FCurveHandle FadeCurve;

	/** The intro animation */
	FCurveSequence IntroAnimation;
	FCurveHandle ScaleCurveX;
	FCurveHandle ScaleCurveY;
	FCurveHandle GlowCurve;
	FCurveSequence ThrobberAnimation;
	/** The completion state change animation */
	FCurveSequence CompletionStateAnimation;

	/** Handle to a throttle request made to ensure the intro animation is smooth in low FPS situations */
	FThrottleRequest ThrottleHandle;
	FTSTicker::FDelegateHandle TickDelegateHandle;

	float InternalTime = 0.0f;
	bool bAutoExpire = false;
};



/////////////////////////////////////////////////
// SNotificationItemImpl

/** A single line in the event message list with additional buttons*/
class SNotificationItemImpl : public SNotificationExtendable
{
public:
	SLATE_BEGIN_ARGS( SNotificationItemImpl )
		: _Text()
		, _Image()	
		, _FadeInDuration(0.5f)
		, _FadeOutDuration(2.f)
		, _ExpireDuration(1.f)
		, _WidthOverride(FOptionalSize())
	{}

		/** The text displayed in this text block */
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ATTRIBUTE(FText, SubText)

		/** Setup information for the buttons on the notification */ 
		SLATE_ARGUMENT(TArray<FNotificationButtonInfo>, ButtonDetails)
		/** The icon image to display next to the text */
		SLATE_ARGUMENT(const FSlateBrush*, Image)
		/** The fade in duration for this element */
		SLATE_ARGUMENT(float, FadeInDuration)
		/** The fade out duration for this element */
		SLATE_ARGUMENT(float, FadeOutDuration)
		/** The duration before a fadeout for this element */
		SLATE_ARGUMENT(float, ExpireDuration)
		/** Controls whether or not to add the animated throbber */
		SLATE_ARGUMENT(bool, bUseThrobber)
		/** Controls whether or not to display the success and fail icons */
		SLATE_ARGUMENT(bool, bUseSuccessFailIcons)
		/** When true the larger bolder font will be used to display the message */
		SLATE_ARGUMENT(bool, bUseLargeFont)
		/** Controls whether or not to display the copy to clipboard button */
		SLATE_ARGUMENT(bool, bUseCopyToClipboard)
		/** When set this forces the width of the box, used to stop resizeing on text change */
		SLATE_ARGUMENT(FOptionalSize, WidthOverride)
		/** When set this will display a check box on the notification; handles getting the current check box state */
		SLATE_ATTRIBUTE(ECheckBoxState, CheckBoxState)
		/** When set this will display a check box on the notification; handles setting the new check box state */
		SLATE_EVENT(FOnCheckStateChanged, CheckBoxStateChanged);
		/** Text to display for the check box message */
		SLATE_ATTRIBUTE(FText, CheckBoxText );
		/** When set this will display as a hyperlink on the right side of the notification. */
		SLATE_EVENT(FSimpleDelegate, Hyperlink)
		/** Text to display for the hyperlink (if Hyperlink is valid) */
		SLATE_ATTRIBUTE(FText, HyperlinkText);

	SLATE_END_ARGS()
	/**
	 * Constructs this widget
	 *
	 * @param InArgs    Declaration from which to construct the widget
	*/
	void Construct( const FArguments& InArgs )
	{
		CompletionState = CS_None;

		Text = InArgs._Text;
		SubText = InArgs._SubText;

		FadeInDuration = InArgs._FadeInDuration;
		FadeOutDuration = InArgs._FadeOutDuration;
		ExpireDuration = InArgs._ExpireDuration;

		ChildSlot
		[
			SNew(SNotificationBackground)
			.Padding(FMargin(16, 8))
			.BorderBackgroundColor(this, &SNotificationItemImpl::GetContentColor)
			.ColorAndOpacity(this, &SNotificationItemImpl::GetContentColorRaw)
			.DesiredSizeScale(this, &SNotificationItemImpl::GetItemScale)
			[
				ConstructInternals(InArgs)
			]
		];
	}
	
	/**
	 * Returns the internals of the notification
	 */
	TSharedRef<SWidget> ConstructInternals( const FArguments& InArgs ) 
	{
		CheckBoxStateChanged = InArgs._CheckBoxStateChanged;
		Hyperlink = InArgs._Hyperlink;
		HyperlinkText = InArgs._HyperlinkText;
		BaseIcon = InArgs._Image;

		// Container for the text and optional interactive widgets (buttons, check box, and hyperlink)
		TSharedRef<SVerticalBox> InteractiveWidgetsBox = SNew(SVerticalBox);
		TSharedRef<SWidget> CopyToClipboardButton = SNullWidget::NullWidget;

		if (InArgs._bUseCopyToClipboard)
		{
			SAssignNew(CopyToClipboardButton, SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			.Padding(0.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("CopyToClipboard", "Copy to Clipboard"))
				.OnClicked(this, &SNotificationItemImpl::OnCopyTextToClipboard)
				.ContentPadding(0.f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
		}

		InteractiveWidgetsBox->AddSlot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(MyTextBlock, STextBlock)
				.Text(Text)
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontBold")))
				.WrapTextAt(InArgs._WidthOverride.IsSet() ? InArgs._WidthOverride.Get()-50.0f : 0.0f )
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(0.f)
			[
				CopyToClipboardButton
			]
		];

		InteractiveWidgetsBox->AddSlot()
		.Padding(FMargin(0.0f, 10.0f, 0.0f, 2.0f))
		.AutoHeight()
		[
			SAssignNew(MySubTextBlock, SRichTextBlock)
			.Text(SubText)
			.TextStyle(FAppStyle::Get(), "NotificationList.WidgetText")
			.Visibility_Lambda([this]() { return SubText.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::SelfHitTestInvisible; })
			.WrapTextAt(InArgs._WidthOverride.IsSet() ? InArgs._WidthOverride.Get() - 50.0f : 0.0f)
			+ SRichTextBlock::HyperlinkDecorator(TEXT("a"),
				FSlateHyperlinkRun::FOnClick::CreateLambda([](const FSlateHyperlinkRun::FMetadata& Metadata)
				{
					const FString* Url = Metadata.Find(TEXT("href"));
					if (Url && !Url->IsEmpty())
					{
						FPlatformProcess::LaunchURL(**Url, nullptr, nullptr);
					}
				}))
		];

		{
			InteractiveWidgetsBox->AddSlot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 10.0f, 0.0f, 2.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SHyperlink)
				.Visibility(this, &SNotificationItemImpl::GetHyperlinkVisibility)
				.Text(this, &SNotificationItemImpl::GetHyperlinkText)
				.OnNavigate(this, &SNotificationItemImpl::OnHyperlinkClicked)
				.ToolTipText(this, &SNotificationItemImpl::GetHyperlinkText)
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
			];
		}

		{
			InteractiveWidgetsBox->AddSlot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 10.0f, 0.0f, 2.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
				.Visibility(this, &SNotificationItemImpl::GetCheckBoxVisibility)
				.IsChecked(InArgs._CheckBoxState)
				.OnCheckStateChanged(CheckBoxStateChanged)
				.Padding(FMargin(2.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(InArgs._CheckBoxText)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
					.ToolTipText(InArgs._CheckBoxText)
				]
			];
		}

		{
			// Adds any buttons that were passed in.
			TSharedRef<SHorizontalBox> ButtonsBox = SNew(SHorizontalBox);

			for (const FNotificationButtonInfo& Button : InArgs._ButtonDetails)
			{
				ButtonsBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SButton)
					.Text(Button.Text)
					.ToolTipText(Button.ToolTip.IsSet() ? Button.ToolTip : Button.Text)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NotificationList.WidgetText"))
					.OnClicked(this, &SNotificationItemImpl::OnButtonClicked, Button.Callback) 
					.Visibility( this, &SNotificationItemImpl::GetButtonVisibility, Button.VisibilityCallback, Button.VisibilityOnNone, Button.VisibilityOnPending, Button.VisibilityOnSuccess, Button.VisibilityOnFail )
					.IsEnabled( this, &SNotificationItemImpl::IsButtonEnabled, Button.IsEnabledCallback )
				];
			}

			InteractiveWidgetsBox->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				ButtonsBox
			];
		}

		return
			SNew(SBox)
			.WidthOverride(InArgs._WidthOverride)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(MakeAttributeSP(this, &SNotificationItemImpl::GetIconPadding))
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("NotificationList.Throbber"))
						.Visibility(InArgs._bUseThrobber ? MakeAttributeSP(this, &SNotificationItemImpl::GetThrobberVisibility) : EVisibility::Collapsed)
						.RenderTransform(this, &SNotificationItemImpl::GetThrobberTransform)
						.RenderTransformPivot(FVector2D(.5f,.5f))
					] 
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(InArgs._bUseSuccessFailIcons ? MakeAttributeSP<const FSlateBrush*>(this, &SNotificationItemImpl::GetNotificationIcon) : BaseIcon)
						.DesiredSizeOverride(FVector2D(32,32))
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(10.f, 0.f, 0.f, 0.f)
				[
					InteractiveWidgetsBox
				]
			];
	}

	/** Sets the text and delegate for the hyperlink */
	virtual void SetHyperlink( const FSimpleDelegate& InHyperlink, const TAttribute< FText >& InHyperlinkText = TAttribute< FText >() ) override
	{
		Hyperlink = InHyperlink;

		// Only replace the text if specified
		if ( InHyperlinkText.IsBound() || InHyperlinkText.IsSet() )
		{
			HyperlinkText = InHyperlinkText;
		}
	}

protected:

	/* Used to determine whether the button is enabled */
	bool IsButtonEnabled( const FNotificationButtonInfo::FIsEnabledDelegate IsEnabledCallback ) const
	{
		return !IsEnabledCallback.IsBound() || IsEnabledCallback.Execute(CompletionState);
	}

	/* Used to determine whether the button is visible */
	EVisibility GetButtonVisibility( const FNotificationButtonInfo::FVisibilityDelegate VisibilityCallback, const EVisibility VisibilityOnNone, const EVisibility VisibilityOnPending, const EVisibility VisibilityOnSuccess, const EVisibility VisibilityOnFail ) const
	{
		if (VisibilityCallback.IsBound())
		{
			return VisibilityCallback.Execute(CompletionState);
		}
		switch ( CompletionState )
		{
		case SNotificationItem::CS_None: return VisibilityOnNone;
		case SNotificationItem::CS_Pending: return VisibilityOnPending;
		case SNotificationItem::CS_Success: return VisibilityOnSuccess;
		case SNotificationItem::CS_Fail: return VisibilityOnFail;
		default:
			check( false );
			return EVisibility::Visible;
		}
	}

	/* Used to determine whether the check box is visible */
	EVisibility GetCheckBoxVisibility() const
	{
		return CheckBoxStateChanged.IsBound() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/* Used to determine whether the hyperlink is visible */
	EVisibility GetHyperlinkVisibility() const
	{
		return Hyperlink.IsBound() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/* Used to fetch the text to display in the hyperlink */
	FText GetHyperlinkText() const
	{
		return HyperlinkText.Get();
	}

	/* Used to fetch the text to copy to the clipboard */
	FText GetClipboardText() const
	{
		FTextBuilder ClipboardTextBuilder;
		if (Text.IsSet())
		{
			ClipboardTextBuilder.AppendLine(Text.Get());
		}

		if (SubText.IsSet())
		{
			ClipboardTextBuilder.AppendLine(SubText.Get());
		}

		return ClipboardTextBuilder.ToText();
	}

private:

	/* Used as a wrapper for the callback, this means any code calling it does not require access to FReply type */
	FReply OnButtonClicked(FSimpleDelegate InCallback)
	{
		InCallback.ExecuteIfBound();
		return FReply::Handled();
	}

	/* Execute the delegate for the hyperlink, if bound */
	void OnHyperlinkClicked() const
	{
		Hyperlink.ExecuteIfBound();
	}

	FReply OnCopyTextToClipboard() const
	{
		FPlatformApplicationMisc::ClipboardCopy(*GetClipboardText().ToString());
		return FReply::Handled();
	}

protected:

	/** When set this will display a check box on the notification; handles setting the new check box state */
	FOnCheckStateChanged CheckBoxStateChanged;

	/** When set this will display as a hyperlink on the right side of the notification. */
	FSimpleDelegate Hyperlink;

	/** Text to display for the hyperlink message */
	TAttribute<FText> HyperlinkText;
};

/////////////////////////////////////////////////
// SNotificationItemExternalImpl

/** A single line in the event message list with the actual content provided by the client */
class SNotificationItemExternalImpl : public SNotificationItemImpl
{
public:
	SLATE_BEGIN_ARGS(SNotificationItemExternalImpl)
		: _FadeInDuration(0.5f)
		, _FadeOutDuration(2.f)
		, _ExpireDuration(1.f)
	{}

		/** The fade in duration for this element */
		SLATE_ATTRIBUTE(float, FadeInDuration)
		/** The fade out duration for this element */
		SLATE_ATTRIBUTE(float, FadeOutDuration)
		/** The duration before a fadeout for this element */
		SLATE_ATTRIBUTE(float, ExpireDuration)
		/** When set this forces the width of the notification */
		SLATE_ARGUMENT(FOptionalSize, WidthOverride)
		/** The widget that provides the notification to display */
		SLATE_ARGUMENT(TSharedPtr<INotificationWidget>, ContentWidget)

	SLATE_END_ARGS()

	/**
	* Constructs this widget
	*
	* @param InArgs    Declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs)
	{
		check(InArgs._ContentWidget.IsValid());

		FadeInDuration = InArgs._FadeInDuration;
		FadeOutDuration = InArgs._FadeOutDuration;
		ExpireDuration = InArgs._ExpireDuration;
		NotificationWidget = InArgs._ContentWidget;

		TSharedRef<SWidget> Internals = SNullWidget::NullWidget;

		if (NotificationWidget->UseNotificationBackground())
		{
			Internals =
					SNew(SNotificationBackground)
					.Padding(FMargin(16, 8))
					.BorderBackgroundColor(this, &SNotificationItemExternalImpl::GetContentColor)
					.ColorAndOpacity(this, &SNotificationItemExternalImpl::GetContentColorRaw)
					.DesiredSizeScale(this, &SNotificationItemExternalImpl::GetItemScale)
					[
						SNew(SBox)
						.WidthOverride(InArgs._WidthOverride)
						[
							NotificationWidget->AsWidget()
						]
					];
		}
		else
		{
			Internals = 
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
				.BorderBackgroundColor(this, &SNotificationItemExternalImpl::GetContentColor)
				.ColorAndOpacity(this, &SNotificationItemExternalImpl::GetContentColorRaw)
				.DesiredSizeScale(this, &SNotificationItemExternalImpl::GetItemScale)
				[
					SNew(SBox)
					.WidthOverride(InArgs._WidthOverride)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							NotificationWidget->AsWidget()
						]
						+ SOverlay::Slot()
						[
							SNew(SBorder)
							.Padding(0)
							.BorderImage(FCoreStyle::Get().GetBrush("NotificationList.ItemBackground_Border_Transparent"))
							.BorderBackgroundColor(this, &SNotificationItemExternalImpl::GetGlowColor)
							.Visibility(EVisibility::SelfHitTestInvisible)
						]
					]
				];
		}

		ChildSlot
		[
			Internals
		];
	}

	/** SNotificationItem interface */
	virtual void SetCompletionState(ECompletionState State) override
	{
		SNotificationItemImpl::SetCompletionState(State);

		NotificationWidget->OnSetCompletionState(State);
	}

private:
	TSharedPtr<INotificationWidget> NotificationWidget;
};

/////////////////////////////////////////////////
// FNotificationInfo

void FNotificationInfo::ShowCopyToClipboadHyperlink()
{
	HyperlinkText = LOCTEXT("CopyToClipboard", "Copy to Clipboard");
	Hyperlink = FSimpleDelegate::CreateLambda(
		[Text = Text, SubText = SubText]()
		{
			TStringBuilder<1024> StrBuilder;
			if (Text.IsSet())
			{
				StrBuilder << Text.Get().ToString();
			}
			if (SubText.IsSet())
			{
				if (StrBuilder.Len() > 0)
				{
					StrBuilder << TEXT("\n");
				}
				StrBuilder << SubText.Get().ToString();
			}

			FPlatformApplicationMisc::ClipboardCopy(StrBuilder.ToString());
		});
}

///////////////////////////////////////////////////
//// SNotificationList

TSharedRef<SNotificationItem> SNotificationList::AddNotification(const FNotificationInfo& Info)
{
	TSharedPtr<SNotificationExtendable> NewItem;

	if (FSlateNotificationManager::Get().AreNotificationsAllowed())
	{
		if (Info.ContentWidget.IsValid())
		{
			NewItem = SNew(SNotificationItemExternalImpl)
				.ContentWidget(Info.ContentWidget)
				.FadeInDuration(Info.FadeInDuration)
				.ExpireDuration(Info.ExpireDuration)
				.FadeOutDuration(Info.FadeOutDuration)
				.WidthOverride(Info.WidthOverride);
		}
		else
		{
			static const FSlateBrush* CachedImage = FAppStyle::Get().GetBrush("UnrealCircle.Thick");

			// Create notification.
			NewItem = SNew(SNotificationItemImpl)
				.Text(Info.Text)
				.SubText(Info.SubText)
				.ButtonDetails(Info.ButtonDetails)
				.Image((Info.Image != nullptr) ? Info.Image : CachedImage)
				.FadeInDuration(Info.FadeInDuration)
				.ExpireDuration(Info.ExpireDuration)
				.FadeOutDuration(Info.FadeOutDuration)
				.bUseThrobber(Info.bUseThrobber)
				.bUseSuccessFailIcons(Info.bUseSuccessFailIcons)
				.bUseLargeFont(Info.bUseLargeFont)
				.bUseCopyToClipboard(Info.bUseCopyToClipboard)
				.WidthOverride(Info.WidthOverride)
				.CheckBoxState(Info.CheckBoxState)
				.CheckBoxStateChanged(Info.CheckBoxStateChanged)
				.CheckBoxText(Info.CheckBoxText)
				.Hyperlink(Info.Hyperlink)
				.HyperlinkText(Info.HyperlinkText);
		}
		
		NewItem->MyList = SharedThis(this);

		MessageItemBoxPtr->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				NewItem.ToSharedRef()
			];

		NewItem->Fadein( Info.bAllowThrottleWhenFrameRateIsLow );

		if (Info.bFireAndForget)
		{
			NewItem->ExpireAndFadeout();
		}

		if (Info.DefaultState != SNotificationItem::ECompletionState::CS_None)
		{
			NewItem->SetCompletionState(Info.DefaultState);
		}
	}
	else
	{
		// When notifications are not allowed we want to return an empty notification.
		NewItem = SNew(SNotificationItemImpl);
	}

	LastNotification = NewItem;

	return NewItem.ToSharedRef();
}

void SNotificationList::NotificationItemFadedOut (const TSharedRef<SNotificationItem>& NotificationItem)
{
	if( ParentWindowPtr.IsValid() )
	{
		// If we are in a single-window per notification situation, we dont want to 
		// remove the NotificationItem straight away, rather we will flag us as done
		// and wait for the FSlateNotificationManager to release the parent window.
		bDone = true;
	}
	else
	{
		// This should remove the last non-local reference to this SNotificationItem.
		// Since there may be many local references on the call stack we are not checking
		// if the reference is unique.
		MessageItemBoxPtr->RemoveSlot(NotificationItem);
	}
}

void SNotificationList::Construct(const FArguments& InArgs)
{
	bDone = false;

	ChildSlot
	[
		SAssignNew(MessageItemBoxPtr, SVerticalBox)
	];
}

#undef LOCTEXT_NAMESPACE 
