// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SPopUpErrorText.h"


void SPopupErrorText::Construct( const FArguments& InArgs )
{
	SComboButton::Construct( SComboButton::FArguments()
		.ComboButtonStyle( FCoreStyle::Get(), "MessageLogListingComboButton" )
		.HasDownArrow(false)
		.ContentPadding(0.f)
		.IsFocusable(false)
		.ButtonContent()
		[
			SAssignNew( HasErrorSymbol, SErrorText )
			.ErrorText(NSLOCTEXT("UnrealEd", "Error", "!"))
			.Font( InArgs._Font )
		]
		.MenuPlacement(MenuPlacement_BelowAnchor)
		.MenuContent()
		[
			SAssignNew( ErrorText, SErrorText )
			.Font( InArgs._Font )
		]
	);
}

void SPopupErrorText::SetError( const FText& InErrorText )
{
	SetError( InErrorText.ToString() );
}

void SPopupErrorText::SetError( const FString& InErrorText )
{
	const bool bHasError = !InErrorText.IsEmpty();

	ErrorText->SetError(InErrorText);
	HasErrorSymbol->SetError( bHasError ? NSLOCTEXT("UnrealEd", "Error", "!") : FText::GetEmpty() );

	this->SetIsOpen( bHasError, false );
}
		
bool SPopupErrorText::HasError() const
{
	return ErrorText->HasError();
}

TSharedRef<SWidget> SPopupErrorText::AsWidget()
{
	return SharedThis(this);
}


