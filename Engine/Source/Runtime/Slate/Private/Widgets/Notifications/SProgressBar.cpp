// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SProgressBar.h"
#include "Rendering/DrawElements.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SProgressBar)


SLATE_IMPLEMENT_WIDGET(SProgressBar)
void SProgressBar::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	struct FInvalidation
	{
		static void UpdateMarquee(SWidget& OwningWidget)
		{
			static_cast<SProgressBar&>(OwningWidget).bNeedsMarqueeTimerUpdate = true;
		}
	};
	
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Percent, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateMarquee));
	
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, FillColorAndOpacity, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BorderPadding, EInvalidateWidgetReason::Layout);
}

SProgressBar::SProgressBar()
	: Percent(*this, TOptional<float>())
	, FillColorAndOpacity(*this, FLinearColor::White)
	, BorderPadding(*this, FVector2D(0,0))
{
}

SProgressBar::~SProgressBar()
{
}

void SProgressBar::Construct( const FArguments& InArgs )
{
	check(InArgs._Style);

	bNeedsMarqueeTimerUpdate = true;
	MarqueeOffset = 0.0f;

	Style = InArgs._Style;

	SetPercent(InArgs._Percent);
	BarFillType = InArgs._BarFillType;
	BarFillStyle = InArgs._BarFillStyle;
	
	BackgroundImage = InArgs._BackgroundImage;
	FillImage = InArgs._FillImage;
	MarqueeImage = InArgs._MarqueeImage;
	
	FillColorAndOpacity.Assign(*this, InArgs._FillColorAndOpacity);
	BorderPadding.Assign(*this, InArgs._BorderPadding);

	SetCanTick(false);

	UpdateMarqueeActiveTimer();
}

void SProgressBar::SetPercent(TAttribute< TOptional<float> > InPercent)
{
	Percent.Assign(*this, MoveTemp(InPercent));
}

void SProgressBar::SetStyle(const FProgressBarStyle* InStyle)
{
	Style = InStyle;

	if (Style == nullptr)
	{
		FArguments Defaults;
		Style = Defaults._Style;
	}

	check(Style);

	UpdateMarqueeActiveTimer();
	Invalidate(EInvalidateWidget::Layout);
}

void SProgressBar::SetBarFillType(EProgressBarFillType::Type InBarFillType)
{
	if(BarFillType != InBarFillType)
	{
		BarFillType = InBarFillType;
		Invalidate(EInvalidateWidget::Paint);
	}
}

void SProgressBar::SetBarFillStyle(EProgressBarFillStyle::Type InBarFillStyle)
{
	if (BarFillStyle != InBarFillStyle)
	{
		BarFillStyle = InBarFillStyle;
		Invalidate(EInvalidateWidget::Paint);
	}
}

void SProgressBar::SetFillColorAndOpacity(TAttribute< FSlateColor > InFillColorAndOpacity)
{
	FillColorAndOpacity.Assign(*this, MoveTemp(InFillColorAndOpacity));
}

void SProgressBar::SetBorderPadding(TAttribute< FVector2D > InBorderPadding)
{
	BorderPadding.Assign(*this, MoveTemp(InBorderPadding));
}

void SProgressBar::SetBackgroundImage(const FSlateBrush* InBackgroundImage)
{
	if(BackgroundImage != InBackgroundImage)
	{
		BackgroundImage = InBackgroundImage;
		Invalidate(EInvalidateWidget::Layout);
	}
}

void SProgressBar::SetFillImage(const FSlateBrush* InFillImage)
{
	if(FillImage != InFillImage)
	{
		FillImage = InFillImage;
		Invalidate(EInvalidateWidget::Layout);
	}
}

void SProgressBar::SetMarqueeImage(const FSlateBrush* InMarqueeImage)
{
	if(MarqueeImage != InMarqueeImage)
	{
		MarqueeImage = InMarqueeImage;
		Invalidate(EInvalidateWidget::Layout);
	}
}

const FSlateBrush* SProgressBar::GetBackgroundImage() const
{
	return BackgroundImage ? BackgroundImage : &Style->BackgroundImage;
}

const FSlateBrush* SProgressBar::GetFillImage() const
{
	return FillImage ? FillImage : &Style->FillImage;
}

const FSlateBrush* SProgressBar::GetMarqueeImage() const
{
	return MarqueeImage ? MarqueeImage : &Style->MarqueeImage;
}

// Returns false if the clipping zone area is zero in which case we should skip drawing
bool PushTransformedClip(FSlateWindowElementList& OutDrawElements, const FGeometry& AllottedGeometry, FVector2D InsetPadding, FVector2D ProgressOrigin, FSlateRect Progress)
{
	const FSlateRenderTransform& Transform = AllottedGeometry.GetAccumulatedRenderTransform();

	const FVector2D MaxSize = AllottedGeometry.GetLocalSize() - ( InsetPadding * 2.0f );

	const FSlateClippingZone ClippingZone(Transform.TransformPoint(InsetPadding + (ProgressOrigin - FVector2D(Progress.Left, Progress.Top)) * MaxSize),
		Transform.TransformPoint(InsetPadding + FVector2D(ProgressOrigin.X + Progress.Right, ProgressOrigin.Y - Progress.Top) * MaxSize),
		Transform.TransformPoint(InsetPadding + FVector2D(ProgressOrigin.X - Progress.Left, ProgressOrigin.Y + Progress.Bottom) * MaxSize),
		Transform.TransformPoint(InsetPadding + (ProgressOrigin + FVector2D(Progress.Right, Progress.Bottom)) * MaxSize));

	if (ClippingZone.HasZeroArea())
	{
		return false;
	}

	OutDrawElements.PushClip(ClippingZone);
	return true;
}

int32 SProgressBar::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	if (bNeedsMarqueeTimerUpdate)
	{
		const_cast<SProgressBar*>(this)->UpdateMarqueeActiveTimer();
	}

	// Used to track the layer ID we will return.
	int32 RetLayerId = LayerId;

	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
	const FSlateBrush* CurrentFillImage = GetFillImage();

	TOptional<float> ProgressFraction = Percent.Get();	
	FVector2D BorderPaddingRef = BorderPadding.Get();

	const FSlateBrush* CurrentBackgroundImage = GetBackgroundImage();

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		RetLayerId++,
		AllottedGeometry.ToPaintGeometry(),
		CurrentBackgroundImage,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * CurrentBackgroundImage->GetTint( InWidgetStyle )
	);
	
	if( ProgressFraction.IsSet() )
	{
		const FLinearColor FillColorAndOpacitySRGB(InWidgetStyle.GetColorAndOpacityTint() * FillColorAndOpacity.Get().GetColor(InWidgetStyle) * CurrentFillImage->GetTint(InWidgetStyle));

		EProgressBarFillType::Type ComputedBarFillType = BarFillType;
		if (GSlateFlowDirection == EFlowDirection::RightToLeft)
		{
			switch (ComputedBarFillType)
			{
			case EProgressBarFillType::LeftToRight:
				ComputedBarFillType = EProgressBarFillType::RightToLeft;
				break;
			case EProgressBarFillType::RightToLeft:
				ComputedBarFillType = EProgressBarFillType::LeftToRight;
				break;
			}
		}

		float MarqueeAnimOffsetX = 0.0f;
		float MarqueeImageSizeX = 0.0f;
		float MarqueeAnimOffsetY = 0.0f;
		float MarqueeImageSizeY = 0.0f;

		if (Style->EnableFillAnimation)
		{
			MarqueeAnimOffsetX = CurrentFillImage->ImageSize.X * MarqueeOffset;
			MarqueeImageSizeX = CurrentFillImage->ImageSize.X;

			MarqueeAnimOffsetY = CurrentFillImage->ImageSize.Y * MarqueeOffset;
			MarqueeImageSizeY = CurrentFillImage->ImageSize.Y;
		}

		bool bScaleWithFillPerc = BarFillStyle == EProgressBarFillStyle::Scale;

		const float ClampedFraction = FMath::Clamp(ProgressFraction.GetValue(), 0.0f, 1.0f);
		switch (ComputedBarFillType)
		{
			case EProgressBarFillType::FillFromCenter:
			{
				float HalfFrac = ClampedFraction / 2.0f;
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0.5, 0.5), FSlateRect(HalfFrac, HalfFrac, HalfFrac, HalfFrac)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X * ClampedFraction, AllottedGeometry.GetLocalSize().Y * ClampedFraction),
							FSlateLayoutTransform(FVector2f((AllottedGeometry.GetLocalSize().X * 0.5f) - ((AllottedGeometry.GetLocalSize().X * (ClampedFraction)) * 0.5f), (AllottedGeometry.GetLocalSize().Y * 0.5f) - ((AllottedGeometry.GetLocalSize().Y * (ClampedFraction)) * 0.5f)))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform()
						);
					}

					// Draw Fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
						);

					OutDrawElements.PopClip();
				}
				break;
			}
			case EProgressBarFillType::FillFromCenterHorizontal:
			{
				float HalfFrac = ClampedFraction / 2.0f;
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0.5, 0), FSlateRect(HalfFrac, 0, HalfFrac, 1)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X * ClampedFraction, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f((AllottedGeometry.GetLocalSize().X * 0.5f) - ((AllottedGeometry.GetLocalSize().X * (ClampedFraction)) * 0.5f), 0.0f))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f(0.0f, 0.0f))
						);
					}

					// Draw Fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
					);

					OutDrawElements.PopClip();
				}
				break;
			}
			case EProgressBarFillType::FillFromCenterVertical:
			{
				float HalfFrac = ClampedFraction / 2.0f;
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0, 0.5), FSlateRect(0, HalfFrac, 1, HalfFrac)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y * ClampedFraction),
							FSlateLayoutTransform(FVector2f(0.0f, (AllottedGeometry.GetLocalSize().Y * 0.5f) - ((AllottedGeometry.GetLocalSize().Y * (ClampedFraction)) * 0.5f)))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f(0.0f, 0.0f))
						);
					}

					// Draw Fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
					);

					OutDrawElements.PopClip();
				}
				break;
			}
			case EProgressBarFillType::TopToBottom:
			{
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0, 0), FSlateRect(0, 0, 1, ClampedFraction)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y * ClampedFraction),
							FSlateLayoutTransform(FVector2f(0.0f, 0.0f))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y + MarqueeImageSizeY),
							FSlateLayoutTransform(FVector2f(0.0f, MarqueeAnimOffsetY - MarqueeImageSizeY))
						);
					}

					// Draw Fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
						);

					OutDrawElements.PopClip();
				}
				break;
			}
			case EProgressBarFillType::BottomToTop:
			{
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0, 1), FSlateRect(0, ClampedFraction, 1, 0)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y * ClampedFraction),
							FSlateLayoutTransform(FVector2f(0.0f, AllottedGeometry.GetLocalSize().Y * (1.0f - ClampedFraction)))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y + MarqueeImageSizeY),
							FSlateLayoutTransform(FVector2f(0.0f, MarqueeAnimOffsetY - MarqueeImageSizeY))
						);
					}

					// Draw Fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
					);

					OutDrawElements.PopClip();
				}
				break;
			}
			case EProgressBarFillType::RightToLeft:
			{
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(1, 0), FSlateRect(ClampedFraction, 0, 0, 1)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X * ClampedFraction, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f(AllottedGeometry.GetLocalSize().X * (1.0f - ClampedFraction), 0.0f))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X + MarqueeImageSizeX, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f(MarqueeAnimOffsetX - MarqueeImageSizeX, 0.0f))
						);
					}

					// Draw Fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
					);

					OutDrawElements.PopClip();
				}
				break;
			}
			case EProgressBarFillType::LeftToRight:
			default:
			{
				if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0, 0), FSlateRect(0, 0, ClampedFraction, 1)))
				{
					FPaintGeometry PaintRect;
					if (bScaleWithFillPerc)
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X * ClampedFraction, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f(0.0f, 0.0f))
						);
					}
					else
					{
						PaintRect = AllottedGeometry.ToPaintGeometry(
							FVector2f(AllottedGeometry.GetLocalSize().X + MarqueeImageSizeX, AllottedGeometry.GetLocalSize().Y),
							FSlateLayoutTransform(FVector2f(MarqueeAnimOffsetX - MarqueeImageSizeX, 0.0f))
						);
					}

					// Draw fill
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						RetLayerId++,
						PaintRect,
						CurrentFillImage,
						DrawEffects,
						FillColorAndOpacitySRGB
					);
					OutDrawElements.PopClip();
				}


				break;
			}
		}
	}
	else
	{
		const FSlateBrush* CurrentMarqueeImage = GetMarqueeImage();
		
		const FLinearColor FillColorAndOpacitySRGB(InWidgetStyle.GetColorAndOpacityTint()* FillColorAndOpacity.Get().GetColor(InWidgetStyle)* CurrentMarqueeImage->GetTint(InWidgetStyle));


		// Draw Marquee
		const float MarqueeAnimOffset = CurrentMarqueeImage->ImageSize.X * MarqueeOffset;
		const float MarqueeImageSize = CurrentMarqueeImage->ImageSize.X;

		if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPaddingRef, FVector2D(0, 0), FSlateRect(0, 0, 1, 1)))
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				RetLayerId++,
				AllottedGeometry.ToPaintGeometry(
					FVector2f( AllottedGeometry.GetLocalSize().X + MarqueeImageSize, AllottedGeometry.GetLocalSize().Y ),
					FSlateLayoutTransform(FVector2f( MarqueeAnimOffset - MarqueeImageSize, 0.0f ))
				),
				CurrentMarqueeImage,
				DrawEffects,
				FillColorAndOpacitySRGB
				);

			OutDrawElements.PopClip();
		}
	}

	return RetLayerId - 1;
}

FVector2D SProgressBar::ComputeDesiredSize( float ) const
{
	return FVector2D(GetMarqueeImage()->ImageSize);
}

void SProgressBar::UpdateMarqueeActiveTimer()
{
	if (!Percent.Get().IsSet() || Style->EnableFillAnimation)
	{
		// If percent is not set or fill animation is enabled then set the timer
		if (!ActiveTimerHandle.IsValid())
		{
			// If we already had a handle, there is no need to register a new one.
			ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SProgressBar::ActiveTick));
		}
	}
	else if (ActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(ActiveTimerHandle.Pin().ToSharedRef());
	}
	bNeedsMarqueeTimerUpdate = false;
}

EActiveTimerReturnType SProgressBar::ActiveTick(double InCurrentTime, float InDeltaTime)
{
	MarqueeOffset = (float)(InCurrentTime - FMath::FloorToDouble(InCurrentTime));

	Invalidate(EInvalidateWidget::Paint);

	return EActiveTimerReturnType::Continue;
}



