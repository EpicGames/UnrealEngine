// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCanvas.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"


SLATE_IMPLEMENT_WIDGET(SCanvas)
void SCanvas::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FSlot::RegisterAttributes(Initializer);
}

void SCanvas::FSlot::RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
{
	TWidgetSlotWithAttributeSupport::RegisterAttributes(AttributeInitializer);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.Size", Size, EInvalidateWidgetReason::Paint);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.Position", Position, EInvalidateWidgetReason::Paint)
		.UpdatePrerequisite("Slot.Size");
}

void SCanvas::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArg)
{
	TWidgetSlotWithAttributeSupport<FSlot>::Construct(SlotOwner, MoveTemp(InArg));
	if (InArg._Size.IsSet())
	{
		Size.Assign(*this, MoveTemp(InArg._Size));
	}
	if (InArg._Position.IsSet())
	{
		Position.Assign(*this, MoveTemp(InArg._Position));
	}

	TAlignmentWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArg));
}


SCanvas::SCanvas()
	: Children(this, GET_MEMBER_NAME_CHECKED(SCanvas, Children))
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

SCanvas::~SCanvas() = default;

void SCanvas::Construct( const SCanvas::FArguments& InArgs )
{
	Children.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));
}

SCanvas::FSlot::FSlotArguments SCanvas::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

SCanvas::FScopedWidgetSlotArguments SCanvas::AddSlot()
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Children, INDEX_NONE };
}

void SCanvas::ClearChildren( )
{
	Children.Empty();
}

int32 SCanvas::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	return Children.Remove(SlotWidget);
}

void SCanvas::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	if (Children.Num() > 0)
	{
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const SCanvas::FSlot& CurChild = Children[ChildIndex];
			const FVector2D Size = CurChild.GetSize();

			//Handle HAlignment
			FVector2D Offset(0.0f, 0.0f);

			switch (CurChild.GetHorizontalAlignment())
			{
			case HAlign_Center:
				Offset.X = -Size.X / 2.0f;
				break;
			case HAlign_Right:
				Offset.X = -Size.X;
				break;
			case HAlign_Fill:
			case HAlign_Left:
				break;
			}

			//handle VAlignment
			switch (CurChild.GetVerticalAlignment())
			{
			case VAlign_Bottom:
				Offset.Y = -Size.Y;
				break;
			case VAlign_Center:
				Offset.Y = -Size.Y / 2.0f;
				break;
			case VAlign_Top:
			case VAlign_Fill:
				break;
			}

			// Add the information about this child to the output list (ArrangedChildren)
			ArrangedChildren.AddWidget( AllottedGeometry.MakeChild(
				// The child widget being arranged
				CurChild.GetWidget(),
				// Child's local position (i.e. position within parent)
				CurChild.GetPosition() + Offset,
				// Child's size
				Size
			));
		}
	}
}

int32 SCanvas::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;

	const bool bForwardedEnabled = ShouldBeEnabled(bParentEnabled);

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];

		if ( !IsChildWidgetCulled(MyCullingRect, CurWidget) )
		{
			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, MyCullingRect, OutDrawElements, MaxLayerId + 1, InWidgetStyle, bForwardedEnabled);

			MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
		}
		else
		{

		}
	}

	return MaxLayerId;
}

FVector2D SCanvas::ComputeDesiredSize( float ) const
{
	// Canvas widgets have no desired size -- their size is always determined by their container
	return FVector2D::ZeroVector;
}

FChildren* SCanvas::GetChildren()
{
	return &Children;
}
