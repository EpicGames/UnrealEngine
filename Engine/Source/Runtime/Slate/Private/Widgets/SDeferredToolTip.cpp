// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDeferredToolTip.h"
#include "Widgets/SToolTip.h"


//////////////////////////////////////////////////////////////////////////
// SDeferredToolTip

SDeferredToolTip::SDeferredToolTip(FOnGetDeferredToolTip InOnGetDeferredToolTip)
	: OnGetDeferredToolTip(InOnGetDeferredToolTip)
{
}

TSharedRef<SWidget> SDeferredToolTip::AsWidget()
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->AsWidget();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDeferredToolTip::GetContentWidget() 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->GetContentWidget();
	}

	return SNullWidget::NullWidget;
}

void SDeferredToolTip::SetContentWidget(const TSharedRef<SWidget>& InContentWidget) 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		CachedToolTip->SetContentWidget(InContentWidget);
	}
}

void SDeferredToolTip::ResetContentWidget() 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		CachedToolTip->ResetContentWidget();
	}
}

bool SDeferredToolTip::IsEmpty() const 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->IsEmpty();
	}

	return true;
}

bool SDeferredToolTip::IsInteractive() const 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->IsInteractive();
	}

	return false;
}

void SDeferredToolTip::OnClosed() 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->OnClosed();
	}
}

void SDeferredToolTip::OnOpening() 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->OnOpening();
	}
}

void SDeferredToolTip::OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		CachedToolTip->OnSetInteractiveWindowLocation(InOutDesiredLocation);
	}
}

void SDeferredToolTip::TryCacheToolTip() const
{
	if (!CachedToolTip && OnGetDeferredToolTip.IsBound())
	{
		CachedToolTip = OnGetDeferredToolTip.Execute();
	}
}


//////////////////////////////////////////////////////////////////////////
// SDeferredToolTipText

SDeferredToolTipText::SDeferredToolTipText(const TAttribute<FText>& InToolTipText)
	: ToolTipText(InToolTipText)
{
}

SDeferredToolTipText::SDeferredToolTipText(const FText& InToolTipText)
	: ToolTipText(InToolTipText)
{
}

TSharedRef<SWidget> SDeferredToolTipText::AsWidget()
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->AsWidget();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDeferredToolTipText::GetContentWidget() 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->GetContentWidget();
	}

	return SNullWidget::NullWidget;
}

void SDeferredToolTipText::SetContentWidget(const TSharedRef<SWidget>& InContentWidget) 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		CachedToolTip->SetContentWidget(InContentWidget);
	}
}

void SDeferredToolTipText::ResetContentWidget() 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		CachedToolTip->ResetContentWidget();
	}
}

bool SDeferredToolTipText::IsEmpty() const 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->IsEmpty();
	}

	return true;
}

bool SDeferredToolTipText::IsInteractive() const 
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		return CachedToolTip->IsInteractive();
	}

	return false;
}

void SDeferredToolTipText::OnClosed() 
{
	// SToolTip does No-op. We choose to not forward no-op to no-op.
}

void SDeferredToolTipText::OnOpening() 
{
	// SToolTip does No-op. We choose to not forward no-op to no-op.
}

void SDeferredToolTipText::OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const
{
	TryCacheToolTip();
	if (CachedToolTip)
	{
		CachedToolTip->OnSetInteractiveWindowLocation(InOutDesiredLocation);
	}
}

void SDeferredToolTipText::TryCacheToolTip() const
{
	if (!CachedToolTip)
	{
		SAssignNew(CachedToolTip, SToolTip)
			.Text(ToolTipText);
	}
}