// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SInvalidationPanel.h"
#include "Rendering/DrawElements.h"
#include "Misc/App.h"
#include "Application/SlateApplicationBase.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Types/ReflectionMetadata.h"
#include "Rendering/SlateObjectReferenceCollector.h"
#include "Widgets/SNullWidget.h"

DECLARE_CYCLE_STAT(TEXT("SInvalidationPanel::Paint"), STAT_SlateInvalidationPaint, STATGROUP_Slate);


void ConsoleVariableEnableInvalidationPanelsChanged(IConsoleVariable*);

/** True if we should allow widgets to be cached in the UI at all. */
static bool bInvalidationPanelsEnabled = true;
FAutoConsoleVariableRef CVarEnableInvalidationPanels(
	TEXT("Slate.EnableInvalidationPanels"),
	bInvalidationPanelsEnabled,
	TEXT("Whether to attempt to cache any widgets through invalidation panels."),
	FConsoleVariableDelegate::CreateStatic(ConsoleVariableEnableInvalidationPanelsChanged));

#if WITH_SLATE_DEBUGGING

static bool bAlwaysInvalidate = false;
FAutoConsoleVariableRef CVarAlwaysInvalidate(
	TEXT("Slate.AlwaysInvalidate"),
	bAlwaysInvalidate,
	TEXT("Forces invalidation panels to cache, but to always invalidate."));

#endif // WITH_SLATE_DEBUGGING

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
namespace DynamicInvalidationCVars
{
	void ConsoleVariableEnableDynamicInvalidationChanged(IConsoleVariable*);

	static int32 DynamicInvalidationOptions = 2;
	static FAutoConsoleVariableRef CVarDynamicInvalidationOptions(
		TEXT("Slate.DynamicInvalidation.Options"),
		DynamicInvalidationOptions,
		TEXT("Toggle Dynamic Invalidation options for Invalidation Panels.\n")
		TEXT(" A Dynamic Invalidation Panel refers to an Invalidation Panel with bUseDynamicInvalidation set to true.\n")
		TEXT(" 0. Disable Dynamic Invalidation and disable caching for all Dynamic Invalidation Panels\n")
		TEXT(" 1. Disable Dynamic Invalidation and makes the Dynamic Invalidation Panels act like regular Invalidation Panels\n")
		TEXT(" 2. Enable Dynamic Invalidation for Dynamic Invalidation Panels\n")
		TEXT(" 3. Forces Dynamic Invalidation on all Invalidation Panels"),
	FConsoleVariableDelegate::CreateStatic(ConsoleVariableEnableDynamicInvalidationChanged)
	);

	/** Is Dynamic Invalidation and caching disabled for all Dynamic Invalidation Panels? */
	bool IsDynamicInvalidationDisabled() { return DynamicInvalidationOptions == 0; }
	/** Is Dynamic Invalidation reverted to regular Invalidation? */
	bool IsDynamicInvalidationForcedStatic() { return DynamicInvalidationOptions == 1; }
	/** Is Dynamic Invalidation enabled? */
	bool IsDynamicInvalidationEnabled() { return DynamicInvalidationOptions == 2; }
	/** Is Dynamic Invalidation forced on all invalidation panels? */
	bool IsDynamicInvalidationForced() { return DynamicInvalidationOptions == 3; }
	
	void ConsoleVariableEnableDynamicInvalidationChanged(IConsoleVariable*)
	{
		DynamicInvalidationOptions = FMath::Clamp(DynamicInvalidationOptions, 0, 3);
#if UE_SLATE_TRACE_ENABLED
		const FString Description =
			IsDynamicInvalidationDisabled()		? TEXT("Disabled") :
			IsDynamicInvalidationForcedStatic()	? TEXT("Disabled: forced Static") :
			IsDynamicInvalidationEnabled()		? TEXT("Enabled") :
			IsDynamicInvalidationForced()		? TEXT("Forced") : TEXT("Unknown");

		UE_TRACE_SLATE_BOOKMARK(TEXT("Dynamic Invalidation changed [%d]: %s"), DynamicInvalidationOptions, *Description)
#endif // UE_SLATE_TRACE_ENABLED
	}
}
#endif


SLATE_IMPLEMENT_WIDGET(SInvalidationPanel)
void SInvalidationPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer) {}

SInvalidationPanel::SInvalidationPanel()
	: HittestGrid(MakeShared<FHittestGrid>())
	, bCanCache(true)
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	, bUseDynamicInvalidation(false)
#endif
	, bPaintedSinceLastPrepass(true)
	, bWasCachable(false)
{
	bHasCustomPrepass = true;
	SetInvalidationRootWidget(*this);
	SetInvalidationRootHittestGrid(HittestGrid.Get());
	SetCanTick(false);
	SetVolatilePrepass(GetCanCache());

	LastIncomingColorAndOpacity = FLinearColor::White;

	FSlateApplicationBase::Get().OnGlobalInvalidationToggled().AddRaw(this, &SInvalidationPanel::OnGlobalInvalidationToggled);
}

void SInvalidationPanel::Construct( const FArguments& InArgs )
{
	ChildSlot
	[
		InArgs._Content.Widget
	];

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	bUseDynamicInvalidation = InArgs._UseDynamicInvalidation;
#endif

#if SLATE_VERBOSE_NAMED_EVENTS
	DebugName = InArgs._DebugName;
	DebugTickName = InArgs._DebugName + TEXT("_Tick");
	DebugPaintName = InArgs._DebugName + TEXT("_Paint");
#endif
}

FChildren* SInvalidationPanel::GetAllChildren()
{
	// The default implementation would call SInvalidationPanel::GetChildren which is always empty if we can cache
	return SCompoundWidget::GetChildren();
}

SInvalidationPanel::~SInvalidationPanel()
{
	InvalidateRootChildOrder();

	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().OnGlobalInvalidationToggled().RemoveAll(this);
	}
}

void ConsoleVariableEnableInvalidationPanelsChanged(IConsoleVariable*)
{
	// If the cache changed, the parent's InvalidationRoot need to rebuild its list
	//since InvalidationPanel cannot be nested in regular mode.
	if (!GSlateEnableGlobalInvalidation)
	{
		FSlateApplicationBase::Get().OnGlobalInvalidationToggled().Broadcast(GSlateEnableGlobalInvalidation);
	}
}

#if WITH_SLATE_DEBUGGING
bool SInvalidationPanel::AreInvalidationPanelsEnabled()
{
	return bInvalidationPanelsEnabled;
}

void SInvalidationPanel::EnableInvalidationPanels(bool bEnable)
{
	if (bInvalidationPanelsEnabled != bEnable)
	{
		bInvalidationPanelsEnabled = bEnable;

		// If the cache changed, the parent's InvalidationRoot need to rebuild its list
		//since InvalidationPanel cannot be nested in regular mode.
		if (!GSlateEnableGlobalInvalidation)
		{
			FSlateApplicationBase::Get().OnGlobalInvalidationToggled().Broadcast(GSlateEnableGlobalInvalidation);
		}
	}
}
#endif

bool SInvalidationPanel::GetCanCache() const
{
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	if (DynamicInvalidationCVars::IsDynamicInvalidationDisabled() & bUseDynamicInvalidation)
	{
		return false;
	}
#endif
	return bCanCache && !GSlateEnableGlobalInvalidation && bInvalidationPanelsEnabled;
}

void SInvalidationPanel::OnGlobalInvalidationToggled(bool bGlobalInvalidationEnabled)
{
	InvalidateRootChildOrder();
	ClearAllFastPathData(true);
	SetVolatilePrepass(GetCanCache());
	Invalidate(EInvalidateWidgetReason::LayoutAndVolatility);
}

bool SInvalidationPanel::UpdateCachePrequisites(FSlateWindowElementList& OutDrawElements, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, int32 LayerId, const FWidgetStyle& InWidgetStyle) const
{
	bool bNeedsRecache = false;

#if WITH_SLATE_DEBUGGING
	bNeedsRecache = bAlwaysInvalidate;
#endif

	// We only need to re-cache if the incoming layer is higher than the maximum layer Id we cached at,
	// we do this so that widgets that appear and live behind your invalidated UI don't constantly invalidate
	// everything above it.
	if (LayerId > LastIncomingLayerId)
	{
		LastIncomingLayerId = LayerId;
		bNeedsRecache = true;
	}

	if ( AllottedGeometry.GetLocalSize() != LastAllottedGeometry.GetLocalSize() || AllottedGeometry.GetAccumulatedRenderTransform() != LastAllottedGeometry.GetAccumulatedRenderTransform() )
	{
		LastAllottedGeometry = AllottedGeometry;
		bNeedsRecache = true;
	}

	// If our clip rect changes size, we've definitely got to invalidate.
	const FVector2D ClipRectSize = MyCullingRect.GetSize().RoundToVector();
	if ( ClipRectSize != LastClipRectSize )
	{
		LastClipRectSize = ClipRectSize;
		bNeedsRecache = true;
	}

	TOptional<FSlateClippingState> ClippingState = OutDrawElements.GetClippingState();
	if (LastClippingState != ClippingState)
	{
		LastClippingState = ClippingState;
		bNeedsRecache = true;
	}

	if (LastIncomingColorAndOpacity != InWidgetStyle.GetColorAndOpacityTint())
	{
		LastIncomingColorAndOpacity = InWidgetStyle.GetColorAndOpacityTint();
		bNeedsRecache = true;
	}
	
	return bNeedsRecache;
}

void SInvalidationPanel::SetCanCache(bool InCanCache)
{
	if (bCanCache != InCanCache)
	{
		bCanCache = InCanCache;
		SetVolatilePrepass(GetCanCache());
		InvalidateRootChildOrder();
	}
}

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
bool SInvalidationPanel::GetUseDynamicInvalidation() const
{
	return bUseDynamicInvalidation;
}

void SInvalidationPanel::SetUseDynamicInvalidation(bool InUseDynamicInvalidation)
{
	bUseDynamicInvalidation = InUseDynamicInvalidation;
}

bool SInvalidationPanel::CanUseDynamicInvalidation() const
{
	const bool bDynamicInvalidationEnabled = bUseDynamicInvalidation && DynamicInvalidationCVars::IsDynamicInvalidationEnabled();
	return DynamicInvalidationCVars::IsDynamicInvalidationForced() | bDynamicInvalidationEnabled;
}
#endif

FChildren* SInvalidationPanel::GetChildren()
{
	if (GetCanCache())
	{
		return &FNoChildren::NoChildrenInstance;
	}
	else
	{
		return SCompoundWidget::GetChildren();
	}
}

#if WITH_SLATE_DEBUGGING
FChildren* SInvalidationPanel::Debug_GetChildrenForReflector()
{
	return SCompoundWidget::GetChildren();
}
#endif

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
bool SInvalidationPanel::SupportsInvalidationRecursive(EInvalidationStrategy InInvalidationStrategy) const
{
	const bool bUseStaticInvalidation = !CanUseDynamicInvalidation();
	if (bUseStaticInvalidation)
	{
		// For an invalidation panel not using dynamic invalidation, we force the caching of all children, so we always support invalidation
		return true;
	}
	return SWidget::SupportsInvalidationRecursive(InInvalidationStrategy);
}
#endif

bool SInvalidationPanel::CanCacheThisFrame() const
{
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION

	const bool bSupportsInvalidationRecursive = SupportsInvalidationRecursive(EInvalidationStrategy::UseCachedValue);
	const bool bCanCacheValue = GetCanCache();
	const bool bCanCacheThisFrame = bCanCacheValue & bSupportsInvalidationRecursive;

#else
	const bool bCanCacheThisFrame = GetCanCache();
#endif

	return bCanCacheThisFrame;
}


UE_TRACE_EVENT_BEGIN(Cpu, SInvalidationPanel_SupportsInvalidationRecursive, NoSync)
	UE_TRACE_EVENT_FIELD(bool, DuringOnPaint)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, CreatedInLocation)
UE_TRACE_EVENT_END()

int32 SInvalidationPanel::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
#if SLATE_VERBOSE_NAMED_EVENTS
	SCOPED_NAMED_EVENT_FSTRING(DebugPaintName, FColor::Purple);
#endif
	SCOPE_CYCLE_COUNTER(STAT_SlateInvalidationPaint);

	bPaintedSinceLastPrepass = true;
	SInvalidationPanel* MutableThis = const_cast<SInvalidationPanel*>(this);

	bool bCanCacheThisFrame;
	{
#if UE_SLATE_TRACE_ENABLED && UE_SLATE_WITH_DYNAMIC_INVALIDATION
		UE_TRACE_LOG_SCOPED_T(Cpu, SInvalidationPanel_SupportsInvalidationRecursive, SlateChannel)
			<< SInvalidationPanel_SupportsInvalidationRecursive.DuringOnPaint(true)
			<< SInvalidationPanel_SupportsInvalidationRecursive.CreatedInLocation(*GetCreatedInLocation().ToString());
#endif
		bCanCacheThisFrame = CanCacheThisFrame();
	}

#if WITH_SLATE_DEBUGGING && UE_SLATE_WITH_DYNAMIC_INVALIDATION
	FSlateDebugging::IncrementInvalidationPanelCount(GetCanCache(), CanUseDynamicInvalidation(), SupportsInvalidationRecursive(EInvalidationStrategy::UseCachedValue));
#endif

	if (bCanCacheThisFrame != bWasCachable)
	{
		MutableThis->InvalidateRootChildOrder();

		bWasCachable = bCanCacheThisFrame;
	}

	if(bCanCacheThisFrame)
	{
		// Copy hit test grid settings from the root
		const bool bHittestCleared = HittestGrid->SetHittestArea(Args.RootGrid.GetGridOrigin(), Args.RootGrid.GetGridSize(), Args.RootGrid.GetGridWindowOrigin());
		HittestGrid->SetOwner(this);
		HittestGrid->SetCullingRect(MyCullingRect);
		FPaintArgs NewArgs = Args.WithNewHitTestGrid(HittestGrid.Get());

		// Copy the current user index into the new grid since nested hit test grids should inherit their parents user id
		NewArgs.GetHittestGrid().SetUserIndex(Args.RootGrid.GetUserIndex());
		check(!GSlateEnableGlobalInvalidation);

		const bool bRequiresRecache = UpdateCachePrequisites(OutDrawElements, AllottedGeometry, MyCullingRect, LayerId, InWidgetStyle);
		if (bHittestCleared || bRequiresRecache)
		{
			// @todo: Overly aggressive?
			MutableThis->InvalidateRootLayout(this);
		}

		// The root widget is our child.  We are not the root because we could be in a parent invalidation panel.  If we are nested in another invalidation panel, our OnPaint was called by that panel
		FSlateInvalidationContext Context(OutDrawElements, InWidgetStyle);
		Context.bParentEnabled = bParentEnabled;
		Context.bAllowFastPathUpdate = true;
		Context.LayoutScaleMultiplier = GetPrepassLayoutScaleMultiplier();
		Context.PaintArgs = &NewArgs;
		Context.IncomingLayerId = LayerId;
		Context.CullingRect = MyCullingRect;

		const FSlateInvalidationResult Result = MutableThis->PaintInvalidationRoot(Context);

		const bool bInheritedHittestability = Args.GetInheritedHittestability();
		const bool bOutgoingHittestability = bInheritedHittestability && GetVisibility().AreChildrenHitTestVisible();

		// add our widgets to the root hit test grid
		if (bOutgoingHittestability)
		{
			Args.GetHittestGrid().AddGrid(HittestGrid);
		}
		return Result.MaxLayerIdPainted;
	}
	else
	{
#if SLATE_VERBOSE_NAMED_EVENTS
		SCOPED_NAMED_EVENT_TEXT("SInvalidationPanel Uncached", FColor::Emerald);
#endif
		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
}

void SInvalidationPanel::SetContent(const TSharedRef< SWidget >& InContent)
{
	ChildSlot
	[
		InContent
	];

	InvalidateRootChildOrder();
}

bool SInvalidationPanel::CustomPrepass(float LayoutScaleMultiplier)
{
	bPaintedSinceLastPrepass = false;

	if (GetCanCache())
	{
		if (NeedsPrepass())
		{
			SetNeedsSlowPath(true);
		}
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
		else if (!NeedsSlowPath())
		{
#if UE_SLATE_TRACE_ENABLED
			UE_TRACE_LOG_SCOPED_T(Cpu, SInvalidationPanel_SupportsInvalidationRecursive, SlateChannel)
				<< SInvalidationPanel_SupportsInvalidationRecursive.DuringOnPaint(false)
				<< SInvalidationPanel_SupportsInvalidationRecursive.CreatedInLocation(*GetCreatedInLocation().ToString());
#endif // UE_SLATE_TRACE_ENABLED

			if (!SupportsInvalidationRecursive(EInvalidationStrategy::UseCachedValue))
			{
				SetNeedsSlowPath(true);
			}
		}
#endif

		ProcessInvalidation();
		if (NeedsSlowPath())
		{
			FChildren* Children = SCompoundWidget::GetChildren();
			Prepass_ChildLoop(LayoutScaleMultiplier, Children);
		}
		return false;
	}
	else
	{
		return true;
	}
}

bool SInvalidationPanel::Advanced_IsInvalidationRoot() const
{
	return GetCanCache();
}

const FSlateInvalidationRoot* SInvalidationPanel::Advanced_AsInvalidationRoot() const
{
	return GetCanCache() ? this : nullptr;
}

TSharedRef<SWidget> SInvalidationPanel::GetRootWidget()
{
	return GetCanCache() ? SCompoundWidget::GetChildren()->GetChildAt(0) : SNullWidget::NullWidget;
}

int32 SInvalidationPanel::PaintSlowPath(const FSlateInvalidationContext& Context)
{
	return SCompoundWidget::OnPaint(*Context.PaintArgs, GetPaintSpaceGeometry(), Context.CullingRect, *Context.WindowElementList, Context.IncomingLayerId, Context.WidgetStyle, Context.bParentEnabled);
}
