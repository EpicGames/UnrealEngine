// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SToolTip.h"
#include "HAL/IConsoleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


static TAutoConsoleVariable<float> StaticToolTipWrapWidth(
	TEXT( "Slate.ToolTipWrapWidth" ),
	1000.0f,
	TEXT( "Width of Slate tool-tips before we wrap the tool-tip text" ) );


float SToolTip::GetToolTipWrapWidth()
{
	return StaticToolTipWrapWidth.GetValueOnAnyThread();
}


void SToolTip::Construct( const FArguments& InArgs )
{
	TextContent = InArgs._Text;
	bIsInteractive = InArgs._IsInteractive;
	Font = InArgs._Font;
	TextMargin = InArgs._TextMargin;
	BorderImage = InArgs._BorderImage;
	OnSetInteractiveWindowLocationDelegate = InArgs._OnSetInteractiveWindowLocation;
	
	SetContentWidget(InArgs._Content.Widget);
}


void SToolTip::SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
{
	if (InContentWidget != SNullWidget::NullWidget)
	{
		// Widget content argument takes precedence over the text content.
		WidgetContent = InContentWidget;
	}

	TSharedPtr< SWidget > PinnedWidgetContent = WidgetContent.Pin();
	if( PinnedWidgetContent.IsValid() )
	{
		ToolTipContent = PinnedWidgetContent;

		// Tool-tip with entirely custom content.  We'll create a border with some padding (as customized by the user), then
		// embed their custom widget right inside the border.  This tool-tip currently has a different styling than tool-tips
		// that only contain text.
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(BorderImage)
			.Padding(TextMargin)
			[
				ToolTipContent.ToSharedRef()
			]
		];

	}
	else
	{
		ToolTipContent =
			SNew( STextBlock )
			.Text( TextContent )
			.Font( Font )
			.ColorAndOpacity( FLinearColor::Black )
			.WrapTextAt_Static( &SToolTip::GetToolTipWrapWidth );

		// Text-only tool-tip.  This tool-tip currently has a different styling than tool-tips with custom content.  We always want basic
		// text tool-tips to look consistent.
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
			.Padding(FMargin(11.0f))
			[
				ToolTipContent.ToSharedRef()
			]
		];
	}
}

void SToolTip::ResetContentWidget()
{
	WidgetContent.Reset();
	SetContentWidget(SNullWidget::NullWidget);
}

bool SToolTip::IsEmpty() const
{
	return !WidgetContent.IsValid() && TextContent.Get().IsEmpty();
}


bool SToolTip::IsInteractive() const
{
	return bIsInteractive.Get();
}

void SToolTip::OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const
{
	if (OnSetInteractiveWindowLocationDelegate.IsBound())
	{
		OnSetInteractiveWindowLocationDelegate.Execute(InOutDesiredLocation);
	}
}