// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SViewport.h"
#include "Rendering/DrawElements.h"
#include "Brushes/SlateColorBrush.h"
#include "Application/SlateApplicationBase.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/HittestGrid.h"
#include "Types/NavigationMetaData.h"

DECLARE_CYCLE_STAT(TEXT("Game UI Tick"), STAT_ViewportTickTime, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Game UI Paint"), STAT_ViewportPaintTime, STATGROUP_Slate);



/* SViewport::FArguments interface
*****************************************************************************/
UE::Slate::FDeprecateVector2DResult SViewport::FArguments::GetDefaultViewportSize()
{
	return FVector2f(320.0f, 240.0f);
}

/* SViewport constructors
 *****************************************************************************/

SViewport::SViewport()
	: ShowDisabledEffect(*this, true)
	, ViewportSize(*this, SViewport::FArguments::GetDefaultViewportSize())
	, bRenderDirectlyToWindow(false)
	, bEnableGammaCorrection(true)
	, bReverseGammaCorrection(false)
	, bEnableStereoRendering(false)
{ }

SViewport::~SViewport() = default;

/* SViewport interface
 *****************************************************************************/

void SViewport::Construct( const FArguments& InArgs )
{
	ShowDisabledEffect.Assign(*this, InArgs._ShowEffectWhenDisabled);
	bRenderDirectlyToWindow = InArgs._RenderDirectlyToWindow;
	bEnableGammaCorrection = InArgs._EnableGammaCorrection;
	bReverseGammaCorrection = InArgs._ReverseGammaCorrection;
	bEnableBlending = InArgs._EnableBlending;
	bEnableStereoRendering = InArgs._EnableStereoRendering;
	bIgnoreTextureAlpha = InArgs._IgnoreTextureAlpha;
	bPreMultipliedAlpha = InArgs._PreMultipliedAlpha;
	ViewportInterface = InArgs._ViewportInterface;
	ViewportSize.Assign(*this, InArgs._ViewportSize);

#if UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
	AddMetadata(MakeShared<FSimulatedNavigationMetaData>(EUINavigationRule::Stop));
#endif

	this->ChildSlot
	[
		InArgs._Content.Widget
	];
}

void SViewport::SetViewportInterface(TSharedRef<ISlateViewport> InViewportInterface)
{
	if (ViewportInterface != InViewportInterface)
	{
		ViewportInterface = InViewportInterface;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SViewport::SetRenderDirectlyToWindow(const bool bInRenderDirectlyToWindow)
{
	if (bRenderDirectlyToWindow != bInRenderDirectlyToWindow)
	{
		bRenderDirectlyToWindow = bInRenderDirectlyToWindow;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SViewport::SetIgnoreTextureAlpha(const bool bInIgnoreTextureAlpha)
{
	if (bIgnoreTextureAlpha != bInIgnoreTextureAlpha)
	{
		bIgnoreTextureAlpha = bInIgnoreTextureAlpha;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

void SViewport::SetActive(bool bActive)
{
	// In game environments the viewport is always active
	if(GIsEditor || IS_PROGRAM)
	{
		if (bActive && !ActiveTimerHandle.IsValid())
		{
			ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SViewport::EnsureTick));
		}
		else if (!bActive && ActiveTimerHandle.IsValid())
		{
			UnRegisterActiveTimer(ActiveTimerHandle.Pin().ToSharedRef());
		}
	}
}

EActiveTimerReturnType SViewport::EnsureTick(double InCurrentTime, float InDeltaTime)
{
	return EActiveTimerReturnType::Continue;
}

int32 SViewport::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	using namespace UE::Slate;

	SCOPED_NAMED_EVENT(SViewport_OnPaint, FColor::Purple);
	SCOPE_CYCLE_COUNTER(STAT_ViewportPaintTime);

	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	bool bShowDisabledEffect = ShowDisabledEffect.Get();
	ESlateDrawEffect DrawEffects = bShowDisabledEffect && !bEnabled ? ESlateDrawEffect::DisabledEffect : ESlateDrawEffect::None;

	// Viewport texture alpha channels are often in an indeterminate state, even after the resolve,
	// so we'll tell the shader to not use the alpha channel when blending
	if( bIgnoreTextureAlpha )
	{
		DrawEffects |= ESlateDrawEffect::IgnoreTextureAlpha;
	}

	// Should we perform gamma correction?
	if( !bEnableGammaCorrection )
	{
		DrawEffects |= ESlateDrawEffect::NoGamma;
	}

	// Should we reverse gamma correction
	if (bReverseGammaCorrection)
	{
		DrawEffects |= ESlateDrawEffect::ReverseGamma;
	}

	// Show we enable blending?
	if( !bEnableBlending )
	{
		DrawEffects |= ESlateDrawEffect::NoBlending;
	}
	// Should we use pre-multiplied alpha?
	else if( bPreMultipliedAlpha )
	{
		DrawEffects |= ESlateDrawEffect::PreMultipliedAlpha;
	}

	TSharedPtr<ISlateViewport> ViewportInterfacePin = ViewportInterface.Pin();

	TUniquePtr<FPaintScope> PaintScope;
	if (ViewportInterfacePin.IsValid())
	{
		PaintScope = ViewportInterfacePin->Paint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ViewportInterfacePin->OnDrawViewport( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Only draw a quad if not rendering directly to the backbuffer
	if( !ShouldRenderDirectly() )
	{
		if( ViewportInterfacePin.IsValid() && ViewportInterfacePin->GetViewportRenderTargetTexture() != nullptr && !ViewportInterfacePin->GetViewportRenderTargetTexture()->Debug_IsDestroyed())
		{
			FSlateDrawElement::MakeViewport( OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), ViewportInterfacePin, DrawEffects, InWidgetStyle.GetColorAndOpacityTint() );
		}
		else
		{
			// Viewport isn't ready yet, so just draw a black box
			static FSlateColorBrush BlackBrush( FColor::Black );
			FSlateDrawElement::MakeBox( OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), &BlackBrush, DrawEffects, BlackBrush.GetTint( InWidgetStyle ) );
		}
	}

	int32 Layer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled );

	if( ViewportInterfacePin.IsValid() && ViewportInterfacePin->IsSoftwareCursorVisible() )
	{
		const FVector2D ToViewportPixelScale = UE::Slate::Viewport::GetViewportLocalToPixelScale(AllottedGeometry, ViewportInterfacePin->GetSize());
		FVector2f CursorPositionLocalSpace = UE::Slate::CastToVector2f(ViewportInterfacePin->GetSoftwareCursorPosition() / ToViewportPixelScale);

		const FVector2f CursorPosScreenSpace = FSlateApplication::Get().GetCursorPos();
		// @todo Slate: why are we calling OnCursorQuery in here?
		FCursorReply Reply = ViewportInterfacePin->OnCursorQuery( AllottedGeometry,
			FPointerEvent(
				FSlateApplicationBase::CursorPointerIndex,
				CursorPosScreenSpace,
				CursorPosScreenSpace,
				FVector2f::ZeroVector,
				TSet<FKey>(),
				FModifierKeysState() )
		 );

		// Check if a cursor widget should be used instead of the standard brush
		TOptional<TSharedRef<SWidget>> MappedCursorWidget = ViewportInterfacePin->OnMapCursor(Reply);
		if (MappedCursorWidget.IsSet())
		{
			TSharedRef<SWidget> CursorWidget = MappedCursorWidget.GetValue();
			CursorWidget->SetVisibility(EVisibility::HitTestInvisible);
			CursorWidget->SlatePrepass(AllottedGeometry.Scale);

			const FVector2f CursorSize = UE::Slate::CastToVector2f(CursorWidget->GetDesiredSize());

			LayerId++;
			CursorWidget->Paint(
				Args,
				AllottedGeometry.MakeChild(CursorSize, FSlateLayoutTransform(CursorPositionLocalSpace - (CursorSize * 0.5f))),
				MyCullingRect,
				OutDrawElements, LayerId, InWidgetStyle, bEnabled);
		}
		else
		{
			EMouseCursor::Type CursorType = Reply.GetCursorType();

			const FSlateBrush* Brush = [CursorType]
			{
				switch (CursorType)
				{
				case EMouseCursor::CardinalCross:
					return FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_CardinalCross"));
				case EMouseCursor::GrabHand:
					return FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_Hand"));
				case EMouseCursor::GrabHandClosed:
				default: // Default is included here for legacy compatability; grab was the default.
					return FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_Grab"));
				}
			}();

			const FVector2f CursorSize = Brush->ImageSize / AllottedGeometry.Scale;

			LayerId++;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry( CursorSize, FSlateLayoutTransform(CursorPositionLocalSpace - (CursorSize * .5f)) ),
				Brush
			);
		}
	}

	// If there are any custom hit testable widgets in the 3D world we need to register their custom hit test path here.
	if ( CustomHitTestPath.IsValid() && Args.GetHittestGrid().ContainsWidget(this))
	{
		Args.GetHittestGrid().InsertCustomHitTestPath(this, CustomHitTestPath.ToSharedRef());
	}

	return Layer;
}

void SViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SCOPE_CYCLE_COUNTER(STAT_ViewportTickTime);

	if ( ViewportInterface.IsValid() )
	{
		ViewportInterface.Pin()->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}


/* SWidget interface
 *****************************************************************************/

FCursorReply SViewport::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnCursorQuery(MyGeometry, CursorEvent) : FCursorReply::Unhandled();
}

TOptional<TSharedRef<SWidget>> SViewport::OnMapCursor(const FCursorReply& CursorReply) const
{
	if (ViewportInterface.IsValid())
	{
		TSharedPtr<ISlateViewport> ViewportPin = ViewportInterface.Pin();

		// When the software cursor is active, OnPaint handles cursor widget drawing
		// at the software cursor position. Don't also let Slate draw it at the
		// hardware cursor position via DrawCursor.
		if (ViewportPin->IsSoftwareCursorVisible())
		{
			return TOptional<TSharedRef<SWidget>>();
		}

		return ViewportPin->OnMapCursor(CursorReply);
	}
	return TOptional<TSharedRef<SWidget>>();
}

FReply SViewport::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnMouseButtonDown(MyGeometry, MouseEvent) : FReply::Unhandled();
}

FReply SViewport::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnMouseButtonUp(MyGeometry, MouseEvent) : FReply::Unhandled();
}

void SViewport::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	if (ViewportInterface.IsValid())
	{
		ViewportInterface.Pin()->OnMouseEnter(MyGeometry, MouseEvent);
	}
}

void SViewport::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	if (ViewportInterface.IsValid())
	{
		ViewportInterface.Pin()->OnMouseLeave(MouseEvent);
	}
}

FReply SViewport::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnMouseMove(MyGeometry, MouseEvent) : FReply::Unhandled();
}

FReply SViewport::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnMouseWheel(MyGeometry, MouseEvent) : FReply::Unhandled();
}

FReply SViewport::OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnMouseButtonDoubleClick(MyGeometry, MouseEvent) : FReply::Unhandled();
}

FReply SViewport::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnKeyDown(MyGeometry, KeyEvent) : FReply::Unhandled();
}

FReply SViewport::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnKeyUp(MyGeometry, KeyEvent) : FReply::Unhandled();
}

FReply SViewport::OnAnalogValueChanged( const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnAnalogValueChanged(MyGeometry, InAnalogInputEvent) : FReply::Unhandled();
}

FReply SViewport::OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& CharacterEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnKeyChar(MyGeometry, CharacterEvent) : FReply::Unhandled();
}

FReply SViewport::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnFocusReceived(InFocusEvent) : FReply::Unhandled();
}

void SViewport::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	if (ViewportInterface.IsValid())
	{
		ViewportInterface.Pin()->OnFocusLost(InFocusEvent);
	}
}

void SViewport::SetContent( TSharedPtr<SWidget> InContent )
{
	ChildSlot
	[
		InContent.IsValid() ? InContent.ToSharedRef() : (TSharedRef<SWidget>)SNullWidget::NullWidget
	];
}

void SViewport::SetCustomHitTestPath( TSharedPtr<ICustomHitTestPath> InCustomHitTestPath )
{
	if (CustomHitTestPath != InCustomHitTestPath)
	{
		CustomHitTestPath = InCustomHitTestPath;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

TSharedPtr<ICustomHitTestPath> SViewport::GetCustomHitTestPath()
{
	return CustomHitTestPath;
}

void SViewport::OnWindowClosed( const TSharedRef<SWindow>& WindowBeingClosed )
{
	if (ViewportInterface.IsValid())
	{
		ViewportInterface.Pin()->OnViewportClosed();
	}
}

FReply SViewport::OnViewportActivated(const FWindowActivateEvent& InActivateEvent)
{
	CachedParentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnViewportActivated(InActivateEvent) : FReply::Unhandled();
}

void SViewport::OnViewportDeactivated(const FWindowActivateEvent& InActivateEvent)
{
	if (ViewportInterface.IsValid())
	{
		ViewportInterface.Pin()->OnViewportDeactivated(InActivateEvent);
	}
}

FReply SViewport::OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnTouchStarted(MyGeometry, InTouchEvent) : FReply::Unhandled();
}

FReply SViewport::OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnTouchMoved(MyGeometry, InTouchEvent) : FReply::Unhandled();
}

FReply SViewport::OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnTouchEnded(MyGeometry, InTouchEvent) : FReply::Unhandled();
}

FReply SViewport::OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnTouchForceChanged(MyGeometry, InTouchEvent) : FReply::Unhandled();
}

FReply SViewport::OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnTouchFirstMove(MyGeometry, InTouchEvent) : FReply::Unhandled();
}

FReply SViewport::OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnTouchGesture(MyGeometry, GestureEvent) : FReply::Unhandled();
}

FReply SViewport::OnMotionDetected( const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent )
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnMotionDetected(MyGeometry, InMotionEvent) : FReply::Unhandled();
}

TOptional<bool> SViewport::OnQueryShowFocus(const EFocusCause InFocusCause) const
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnQueryShowFocus(InFocusCause) : TOptional<bool>();
}

FPopupMethodReply SViewport::OnQueryPopupMethod() const
{
	TSharedPtr<ISlateViewport> PinnedInterface = ViewportInterface.Pin();
	if (PinnedInterface.IsValid())
	{
		return PinnedInterface->OnQueryPopupMethod();
	}
	else
	{
		return FPopupMethodReply::UseMethod(EPopupMethod::CreateNewWindow);
	}	
}

void SViewport::OnFinishedPointerInput()
{
	TSharedPtr<ISlateViewport> PinnedInterface = ViewportInterface.Pin();
	if (PinnedInterface.IsValid())
	{
		PinnedInterface->OnFinishedPointerInput();
	}
}

void SViewport::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	SCompoundWidget::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
	if (ArrangedChildren.Allows3DWidgets() && CustomHitTestPath.IsValid())
	{
		CustomHitTestPath->ArrangeCustomHitTestChildren(ArrangedChildren);
	}
}

TOptional<FVirtualPointerPosition> SViewport::TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& MyGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const
{
	if( CustomHitTestPath.IsValid() )
	{
		return CustomHitTestPath->TranslateMouseCoordinateForCustomHitTestChild( ChildWidget, MyGeometry, ScreenSpaceMouseCoordinate, LastScreenSpaceMouseCoordinate );
	}

	return TOptional<FVirtualPointerPosition>();
}

FNavigationReply SViewport::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	return ViewportInterface.IsValid() ? ViewportInterface.Pin()->OnNavigation(MyGeometry, InNavigationEvent) : FNavigationReply::Stop();
}

FVector2D UE::Slate::Viewport::GetViewportLocalToPixelScale(const FGeometry& InGeometry, const FVector2D& InViewportSize)
{
	const FDeprecateVector2DResult SlateSize = InGeometry.GetAbsoluteSize();

	// Check for invalid sizes, return identity scalar in such a case
	if (SlateSize.IsNearlyZero() || InViewportSize.IsNearlyZero())
	{
		return FVector2D(1.0f, 1.0f);
	}

	return InGeometry.Scale * (InViewportSize / SlateSize);
}
