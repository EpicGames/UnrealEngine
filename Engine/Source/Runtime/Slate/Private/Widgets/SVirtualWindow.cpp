// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SVirtualWindow.h"
#include "Input/HittestGrid.h"
#include "Layout/ArrangedChildren.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SlateDrawBuffer.h"

#include "Widgets/LayerManager/STooltipPresenter.h"
#include "Widgets/Layout/SPopup.h"

void SVirtualWindow::Construct(const FArguments& InArgs)
{
	bIsPopupWindow = true;
	bVirtualWindow = true;
	bIsFocusable = false;
	bShouldResolveDeferred = true;
	SetCachedSize(InArgs._Size);
	SetNativeWindow(MakeShareable(new FGenericWindow()));

	ConstructWindowInternals();

	WindowOverlay->AddSlot()
		[
			SNew(SPopup)
			[
				SAssignNew(TooltipPresenter, STooltipPresenter)
			]
		];

	SetContent(SNullWidget::NullWidget);
}

FPopupMethodReply SVirtualWindow::OnQueryPopupMethod() const
{
	return FPopupMethodReply::UseMethod(EPopupMethod::UseCurrentWindow)
		.SetShouldThrottle(EShouldThrottle::No);
}

bool SVirtualWindow::OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent)
{
	TooltipPresenter->SetContent(TooltipContent);

	return true;
}

void SVirtualWindow::SetShouldResolveDeferred(bool bResolve)
{
	bShouldResolveDeferred = bResolve;
}

void SVirtualWindow::SetIsFocusable(bool bFocusable)
{
	bIsFocusable = bFocusable;
}

bool SVirtualWindow::SupportsKeyboardFocus() const
{
	return bIsFocusable;
}

bool SVirtualWindow::Advanced_IsInvalidationRoot() const
{
	return bAllowFastUpdate;
}

FVector2D SVirtualWindow::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
}

int32 SVirtualWindow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (bShouldResolveDeferred)
	{
		OutDrawElements.BeginDeferredGroup();
	}

	// We intentionally skip SWindow's OnPaint, since we are going to do our own handling of deferred groups.
	int32 MaxLayer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	if (bShouldResolveDeferred)
	{
		OutDrawElements.EndDeferredGroup();
	}

	return MaxLayer;
}

void SVirtualWindow::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	SWindow::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

	// @HACK VREDITOR - otherwise popup layers don't work in nested child windows, in tab managers and such.
	if (ArrangedChildren.Allows3DWidgets())
	{
		const TArray< TSharedRef<SWindow> >& WindowChildren = GetChildWindows();
		for (int32 ChildIndex = 0; ChildIndex < WindowChildren.Num(); ++ChildIndex)
		{
			const TSharedRef<SWindow>& ChildWindow = WindowChildren[ChildIndex];
			FGeometry ChildWindowGeometry = ChildWindow->GetWindowGeometryInWindow();
			ChildWindow->ArrangeChildren(ChildWindowGeometry, ArrangedChildren);
		}
	}
}