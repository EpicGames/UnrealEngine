// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWeakWidget.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"


SWeakWidget::SWeakWidget()
	: WeakChild(this)
{
}

SWeakWidget::~SWeakWidget() = default;

void SWeakWidget::Construct(const FArguments& InArgs)
{
	SetCanTick(false);
	WeakChild.AttachWidget( InArgs._PossiblyNullContent );
}

FVector2D SWeakWidget::ComputeDesiredSize( float ) const
{
	if (WeakChild.Num() > 0)
	{
		TSharedRef<SWidget> ReferencedWidget = WeakChild.GetWidget();

		if (ReferencedWidget != SNullWidget::NullWidget && ReferencedWidget->GetVisibility() != EVisibility::Collapsed)
		{
			return ReferencedWidget->GetDesiredSize();
		}
	}

	return FVector2D::ZeroVector;
}

void SWeakWidget::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	if (WeakChild.Num() > 0)
	{
		// We just want to show the child that we are presenting. Always stretch it to occupy all of the space.
		TSharedRef<SWidget> MyContent = WeakChild.GetWidget();

		if (MyContent != SNullWidget::NullWidget && ArrangedChildren.Accepts(MyContent->GetVisibility()))
		{
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(
				MyContent,
				FVector2D(0, 0),
				AllottedGeometry.GetLocalSize()
			));
		}
	}
}

FChildren* SWeakWidget::GetChildren()
{
	return &WeakChild;
}

int32 SWeakWidget::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// Just draw the children.
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(AllottedGeometry, ArrangedChildren);

	// There may be zero elements in this array if our child collapsed/hidden
	if( ArrangedChildren.Num() > 0 )
	{
		check( ArrangedChildren.Num() == 1 );
		FArrangedWidget& TheChild = ArrangedChildren[0];

		return TheChild.Widget->Paint( 
			Args.WithNewParent(this), 
			TheChild.Geometry, 
			MyCullingRect, 
			OutDrawElements, 
			LayerId + 1,
			InWidgetStyle, 
			ShouldBeEnabled( bParentEnabled ) );
	}
	return LayerId;
}

void SWeakWidget::SetContent(const TSharedRef<SWidget>& InWidget)
{
	WeakChild.AttachWidget( InWidget );
}

bool SWeakWidget::ChildWidgetIsValid() const
{
	return WeakChild.Num() > 0;
}
