// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/ISlateEditableTextWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISlateEditableTextWidget)

FMoveCursor FMoveCursor::Cardinal(ECursorMoveGranularity Granularity, FIntPoint Direction, ECursorAction Action)
{
	return FMoveCursor(Granularity, ECursorMoveMethod::Cardinal, Direction, 1.0f, Action);
}

FMoveCursor FMoveCursor::ViaScreenPointer(FVector2D LocalPosition, float GeometryScale, ECursorAction Action)
{
	return FMoveCursor(ECursorMoveGranularity::Character, ECursorMoveMethod::ScreenPosition, LocalPosition, GeometryScale, Action);
}

ECursorMoveMethod FMoveCursor::GetMoveMethod() const
{
	return Method;
}

bool FMoveCursor::IsVerticalMovement() const
{
	return DirectionOrPosition.Y != 0.0f;
}

bool FMoveCursor::IsHorizontalMovement() const
{
	return DirectionOrPosition.X != 0.0f;
}

FIntPoint FMoveCursor::GetMoveDirection() const
{
	return FIntPoint(DirectionOrPosition.X, DirectionOrPosition.Y);
}

ECursorAction FMoveCursor::GetAction() const
{
	return Action;
}

FVector2D FMoveCursor::GetLocalPosition() const
{
	return FVector2D(DirectionOrPosition);
}

ECursorMoveGranularity FMoveCursor::GetGranularity() const
{
	return Granularity;
}

float FMoveCursor::GetGeometryScale() const
{
	return GeometryScale;
}

FMoveCursor::FMoveCursor(ECursorMoveGranularity InGranularity, ECursorMoveMethod InMethod, FVector2D InDirectionOrPosition, float InGeometryScale, ECursorAction InAction)
	: Granularity(InGranularity)
	, Method(InMethod)
	, DirectionOrPosition(InDirectionOrPosition)
	, Action(InAction)
	, GeometryScale(InGeometryScale)
{
	// Movement actions are assumed to be exclusively vertical or exclusively horizontal
	// There is no great reason to assume this, but a lot of the code was written to deal with
	// events, which must be one or the other.
	ensure(Method == ECursorMoveMethod::ScreenPosition || (Method == ECursorMoveMethod::Cardinal && IsVerticalMovement() != IsHorizontalMovement()));
}
