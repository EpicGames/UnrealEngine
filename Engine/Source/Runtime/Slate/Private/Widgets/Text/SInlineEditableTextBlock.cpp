// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextEditHelper.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableTextBox.h"

SLATE_IMPLEMENT_WIDGET(SInlineEditableTextBlock)
void SInlineEditableTextBlock::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaximumLengthAttribute, [](const SWidget& OwningWidget)
	{
		const SInlineEditableTextBlock& InlineTextBlock = static_cast<const SInlineEditableTextBlock&>(OwningWidget);
		if (InlineTextBlock.IsInEditMode())
		{
			const bool bIsProcessingAttributeUpdate = InlineTextBlock.GetProxyHandle().IsValid(InlineTextBlock) &&
				InlineTextBlock.GetProxyHandle().GetInvalidationRootHandle().GetInvalidationRoot() &&
				InlineTextBlock.GetProxyHandle().GetInvalidationRootHandle().GetInvalidationRoot()->IsProcessingAttributeUpdate();

			if (bIsProcessingAttributeUpdate)
			{
				// We want to retrigger an OnTextChanged event with the text of the edited textbox to revalidate the text.
				// It cannot be retriggered right away while attributes are updated if OnTextChanged ends up creating or destroying the error
				// message as this would lead to a crash. We also cannot register a timer at this point, so we delay until next tick.
				bool bHadError = InlineTextBlock.HasTextBoxError();
				FText ErrorMessage;
				bool bHasError = !InlineTextBlock.IsTextAllowed(InlineTextBlock.GetEditableText(), ErrorMessage);

				if (bHadError != bHasError)
				{
					// a popup widget would end up being created or destroyed, so we need to delay its creation/destruction
					InlineTextBlock.bNeedsMaximumLengthChecked = true;
					return EInvalidateWidgetReason::Paint;
				}
			}
			
			const_cast<SInlineEditableTextBlock&>(InlineTextBlock).OnTextChanged(InlineTextBlock.GetEditableText());
		}
		return EInvalidateWidgetReason::None;
	});
}

SInlineEditableTextBlock::SInlineEditableTextBlock()
	: MaximumLengthAttribute(*this)
{
}


void SInlineEditableTextBlock::Construct( const FArguments& InArgs )
{
	check(InArgs._Style);
	SetCanTick(false);

	OnTextCommittedDelegate = InArgs._OnTextCommitted;
	IsSelected = InArgs._IsSelected;
	OnVerifyTextChanged = InArgs._OnVerifyTextChanged;
	Text = InArgs._Text;
	HintText = InArgs._HintText;
	bIsReadOnly = InArgs._IsReadOnly;
	bNeedsMaximumLengthChecked = false;

	bIsMultiLine = InArgs._MultiLine;
	bDelayedLeftClickEntersEditMode = InArgs._DelayedLeftClickEntersEditMode;
	DoubleSelectDelay = InArgs._DoubleSelectDelay;

	OnEnterEditingMode = InArgs._OnEnterEditingMode;
	OnExitEditingMode = InArgs._OnExitEditingMode;

	ChildSlot
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
			
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(TextBlock, STextBlock)
			.Text(Text)
			.TextStyle( &InArgs._Style->TextStyle )
			.Font(InArgs._Font)
			.ColorAndOpacity( InArgs._ColorAndOpacity )
			.ShadowColorAndOpacity( InArgs._ShadowColorAndOpacity )
			.ShadowOffset( InArgs._ShadowOffset )
			.HighlightText( InArgs._HighlightText )
			.ToolTipText( InArgs._ToolTipText )
			.WrapTextAt( InArgs._WrapTextAt )
			.AutoWrapText( InArgs._AutoWrapNonEditText )
			.Justification( InArgs._Justification )
			.LineBreakPolicy( InArgs._LineBreakPolicy )
			.OverflowPolicy(InArgs._OverflowPolicy)
		]
	];

#if WITH_FANCY_TEXT
	if( bIsMultiLine )
	{
		SAssignNew(MultiLineTextBox, SMultiLineEditableTextBox)
			.Text(InArgs._Text)
			.HintText(InArgs._HintText)
			.Style(&InArgs._Style->EditableTextBoxStyle)
			.Font(InArgs._Font)
			.ToolTipText(InArgs._ToolTipText)
			.OnBeginTextEdit(InArgs._OnBeginTextEdit)
			.OnTextChanged(this, &SInlineEditableTextBlock::OnTextChanged)
			.OnTextCommitted(this, &SInlineEditableTextBlock::OnTextBoxCommitted)
			.WrapTextAt(InArgs._WrapTextAt)
			.AutoWrapText(InArgs._AutoWrapMultilineEditText)
			.Justification(InArgs._Justification)
			.SelectAllTextWhenFocused(true)
			.ClearKeyboardFocusOnCommit(true)
			.RevertTextOnEscape(true)
			.ModiferKeyForNewLine(InArgs._ModiferKeyForNewLine)
			.OverflowPolicy(InArgs._OverflowPolicy);
	}
	else
#endif //WITH_FANCY_TEXT
	{
		SAssignNew(TextBox, SEditableTextBox)
			.Text(InArgs._Text)
			.HintText(InArgs._HintText)
			.Style(&InArgs._Style->EditableTextBoxStyle)
			.Font(InArgs._Font)
			.ToolTipText(InArgs._ToolTipText)
			.OnBeginTextEdit(InArgs._OnBeginTextEdit)
			.OnTextChanged(this, &SInlineEditableTextBlock::OnTextChanged)
			.OnTextCommitted(this, &SInlineEditableTextBlock::OnTextBoxCommitted)
			.SelectAllTextWhenFocused(true)
			.ClearKeyboardFocusOnCommit(false)
			.OverflowPolicy(InArgs._OverflowPolicy);
	}

	SetMaximumLength(InArgs._MaximumLength);
}

int32 SInlineEditableTextBlock::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (bNeedsMaximumLengthChecked)
	{
		const_cast<SInlineEditableTextBlock*>(this)->OnTextChanged(GetEditableText());
	}
	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

SInlineEditableTextBlock::~SInlineEditableTextBlock()
{
	if(IsInEditMode())
	{
		// Clear the error so it will vanish.
		SetTextBoxError( FText::GetEmpty() );
	}
}

void SInlineEditableTextBlock::CancelEditMode()
{
	ExitEditingMode();

	// Get the text from source again.
	SetEditableText(Text);
}

bool SInlineEditableTextBlock::SupportsKeyboardFocus() const
{
	//Can not have keyboard focus if it's status of being selected is managed by another widget.
	return !IsSelected.IsBound();
}

void SInlineEditableTextBlock::EnterEditingMode()
{
	if(!bIsReadOnly.Get() && FSlateApplication::Get().HasAnyMouseCaptor() == false)
	{
		if(TextBlock->GetVisibility() == EVisibility::Visible)
		{
			OnEnterEditingMode.ExecuteIfBound();

			const FText CurrentText = TextBlock->GetText();
			SetEditableText( CurrentText );

			TSharedPtr<SWidget> ActiveTextBox = GetEditableTextWidget();
			HorizontalBox->AddSlot()
				[
					ActiveTextBox.ToSharedRef()
				];

			WidgetToFocus = FSlateApplication::Get().GetKeyboardFocusedWidget();
			FSlateApplication::Get().SetKeyboardFocus(ActiveTextBox, EFocusCause::SetDirectly);

			TextBlock->SetVisibility(EVisibility::Collapsed);

			// We retrigger an OnTextChanged event to revalidate the text
			OnTextChanged(CurrentText);
		}
	}
}

void SInlineEditableTextBlock::ExitEditingMode()
{
	OnExitEditingMode.ExecuteIfBound();

	HorizontalBox->RemoveSlot( GetEditableTextWidget().ToSharedRef() );
	TextBlock->SetVisibility(EVisibility::Visible);
	// Clear the error so it will vanish.
	SetTextBoxError( FText::GetEmpty() );
	bNeedsMaximumLengthChecked = false;

	TSharedPtr<SWidget> CurrentFocus = FSlateApplication::Get().GetKeyboardFocusedWidget();
	if (CurrentFocus.IsValid())
	{
		if (CurrentFocus == GetEditableTextWidget())
		{
			// Restore the original widget focus
			TSharedPtr<SWidget> WidgetToFocusPin = WidgetToFocus.Pin();
			if (WidgetToFocusPin.IsValid())
			{
				FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPin, EFocusCause::SetDirectly);
			}
			else
			{
				FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
			}
		}
	}

	WidgetToFocus.Reset();
}

bool SInlineEditableTextBlock::IsInEditMode() const
{
	return TextBlock->GetVisibility() == EVisibility::Collapsed;
}

void SInlineEditableTextBlock::SetReadOnly(const TAttribute<bool>& bInIsReadOnly)
{
	bIsReadOnly = bInIsReadOnly;
}

void SInlineEditableTextBlock::SetMaximumLength(TAttribute<int32> InMaximumLength)
{
	MaximumLengthAttribute.Assign(*this, MoveTemp(InMaximumLength));
}

void SInlineEditableTextBlock::SetText( const TAttribute< FText >& InText )
{
	Text = InText;
	TextBlock->SetText( Text );
	SetEditableText( Text );
}

void SInlineEditableTextBlock::SetText( const FString& InText )
{
	Text = FText::FromString( InText );
	TextBlock->SetText( Text );
	SetEditableText( Text );
}

FText SInlineEditableTextBlock::GetText() const
{
	return
#if WITH_FANCY_TEXT
	bIsMultiLine ? MultiLineTextBox->GetText() :
#endif //WITH_FANCY_TEXT
				 TextBox->GetText();
}

void SInlineEditableTextBlock::SetHighlightText( const TAttribute< FText >& InText )
{
	TextBlock->SetHighlightText(InText);
}

void SInlineEditableTextBlock::SetWrapTextAt( const TAttribute<float>& InWrapTextAt )
{
	TextBlock->SetWrapTextAt( InWrapTextAt );
}

void SInlineEditableTextBlock::SetOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy)
{
	TextBlock->SetOverflowPolicy(InOverflowPolicy);
}

void SInlineEditableTextBlock::SetOnTextCommitted(FOnTextCommitted InOnTextCommitted)
{
	OnTextCommittedDelegate = MoveTemp(InOnTextCommitted);
}

void SInlineEditableTextBlock::SetOnEnterEditingMode(FSimpleDelegate InOnEnterEditingMode)
{
	OnEnterEditingMode = MoveTemp(InOnEnterEditingMode);
}

void SInlineEditableTextBlock::SetOnExitEditingMode(FSimpleDelegate InOnExitEditingMode)
{
	OnExitEditingMode = MoveTemp(InOnExitEditingMode);
}

void SInlineEditableTextBlock::SetOnExitEditingMode(FIsSelected InIsSelected)
{
	IsSelected = MoveTemp(InIsSelected);
}

void SInlineEditableTextBlock::SetOnVerifyTextChanged(FOnVerifyTextChanged InOnVerifyTextChanged)
{
	OnVerifyTextChanged = MoveTemp(InOnVerifyTextChanged);
}

FReply SInlineEditableTextBlock::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!bDelayedLeftClickEntersEditMode || DoubleSelectDelay == 0.0f || !MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) || MouseEvent.IsControlDown() || MouseEvent.IsShiftDown())
	{
		return FReply::Unhandled();
	}

	if (IsSelected.IsBound())
	{
		if (IsSelected.Execute() && !bIsReadOnly.Get() && !ActiveTimerHandle.IsValid())
		{
			ActiveTimerHandle = RegisterActiveTimer(DoubleSelectDelay, FWidgetActiveTimerDelegate::CreateSP(this, &SInlineEditableTextBlock::TriggerEditMode));
		}
	}
	else
	{
		// The widget is not managed by another widget, so handle the mouse input and enter edit mode if ready.
		if (HasKeyboardFocus() && !bIsReadOnly.Get())
		{
			EnterEditingMode();
			return FReply::Handled();
		}
	}

	// Do not handle the mouse input, this will allow for drag and dropping events to trigger.
	return FReply::Unhandled();
}

FReply SInlineEditableTextBlock::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Cancel during a drag over event, otherwise the widget may enter edit mode.
	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
	return FReply::Unhandled();
}

FReply SInlineEditableTextBlock::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if (DoubleSelectDelay == 0.0f)
	{
		if (IsSelected.IsBound())
		{
			if (IsSelected.Execute() && !bIsReadOnly.Get() && !FSlateApplication::Get().IsDragDropping())
			{
				EnterEditingMode();
			}
		}
		else if (HasKeyboardFocus() && !bIsReadOnly.Get())
		{
			EnterEditingMode();
		}
		
		return FReply::Handled();
	}
	
	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
	return FReply::Unhandled();
}

EActiveTimerReturnType SInlineEditableTextBlock::TriggerEditMode(double InCurrentTime, float InDeltaTime)
{
	// If someone clicks on us then clicks again and starts dragging it away quickly we might not get OnDragOver
	// called (which cancels the timer), meaning the trigger timer will end up firing later on while they're dragging 
	// something around, which is unexpected. So as an extra safe guard, we'll just avoid entering edit 
	// mode while drag dropping (and parent visibility might mean we never get drag events called on us at all anyway)
	// Note: Doing this check in here rather than in EnterEditingMode just in case a caller actually did want to enter 
	// editing mode during a drag drop, given it's public API
	if (!FSlateApplication::Get().IsDragDropping())
	{
		EnterEditingMode();
	}

	return EActiveTimerReturnType::Stop;
}

FReply SInlineEditableTextBlock::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if(InKeyEvent.GetKey() == EKeys::F2)
	{
		EnterEditingMode();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SInlineEditableTextBlock::OnTextChanged(const FText& InText)
{
	if(IsInEditMode())
	{
		FText OutErrorMessage;
		if(!IsTextAllowed(InText, OutErrorMessage))
		{
			SetTextBoxError( OutErrorMessage );
		}
		else
		{
			SetTextBoxError( FText::GetEmpty() );
		}
	}
	bNeedsMaximumLengthChecked = false;
}

bool SInlineEditableTextBlock::IsTextAllowed(const FText& InText, FText& OutErrorMessage) const
{
	return FTextEditHelper::VerifyTextLength(InText, OutErrorMessage, MaximumLengthAttribute.Get()) &&
		(!OnVerifyTextChanged.IsBound() || OnVerifyTextChanged.Execute(InText, OutErrorMessage));
}

void SInlineEditableTextBlock::OnTextBoxCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if(InCommitType == ETextCommit::OnCleared)
	{
		FText SourceTextInEditMode = Text.Get();
		CancelEditMode();
		// Commit the name, certain actions might need to be taken by the bound function
		OnTextCommittedDelegate.ExecuteIfBound(SourceTextInEditMode, InCommitType);
	}
	else if(IsInEditMode())
	{
		if(InCommitType == ETextCommit::OnEnter)
		{
			FText OutErrorMessage;
			if (!IsTextAllowed(InText, OutErrorMessage))
			{
				// Display as an error.
				SetTextBoxError( OutErrorMessage );
				return;
			}
		}
		else if(InCommitType == ETextCommit::OnUserMovedFocus || InCommitType == ETextCommit::Default)
		{
			FText OutErrorMessage;
			if (!IsTextAllowed(InText, OutErrorMessage))
			{
				CancelEditMode();

				// Commit the name, certain actions might need to be taken by the bound function
				OnTextCommittedDelegate.ExecuteIfBound(Text.Get(), InCommitType);
				
				return;
			}
		}
		else // When the user removes all focus from the window, revert the name
		{
			CancelEditMode();

			// Commit the name, certain actions might need to be taken by the bound function
			OnTextCommittedDelegate.ExecuteIfBound(Text.Get(), InCommitType);
			return;
		}
		
		ExitEditingMode();

		OnTextCommittedDelegate.ExecuteIfBound(InText, InCommitType);

		if ( !Text.IsBound() )
		{
			TextBlock->SetText( Text );
		}
	}
}

TSharedPtr<SWidget> SInlineEditableTextBlock::GetEditableTextWidget() const
{
#if WITH_FANCY_TEXT
	if( bIsMultiLine )
	{
		return MultiLineTextBox;
	}
	else
#endif //WITH_FANCY_TEXT
	{
		return TextBox;
	}
}

void SInlineEditableTextBlock::SetHintText( const TAttribute< FText >& InHintText )
{
	HintText = InHintText;
#if WITH_FANCY_TEXT
	bIsMultiLine ? MultiLineTextBox->SetHintText(InHintText) :
#endif //WITH_FANCY_TEXT
	TextBox->SetHintText(InHintText);
}


void SInlineEditableTextBlock::SetEditableText( const TAttribute< FText >& InNewText )
{
#if WITH_FANCY_TEXT
	bIsMultiLine ? MultiLineTextBox->SetText( Text ) :
#endif //WITH_FANCY_TEXT
				 TextBox->SetText( Text );
}

FText SInlineEditableTextBlock::GetEditableText() const
{
	return 
#if WITH_FANCY_TEXT
	bIsMultiLine ? MultiLineTextBox->GetText() :
#endif //WITH_FANCY_TEXT
				 TextBox->GetText();
}

void SInlineEditableTextBlock::SetTextBoxError( const FText& ErrorText )
{
#if WITH_FANCY_TEXT
	bIsMultiLine ? MultiLineTextBox->SetError( ErrorText ) :
#endif //WITH_FANCY_TEXT
				 TextBox->SetError( ErrorText );
}

bool SInlineEditableTextBlock::HasTextBoxError() const
{
	return 
#if WITH_FANCY_TEXT
	bIsMultiLine ? MultiLineTextBox->HasError() :
#endif //WITH_FANCY_TEXT
				 TextBox->HasError();
}
