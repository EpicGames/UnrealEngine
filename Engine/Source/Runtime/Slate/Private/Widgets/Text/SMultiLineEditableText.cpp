// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/SMultiLineEditableText.h"
#include "Rendering/DrawElements.h"
#include "Types/SlateConstants.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_FANCY_TEXT

#include "Framework/Text/TextEditHelper.h"
#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Widgets/Text/SlateEditableTextLayout.h"
#include "Widgets/Text/SlateTextBlockLayout.h"
#include "Types/ReflectionMetadata.h"
#include "Types/TrackedMetaData.h"

SLATE_IMPLEMENT_WIDGET(SMultiLineEditableText)
void SMultiLineEditableText::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	struct FInvalidation
	{
		static void SynchronizeTextStyle(SWidget& OwningWidget)
		{
			SMultiLineEditableText& MultiLineEditableText = static_cast<SMultiLineEditableText&>(OwningWidget);
			MultiLineEditableText.SynchronizeTextStyle();
		}
	};

	FSlateEditableTextLayout::RegisterNestedAttributes<&SMultiLineEditableText::SlateEditableTextLayout>(AttributeInitializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, FontAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::SynchronizeTextStyle));
}

SMultiLineEditableText::SMultiLineEditableText()
	: SlateEditableTextLayout(*this)
	, bSelectAllTextWhenFocused(false)
	, AmountScrolledWhileRightMouseDown(0.0f)
	, bIsSoftwareCursor(false)
	, FontAttribute(*this)
{
}

void SMultiLineEditableText::Construct( const FArguments& InArgs )
{
	OnIsTypedCharValid = InArgs._OnIsTypedCharValid;
	OnTextChangedCallback = InArgs._OnTextChanged;
	OnTextCommittedCallback = InArgs._OnTextCommitted;
	OnCursorMovedCallback = InArgs._OnCursorMoved;
	bAllowMultiLine = InArgs._AllowMultiLine;
	bSelectAllTextWhenFocused = InArgs._SelectAllTextWhenFocused;
	bClearTextSelectionOnFocusLoss = InArgs._ClearTextSelectionOnFocusLoss;
	bClearKeyboardFocusOnCommit = InArgs._ClearKeyboardFocusOnCommit;
	bAllowContextMenu = InArgs._AllowContextMenu;
	bSelectWordOnMouseDoubleClick = InArgs._SelectWordOnMouseDoubleClick;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	bRevertTextOnEscape = InArgs._RevertTextOnEscape;
	VirtualKeyboardOptions = InArgs._VirtualKeyboardOptions;
	VirtualKeyboardTrigger = InArgs._VirtualKeyboardTrigger;
	VirtualKeyboardDismissAction = InArgs._VirtualKeyboardDismissAction;
	OnHScrollBarUserScrolled = InArgs._OnHScrollBarUserScrolled;
	OnVScrollBarUserScrolled = InArgs._OnVScrollBarUserScrolled;
	OnKeyCharHandler = InArgs._OnKeyCharHandler;
	OnKeyDownHandler = InArgs._OnKeyDownHandler;
	ModiferKeyForNewLine = InArgs._ModiferKeyForNewLine;

	HScrollBar = InArgs._HScrollBar;

    if (HScrollBar.IsValid())
	{
		HScrollBar->SetUserVisibility(EVisibility::Collapsed);
		HScrollBar->SetOnUserScrolled(FOnUserScrolled::CreateSP(this, &SMultiLineEditableText::OnHScrollBarMoved));
	}

	VScrollBar = InArgs._VScrollBar;
	if (VScrollBar.IsValid())
	{
		VScrollBar->SetUserVisibility(EVisibility::Collapsed);
		VScrollBar->SetOnUserScrolled(FOnUserScrolled::CreateSP(this, &SMultiLineEditableText::OnVScrollBarMoved));
	}

	TSharedPtr<ITextLayoutMarshaller> Marshaller = InArgs._Marshaller;
	if (!Marshaller.IsValid())
	{
		Marshaller = FPlainTextLayoutMarshaller::Create();
	}

	SlateEditableTextLayout.Construct(InArgs._Text, *InArgs._TextStyle, InArgs._TextShapingMethod, InArgs._TextFlowDirection, InArgs._CreateSlateTextLayout, Marshaller.ToSharedRef(), Marshaller.ToSharedRef());
	SlateEditableTextLayout.SetHintText(InArgs._HintText);
	SlateEditableTextLayout.SetSearchText(InArgs._SearchText);
	SlateEditableTextLayout.SetTextWrapping(InArgs._WrapTextAt, InArgs._AutoWrapText, InArgs._WrappingPolicy);
	SlateEditableTextLayout.SetMargin(InArgs._Margin);
	SlateEditableTextLayout.SetJustification(InArgs._Justification);
	SlateEditableTextLayout.SetLineHeightPercentage(InArgs._LineHeightPercentage);
	SlateEditableTextLayout.SetDebugSourceInfo(TAttribute<FString>::Create(TAttribute<FString>::FGetter::CreateLambda([this]{ return FReflectionMetaData::GetWidgetDebugInfo(this); })));
	SlateEditableTextLayout.SetOverflowPolicy(InArgs._OverflowPolicy);
	SlateEditableTextLayout.SetFontFacesLoadingPaintPolicy(InArgs._FontFacesLoadingPaintPolicy);
	if (InArgs._OnAllFontFacesFinishLoading.IsBound())
	{
		SlateEditableTextLayout.SetOnAllFontFacesFinishLoadingDelegate(InArgs._OnAllFontFacesFinishLoading);
	}

	SetIsReadOnly(InArgs._IsReadOnly);
	SetFont(InArgs._Font);

	// build context menu extender
	MenuExtender = MakeShareable(new FExtender);
	MenuExtender->AddMenuExtension("EditText", EExtensionHook::Before, TSharedPtr<FUICommandList>(), InArgs._ContextMenuExtender);

	AddMetadata(MakeShared<FTrackedMetaData>(this, FName("EditableText")));
}

void SMultiLineEditableText::GetCurrentTextLine(FString& OutTextLine) const
{
	SlateEditableTextLayout.GetCurrentTextLine(OutTextLine);
}

void SMultiLineEditableText::GetTextLine(const int32 InLineIndex, FString& OutTextLine) const
{
	SlateEditableTextLayout.GetTextLine(InLineIndex, OutTextLine);
}

void SMultiLineEditableText::SetText(TAttribute<FText> InText)
{
	SlateEditableTextLayout.SetText(MoveTemp(InText));
}

int32 SMultiLineEditableText::GetTextLineCount() const
{
	return SlateEditableTextLayout.GetTextLineCount();
}

FText SMultiLineEditableText::GetText() const
{
	return SlateEditableTextLayout.GetText();
}

FText SMultiLineEditableText::GetPlainText() const
{
	return SlateEditableTextLayout.GetPlainText();
}

void SMultiLineEditableText::SetHintText(TAttribute<FText> InHintText)
{
	SlateEditableTextLayout.SetHintText(MoveTemp(InHintText));
}

FText SMultiLineEditableText::GetHintText() const
{
	return SlateEditableTextLayout.GetHintText();
}

void SMultiLineEditableText::SetSearchText(TAttribute<FText> InSearchText)
{
	SlateEditableTextLayout.SetSearchText(MoveTemp(InSearchText));
}

FText SMultiLineEditableText::GetSearchText() const
{
	return SlateEditableTextLayout.GetSearchText();
}

int32 SMultiLineEditableText::GetSearchResultIndex() const
{
	return SlateEditableTextLayout.GetSearchResultIndex();
}

int32 SMultiLineEditableText::GetNumSearchResults() const
{
	return SlateEditableTextLayout.GetNumSearchResults();
}

void SMultiLineEditableText::SetTextStyle(const FTextBlockStyle* InTextStyle)
{
	if (InTextStyle)
	{
		SlateEditableTextLayout.SetTextStyle(*InTextStyle);
	}
	else
	{
		const FArguments Defaults;
		check(Defaults._TextStyle);
		SlateEditableTextLayout.SetTextStyle(*Defaults._TextStyle);
	}
}

void SMultiLineEditableText::SetFont(TAttribute<FSlateFontInfo> InNewFont)
{
	bIsFontAttrSet = InNewFont.IsSet();
	FontAttribute.Assign(*this, MoveTemp(InNewFont));
}

FSlateFontInfo SMultiLineEditableText::GetFont() const
{
	return bIsFontAttrSet ? FontAttribute.Get() : SlateEditableTextLayout.GetTextStyle().Font;
}

void SMultiLineEditableText::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	SlateEditableTextLayout.SetTextShapingMethod(InTextShapingMethod);
}

void SMultiLineEditableText::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	SlateEditableTextLayout.SetTextFlowDirection(InTextFlowDirection);
}

void SMultiLineEditableText::SetWrapTextAt(TAttribute<float> InWrapTextAt)
{
	SlateEditableTextLayout.SetWrapTextAt(MoveTemp(InWrapTextAt));
}

void SMultiLineEditableText::SetAutoWrapText(TAttribute<bool> InAutoWrapText)
{
	SlateEditableTextLayout.SetAutoWrapText(MoveTemp(InAutoWrapText));
}

void SMultiLineEditableText::SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy)
{
	SlateEditableTextLayout.SetWrappingPolicy(MoveTemp(InWrappingPolicy));
}

void SMultiLineEditableText::SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage)
{
	SlateEditableTextLayout.SetLineHeightPercentage(MoveTemp(InLineHeightPercentage));
}

void SMultiLineEditableText::SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine)
{
	SlateEditableTextLayout.SetApplyLineHeightToBottomLine(MoveTemp(InApplyLineHeightToBottomLine));
}

void SMultiLineEditableText::SetMargin(TAttribute<FMargin> InMargin)
{
	SlateEditableTextLayout.SetMargin(MoveTemp(InMargin));
}

void SMultiLineEditableText::SetJustification(TAttribute<ETextJustify::Type> InJustification)
{
	SlateEditableTextLayout.SetJustification(MoveTemp(InJustification));
}

void SMultiLineEditableText::SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy)
{
	SlateEditableTextLayout.SetOverflowPolicy(InOverflowPolicy);
}

void SMultiLineEditableText::SetAllowContextMenu(TAttribute<bool> InAllowContextMenu)
{
	bAllowContextMenu = MoveTemp(InAllowContextMenu);
}

void SMultiLineEditableText::SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction)
{
	VirtualKeyboardDismissAction = MoveTemp(InVirtualKeyboardDismissAction);
}

void SMultiLineEditableText::SetIsReadOnly(TAttribute<bool> InIsReadOnly)
{
	SlateEditableTextLayout.SetIsReadOnly(MoveTemp(InIsReadOnly));
}

void SMultiLineEditableText::SetSelectAllTextWhenFocused(TAttribute<bool> InSelectAllTextWhenFocused)
{
	bSelectAllTextWhenFocused = MoveTemp(InSelectAllTextWhenFocused);
}

void SMultiLineEditableText::SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick)
{
	bSelectWordOnMouseDoubleClick = MoveTemp(InSelectWordOnMouseDoubleClick);
}

void SMultiLineEditableText::SetClearTextSelectionOnFocusLoss(TAttribute<bool> InClearTextSelectionOnFocusLoss)
{
	bClearTextSelectionOnFocusLoss = MoveTemp(InClearTextSelectionOnFocusLoss);
}

void SMultiLineEditableText::SetRevertTextOnEscape(TAttribute<bool> InRevertTextOnEscape)
{
	bRevertTextOnEscape = MoveTemp(InRevertTextOnEscape);
}

void SMultiLineEditableText::SetClearKeyboardFocusOnCommit(TAttribute<bool> InClearKeyboardFocusOnCommit)
{
	bClearKeyboardFocusOnCommit = MoveTemp(InClearKeyboardFocusOnCommit);
}

void SMultiLineEditableText::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	SlateEditableTextLayout.SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
}

void SMultiLineEditableText::OnHScrollBarMoved(const float InScrollOffsetFraction)
{
	SlateEditableTextLayout.SetHorizontalScrollFraction(InScrollOffsetFraction);
	OnHScrollBarUserScrolled.ExecuteIfBound(InScrollOffsetFraction);
}

void SMultiLineEditableText::OnVScrollBarMoved(const float InScrollOffsetFraction)
{
	SlateEditableTextLayout.SetVerticalScrollFraction(InScrollOffsetFraction);
	OnVScrollBarUserScrolled.ExecuteIfBound(InScrollOffsetFraction);
}

bool SMultiLineEditableText::IsWidgetConstructed() const
{
	return IsConstructed();
}

bool SMultiLineEditableText::IsTextReadOnly() const
{
	return SlateEditableTextLayout.GetIsReadOnly();
}

bool SMultiLineEditableText::IsTextPassword() const
{
	return false;
}

bool SMultiLineEditableText::IsMultiLineTextEdit() const
{
	return bAllowMultiLine.Get(true);
}

bool SMultiLineEditableText::IsIntegratedKeyboardEnabled() const
{
	return false;
}

bool SMultiLineEditableText::ShouldJumpCursorToEndWhenFocused() const
{
	return false;
}

bool SMultiLineEditableText::ShouldSelectAllTextWhenFocused() const
{
	return bSelectAllTextWhenFocused.Get(false);
}

bool SMultiLineEditableText::ShouldClearTextSelectionOnFocusLoss() const
{
	return bClearTextSelectionOnFocusLoss.Get(false);
}

bool SMultiLineEditableText::ShouldRevertTextOnEscape() const
{
	return bRevertTextOnEscape.Get(false);
}

bool SMultiLineEditableText::ShouldClearKeyboardFocusOnCommit() const
{
	return bClearKeyboardFocusOnCommit.Get(false);
}

bool SMultiLineEditableText::ShouldSelectAllTextOnCommit() const
{
	return false;
}

bool SMultiLineEditableText::ShouldSelectWordOnMouseDoubleClick() const
{
	return bSelectWordOnMouseDoubleClick.Get(true);
}

bool SMultiLineEditableText::CanInsertCarriageReturn() const
{
	return FSlateApplication::Get().GetModifierKeys().AreModifersDown(ModiferKeyForNewLine);
}

bool SMultiLineEditableText::CanTypeCharacter(const TCHAR InChar) const
{
	if (OnIsTypedCharValid.IsBound())
	{
		return OnIsTypedCharValid.Execute(InChar);
	}

	return InChar != TEXT('\t');
}

void SMultiLineEditableText::EnsureActiveTick()
{
	TSharedPtr<FActiveTimerHandle> ActiveTickTimerPin = ActiveTickTimer.Pin();
	if (ActiveTickTimerPin.IsValid())
	{
		return;
	}

	auto DoActiveTick = [this](double InCurrentTime, float InDeltaTime) -> EActiveTimerReturnType
	{
		// Continue if we still have focus, otherwise treat as a fire-and-forget Tick() request
		const bool bShouldAppearFocused = HasKeyboardFocus() || SlateEditableTextLayout.HasActiveContextMenu();
		return (bShouldAppearFocused) ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
	};

	const float TickPeriod = EditableTextDefs::BlinksPerSecond * 0.5f;
	ActiveTickTimer = RegisterActiveTimer(TickPeriod, FWidgetActiveTimerDelegate::CreateLambda(DoActiveTick));
}

EKeyboardType SMultiLineEditableText::GetVirtualKeyboardType() const
{
	return Keyboard_Default;
}

FVirtualKeyboardOptions SMultiLineEditableText::GetVirtualKeyboardOptions() const
{
	return VirtualKeyboardOptions;
}

EVirtualKeyboardTrigger SMultiLineEditableText::GetVirtualKeyboardTrigger() const
{
	return VirtualKeyboardTrigger.Get();
}

EVirtualKeyboardDismissAction SMultiLineEditableText::GetVirtualKeyboardDismissAction() const
{
	return VirtualKeyboardDismissAction.Get();
}

TSharedRef<SWidget> SMultiLineEditableText::GetSlateWidget()
{
	return AsShared();
}

TSharedPtr<SWidget> SMultiLineEditableText::GetSlateWidgetPtr()
{
	if (DoesSharedInstanceExist())
	{
		return AsShared();
	}
	return nullptr;
}

TSharedPtr<SWidget> SMultiLineEditableText::BuildContextMenuContent() const
{
	if (!bAllowContextMenu.Get())
	{
		return nullptr;
	}

	if (OnContextMenuOpening.IsBound())
	{
		return OnContextMenuOpening.Execute();
	}

	return SlateEditableTextLayout.BuildDefaultContextMenu(MenuExtender);
}

void SMultiLineEditableText::OnBeginTextEdit(const FText& InText)
{
	OnBeginTextEditCallback.ExecuteIfBound(InText);
}

void SMultiLineEditableText::OnTextChanged(const FText& InText)
{
	OnTextChangedCallback.ExecuteIfBound(InText);
}

void SMultiLineEditableText::OnTextCommitted(const FText& InText, const ETextCommit::Type InTextAction)
{
	OnTextCommittedCallback.ExecuteIfBound(InText, InTextAction);
}

void SMultiLineEditableText::OnCursorMoved(const FTextLocation& InLocation)
{
	OnCursorMovedCallback.ExecuteIfBound(InLocation);
	Invalidate(EInvalidateWidget::Layout);
}

float SMultiLineEditableText::UpdateAndClampHorizontalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride)
{
	if (HScrollBar.IsValid())
	{
		// Horizontal scroll is unnecessary when auto wrap is enabled.
		// Otherwise if allowed, having both enabled will cause a wrap+scroll relayout feedback loop which can result in jittering
		const bool bAutoWrapEnabled = SlateEditableTextLayout.GetAutoWrapText();

		if (bAutoWrapEnabled)
		{
			HScrollBar->SetUserVisibility(EVisibility::Collapsed);
		}
		else
		{
			HScrollBar->SetState(InViewOffset, InViewFraction);
			HScrollBar->SetUserVisibility(InVisibilityOverride);
			if (!HScrollBar->IsNeeded())
			{
				// We cannot scroll, so ensure that there is no offset
				return 0.0f;
			}
		}
	}

	return SlateEditableTextLayout.GetScrollOffset().X;
}

float SMultiLineEditableText::UpdateAndClampVerticalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride)
{
	if (VScrollBar.IsValid())
	{
		VScrollBar->SetState(InViewOffset, InViewFraction);
		VScrollBar->SetUserVisibility(InVisibilityOverride);
		if (!VScrollBar->IsNeeded())
		{
			// We cannot scroll, so ensure that there is no offset
			return 0.0f;
		}
	}

	return SlateEditableTextLayout.GetScrollOffset().Y;
}

void SMultiLineEditableText::BeginEditTransaction()
{
	SlateEditableTextLayout.BeginEditTransaction();
}

void SMultiLineEditableText::EndEditTransaction()
{
	SlateEditableTextLayout.EndEditTransaction();
}

FReply SMultiLineEditableText::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	SlateEditableTextLayout.HandleFocusReceived(InFocusEvent);
	return FReply::Handled();
}

void SMultiLineEditableText::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	bIsSoftwareCursor = false;
	SlateEditableTextLayout.HandleFocusLost(InFocusEvent);
}

bool SMultiLineEditableText::AnyTextSelected() const
{
	return SlateEditableTextLayout.AnyTextSelected();
}

void SMultiLineEditableText::SelectAllText()
{
	SlateEditableTextLayout.SelectAllText();
}

void SMultiLineEditableText::SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation)
{
	SlateEditableTextLayout.SelectText(InSelectionStart, InCursorLocation);
}

void SMultiLineEditableText::ClearSelection()
{
	SlateEditableTextLayout.ClearSelection();
}

FText SMultiLineEditableText::GetSelectedText() const
{
	return SlateEditableTextLayout.GetSelectedText();
}

FTextSelection SMultiLineEditableText::GetSelection() const
{
	return SlateEditableTextLayout.GetSelection();
}

void SMultiLineEditableText::DeleteSelectedText()
{
	SlateEditableTextLayout.DeleteSelectedText();
}

void SMultiLineEditableText::InsertTextAtCursor(const FText& InText)
{
	SlateEditableTextLayout.InsertTextAtCursor(InText.ToString());
}

void SMultiLineEditableText::InsertTextAtCursor(const FString& InString)
{
	SlateEditableTextLayout.InsertTextAtCursor(InString);
}

void SMultiLineEditableText::InsertRunAtCursor(TSharedRef<IRun> InRun)
{
	SlateEditableTextLayout.InsertRunAtCursor(MoveTemp(InRun));
}

void SMultiLineEditableText::GoTo(const FTextLocation& NewLocation)
{
	SlateEditableTextLayout.GoTo(NewLocation);
}

void SMultiLineEditableText::GoTo(const ETextLocation NewLocation)
{
	SlateEditableTextLayout.GoTo(NewLocation);
}

void SMultiLineEditableText::ScrollTo(const FTextLocation& NewLocation)
{
	SlateEditableTextLayout.ScrollTo(NewLocation);
}

void SMultiLineEditableText::ScrollTo(const ETextLocation NewLocation)
{
	SlateEditableTextLayout.ScrollTo(NewLocation);
}

void SMultiLineEditableText::ApplyToSelection(const FRunInfo& InRunInfo, const FTextBlockStyle& InStyle)
{
	SlateEditableTextLayout.ApplyToSelection(InRunInfo, InStyle);
}

void SMultiLineEditableText::BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase, const bool InReverse)
{
	SlateEditableTextLayout.BeginSearch(InSearchText, InSearchCase, InReverse);
}

void SMultiLineEditableText::AdvanceSearch(const bool InReverse)
{
	SlateEditableTextLayout.AdvanceSearch(InReverse);
}

TSharedPtr<const IRun> SMultiLineEditableText::GetRunUnderCursor() const
{
	return SlateEditableTextLayout.GetRunUnderCursor();
}

TArray<TSharedRef<const IRun>> SMultiLineEditableText::GetSelectedRuns() const
{
	return SlateEditableTextLayout.GetSelectedRuns();
}

FTextLocation SMultiLineEditableText::GetCursorLocation() const
{
	return SlateEditableTextLayout.GetCursorLocation();
}

TCHAR SMultiLineEditableText::GetCharacterAt(const FTextLocation& Location) const
{
	return SlateEditableTextLayout.GetCharacterAt(Location);
}

TSharedPtr<const SScrollBar> SMultiLineEditableText::GetHScrollBar() const
{
	return HScrollBar;
}

TSharedPtr<const SScrollBar> SMultiLineEditableText::GetVScrollBar() const
{
	return VScrollBar;
}

void SMultiLineEditableText::Refresh()
{
	SlateEditableTextLayout.Refresh();
}

void SMultiLineEditableText::ForceScroll(int32 UserIndex, float ScrollAxisMagnitude)
{
	const FGeometry& CachedGeom = GetCachedGeometry();
	FVector2f ScrollPos = (CachedGeom.LocalToAbsolute(FVector2f::ZeroVector) + CachedGeom.LocalToAbsolute(CachedGeom.GetLocalSize())) * 0.5f;
	TSet<FKey> PressedKeys;

	OnMouseWheel(CachedGeom, FPointerEvent(UserIndex, 0, ScrollPos, ScrollPos, PressedKeys, EKeys::Invalid, ScrollAxisMagnitude, FModifierKeysState()));
}

void SMultiLineEditableText::SynchronizeTextStyle()
{
	// Has the style used for this editable text changed?
	bool bTextStyleChanged = false;
	FTextBlockStyle NewTextStyle = SlateEditableTextLayout.GetTextStyle();

	// Sync from the font override
	if (bIsFontAttrSet)
	{
		const FSlateFontInfo& NewFontInfo = FontAttribute.Get();
		if (!NewTextStyle.Font.IsIdenticalTo(NewFontInfo))
		{
			NewTextStyle.Font = NewFontInfo;
			bTextStyleChanged = true;
		}
	}

	if (bTextStyleChanged)
	{
		SlateEditableTextLayout.SetTextStyle(NewTextStyle);
		SlateEditableTextLayout.ForceRefreshTextLayout(SlateEditableTextLayout.GetEditableText());
	}
}

void SMultiLineEditableText::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SlateEditableTextLayout.Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

int32 SMultiLineEditableText::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const FTextBlockStyle& EditableTextStyle = SlateEditableTextLayout.GetTextStyle();
	const FLinearColor ForegroundColor = EditableTextStyle.ColorAndOpacity.GetColor(InWidgetStyle);

	FWidgetStyle TextWidgetStyle = FWidgetStyle(InWidgetStyle)
		.SetForegroundColor(ForegroundColor);

	const bool bAutoWrap = SlateEditableTextLayout.GetAutoWrapText();
	if (bAutoWrap && SlateEditableTextLayout.GetComputedWrappingWidth() == 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Slate_AutoWrapRecompute)
		const_cast<FSlateEditableTextLayout&>(SlateEditableTextLayout).SetCachedSize(AllottedGeometry);
		const_cast<SMultiLineEditableText*>(this)->CacheDesiredSize(GetPrepassLayoutScaleMultiplier());
	}

	LayerId = SlateEditableTextLayout.OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, TextWidgetStyle, ShouldBeEnabled(bParentEnabled));

	if (bIsSoftwareCursor)
	{
		const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_Grab"));
		const FVector2f CursorSize = Brush->ImageSize / AllottedGeometry.Scale;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(CursorSize, FSlateLayoutTransform(SoftwareCursorPosition - (CursorSize *.5f))),
			Brush
			);
	}

	return LayerId;
}

void SMultiLineEditableText::CacheDesiredSize(float LayoutScaleMultiplier)
{
	SynchronizeTextStyle();
	SlateEditableTextLayout.CacheDesiredSize(LayoutScaleMultiplier);
	SWidget::CacheDesiredSize(LayoutScaleMultiplier);
}

FVector2D SMultiLineEditableText::ComputeDesiredSize( float LayoutScaleMultiplier ) const
{
	return SlateEditableTextLayout.ComputeDesiredSize(LayoutScaleMultiplier);
}

FChildren* SMultiLineEditableText::GetChildren()
{
	return SlateEditableTextLayout.GetChildren();
}

void SMultiLineEditableText::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	SlateEditableTextLayout.OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

bool SMultiLineEditableText::SupportsKeyboardFocus() const
{
	return true;
}

FReply SMultiLineEditableText::OnKeyChar( const FGeometry& MyGeometry,const FCharacterEvent& InCharacterEvent )
{
	FReply Reply = FReply::Unhandled();

	// First call the user defined key handler, there might be overrides to normal functionality
	if (OnKeyCharHandler.IsBound())
	{
		Reply = OnKeyCharHandler.Execute(MyGeometry, InCharacterEvent);
	}

	if (!Reply.IsEventHandled())
	{
		Reply = SlateEditableTextLayout.HandleKeyChar(InCharacterEvent);
	}

	return Reply;
}

FReply SMultiLineEditableText::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	FReply Reply = FReply::Unhandled();

	// First call the user defined key handler, there might be overrides to normal functionality
	if (OnKeyDownHandler.IsBound())
	{
		Reply = OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
	}

	if (!Reply.IsEventHandled())
	{
		Reply = SlateEditableTextLayout.HandleKeyDown(InKeyEvent);
	}

	return Reply;
}

FReply SMultiLineEditableText::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	return SlateEditableTextLayout.HandleKeyUp(InKeyEvent);
}

FReply SMultiLineEditableText::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		AmountScrolledWhileRightMouseDown = 0.0f;
	}

	return SlateEditableTextLayout.HandleMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SMultiLineEditableText::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bool bWasRightClickScrolling = IsRightClickScrolling();
		AmountScrolledWhileRightMouseDown = 0.0f;

		if (bWasRightClickScrolling)
		{
			bIsSoftwareCursor = false;
			const FVector2f CursorPosition = MyGeometry.LocalToAbsolute(SoftwareCursorPosition);
			const FIntPoint OriginalMousePos(CursorPosition.X, CursorPosition.Y);
			return FReply::Handled().ReleaseMouseCapture().SetMousePos(OriginalMousePos);
		}
	}

	return SlateEditableTextLayout.HandleMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SMultiLineEditableText::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		const float ScrollByAmount = MouseEvent.GetCursorDelta().Y / MyGeometry.Scale;

		// If scrolling with the right mouse button, we need to remember how much we scrolled.
		// If we did not scroll at all, we will bring up the context menu when the mouse is released.
		AmountScrolledWhileRightMouseDown += FMath::Abs(ScrollByAmount);

		if (IsRightClickScrolling())
		{
			const FVector2f PreviousScrollOffset = SlateEditableTextLayout.GetScrollOffset();

			FVector2f NewScrollOffset = PreviousScrollOffset;
			NewScrollOffset.Y -= ScrollByAmount;
			SlateEditableTextLayout.SetScrollOffset(NewScrollOffset, MyGeometry);

			if (!bIsSoftwareCursor)
			{
				SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
				bIsSoftwareCursor = true;
			}

			if (PreviousScrollOffset.Y != NewScrollOffset.Y)
			{
				const float ScrollMax = SlateEditableTextLayout.GetSize().Y - MyGeometry.GetLocalSize().Y;
				const float ScrollbarOffset = (ScrollMax != 0.0f) ? NewScrollOffset.Y / ScrollMax : 0.0f;
				OnVScrollBarUserScrolled.ExecuteIfBound(ScrollbarOffset);
				SoftwareCursorPosition.Y += (PreviousScrollOffset.Y - NewScrollOffset.Y);
			}

			return FReply::Handled().UseHighPrecisionMouseMovement(AsShared());
		}
	}

	return SlateEditableTextLayout.HandleMouseMove(MyGeometry, MouseEvent);
}

FReply SMultiLineEditableText::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (VScrollBar.IsValid() && VScrollBar->IsNeeded())
	{
		const float ScrollAmount = -MouseEvent.GetWheelDelta() * GetGlobalScrollAmount();

		const FVector2f PreviousScrollOffset = SlateEditableTextLayout.GetScrollOffset();

		FVector2f NewScrollOffset = PreviousScrollOffset;
		NewScrollOffset.Y += ScrollAmount;
		SlateEditableTextLayout.SetScrollOffset(NewScrollOffset, MyGeometry);

		if (PreviousScrollOffset.Y != NewScrollOffset.Y)
		{
			const float ScrollMax = SlateEditableTextLayout.GetSize().Y - MyGeometry.GetLocalSize().Y;
			const float ScrollbarOffset = (ScrollMax != 0.0f) ? NewScrollOffset.Y / ScrollMax : 0.0f;
			OnVScrollBarUserScrolled.ExecuteIfBound(ScrollbarOffset);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SMultiLineEditableText::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SlateEditableTextLayout.HandleMouseButtonDoubleClick(MyGeometry, MouseEvent);
}

FCursorReply SMultiLineEditableText::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (IsRightClickScrolling() && CursorEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		return FCursorReply::Cursor(EMouseCursor::None);
	}
	else
	{
		return FCursorReply::Cursor(EMouseCursor::TextEditBeam);
	}
}

bool SMultiLineEditableText::IsInteractable() const
{
	return IsEnabled();
}

bool SMultiLineEditableText::ComputeVolatility() const
{
	return SWidget::ComputeVolatility()
		|| SlateEditableTextLayout.ComputeVolatility();
}

bool SMultiLineEditableText::IsRightClickScrolling() const
{
	return AmountScrolledWhileRightMouseDown >= FSlateApplication::Get().GetDragTriggerDistance() && VScrollBar.IsValid() && VScrollBar->IsNeeded();
}

#endif //WITH_FANCY_TEXT
