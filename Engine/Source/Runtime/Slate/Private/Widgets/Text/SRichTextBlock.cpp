// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/SRichTextBlock.h"

#if WITH_FANCY_TEXT

#include "Widgets/Text/SlateTextBlockLayout.h"
#include "Framework/Text/RichTextMarkupProcessing.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Types/ReflectionMetadata.h"

SRichTextBlock::SRichTextBlock()
{
}

SRichTextBlock::~SRichTextBlock()
{
	// Needed to avoid "deletion of pointer to incomplete type 'FSlateTextBlockLayout'; no destructor called" error when using TUniquePtr
}

void SRichTextBlock::Construct( const FArguments& InArgs )
{
	BoundText = InArgs._Text;
	HighlightText = InArgs._HighlightText;

	TextStyle = *InArgs._TextStyle;
	WrapTextAt = InArgs._WrapTextAt;
	AutoWrapText = InArgs._AutoWrapText;
	WrappingPolicy = InArgs._WrappingPolicy;
	TransformPolicy = InArgs._TransformPolicy;
	Margin = InArgs._Margin;
	LineHeightPercentage = InArgs._LineHeightPercentage;
	ApplyLineHeightToBottomLine = InArgs._ApplyLineHeightToBottomLine;
	Justification = InArgs._Justification;
	MinDesiredWidth = InArgs._MinDesiredWidth;

	{
		TSharedPtr<IRichTextMarkupParser> Parser = InArgs._Parser;
		if ( !Parser.IsValid() )
		{
			Parser = FDefaultRichTextMarkupParser::GetStaticInstance();
		}

		Marshaller = InArgs._Marshaller;
		if (!Marshaller.IsValid())
		{
			Marshaller = FRichTextLayoutMarshaller::Create(Parser, nullptr, InArgs._Decorators, InArgs._DecoratorStyleSet);
		}
		
		for (const TSharedRef< ITextDecorator >& Decorator : InArgs.InlineDecorators)
		{
			Marshaller->AppendInlineDecorator(Decorator);
		}

		TextLayoutCache = MakeUnique<FSlateTextBlockLayout>(this, TextStyle, InArgs._TextShapingMethod, InArgs._TextFlowDirection, InArgs._CreateSlateTextLayout, Marshaller.ToSharedRef(), nullptr);
		TextLayoutCache->SetDebugSourceInfo(TAttribute<FString>::Create(TAttribute<FString>::FGetter::CreateLambda([this]{ return FReflectionMetaData::GetWidgetDebugInfo(this); })));
		TextLayoutCache->SetTextOverflowPolicy(InArgs._OverflowPolicy.IsSet() ? InArgs._OverflowPolicy : TextStyle.OverflowPolicy);
		TextLayoutCache->SetFontFacesLoadingPaintPolicy(InArgs._FontFacesLoadingPaintPolicy);
		if (InArgs._OnAllFontFacesFinishLoading.IsBound())
		{
			TextLayoutCache->SetOnAllFontFacesFinishLoading(InArgs._OnAllFontFacesFinishLoading);
		}
	}

	SetCanTick(false);
}

int32 SRichTextBlock::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector2D LastDesiredSize = TextLayoutCache->GetDesiredSize();

	const FGeometry TextBlockScaledGeometry = AllottedGeometry.MakeChild(AllottedGeometry.GetLocalSize() / TextBlockScale, FSlateLayoutTransform(TextBlockScale));

	// OnPaint will also update the text layout cache if required
	LayerId = TextLayoutCache->OnPaint(Args, TextBlockScaledGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));

	const FVector2D NewDesiredSize = TextLayoutCache->GetDesiredSize();

	// HACK: Due to the nature of wrapping and layout, we may have been arranged in a different box than what we were cached with.  Which
	// might update wrapping, so make sure we always set the desired size to the current size of the text layout, which may have changed
	// during paint.
	bool bCanWrap = WrapTextAt.Get() > 0 || AutoWrapText.Get();

	if (bCanWrap && !NewDesiredSize.Equals(LastDesiredSize))
	{
		const_cast<SRichTextBlock*>(this)->Invalidate(EInvalidateWidget::Layout);
	}

	return LayerId;
}

FVector2D SRichTextBlock::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	// ComputeDesiredSize will also update the text layout cache if required
	const FVector2D TextSize = TextLayoutCache->ComputeDesiredSize(
		FSlateTextBlockLayout::FWidgetDesiredSizeArgs(BoundText.Get(), HighlightText.Get(), WrapTextAt.Get(), AutoWrapText.Get(), WrappingPolicy.Get(), TransformPolicy.Get(), Margin.Get(), LineHeightPercentage.Get(), ApplyLineHeightToBottomLine.Get(), Justification.Get()),
		LayoutScaleMultiplier * TextBlockScale, TextStyle) * TextBlockScale;

	return FVector2D(FMath::Max(TextSize.X, MinDesiredWidth.Get()), TextSize.Y);
}

FChildren* SRichTextBlock::GetChildren()
{
	return TextLayoutCache->GetChildren();
}

void SRichTextBlock::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const FGeometry TextBlockScaledGeometry = AllottedGeometry.MakeChild(AllottedGeometry.GetLocalSize() / TextBlockScale, FSlateLayoutTransform(TextBlockScale));

	TextLayoutCache->ArrangeChildren(TextBlockScaledGeometry, ArrangedChildren);
}

void SRichTextBlock::SetText( const TAttribute<FText>& InTextAttr )
{
	if (!BoundText.IsBound() && !InTextAttr.IsBound() && InTextAttr.Get().IdenticalTo(BoundText.Get(), ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		return;
	}

	BoundText = InTextAttr;
	Invalidate(EInvalidateWidget::LayoutAndVolatility | EInvalidateWidgetReason::Prepass);
}

void SRichTextBlock::SetHighlightText( const TAttribute<FText>& InHighlightText )
{
	if (!HighlightText.IsBound() && !InHighlightText.IsBound() && InHighlightText.Get().IdenticalTo(HighlightText.Get(), ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		return;
	}

	HighlightText = InHighlightText;
	Invalidate(EInvalidateWidget::LayoutAndVolatility);
}

void SRichTextBlock::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	TextLayoutCache->SetTextShapingMethod(InTextShapingMethod);
	Invalidate(EInvalidateWidget::Layout);
}

void SRichTextBlock::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	TextLayoutCache->SetTextFlowDirection(InTextFlowDirection);
	Invalidate(EInvalidateWidget::Layout);
}

void SRichTextBlock::SetWrapTextAt(const TAttribute<float>& InWrapTextAt)
{
	SetAttribute(WrapTextAt, InWrapTextAt, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetAutoWrapText(const TAttribute<bool>& InAutoWrapText)
{
	SetAttribute(AutoWrapText, InAutoWrapText, EInvalidateWidgetReason::Prepass);
}

void SRichTextBlock::SetWrappingPolicy(const TAttribute<ETextWrappingPolicy>& InWrappingPolicy)
{
	SetAttribute(WrappingPolicy, InWrappingPolicy, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetTransformPolicy(const TAttribute<ETextTransformPolicy>& InTransformPolicy)
{
	SetAttribute(TransformPolicy, InTransformPolicy, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetLineHeightPercentage(const TAttribute<float>& InLineHeightPercentage)
{
	SetAttribute(LineHeightPercentage, InLineHeightPercentage, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetApplyLineHeightToBottomLine(const TAttribute<bool>& InApplyLineHeightToBottomLine)
{
	SetAttribute(ApplyLineHeightToBottomLine, InApplyLineHeightToBottomLine, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetMargin(const TAttribute<FMargin>& InMargin)
{
	SetAttribute(Margin, InMargin, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetJustification(const TAttribute<ETextJustify::Type>& InJustification)
{
	SetAttribute(Justification, InJustification, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetTextStyle(const FTextBlockStyle& InTextStyle)
{
	TextStyle = InTextStyle;
	Invalidate(EInvalidateWidget::Layout);
}

void SRichTextBlock::SetMinDesiredWidth(const TAttribute<float>& InMinDesiredWidth)
{
	SetAttribute(MinDesiredWidth, InMinDesiredWidth, EInvalidateWidgetReason::Layout);
}

void SRichTextBlock::SetDecoratorStyleSet(const ISlateStyle* NewDecoratorStyleSet)
{
	if (Marshaller.IsValid())
	{
		Marshaller->SetDecoratorStyleSet(NewDecoratorStyleSet);
		Refresh();
	}
}

void SRichTextBlock::SetDecorators(TArrayView<TSharedRef<ITextDecorator>> InDecorators)
{
	if (Marshaller.IsValid())
	{
		Marshaller->SetDecorators(InDecorators);
		Refresh();
	}
}

void SRichTextBlock::SetOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy)
{
	TextLayoutCache->SetTextOverflowPolicy(InOverflowPolicy);
	Invalidate(EInvalidateWidget::Layout);
}

void SRichTextBlock::SetTextBlockScale(const float NewTextBlockScale)
{
	if (TextBlockScale != NewTextBlockScale)
	{
		TextBlockScale = NewTextBlockScale;
		Invalidate(EInvalidateWidget::Prepass);
	}
}

void SRichTextBlock::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	if (TextLayoutCache)
	{
		TextLayoutCache->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
	}
}

void SRichTextBlock::Refresh()
{
	TextLayoutCache->DirtyContent();
	Invalidate(EInvalidateWidget::Layout);
}

bool SRichTextBlock::ComputeVolatility() const
{
	return SWidget::ComputeVolatility() 
		|| BoundText.IsBound()
		|| HighlightText.IsBound()
		|| WrapTextAt.IsBound()
		|| AutoWrapText.IsBound()
		|| WrappingPolicy.IsBound()
		|| TransformPolicy.IsBound()
		|| Margin.IsBound()
		|| Justification.IsBound()
		|| LineHeightPercentage.IsBound()
		|| MinDesiredWidth.IsBound();
}

#endif //WITH_FANCY_TEXT
