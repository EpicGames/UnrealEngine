// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/STextBlock.h"
#include "SlateGlobals.h"
#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Widgets/Text/SlateTextBlockLayout.h"
#include "Types/ReflectionMetadata.h"
#include "Rendering/DrawElements.h"
#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontMeasure.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

DECLARE_CYCLE_STAT(TEXT("STextBlock::OnPaint Time"), Stat_SlateTextBlockOnPaint, STATGROUP_SlateVerbose)
DECLARE_CYCLE_STAT(TEXT("STextBlock::ComputeDesiredSize"), Stat_SlateTextBlockCDS, STATGROUP_SlateVerbose)


SLATE_IMPLEMENT_WIDGET(STextBlock)
void STextBlock::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	//~ If we are in SimpleTextMode the invalidation reason can be different.
	struct FInvalidation
	{
		static EInvalidateWidgetReason GetInvalidationNoneIfSimpleTextMode(const SWidget& OwningWidget)
		{
			return static_cast<const STextBlock&>(OwningWidget).bSimpleTextMode ? EInvalidateWidgetReason::None : EInvalidateWidgetReason::Layout;
		}
		static EInvalidateWidgetReason GetInvalidationPaintIfSimpleTextMode(const SWidget& OwningWidget)
		{
			return static_cast<const STextBlock&>(OwningWidget).bSimpleTextMode ? EInvalidateWidgetReason::Paint : EInvalidateWidgetReason::Layout;
		}

		static void UpdateTextStyle(SWidget& OwningWidget)
		{
			// see ComputeDesiredSize. We call UpdateTextStyle when the style changes.
			static_cast<STextBlock&>(OwningWidget).bTextLayoutUpdateTextStyle = true;
		}

		static void UpdateDesiredSize(SWidget& OwningWidget)
		{
			// We call ComputeDesiredSize when the text changes.
			static_cast<STextBlock&>(OwningWidget).bTextLayoutUpdateDesiredSize = true;
		}
	};

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BoundText, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Font, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, StrikeBrush, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ColorAndOpacity, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShadowOffset, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShadowColorAndOpacity, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HighlightColor, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HighlightShape, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateTextStyle));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HighlightText, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, WrapTextAt, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, AutoWrapText, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, WrappingPolicy, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, TransformPolicy, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Margin, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, Justification, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, LineHeightPercentage, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ApplyLineHeightToBottomLine, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::UpdateDesiredSize));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredWidth, EInvalidateWidgetReason::Layout);
}

STextBlock::STextBlock()
	: BoundText(*this)
	, Font(*this)
	, StrikeBrush(*this)
	, ColorAndOpacity(*this)
	, ShadowOffset(*this, FVector2D::ZeroVector)
	, ShadowColorAndOpacity(*this, FLinearColor(ForceInit))
	, HighlightColor(*this, FLinearColor(ForceInit))
	, HighlightShape(*this)
	, HighlightText(*this)
	, WrapTextAt(*this, 0.0f)
	, AutoWrapText(*this, false)
	, WrappingPolicy(*this, ETextWrappingPolicy::DefaultWrapping)
	, TransformPolicy(*this)
	, Margin(*this)
	, Justification(*this)
	, LineHeightPercentage(*this, 1.0f)
	, ApplyLineHeightToBottomLine(*this, true)
	, MinDesiredWidth(*this, 0.0f)
	, Union_Flags(0)
	, bSimpleTextMode(false)
{
	SetCanTick(false);
	bCanSupportFocus = false;
	bTextLayoutUpdateTextStyle = true;
	bTextLayoutUpdateDesiredSize = true;

#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = false;
#endif
}

STextBlock::~STextBlock()
{
	// Needed to avoid "deletion of pointer to incomplete type 'FSlateTextBlockLayout'; no destructor called" error when using TUniquePtr
}

void STextBlock::Construct( const FArguments& InArgs )
{
	TextStyle = *InArgs._TextStyle;

	SetHighlightText(InArgs._HighlightText);
	SetWrapTextAt(InArgs._WrapTextAt);
	SetAutoWrapText(InArgs._AutoWrapText);
	SetWrappingPolicy(InArgs._WrappingPolicy);

	SetTransformPolicy(InArgs._TransformPolicy);

	SetMargin(InArgs._Margin);
	SetLineHeightPercentage(InArgs._LineHeightPercentage);
	SetApplyLineHeightToBottomLine(InArgs._ApplyLineHeightToBottomLine);
	SetJustification(InArgs._Justification);
	SetMinDesiredWidth(InArgs._MinDesiredWidth);

	SetFont(InArgs._Font);
	SetStrikeBrush(InArgs._StrikeBrush);
	SetColorAndOpacity(InArgs._ColorAndOpacity);
	SetShadowOffset(InArgs._ShadowOffset);
	SetShadowColorAndOpacity(InArgs._ShadowColorAndOpacity);
	SetHighlightColor(InArgs._HighlightColor);
	SetHighlightShape(InArgs._HighlightShape);

	bSimpleTextMode = InArgs._SimpleTextMode;

	if (InArgs._OnDoubleClicked.IsBound())
	{
		SetOnMouseDoubleClick(InArgs._OnDoubleClicked);
	}

	SetText(InArgs._Text);
	//if(!bSimpleTextMode)
	{
		// We use a dummy style here (as it may not be safe to call the delegates used to compute the style), but the correct style is set by ComputeDesiredSize
		TextLayoutCache = MakeUnique<FSlateTextBlockLayout>(this, FTextBlockStyle::GetDefault(), InArgs._TextShapingMethod, InArgs._TextFlowDirection, FCreateSlateTextLayout(), FPlainTextLayoutMarshaller::Create(), InArgs._LineBreakPolicy);
		TextLayoutCache->SetDebugSourceInfo(TAttribute<FString>::Create(TAttribute<FString>::FGetter::CreateLambda([this] { return FReflectionMetaData::GetWidgetDebugInfo(this); })));
		TextLayoutCache->SetTextOverflowPolicy(InArgs._OverflowPolicy.IsSet() ? InArgs._OverflowPolicy : TextStyle.OverflowPolicy);
		TextLayoutCache->SetFontFacesLoadingPaintPolicy(InArgs._FontFacesLoadingPaintPolicy);
	}

	if (InArgs._OnAllFontFacesFinishLoading.IsBound() && TextLayoutCache)
	{
		TextLayoutCache->SetOnAllFontFacesFinishLoading(InArgs._OnAllFontFacesFinishLoading);
	}
}

FSlateFontInfo STextBlock::GetFont() const
{
	return GetFontRef();
}

const FSlateFontInfo& STextBlock::GetFontRef() const
{
	return bIsAttributeFontSet ? Font.Get() : TextStyle.Font;
}

const FSlateBrush* STextBlock::GetStrikeBrush() const
{
	return bIsAttributeStrikeBrushSet ? StrikeBrush.Get() : &TextStyle.StrikeBrush;
}

FSlateColor STextBlock::GetColorAndOpacity() const
{
	return GetColorAndOpacityRef();
}

const FSlateColor& STextBlock::GetColorAndOpacityRef() const
{
	return bIsAttributeColorAndOpacitySet ? ColorAndOpacity.Get() : TextStyle.ColorAndOpacity;
}

FVector2f STextBlock::GetShadowOffset() const
{
	return bIsAttributeShadowOffsetSet ? UE::Slate::CastToVector2f(ShadowOffset.Get()) : FVector2f(TextStyle.ShadowOffset);
}

FLinearColor STextBlock::GetShadowColorAndOpacity() const
{
	return GetShadowColorAndOpacityRef();
}

const FLinearColor& STextBlock::GetShadowColorAndOpacityRef() const
{
	return bIsAttributeShadowColorAndOpacitySet ? ShadowColorAndOpacity.Get() : TextStyle.ShadowColorAndOpacity;
}

FSlateColor STextBlock::GetHighlightColor() const
{
	return bIsAttributeHighlightColorSet ? HighlightColor.Get() : TextStyle.HighlightColor;
}

const FSlateBrush* STextBlock::GetHighlightShape() const
{
	return bIsAttributeHighlightShapeSet ? HighlightShape.Get() : &TextStyle.HighlightShape;
}

ETextTransformPolicy STextBlock::GetTransformPolicyImpl() const
{
	return bIsAttributeTransformPolicySet ? TransformPolicy.Get() : TextStyle.TransformPolicy;
}

FMargin STextBlock::GetMargin() const
{
	return Margin.Get();
}

float STextBlock::GetMinDesiredWidth() const
{
	return MinDesiredWidth.Get();
}

void STextBlock::InvalidateText(EInvalidateWidgetReason InvalidateReason)
{
	if (bSimpleTextMode && EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Layout))
	{
		CachedSimpleDesiredSize.Reset();
	}

	Invalidate(InvalidateReason);
}

void STextBlock::SetText(TAttribute<FText> InText)
{
	// Cache the IsBound.
	//When the attribute is not bound, we need to go through all the other bound property to check if it is bound.
	bIsAttributeBoundTextBound = InText.IsBound();
	BoundText.Assign(*this, MoveTemp(InText));
}

void STextBlock::SetHighlightText(TAttribute<FText> InText)
{
	HighlightText.Assign(*this, MoveTemp(InText));
}

int32 STextBlock::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	SCOPE_CYCLE_COUNTER(Stat_SlateTextBlockOnPaint);

	if (bSimpleTextMode)
	{
		// Draw the optional shadow
		const FLinearColor LocalShadowColorAndOpacity = GetShadowColorAndOpacity();
		const FVector2f LocalShadowOffset = GetShadowOffset();
		const bool ShouldDropShadow = LocalShadowColorAndOpacity.A > 0.f && LocalShadowOffset.SizeSquared() > 0.f;

		const bool bShouldBeEnabled = ShouldBeEnabled(bParentEnabled);

		const FText& LocalText = BoundText.Get();
		FSlateFontInfo LocalFont = GetFont();

		if (ShouldDropShadow)
		{
			const int32 OutlineSize = LocalFont.OutlineSettings.OutlineSize;
			if (!LocalFont.OutlineSettings.bApplyOutlineToDropShadows)
			{
				LocalFont.OutlineSettings.OutlineSize = 0;
			}

			FSlateDrawElement::MakeText(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToOffsetPaintGeometry(LocalShadowOffset),
				LocalText,
				LocalFont,
				bShouldBeEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				InWidgetStyle.GetColorAndOpacityTint() * LocalShadowColorAndOpacity
			);

			// Restore outline size for main text
			LocalFont.OutlineSettings.OutlineSize = OutlineSize;

			// actual text should appear above the shadow
			++LayerId;
		}

		// Draw the text itself
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LocalText,
			LocalFont,
			bShouldBeEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			InWidgetStyle.GetColorAndOpacityTint() * GetColorAndOpacity().GetColor(InWidgetStyle)
			);
	}
	else
	{
		const FVector2D LastDesiredSize = TextLayoutCache->GetDesiredSize();

		// OnPaint will also update the text layout cache if required
		UpdateTextBlockLayout(TextLayoutCache->GetLayoutScale());
		LayerId = TextLayoutCache->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));

		// HACK: Due to the nature of wrapping and layout, we may have been arranged in a different box than what we were cached with.  Which
		// might update wrapping, so make sure we always set the desired size to the current size of the text layout, which may have changed
		// during paint.
		const bool bCanWrap = WrapTextAt.Get() > 0 || AutoWrapText.Get();
		const FVector2D NewDesiredSize = TextLayoutCache->GetDesiredSize();
		if (bCanWrap && !NewDesiredSize.Equals(LastDesiredSize))
		{
			const_cast<STextBlock*>(this)->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}

	return LayerId;
}

FVector2D STextBlock::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	SCOPE_CYCLE_COUNTER(Stat_SlateTextBlockCDS);

	if (bSimpleTextMode)
	{
		const FVector2D LocalShadowOffset = FVector2D(GetShadowOffset());

		const float LocalOutlineSize = (float)(GetFont().OutlineSettings.OutlineSize);

		// Account for the outline width impacting both size of the text by multiplying by 2
		// Outline size in Y is accounted for in MaxHeight calculation in Measure()
		const FVector2D ComputedOutlineSize(LocalOutlineSize * 2.f, LocalOutlineSize);
		const FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(BoundText.Get(), GetFont()) + ComputedOutlineSize + LocalShadowOffset;

		CachedSimpleDesiredSize = FVector2f(FMath::Max(MinDesiredWidth.Get(), TextSize.X), TextSize.Y);
		return FVector2D(CachedSimpleDesiredSize.GetValue());
	}
	else
	{
		bTextLayoutUpdateDesiredSize = true; // force ComputeDesiredSize
		UpdateTextBlockLayout(LayoutScaleMultiplier);
		const FVector2D TextSize = TextLayoutCache->GetDesiredSize();
		return FVector2D(FMath::Max(MinDesiredWidth.Get(), TextSize.X), TextSize.Y);
	}
}

void STextBlock::SetFont(TAttribute<FSlateFontInfo> InFont)
{
	bIsAttributeFontSet = InFont.IsSet();
	Font.Assign(*this, MoveTemp(InFont));
}

void STextBlock::SetStrikeBrush(TAttribute<const FSlateBrush*> InStrikeBrush)
{
	bIsAttributeStrikeBrushSet = InStrikeBrush.IsSet();
	StrikeBrush.Assign(*this, MoveTemp(InStrikeBrush));
}

void STextBlock::SetColorAndOpacity(TAttribute<FSlateColor> InColorAndOpacity)
{
	bIsAttributeColorAndOpacitySet = InColorAndOpacity.IsSet();
	ColorAndOpacity.Assign(*this, MoveTemp(InColorAndOpacity));
}

void STextBlock::SetTextStyle(const FTextBlockStyle* InTextStyle, bool bUpdateTextLayout)
{
	if (InTextStyle)
	{
		TextStyle = *InTextStyle;
	}
	else
	{
		FArguments Defaults;
		TextStyle = *Defaults._TextStyle;
	}

	if (bUpdateTextLayout)
	{
		bTextLayoutUpdateTextStyle = true;
	}

	InvalidateText(EInvalidateWidgetReason::Layout);
}

void STextBlock::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	if (!bSimpleTextMode)
	{
		TextLayoutCache->SetTextShapingMethod(InTextShapingMethod);
		InvalidateText(EInvalidateWidgetReason::Layout);
	}
}

void STextBlock::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	if(!bSimpleTextMode)
	{
		TextLayoutCache->SetTextFlowDirection(InTextFlowDirection);
		InvalidateText(EInvalidateWidgetReason::Layout);
	}
}

void STextBlock::SetWrapTextAt(TAttribute<float> InWrapTextAt)
{
	WrapTextAt.Assign(*this, MoveTemp(InWrapTextAt), 0.f);
}

void STextBlock::SetAutoWrapText(TAttribute<bool> InAutoWrapText)
{
	AutoWrapText.Assign(*this, MoveTemp(InAutoWrapText), 0.f);
}

void STextBlock::SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy)
{
	WrappingPolicy.Assign(*this, MoveTemp(InWrappingPolicy));
}

void STextBlock::SetTransformPolicy(TAttribute<ETextTransformPolicy> InTransformPolicy)
{
	bIsAttributeTransformPolicySet = InTransformPolicy.IsSet();
	TransformPolicy.Assign(*this, MoveTemp(InTransformPolicy));
}

void STextBlock::SetOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy)
{
	TextLayoutCache->SetTextOverflowPolicy(InOverflowPolicy);
	InvalidateText(EInvalidateWidgetReason::Layout);
}

void STextBlock::SetShadowOffset(TAttribute<FVector2D> InShadowOffset)
{
	bIsAttributeShadowOffsetSet = InShadowOffset.IsSet();
	ShadowOffset.Assign(*this, MoveTemp(InShadowOffset));
}

void STextBlock::SetShadowColorAndOpacity(TAttribute<FLinearColor> InShadowColorAndOpacity)
{
	bIsAttributeShadowColorAndOpacitySet = InShadowColorAndOpacity.IsSet();
	ShadowColorAndOpacity.Assign(*this, MoveTemp(InShadowColorAndOpacity));
}

void STextBlock::SetHighlightColor(TAttribute<FLinearColor> InHighlightColor)
{
	bIsAttributeHighlightColorSet = InHighlightColor.IsSet();
	HighlightColor.Assign(*this, MoveTemp(InHighlightColor));
}

void STextBlock::SetHighlightShape(TAttribute<const FSlateBrush*> InHighlightShape)
{
	bIsAttributeHighlightShapeSet = InHighlightShape.IsSet();
	HighlightShape.Assign(*this, MoveTemp(InHighlightShape));
}

void STextBlock::SetMinDesiredWidth(TAttribute<float> InMinDesiredWidth)
{
	MinDesiredWidth.Assign(*this, MoveTemp(InMinDesiredWidth), 0.f);
}

void STextBlock::SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage)
{
	LineHeightPercentage.Assign(*this, MoveTemp(InLineHeightPercentage));
}

void STextBlock::SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine)
{
	ApplyLineHeightToBottomLine.Assign(*this, MoveTemp(InApplyLineHeightToBottomLine));
}

void STextBlock::SetMargin(TAttribute<FMargin> InMargin)
{
	Margin.Assign(*this, MoveTemp(InMargin));
}

void STextBlock::SetJustification(TAttribute<ETextJustify::Type> InJustification)
{
	Justification.Assign(*this, MoveTemp(InJustification));
}

void STextBlock::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	if (TextLayoutCache)
	{
		TextLayoutCache->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
	}
}

FTextBlockStyle STextBlock::GetComputedTextStyle() const
{
	FTextBlockStyle ComputedStyle = TextStyle;
	ComputedStyle.SetFont( GetFontRef() );
	if (const FSlateBrush* const ComputedStrikeBrush = GetStrikeBrush())
	{
		ComputedStyle.SetStrikeBrush(*ComputedStrikeBrush);
	}
	ComputedStyle.SetColorAndOpacity( GetColorAndOpacity() );
	ComputedStyle.SetShadowOffset( GetShadowOffset() );
	ComputedStyle.SetShadowColorAndOpacity( GetShadowColorAndOpacity() );
	ComputedStyle.SetHighlightColor( GetHighlightColor() );
	if (const FSlateBrush* const ComputedHighlightShape = GetHighlightShape())
	{
		ComputedStyle.SetHighlightShape(*ComputedHighlightShape);
	}
	return ComputedStyle;
}

void STextBlock::UpdateTextBlockLayout(float LayoutScaleMultiplier) const
{
	if (bTextLayoutUpdateTextStyle)
	{
		TextLayoutCache->UpdateTextStyle(GetComputedTextStyle());
	}

	if (bTextLayoutUpdateDesiredSize || bTextLayoutUpdateTextStyle)
	{
		// ComputeDesiredSize will also update the text layout cache if required
		FSlateTextBlockLayout::FWidgetDesiredSizeArgs DesiredSizeArgs(
			BoundText.Get(),
			HighlightText.Get(),
			WrapTextAt.Get(),
			AutoWrapText.Get(),
			WrappingPolicy.Get(),
			GetTransformPolicyImpl(),
			Margin.Get(),
			LineHeightPercentage.Get(),
			ApplyLineHeightToBottomLine.Get(),
			Justification.Get()
		);
		TextLayoutCache->ComputeDesiredSize(DesiredSizeArgs, LayoutScaleMultiplier);

		bTextLayoutUpdateDesiredSize = false;
		bTextLayoutUpdateTextStyle = false;
	}
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> STextBlock::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleTextBlock(SharedThis(this)));
}

TOptional<FText> STextBlock::GetDefaultAccessibleText(EAccessibleType AccessibleType) const
{
	return BoundText.Get();
}
#endif
