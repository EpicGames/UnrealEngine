// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/STextScroller.h"
#include "Widgets/Text/STextBlock.h"

#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"

void STextScroller::Construct(const FArguments& InArgs)
{
	ScrollOptions = InArgs._ScrollOptions;
	ScrollOrientation = InArgs._ScrollOrientation;

	// We don't tick, we use an active ticker.
	SetCanTick(false);

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

void STextScroller::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if (ArrangedChildren.Accepts(ChildVisibility))
	{
		const FVector2D ThisContentScale = GetContentScale();
		const FMargin SlotPadding(LayoutPaddingWithFlow(GSlateFlowDirection, ChildSlot.GetPadding()));
		const AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(GSlateFlowDirection, AllottedGeometry.GetLocalSize().X, ChildSlot, SlotPadding, ThisContentScale.X, ScrollOrientation != Orient_Horizontal);
		const AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, ChildSlot, SlotPadding, ThisContentScale.Y,  ScrollOrientation != Orient_Vertical);
		const FVector2D MinimumSize = ChildSlot.GetWidget()->GetDesiredSize();

		ArrangedChildren.AddWidget(ChildVisibility, AllottedGeometry.MakeChild(
			ChildSlot.GetWidget(),
			FVector2D(XResult.Offset, YResult.Offset),
			FVector2D(XResult.Size, YResult.Size)
		));
	}
}

int32 STextScroller::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const int AxisIndex = ScrollOrientation == Orient_Vertical ? 1 : 0;
	{
		STextScroller* MutableThis = const_cast<STextScroller*>(this);

		const float VisibleGeometry = AllottedGeometry.GetLocalSize()[AxisIndex];
		const float DesiredGeometry = VisibleGeometry == 0.0f ? VisibleGeometry : ChildSlot.GetWidget()->GetDesiredSize()[AxisIndex];

		if (DesiredGeometry > (VisibleGeometry + 2.0f) && IsScrollingEnabled())
		{
			if (!ActiveTimerHandle.IsValid())
			{
				MutableThis->TickerState = ETickerState::StartTicking;
				MutableThis->ActiveTimerHandle = MutableThis->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(MutableThis, &STextScroller::OnScrollTextTick));
			}
		}
		else if (ActiveTimerHandle.IsValid())
		{
			MutableThis->TickerState = ETickerState::StopTicking;
		}
	}

	if (ScrollOffset != 0)
	{
		const float ScrollDirection = GSlateFlowDirection == EFlowDirection::LeftToRight || ScrollOrientation == Orient_Vertical ? -1 : 1;
		FVector2D GeometryOffset = FVector2D::ZeroVector;
		GeometryOffset[AxisIndex] = ScrollOffset * ScrollDirection;
		const FGeometry ScrolledGeometry = AllottedGeometry.MakeChild(FSlateRenderTransform(GeometryOffset));
		return SCompoundWidget::OnPaint(Args, ScrolledGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
	else
	{
		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
}

void STextScroller::ResetScrollState()
{
	FontAlpha = 1.f;
	TimeElapsed = 0.f;
	ScrollOffset = 0.f;
	// If suspended, make sure we stay suspended until receiving an explicit call to resume
	if (IsScrollingEnabled())
	{
		ActiveState = EActiveState::Start;
	}
	SetRenderOpacity(1.0f);
}

void STextScroller::StartScrolling()
{
	ActiveState = EActiveState::Start;
	ResetScrollState();
}

void STextScroller::SuspendScrolling()
{
	ActiveState = EActiveState::Suspend;
	ResetScrollState();
}

EActiveTimerReturnType STextScroller::OnScrollTextTick(double CurrentTime, float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_STextScroller_OnTick);

	check(TickerState != ETickerState::None);
	switch (TickerState)
	{
	case ETickerState::StartTicking:
		if (ScrollOrientation == Orient_Vertical)
		{
			ChildSlot.VAlign(VAlign_Top);
		}
		else
		{
			// If we need to scroll, then it's imperative that we arrange from the left, rather than fill, so that we flip to right aligning and scrolling
			// to the right (potentially).
			SetFlowDirectionPreference(EFlowDirectionPreference::Culture);
			ChildSlot.HAlign(HAlign_Left);
		}

		TickerState = ETickerState::Ticking;
		return EActiveTimerReturnType::Continue; // Defer to the next tick so that the layout is correct

	case ETickerState::StopTicking:
		if (ScrollOrientation == Orient_Vertical)
		{
			ChildSlot.VAlign(VAlign_Fill);
		}
		else
		{
			// If we no longer need to scroll, just inherit the flow direction.
			SetFlowDirectionPreference(EFlowDirectionPreference::Inherit);
			ChildSlot.HAlign(HAlign_Fill);
		}

		TickerState = ETickerState::None;
		ResetScrollState();
		ActiveTimerHandle.Reset();
		return EActiveTimerReturnType::Stop; // Ticking is no longer required

	default:
		break;
	}
	
	const int AxisIndex = ScrollOrientation == Orient_Vertical ? 1 : 0;
	const float ContentSize = ChildSlot.GetWidget()->GetDesiredSize()[AxisIndex];
	TimeElapsed += DeltaTime;

	switch (ActiveState)
	{
	case EActiveState::FadeIn:
	{
		FontAlpha = FMath::Clamp<float>(TimeElapsed / ScrollOptions.FadeInDelay, 0.f, 1.f);
		if (TimeElapsed >= ScrollOptions.FadeInDelay)
		{
			FontAlpha = 1.f;
			TimeElapsed = 0.f;
			ScrollOffset = 0.f;
			ActiveState = EActiveState::Start;
		}
		break;
	}
	case EActiveState::Start:
	{
		TimeElapsed = 0.f;
		ScrollOffset = 0.f;
		ActiveState = EActiveState::StartWait;
		break;
	}
	case EActiveState::StartWait:
	{
		if (TimeElapsed >= ScrollOptions.StartDelay)
		{
			TimeElapsed = 0.f;
			ScrollOffset = 0.f;
			ActiveState = EActiveState::Scrolling;
		}
		break;
	}
	case EActiveState::Scrolling:
	{
		ScrollOffset += ScrollOptions.Speed * DeltaTime;
		if ((ScrollOffset + GetCachedGeometry().GetLocalSize()[AxisIndex]) >= ContentSize)
		{
			TimeElapsed = 0.0f;
			ActiveState = EActiveState::Stop;
		}
		break;
	}
	case EActiveState::Stop:
	{
		TimeElapsed = 0.f;
		ActiveState = EActiveState::StopWait;
		break;
	}
	case EActiveState::StopWait:
	{
		if (TimeElapsed >= ScrollOptions.EndDelay)
		{
			TimeElapsed = 0.f;
			ActiveState = EActiveState::FadeOut;
		}
		break;
	}
	case EActiveState::FadeOut:
	{
		FontAlpha = 1.0f - FMath::Clamp<float>(TimeElapsed / ScrollOptions.FadeOutDelay, 0.f, 1.f);
		if (TimeElapsed >= ScrollOptions.FadeOutDelay)
		{
			FontAlpha = 0.0f;
			TimeElapsed = 0.0f;
			ScrollOffset = 0.0f;
			ActiveState = EActiveState::FadeIn;
		}
		break;
	}
	}

	SetRenderOpacity(FontAlpha);
	Invalidate(EInvalidateWidgetReason::Paint);

	return EActiveTimerReturnType::Continue;
}
