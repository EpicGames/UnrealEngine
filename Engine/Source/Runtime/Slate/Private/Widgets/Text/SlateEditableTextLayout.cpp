// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/SlateEditableTextLayout.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Text/TextHitPoint.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Text/SlatePasswordRun.h"
#include "Widgets/Text/SlateTextBlockLayout.h"
#include "Framework/Text/TextEditHelper.h"
#include "Framework/Commands/GenericCommands.h"
#include "Internationalization/BreakIterator.h"
#include "HAL/PlatformApplicationMisc.h"

/**
 * Line terminator used for the TSF/IME (ITextInputMethodContext) ACP coordinate space.
 * The platform default LINE_TERMINATOR is "\r\n" (2 chars) on Windows, but the Korean (and other
 * composition-based) IME calculates ACP indices assuming a single-character line break. Exposing a
 * 2-char line break leaves an unaddressable position between '\r' and '\n', which desyncs composition.
 * Every method that talks to TSF must use this single-character terminator so the ACP space matches 1:1.
 */
static const TCHAR* const GIMETSFLineTerminator = TEXT("\n");

namespace SlateEditableTextLayoutIME
{
	/**
	 * Computes the minimal changed sub-range between two strings by trimming their common prefix and
	 * common suffix, expressed in the half-open form TSF's NotifyTextChanged() expects:
	 *     acpStart  = OutBeginIndex
	 *     acpOldEnd = OutBeginIndex + OutOldLength
	 *     acpNewEnd = OutBeginIndex + OutNewLength
	 * Reporting only the touched span (rather than the whole document) avoids the overflow-like
	 * behaviour TSF exhibits on full-document change reports and keeps composition anchoring stable.
	 *
	 * Both strings must already be in the same coordinate space the rest of the IME path uses, i.e. a
	 * single-character (\n) line terminator - see GIMETSFLineTerminator.
	 *
	 * @return true if the strings differ (out-params then describe a non-empty edit); false if identical.
	 */
	static bool ComputeMinimalChangedRange(const FString& OldString, const FString& NewString, int32& OutBeginIndex, int32& OutOldLength, int32& OutNewLength)
	{
		const int32 OldLen = OldString.Len();
		const int32 NewLen = NewString.Len();
		const int32 MinLen = FMath::Min(OldLen, NewLen);

		int32 CommonPrefixLen = 0;
		while (CommonPrefixLen < MinLen && OldString[CommonPrefixLen] == NewString[CommonPrefixLen])
		{
			++CommonPrefixLen;
		}

		int32 OldSuffixStart = OldLen;
		int32 NewSuffixStart = NewLen;
		while (OldSuffixStart > CommonPrefixLen && NewSuffixStart > CommonPrefixLen
			&& OldString[OldSuffixStart - 1] == NewString[NewSuffixStart - 1])
		{
			--OldSuffixStart;
			--NewSuffixStart;
		}

		OutBeginIndex = CommonPrefixLen;
		OutOldLength = OldSuffixStart - CommonPrefixLen;
		OutNewLength = NewSuffixStart - CommonPrefixLen;

		// Identical strings collapse to a zero-length edit; let the caller skip the notification entirely.
		return OutOldLength != 0 || OutNewLength != 0;
	}
}

/**
 * Ensure that text transactions are always completed.
 * i.e. never forget to call EndEditTransaction();
 */
struct FScopedEditableTextTransaction
{
public:
	explicit FScopedEditableTextTransaction(FSlateEditableTextLayout& InEditableTextLayout)
		: EditableTextLayout(&InEditableTextLayout)
	{
		EditableTextLayout->BeginEditTransaction();
	}

	~FScopedEditableTextTransaction()
	{
		EditableTextLayout->EndEditTransaction();
	}

private:
	FSlateEditableTextLayout* EditableTextLayout;
};

namespace
{

FORCEINLINE FReply BoolToReply(const bool bHandled)
{
	return (bHandled) ? FReply::Handled() : FReply::Unhandled();
}

bool IsCharAllowed(const TCHAR InChar)
{
	// Certain characters are not allowed
	if (InChar == TEXT('\t'))
	{
		return true;
	}
	else if (InChar <= 0x1F)
	{
		return false;
	}
	return true;
}

}

void FSlateEditableTextLayout::Construct(TAttribute<FText> InInitialText, FTextBlockStyle InTextStyle, const TOptional<ETextShapingMethod>& InTextShapingMethod, const TOptional<ETextFlowDirection>& InTextFlowDirection, const FCreateSlateTextLayout& InCreateSlateTextLayout, TSharedRef<ITextLayoutMarshaller> InTextMarshaller, TSharedRef<ITextLayoutMarshaller> InHintTextMarshaller)
{
	CreateSlateTextLayout = InCreateSlateTextLayout;
	if (!CreateSlateTextLayout.IsBound())
	{
		CreateSlateTextLayout.BindLambda([](SWidget* InOwningWidget, const FTextBlockStyle& InDefaultTextStyle) 
			{
				return FSlateTextLayout::Create(InOwningWidget, InDefaultTextStyle);
			});
	}

	const TSharedRef<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidget();

	Marshaller = MoveTemp(InTextMarshaller);
	HintMarshaller = MoveTemp(InHintTextMarshaller);
	TextStyle = MoveTemp(InTextStyle);
	HintTextStyle = TextStyle;
	TextLayout = CreateSlateTextLayout.Execute(&OwnerSlateWidget.Get(), TextStyle);

	DebugSourceInfo = FString();

	SearchCase = ESearchCase::IgnoreCase;

	GraphemeBreakIterator = GSlateUseSharedBreakIterator ? FBreakIterator::GetCharacterBoundaryIterator() : FBreakIterator::CreateCharacterBoundaryIterator();

	BoundText.Assign(OwnerSlateWidget.Get(), MoveTemp(InInitialText), ESlateAttributeBindAction::Initialize);
	PreviousText = BoundText.Get();

	// Set the initial text - the same as SetText, but doesn't call OnTextChanged as that can cause a crash during construction
	{
		const bool bIsPassword = OwnerWidget->IsTextPassword();
		TextLayout->SetIsPassword(bIsPassword);

		const FText& InitialTextToSet = BoundText.Get();
		SetEditableText(InitialTextToSet, true);

		// Update the cached BoundText value to prevent it triggering another SetEditableText update again next Tick
		BoundTextLastTick = FTextSnapshot(InitialTextToSet);
		bWasPasswordLastTick = bIsPassword;
	}

	if (InTextShapingMethod.IsSet())
	{
		TextLayout->SetTextShapingMethod(InTextShapingMethod.GetValue());
	}

	if (InTextFlowDirection.IsSet())
	{
		TextLayout->SetTextFlowDirection(InTextFlowDirection.GetValue());
	}

	VirtualKeyboardEntry = FVirtualKeyboardEntry::Create(*this);

	bHasRegisteredTextInputMethodContext = false;
	TextInputMethodContext = FTextInputMethodContext::Create(*this);

	CursorLineHighlighter = SlateEditableTextTypes::FCursorLineHighlighter::Create(&CursorInfo);
	TextCompositionHighlighter = SlateEditableTextTypes::FTextCompositionHighlighter::Create();
	TextSelectionHighlighter = SlateEditableTextTypes::FTextSelectionHighlighter::Create();
	SearchSelectionHighlighter = SlateEditableTextTypes::FTextSearchHighlighter::Create();

	ScrollOffset = FVector2f::ZeroVector;
	PreferredCursorScreenOffsetInLine = 0.0f;
	SelectionStart = TOptional<FTextLocation>();
	CurrentUndoLevel = INDEX_NONE;
	NumTransactionsOpened = 0;

	bIsDragSelecting = false;
	bWasFocusedByLastMouseDown = false;
	bHasDragSelectedSinceFocused = false;
	bTextChangedByVirtualKeyboard = false;
	bTextCommittedByVirtualKeyboard = false;
	bSelectionChangedExternally = false;
	bForceBoundTextReview = false;
	VirtualKeyboardTextCommitType = ETextCommit::Default;

	CachedSize = FVector2f::ZeroVector;

	auto ExecuteDeleteAction = [this]()
	{
		BeginEditTransaction();
		DeleteSelectedText();
		EndEditTransaction();
	};

	UICommandList = MakeShareable(new FUICommandList());

	UICommandList->MapAction(FGenericCommands::Get().Undo,
		FExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::Undo),
		FCanExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CanExecuteUndo),
		EUIActionRepeatMode::RepeatEnabled
		);

	UICommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CutSelectedTextToClipboard),
		FCanExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CanExecuteCut)
		);

	UICommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::PasteTextFromClipboard),
		FCanExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CanExecutePaste),
		EUIActionRepeatMode::RepeatEnabled
		);

	UICommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CopySelectedTextToClipboard),
		FCanExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CanExecuteCopy)
		);

	UICommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateLambda(ExecuteDeleteAction),
		FCanExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CanExecuteDelete)
		);

	UICommandList->MapAction(FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::SelectAllText),
		FCanExecuteAction::CreateRaw(this, &FSlateEditableTextLayout::CanExecuteSelectAll)
		);
}

FSlateEditableTextLayout::~FSlateEditableTextLayout()
{
	if (ActiveContextMenu.IsValid())
	{
		ActiveContextMenu.Dismiss();
	}

	ITextInputMethodSystem* const TextInputMethodSystem = FSlateApplication::IsInitialized() ? FSlateApplication::Get().GetTextInputMethodSystem() : nullptr;
	if (TextInputMethodSystem && bHasRegisteredTextInputMethodContext)
	{
		TSharedRef<FTextInputMethodContext> TextInputMethodContextRef = TextInputMethodContext.ToSharedRef();
		
		// Make sure that the context is marked as dead to avoid any further IME calls from trying to mutate our dying owner widget
		TextInputMethodContextRef->KillContext();

		if (TextInputMethodSystem->IsActiveContext(TextInputMethodContextRef))
		{
			// This can happen if an entire tree of widgets is culled, as Slate isn't notified of the focus loss, the widget is just deleted
			TextInputMethodSystem->DeactivateContext(TextInputMethodContextRef);
		}

		TextInputMethodSystem->UnregisterContext(TextInputMethodContextRef);
	}

	if(FSlateApplication::IsInitialized() && FPlatformApplicationMisc::RequiresVirtualKeyboard() && 
	   OwnerWidget->GetSlateWidgetPtr().IsValid() && OwnerWidget->GetSlateWidgetPtr()->HasAnyUserFocus().IsSet())
	{
		FSlateApplication::Get().ShowVirtualKeyboard(false, 0);
	}
}

void FSlateEditableTextLayout::SetText(TAttribute<FText> InText)
{
	bForceBoundTextReview = true;
	BoundText.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InText), FText::GetEmpty(), ESlateAttributeBindAction::Initialize);

	if (bForceBoundTextReview)
	{
		// We would only arrive here if OnBoundTextChanged did not trigger (because the BoundText technically did not change),
		// but we still want to make sure the EditableText is aligning so we call it manually
		OnBoundTextChanged();
	}
}

int32 FSlateEditableTextLayout::GetTextLineCount() const
{
	return TextLayout->GetLineCount();
}

FText FSlateEditableTextLayout::GetText() const
{
	SLATE_CROSS_THREAD_CHECK();

	return BoundText.Get();
}

void FSlateEditableTextLayout::SetHintText(TAttribute<FText> InHintText)
{
	const TSharedRef<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidget();

	HintText.Assign(OwnerSlateWidget.Get(), MoveTemp(InHintText), FText::GetEmpty(), ESlateAttributeBindAction::Initialize);

	// If we have hint text that is either non-empty or bound to a delegate, we'll also need to make the hint text layout
	if (HintText.IsBound(OwnerSlateWidget.Get()) || !HintText.Get().IsEmpty())
	{
		HintTextLayout = MakeUnique<FSlateTextBlockLayout>(&OwnerSlateWidget.Get(), HintTextStyle, TextLayout->GetTextShapingMethod(), TextLayout->GetTextFlowDirection(), CreateSlateTextLayout, HintMarshaller.ToSharedRef(), nullptr);
		HintTextLayout->SetDebugSourceInfo(DebugSourceInfo);
	}
	else
	{
		HintTextLayout.Reset();
	}
}

FText FSlateEditableTextLayout::GetHintText() const
{
	return HintText.Get();
}

void FSlateEditableTextLayout::SetSearchText(TAttribute<FText> InSearchText)
{
	BoundSearchText.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InSearchText));
}

FText FSlateEditableTextLayout::GetSearchText() const
{
	return SearchText;
}

int32 FSlateEditableTextLayout::GetSearchResultIndex() const
{
	return CurrentSearchResultIndex;
}

int32 FSlateEditableTextLayout::GetNumSearchResults() const
{
	return SearchResultToIndexMap.Num();
}

void FSlateEditableTextLayout::SetTextStyle(const FTextBlockStyle& InTextStyle)
{
	TextStyle = InTextStyle;
	HintTextStyle = TextStyle;

	TextLayout->SetDefaultTextStyle(TextStyle);
	Marshaller->MakeDirty(); // will regenerate the text using the new default style

	// Using Layout as changing text block size can affect the size.
	OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::Layout);
}

const FTextBlockStyle& FSlateEditableTextLayout::GetTextStyle() const
{
	return TextStyle;
}

void FSlateEditableTextLayout::SetHintTextOpacity(TAttribute<float> InOpacity)
{
	HintTextOpacity.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InOpacity));
}

float FSlateEditableTextLayout::GetHintTextOpacity() const
{
	return HintTextOpacity.Get();
}

void FSlateEditableTextLayout::SetIsReadOnly(TAttribute<bool> InIsReadOnly)
{
	bIsReadOnlyAttribute.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InIsReadOnly));
}

bool FSlateEditableTextLayout::GetIsReadOnly() const
{
	return bIsReadOnlyAttribute.Get();
}

void FSlateEditableTextLayout::SetCursorBrush(const TAttribute<const FSlateBrush*>& InCursorBrush)
{
	CursorLineHighlighter->SetCursorBrush(InCursorBrush);
}

void FSlateEditableTextLayout::SetCompositionBrush(const TAttribute<const FSlateBrush*>& InCompositionBrush)
{
	TextCompositionHighlighter->SetCompositionBrush(InCompositionBrush);
}

FText FSlateEditableTextLayout::GetPlainText() const
{
	SLATE_CROSS_THREAD_CHECK();

	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

	const int32 NumberOfLines = Lines.Num();
	if (NumberOfLines > 0)
	{
		FString SelectedText;
		const FTextSelection Selection(FTextLocation(0, 0), FTextLocation(NumberOfLines - 1, Lines[NumberOfLines - 1].Text->Len()));
		TextLayout->GetSelectionAsText(SelectedText, Selection);

		return FText::FromString(MoveTemp(SelectedText));
	}

	return FText::GetEmpty();
}

bool FSlateEditableTextLayout::SetEditableText(const FText& TextToSet, const bool bForce)
{
	SLATE_CROSS_THREAD_CHECK();

	bool bHasTextChanged = bForce;
	if (!bHasTextChanged)
	{
		const FText EditedText = GetEditableText();
		bHasTextChanged = !EditedText.ToString().Equals(TextToSet.ToString(), ESearchCase::CaseSensitive);
	}

	if (bHasTextChanged)
	{
		const FString& TextToSetString = TextToSet.ToString();

		ClearSelection();
		TextLayout->ClearLines();

		TextLayout->ClearLineHighlights();
		TextLayout->ClearRunRenderers();

		Marshaller->SetText(TextToSetString, *TextLayout);

		Marshaller->ClearDirty();

		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
		if (Lines.Num() == 0)
		{
			TSharedRef< FString > LineText = MakeShareable(new FString());

			// Create an empty run
			TArray< TSharedRef< IRun > > Runs;
			Runs.Add(CreateTextOrPasswordRun(FRunInfo(), LineText, TextStyle));

			TextLayout->AddLine(FTextLayout::FNewLineData(MoveTemp(LineText), MoveTemp(Runs)));
		}

		{
			const FTextLocation OldCursorPos = CursorInfo.GetCursorInteractionLocation();

			// Make sure the cursor is still at a valid location
			if (OldCursorPos.GetLineIndex() >= Lines.Num() || OldCursorPos.GetOffset() > Lines[OldCursorPos.GetLineIndex()].Text->Len())
			{
				const int32 LastLineIndex = Lines.Num() - 1;
				const FTextLocation NewCursorPosition = FTextLocation(LastLineIndex, Lines[LastLineIndex].Text->Len());

				CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
				UpdatePreferredCursorScreenOffsetInLine();
				UpdateCursorHighlight();
			}
		}

		OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::Layout);

		return true;
	}

	return false;
}

FText FSlateEditableTextLayout::GetEditableText() const
{
	FString EditedText;
	Marshaller->GetText(EditedText, *TextLayout);
	return FText::FromString(MoveTemp(EditedText));
}

FText FSlateEditableTextLayout::GetSelectedText() const
{
	if (AnyTextSelected())
	{
		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
		FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		FString SelectedText;
		TextLayout->GetSelectionAsText(SelectedText, Selection);

		return FText::FromString(MoveTemp(SelectedText));
	}

	return FText::GetEmpty();
}

FTextSelection FSlateEditableTextLayout::GetSelection() const
{
	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	const FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
	const FTextSelection Selection(SelectionLocation, CursorInteractionPosition);
	
	return Selection;
}

void FSlateEditableTextLayout::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	const ETextShapingMethod NewShapingMethod = InTextShapingMethod.Get(GetDefaultTextShapingMethod());
	if (NewShapingMethod != TextLayout->GetTextShapingMethod())
	{
		TextLayout->SetTextShapingMethod(NewShapingMethod);

		OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::Layout);
	}
}

void FSlateEditableTextLayout::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	const ETextFlowDirection NewFlowDirection = InTextFlowDirection.Get(GetDefaultTextFlowDirection());
	if (NewFlowDirection != TextLayout->GetTextFlowDirection())
	{
		TextLayout->SetTextFlowDirection(NewFlowDirection);

		OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::Layout);
	}
}

void FSlateEditableTextLayout::SetTextWrapping(TAttribute<float> InWrapTextAt, TAttribute<bool> InAutoWrapText, TAttribute<ETextWrappingPolicy> InWrappingPolicy)
{
	const TSharedRef<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidget();

	WrapTextAt.Assign(OwnerSlateWidget.Get(), MoveTemp(InWrapTextAt));
	AutoWrapText.Assign(OwnerSlateWidget.Get(), MoveTemp(InAutoWrapText));
	WrappingPolicy.Assign(OwnerSlateWidget.Get(), MoveTemp(InWrappingPolicy));
}

void FSlateEditableTextLayout::SetWrapTextAt(TAttribute<float> InWrapTextAt)
{
	WrapTextAt.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InWrapTextAt));
}

void FSlateEditableTextLayout::SetAutoWrapText(TAttribute<bool> InAutoWrapText)
{
	AutoWrapText.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InAutoWrapText));
}

void FSlateEditableTextLayout::SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy)
{
	WrappingPolicy.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InWrappingPolicy));
}

void FSlateEditableTextLayout::SetMargin(TAttribute<FMargin> InMargin)
{
	Margin.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InMargin));
}

void FSlateEditableTextLayout::SetJustification(TAttribute<ETextJustify::Type> InJustification)
{
	Justification.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InJustification));
}

ETextJustify::Type FSlateEditableTextLayout::GetJustification() const
{
	return Justification.Get();
}

void FSlateEditableTextLayout::SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage)
{
	LineHeightPercentage.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InLineHeightPercentage));
}

void FSlateEditableTextLayout::SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine)
{
	ApplyLineHeightToBottomLine.Assign(OwnerWidget->GetSlateWidget().Get(), MoveTemp(InApplyLineHeightToBottomLine));
}

void FSlateEditableTextLayout::SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy)
{
	if(OverflowPolicyOverride != InOverflowPolicy)
	{
		OverflowPolicyOverride = InOverflowPolicy;
		TextLayout->SetTextOverflowPolicy(OverflowPolicyOverride);
		if (HintTextLayout.IsValid())
		{
			HintTextLayout->SetTextOverflowPolicy(OverflowPolicyOverride);
		}

		OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::Layout);
	}
}

void FSlateEditableTextLayout::SetDebugSourceInfo(const TAttribute<FString>& InDebugSourceInfo)
{
	DebugSourceInfo = InDebugSourceInfo;

	TextLayout->SetDebugSourceInfo(DebugSourceInfo);

	if (HintTextLayout.IsValid())
	{
		HintTextLayout->SetDebugSourceInfo(DebugSourceInfo);
	}
}

TSharedRef<IVirtualKeyboardEntry> FSlateEditableTextLayout::GetVirtualKeyboardEntry() const
{
	return VirtualKeyboardEntry.ToSharedRef();
}

TSharedRef<ITextInputMethodContext> FSlateEditableTextLayout::GetTextInputMethodContext() const
{
	return TextInputMethodContext.ToSharedRef();
}

void FSlateEditableTextLayout::EnableTextInputMethodContext()
{
	ITextInputMethodSystem* const TextInputMethodSystem = FSlateApplication::Get().GetTextInputMethodSystem();
	if (TextInputMethodSystem)
	{
		if (!bHasRegisteredTextInputMethodContext)
		{
			bHasRegisteredTextInputMethodContext = true;

			TextInputMethodChangeNotifier = TextInputMethodSystem->RegisterContext(TextInputMethodContext.ToSharedRef());
			if (TextInputMethodChangeNotifier.IsValid())
			{
				TextInputMethodChangeNotifier->NotifyLayoutChanged(ITextInputMethodChangeNotifier::ELayoutChangeType::Created);
			}
		}

		TextInputMethodContext->CacheWindow();

		// Make sure to set Native OS window focus as well to ensure IME support
		if (TSharedPtr<FGenericWindow> NativeWindow = TextInputMethodContext->GetWindow())
		{
			NativeWindow->SetWindowFocus();
		}

		TextInputMethodSystem->ActivateContext(TextInputMethodContext.ToSharedRef());
	}
}

bool FSlateEditableTextLayout::Refresh()
{
	const FText& TextToSet = BoundText.Get();
	return RefreshImpl(&TextToSet);
}

bool FSlateEditableTextLayout::RefreshImpl(const FText* InTextToSet, const bool bForce)
{
	SLATE_CROSS_THREAD_CHECK();

	bool bHasSetText = false;

	const bool bIsPassword = OwnerWidget->IsTextPassword();
	TextLayout->SetIsPassword(bIsPassword);

	if (InTextToSet && (bForce || !BoundTextLastTick.IdenticalTo(*InTextToSet)))
	{
		// The pointer used by the bound text has changed, however the text may still be the same - check that now
		if (bForce || !BoundTextLastTick.IsDisplayStringEqualTo(*InTextToSet))
		{
			// The source text has changed, so update the internal editable text
			bHasSetText = SetEditableText(*InTextToSet, bForce);
		}

		// Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next Tick
		BoundTextLastTick = FTextSnapshot(*InTextToSet);
	}

	if (!bHasSetText && (Marshaller->IsDirty() || bIsPassword != bWasPasswordLastTick))
	{
		ForceRefreshTextLayout((InTextToSet) ? *InTextToSet : GetEditableText());
		bHasSetText = true;
	}

	bWasPasswordLastTick = bIsPassword;

	if (bHasSetText)
	{
		TextLayout->UpdateIfNeeded();
	}

	return bHasSetText;
}

void FSlateEditableTextLayout::ForceRefreshTextLayout(const FText& CurrentText)
{
	// Marshallers shouldn't inject any visible characters into the text, but SetEditableText will clear the current selection
	// so we need to back that up here so we don't cause the cursor to jump around
	const TOptional<FTextLocation> OldSelectionStart = SelectionStart;
	const SlateEditableTextTypes::FCursorInfo OldCursorInfo = CursorInfo;

	SetEditableText(CurrentText, true);

	SelectionStart = OldSelectionStart;
	CursorInfo = OldCursorInfo;
	UpdateCursorHighlight();

	TextLayout->UpdateIfNeeded();
}

void FSlateEditableTextLayout::BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase, const bool InReverse)
{
	SearchText = InSearchText;
	SearchCase = InSearchCase;
	AdvanceSearch(InReverse);
}

void FSlateEditableTextLayout::AdvanceSearch(const bool InReverse)
{
	//FirstMatchedLocation used as a key to find the index of the first matched string among all matches
	FTextLocation FirstMatchedLocation(INDEX_NONE, INDEX_NONE);
	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	
	if (!SearchText.IsEmpty())
	{
		const FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
		const FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		const FTextLocation SearchStartLocation = InReverse ? Selection.GetBeginning() : Selection.GetEnd();

		const FString& SearchTextString = SearchText.ToString();
		const int32 SearchTextLength = SearchTextString.Len();
		const TArray<FTextLayout::FLineModel>& Lines = TextLayout->GetLineModels();

		int32 CurrentLineIndex = SearchStartLocation.GetLineIndex();
		int32 CurrentLineOffset = SearchStartLocation.GetOffset();

		int32 NumLinesSearched = 0;
		
		do
		{
			const FTextLayout::FLineModel& Line = Lines[CurrentLineIndex];

			bool bShouldSearchLine = true;
			
			if (!InReverse && CurrentLineOffset >= Line.Text->Len() )
			{
				// CurrentLineOffset needs to be less than len(),
				// otherwise, Find() clamps it to len() - 1 (see FString::Find()),
				// and if there is a match at len() - 1, the search gets stuck
				// 
				// for example, for text "[cursor]abcd", len() = 4, len() - 1 = 3
				// if you search for 'd', after the first advance,
				// we get "abcd[cursor]", where CurrentLineOffset = 4
				// if we advance one more time with Find('d', offset = 4)
				// internally it becomes a search starting from len() - 1 = 3,
				// since at index 3 there is a 'd' match, the CurrentSearchBegin is 3
				// as a result, cursor position is set to 3 + len('d') = 4, again
				bShouldSearchLine = false;
			}

			if (bShouldSearchLine)
			{
				// Do we have a match on this line?
				const int32 CurrentSearchBegin = Line.Text->Find(SearchTextString, SearchCase, InReverse ? ESearchDir::FromEnd : ESearchDir::FromStart, CurrentLineOffset);
				if (CurrentSearchBegin != INDEX_NONE)
				{
					SelectionStart = FTextLocation(CurrentLineIndex, CurrentSearchBegin);
					CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, FTextLocation(CurrentLineIndex, CurrentSearchBegin + SearchTextLength));

					FirstMatchedLocation = SelectionStart.GetValue();
					break;
				}
			}

			if (InReverse)
			{
				// Advance and loop the line (the outer loop will break once we loop fully around)
				--CurrentLineIndex;
				if (CurrentLineIndex < 0)
				{
					CurrentLineIndex = Lines.Num() - 1;
				}
				CurrentLineOffset = Lines[CurrentLineIndex].Text->Len();
			}
			else
			{
				// Advance and loop the line (the outer loop will break once we loop fully around)
				++CurrentLineIndex;
				if (CurrentLineIndex == Lines.Num())
				{
					CurrentLineIndex = 0;
				}
				CurrentLineOffset = 0;
			}

			NumLinesSearched++;

		}while(NumLinesSearched <= Lines.Num());
		// use "<=" because if we start a search from the middle of a line
		// the search should wrap around and search from the beginning of the line
		// so loop twice even if there is only a single line
	}

	UpdateCursorHighlight();

	// UpdateCursorHighlight() ensures SearchResultToIndexMap is up to date
	CurrentSearchResultIndex = 0;
	if (FirstMatchedLocation.IsValid())
	{
		int32* Index = SearchResultToIndexMap.Find(FirstMatchedLocation);
		if (ensure(Index))
		{
			CurrentSearchResultIndex = *Index;
		}
		
		// PositionToScrollIntoView is set to cursor position in UpdateCursorHighlight();
		// Scrolling to cursor position directly does not always produce a good result
		// because you can have only the last letter of the matched text in the view
		// and most of the text out of view. The following code addresses this problem
		const FTextLocation LineStart(GetSelection().GetBeginning().GetLineIndex(), 0);
		const FVector2D LocalLineStartLocation = TextLayout->GetLocationAt(LineStart, false) / TextLayout->GetScale();

		const FVector2D LocalSelectionBeginLocation = TextLayout->GetLocationAt(GetSelection().GetBeginning(), false) / TextLayout->GetScale();

		const FVector2D LocalSelectionEndLocation = TextLayout->GetLocationAt(GetSelection().GetEnd(), false) / TextLayout->GetScale();
		
		// Only apply extra scrolling if we are going from right to left 
		if (LocalSelectionBeginLocation.X < 0.0f)
		{
			const float DistanceFromSelectionEndToLineStart = LocalSelectionEndLocation.X - LocalLineStartLocation.X;

			if (DistanceFromSelectionEndToLineStart < TextLayout->GetViewSize().X)
			{
				// Scroll to line start if both the matched text and line start can fit into the view
				PositionToScrollIntoView = SlateEditableTextTypes::FScrollInfo(LineStart, SlateEditableTextTypes::ECursorAlignment::Left);
			}
			else
			{
				// Otherwise, just apply minimal scrolling such that the entirety of the matched text is in the view
				PositionToScrollIntoView = SlateEditableTextTypes::FScrollInfo(GetSelection().GetBeginning(), SlateEditableTextTypes::ECursorAlignment::Left);
			}
		}
	}
}

UE::Slate::FDeprecateVector2DResult FSlateEditableTextLayout::SetHorizontalScrollFraction(const float InScrollOffsetFraction)
{
	ScrollOffset.X = FMath::Clamp<float>(InScrollOffsetFraction, 0.0, 1.0) * TextLayout->GetSize().X;
	return ScrollOffset;
}

UE::Slate::FDeprecateVector2DResult FSlateEditableTextLayout::SetVerticalScrollFraction(const float InScrollOffsetFraction)
{
	ScrollOffset.Y = FMath::Clamp<float>(InScrollOffsetFraction, 0.0, 1.0) * TextLayout->GetSize().Y;
	return ScrollOffset;
}

UE::Slate::FDeprecateVector2DResult FSlateEditableTextLayout::SetScrollOffset(const UE::Slate::FDeprecateVector2DParameter& InScrollOffset, const FGeometry& InGeometry)
{
	const FVector2f ContentSize = UE::Slate::CastToVector2f(TextLayout->GetSize());
	ScrollOffset.X = FMath::Clamp(InScrollOffset.X, 0.0f, ContentSize.X - InGeometry.GetLocalSize().X);
	ScrollOffset.Y = FMath::Clamp(InScrollOffset.Y, 0.0f, ContentSize.Y - InGeometry.GetLocalSize().Y);
	return ScrollOffset;
}

UE::Slate::FDeprecateVector2DResult FSlateEditableTextLayout::GetScrollOffset() const
{
	return ScrollOffset;
}


float FSlateEditableTextLayout::GetComputedWrappingWidth() const
{
	return TextLayout->GetWrappingWidth();
}

bool FSlateEditableTextLayout::GetAutoWrapText() const
{
	return AutoWrapText.Get();
}

bool FSlateEditableTextLayout::HandleFocusReceived(const FFocusEvent& InFocusEvent)
{
	if (ActiveContextMenu.IsValid())
	{
		return false;
	}

	// We need to Tick() while we have focus to keep some things up-to-date
	OwnerWidget->EnsureActiveTick();
	
	if (FPlatformApplicationMisc::RequiresVirtualKeyboard())
	{
		if (!OwnerWidget->IsTextReadOnly())
		{
			if ( (InFocusEvent.GetCause() == EFocusCause::Mouse && OwnerWidget->GetVirtualKeyboardTrigger() == EVirtualKeyboardTrigger::OnFocusByPointer) ||
				 (OwnerWidget->GetVirtualKeyboardTrigger() == EVirtualKeyboardTrigger::OnAllFocusEvents))
			{
				// @TODO: Create ITextInputMethodSystem derivations for mobile
				FSlateApplication::Get().ShowVirtualKeyboard(true, InFocusEvent.GetUser(), VirtualKeyboardEntry);
			}
		}
	}
	else
	{
		EnableTextInputMethodContext();
	}

	// Make sure we have the correct text (we might have been collapsed and have missed updates due to not being ticked)
	LoadText();

	// Store undo state to use for escape key reverts
	MakeUndoState(OriginalText);

	// Jump to the end of the document?
	if (InFocusEvent.GetCause() != EFocusCause::Mouse && InFocusEvent.GetCause() != EFocusCause::OtherWidgetLostFocus && OwnerWidget->ShouldJumpCursorToEndWhenFocused())
	{
		GoTo(ETextLocation::EndOfDocument);
	}

	// Select All Text for non-mouse events (mouse events are handled by HandleMouseButtonUp)
	if (InFocusEvent.GetCause() != EFocusCause::Mouse && OwnerWidget->ShouldSelectAllTextWhenFocused())
	{
		SelectAllText();
	}

	UpdateCursorHighlight();

	// UpdateCursorHighlight always tries to scroll to the cursor, but we don't want that to happen when we 
	// gain focus since it can cause the scroll position to jump unexpectedly
	// If we gained focus via a mouse click that moved the cursor, then MoveCursor will already take care
	// of making sure that gets scrolled into view
	PositionToScrollIntoView.Reset();

	// Focus change affects volatility, so update that too
	OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::LayoutAndVolatility);

	// Force update of last cursor interaction time to ensure the caret blinks at the correct frequency regardless of how focus is set.
	CursorInfo.UpdateLastCursorInteractionTime();

	OwnerWidget->OnBeginTextEdit(BoundText.Get());
	return true;
}

bool FSlateEditableTextLayout::HandleFocusLost(const FFocusEvent& InFocusEvent)
{
	if (ActiveContextMenu.IsValid())
	{
		return false;
	}

	if (FPlatformApplicationMisc::RequiresVirtualKeyboard())
	{
		FSlateApplication::Get().ShowVirtualKeyboard(false, InFocusEvent.GetUser());
	}
	else
	{
		ITextInputMethodSystem* const TextInputMethodSystem = FSlateApplication::Get().GetTextInputMethodSystem();
		if (TextInputMethodSystem && bHasRegisteredTextInputMethodContext)
		{
			if (TextInputMethodContext->IsComposing() && TextInputMethodChangeNotifier.IsValid())
			{
				TextInputMethodChangeNotifier->CancelComposition();
			}
			TextInputMethodSystem->DeactivateContext(TextInputMethodContext.ToSharedRef());
		}
	}

	// Clear selection unless activating a new window (otherwise can't copy and past on right click)
	if (OwnerWidget->ShouldClearTextSelectionOnFocusLoss() && InFocusEvent.GetCause() != EFocusCause::WindowActivate)
	{
		ClearSelection();
	}

	if (!OwnerWidget->IsTextReadOnly())
	{
		// When focus is lost let anyone who is interested that text was committed
		// See if user explicitly tabbed away or moved focus
		ETextCommit::Type TextAction;
		switch (InFocusEvent.GetCause())
		{
		case EFocusCause::Navigation:
		case EFocusCause::Mouse:
			TextAction = ETextCommit::OnUserMovedFocus;
			break;

		case EFocusCause::Cleared:
			TextAction = ETextCommit::OnCleared;
			break;

		default:
			TextAction = ETextCommit::Default;
			break;
		}

		// Always clear the local undo chain on commit
		ClearUndoStates();

		const FText EditedText = GetEditableText();

		OwnerWidget->OnTextCommitted(EditedText, TextAction);
	}

	// Reload underlying value now it is committed  (commit may alter the value) 
	// so it can be re-displayed in the edit box
	LoadText();

	UpdateCursorHighlight();

	// UpdateCursorHighlight always tries to scroll to the cursor, but we don't want that to happen when we 
	// lose focus since it can cause the scroll position to jump unexpectedly
	PositionToScrollIntoView.Reset();

	// Focus change affects volatility, so update that too
	OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::LayoutAndVolatility);

	return true;
}

FReply FSlateEditableTextLayout::HandleKeyChar(const FCharacterEvent& InCharacterEvent)
{
	// Check for special characters
	const TCHAR Character = InCharacterEvent.GetCharacter();

	switch (Character)
	{
	case TCHAR(8):		// Backspace
		if (!OwnerWidget->IsTextReadOnly())
		{
			FScopedEditableTextTransaction TextTransaction(*this);
			return BoolToReply(HandleBackspace());
		}
		break;

	case TCHAR('\n'):	// Newline (Ctrl+Enter), we handle adding new lines via HandleCarriageReturn rather than processing newline characters
		return FReply::Handled();

	case 1:				// Swallow Ctrl+A, we handle that through OnKeyDown
	case 3:				// Swallow Ctrl+C, we handle that through OnKeyDown
	case 13:			// Swallow Enter, we handle that through OnKeyDown
	case 22:			// Swallow Ctrl+V, we handle that through OnKeyDown
	case 24:			// Swallow Ctrl+X, we handle that through OnKeyDown
	case 25:			// Swallow Ctrl+Y, we handle that through OnKeyDown
	case 26:			// Swallow Ctrl+Z, we handle that through OnKeyDown
	case 27:			// Swallow ESC, we handle that through OnKeyDown
	case 127:			// Swallow CTRL+Backspace, we handle that through OnKeyDown
		return FReply::Handled();

	default:
		// Type the character, but only if it is allowed.
		if (!OwnerWidget->IsTextReadOnly() && OwnerWidget->CanTypeCharacter(Character))
		{
			FScopedEditableTextTransaction TextTransaction(*this);
			return BoolToReply(HandleTypeChar(Character));
		}
		break;
	}

	return FReply::Unhandled();
}

FReply FSlateEditableTextLayout::HandleKeyDown(const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();

	const FKey Key = InKeyEvent.GetKey();

	if (Key == EKeys::Left)
	{
		if (OwnerWidget->IsTextPassword() && InKeyEvent.IsControlDown())
		{
			// If the text is sensitive, we should not clue the user in to where word breaks are
			JumpTo(ETextLocation::BeginningOfLine, InKeyEvent.IsShiftDown() ? ECursorAction::SelectText : ECursorAction::MoveCursor);
			Reply = FReply::Handled();
		}
		else
		{
			Reply = BoolToReply(MoveCursor(FMoveCursor::Cardinal(
				// Ctrl moves a whole word instead of one character.	
				InKeyEvent.IsControlDown() ? ECursorMoveGranularity::Word : ECursorMoveGranularity::Character,
				// Move left
				FIntPoint(-1, 0),
				// Shift selects text.	
				InKeyEvent.IsShiftDown() ? ECursorAction::SelectText : ECursorAction::MoveCursor
			)));
		}
	}
	else if (Key == EKeys::Right)
	{
		if (OwnerWidget->IsTextPassword() && InKeyEvent.IsControlDown())
		{
			// If the text is sensitive, we should not clue the user in to where word breaks are
			JumpTo(ETextLocation::EndOfLine, InKeyEvent.IsShiftDown() ? ECursorAction::SelectText : ECursorAction::MoveCursor);
			Reply = FReply::Handled();
		}
		else
		{
			Reply = BoolToReply(MoveCursor(FMoveCursor::Cardinal(
				// Ctrl moves a whole word instead of one character.	
				InKeyEvent.IsControlDown() ? ECursorMoveGranularity::Word : ECursorMoveGranularity::Character,
				// Move right
				FIntPoint(+1, 0),
				// Shift selects text.	
				InKeyEvent.IsShiftDown() ? ECursorAction::SelectText : ECursorAction::MoveCursor
			)));
		}
	}
	else if (Key == EKeys::Up)
	{
		Reply = BoolToReply(MoveCursor(FMoveCursor::Cardinal(
			ECursorMoveGranularity::Character,
			// Move up
			FIntPoint(0, -1),
			// Shift selects text.	
			InKeyEvent.IsShiftDown() ? ECursorAction::SelectText : ECursorAction::MoveCursor
			)));
	}
	else if (Key == EKeys::Down)
	{
		Reply = BoolToReply(MoveCursor(FMoveCursor::Cardinal(
			ECursorMoveGranularity::Character,
			// Move down
			FIntPoint(0, +1),
			// Shift selects text.	
			InKeyEvent.IsShiftDown() ? ECursorAction::SelectText : ECursorAction::MoveCursor
			)));
	}
	else if (Key == EKeys::Home)
	{
		// Go to the beginning of the document; select text if Shift is down.
		JumpTo(
			(InKeyEvent.IsControlDown()) ? ETextLocation::BeginningOfDocument : ETextLocation::BeginningOfLine,
			(InKeyEvent.IsShiftDown()) ? ECursorAction::SelectText : ECursorAction::MoveCursor);
		Reply = FReply::Handled();
	}
	else if (Key == EKeys::End)
	{
		// Go to the end of the document; select text if Shift is down.
		JumpTo(
			(InKeyEvent.IsControlDown()) ? ETextLocation::EndOfDocument : ETextLocation::EndOfLine,
			(InKeyEvent.IsShiftDown()) ? ECursorAction::SelectText : ECursorAction::MoveCursor);
		Reply = FReply::Handled();
	}
	else if (Key == EKeys::PageUp)
	{
		// Go to the previous page of the document document; select text if Shift is down.
		JumpTo(
			ETextLocation::PreviousPage,
			(InKeyEvent.IsShiftDown()) ? ECursorAction::SelectText : ECursorAction::MoveCursor);
		Reply = FReply::Handled();
	}
	else if (Key == EKeys::PageDown)
	{
		// Go to the next page of the document document; select text if Shift is down.
		JumpTo(
			ETextLocation::NextPage,
			(InKeyEvent.IsShiftDown()) ? ECursorAction::SelectText : ECursorAction::MoveCursor);
		Reply = FReply::Handled();
	}
	else if (Key == EKeys::Enter && !OwnerWidget->IsTextReadOnly())
	{
		FScopedEditableTextTransaction TextTransaction(*this);
		HandleCarriageReturn(InKeyEvent.IsRepeat(), InKeyEvent.GetUserIndex());
		Reply = FReply::Handled();
	}
	else if (Key == EKeys::Tab && OwnerWidget->CanTypeCharacter(TEXT('\t')))
	{
		Reply = FReply::Handled();
	}
	else if (Key == EKeys::Escape)
	{
		Reply = BoolToReply(HandleEscape());
	}

	// @Todo: Slate keybindings support more than one set of keys. 
	//Alternate key for cut (Shift+Delete)
	else if (Key == EKeys::Delete && InKeyEvent.IsShiftDown() && CanExecuteCut())
	{
		// Cut text to clipboard
		CutSelectedTextToClipboard();
		Reply = FReply::Handled();
	}
	// This must come after the Cut hotkey or else Cut is unreachable
	else if (Key == EKeys::Delete && !OwnerWidget->IsTextReadOnly())
	{
		// @Todo: Slate keybindings support more than one set of keys. 
		// Delete to next word boundary (Ctrl+Delete), only if there is no Text Selected in that case we carry on with a normal delete.
		if (!AnyTextSelected() && InKeyEvent.IsControlDown() && !InKeyEvent.IsAltDown() && !InKeyEvent.IsShiftDown())
		{
			if (OwnerWidget->IsTextPassword())
			{
				// If the text is sensitive, we should not clue the user in to where word breaks are
				JumpTo(ETextLocation::EndOfLine, ECursorAction::SelectText);
			}
			else
			{
				MoveCursor(FMoveCursor::Cardinal(
					ECursorMoveGranularity::Word,
					// Move right
					FIntPoint(+1, 0),
					// selects text.	
					ECursorAction::SelectText
				));
			}
		}

		FScopedEditableTextTransaction TextTransaction(*this);
		Reply = BoolToReply(HandleDelete());
	}
	// @Todo: Slate keybindings support more than one set of keys. 
	// Alternate key for copy (Ctrl+Insert) 
	else if (Key == EKeys::Insert && InKeyEvent.IsControlDown() && CanExecuteCopy())
	{
		// Copy text to clipboard
		CopySelectedTextToClipboard();
		Reply = FReply::Handled();
	}

	// @Todo: Slate keybindings support more than one set of keys. 
	// Alternate key for paste (Shift+Insert) 
	else if (Key == EKeys::Insert && InKeyEvent.IsShiftDown() && CanExecutePaste())
	{
		// Paste text from clipboard
		PasteTextFromClipboard();
		Reply = FReply::Handled();
	}

	// @Todo: Slate keybindings support more than one set of keys. 
	//Alternate key for undo (Alt+Backspace)
	else if (CanExecuteUndo() && Key == EKeys::BackSpace && InKeyEvent.IsAltDown() && !InKeyEvent.IsShiftDown())
	{
		// Undo
		Undo();
		Reply = FReply::Handled();
	}

	// Ctrl+Y (or Ctrl+Shift+Z, or Alt+Shift+Backspace) to redo
	else if (CanExecuteRedo() && ((Key == EKeys::Y && InKeyEvent.IsControlDown()) ||
		(Key == EKeys::Z && InKeyEvent.IsControlDown() && InKeyEvent.IsShiftDown()) ||
		(Key == EKeys::BackSpace && InKeyEvent.IsAltDown() && InKeyEvent.IsShiftDown())))
	{
		// Redo
		Redo();
		Reply = FReply::Handled();
	}

	// @Todo: Slate keybindings support more than one set of keys. 
	// Delete to previous word boundary (Ctrl+Backspace)
	else if (Key == EKeys::BackSpace && InKeyEvent.IsControlDown() && !InKeyEvent.IsAltDown() && !InKeyEvent.IsShiftDown() && !OwnerWidget->IsTextReadOnly())
	{
		FScopedEditableTextTransaction TextTransaction(*this);

		if (OwnerWidget->IsTextPassword())
		{
			// If the text is sensitive, we should not clue the user in to where word breaks are
			JumpTo(ETextLocation::BeginningOfLine, ECursorAction::SelectText);
		}
		else
		{
			MoveCursor(FMoveCursor::Cardinal(
				ECursorMoveGranularity::Word,
				// Move left
				FIntPoint(-1, 0),
				ECursorAction::SelectText
			));
		}
		Reply = BoolToReply(HandleBackspace());
	}

	// @Todo: Slate keybindings support more than one set of keys. 
	// Begin search (Ctrl+[Shift]+F3)
	else if (Key == EKeys::F3 && InKeyEvent.IsControlDown() && !InKeyEvent.IsAltDown())
	{
		BeginSearch(GetSelectedText(), ESearchCase::IgnoreCase, InKeyEvent.IsShiftDown());
		Reply = FReply::Handled();
	}

	// @Todo: Slate keybindings support more than one set of keys. 
	// Advance search ([Shift]+F3)
	else if (Key == EKeys::F3 && !InKeyEvent.IsControlDown() && !InKeyEvent.IsAltDown())
	{
		AdvanceSearch(InKeyEvent.IsShiftDown());
		Reply = FReply::Handled();
	}

	else if (!InKeyEvent.IsAltDown() && !InKeyEvent.IsControlDown() && InKeyEvent.GetKey() != EKeys::Tab && InKeyEvent.GetCharacter() != 0)
	{
		// Shift and a character was pressed or a single character was pressed.  We will type something in an upcoming OnKeyChar event.  
		// Absorb this event so it is not bubbled and handled by other widgets that could have something bound to the key press.
		Reply = FReply::Handled();
	}

	else if (InKeyEvent.IsRightAltDown() && InKeyEvent.GetCharacter() != 0)
	{
		// Alt Gr button and a single character was pressed. We will type something in an upcoming OnKeyChar event.  
		// Absorb this event so it is not bubbled and handled by other widgets that could have something bound to the key press.
		Reply = FReply::Handled();
	}
	
	if (!Reply.IsEventHandled())
	{
		// Process key-bindings if the event wasn't already handled
		if (UICommandList->ProcessCommandBindings(InKeyEvent))
		{
			Reply = FReply::Handled();
		}
	}

	return Reply;
}

FReply FSlateEditableTextLayout::HandleKeyUp(const FKeyEvent& InKeyEvent)
{
	if (FPlatformApplicationMisc::RequiresVirtualKeyboard() && FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		if (!OwnerWidget->IsTextReadOnly())
		{
			// @TODO: Create ITextInputMethodSystem derivations for mobile
			FSlateApplication::Get().ShowVirtualKeyboard(true, InKeyEvent.GetUserIndex(), VirtualKeyboardEntry);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply FSlateEditableTextLayout::HandleMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = FReply::Unhandled();
	// If the mouse is already captured, then don't allow a new action to be taken
	if (!OwnerWidget->GetSlateWidget()->HasMouseCapture())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton ||
			InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Am I getting focus right now?
			const bool bIsGettingFocus = !OwnerWidget->GetSlateWidget()->HasAnyUserFocus().IsSet();
			if (bIsGettingFocus)
			{
				// We might be receiving keyboard focus due to this event.  Because the keyboard focus received callback
				// won't fire until after this function exits, we need to make sure our widget's state is in order early

				// Assume we'll be given keyboard focus, so load text for editing
				LoadText();

				// Reset 'mouse has moved' state.  We'll use this in OnMouseMove to determine whether we
				// should reset the selection range to the caret's position.
				bWasFocusedByLastMouseDown = true;
			}

			if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				if (InMouseEvent.IsShiftDown())
				{
					MoveCursor(FMoveCursor::ViaScreenPointer(MyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition()), MyGeometry.Scale, ECursorAction::SelectText));
				}
				else
				{
					// Deselect any text that was selected
					ClearSelection();
					MoveCursor(FMoveCursor::ViaScreenPointer(MyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition()), MyGeometry.Scale, ECursorAction::MoveCursor));
				}

				// Start drag selection
				bIsDragSelecting = true;
			}
			else if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
			{
				// If the user right clicked on a character that wasn't already selected, we'll clear
				// the selection
				if (AnyTextSelected() && !IsTextSelectedAt(MyGeometry, InMouseEvent.GetScreenSpacePosition()))
				{
					// Deselect any text that was selected
					ClearSelection();
					MoveCursor(FMoveCursor::ViaScreenPointer(MyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition()), MyGeometry.Scale, ECursorAction::MoveCursor));
				}
			}

			// Right clicking to summon context menu, but we'll do that on mouse-up.
			Reply = FReply::Handled();
			Reply.CaptureMouse(OwnerWidget->GetSlateWidget());
			Reply.SetUserFocus(OwnerWidget->GetSlateWidget(), EFocusCause::Mouse);
		}
	}

	return Reply;
}

FReply FSlateEditableTextLayout::HandleMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = FReply::Unhandled();

	// The mouse must have been captured by either left or right button down before we'll process mouse ups
	if (OwnerWidget->GetSlateWidget()->HasMouseCapture())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton ||
			InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (!bWasFocusedByLastMouseDown)
			{
				// On platforms using a virtual keyboard open the virtual keyboard again 
				if (FPlatformApplicationMisc::RequiresVirtualKeyboard())
				{
					if (!OwnerWidget->IsTextReadOnly())
					{
						if (OwnerWidget->GetVirtualKeyboardTrigger() == EVirtualKeyboardTrigger::OnAllFocusEvents ||
							OwnerWidget->GetVirtualKeyboardTrigger() == EVirtualKeyboardTrigger::OnFocusByPointer)
						{
								FSlateApplication::Get().ShowVirtualKeyboard(true, InMouseEvent.GetUserIndex(), VirtualKeyboardEntry.ToSharedRef());
						}
					}
				}
			}
		}

		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsDragSelecting)
		{
			// No longer drag-selecting
			bIsDragSelecting = false;

			// If we received keyboard focus on this click, then we'll want to select all of the text
			// when the user releases the mouse button, unless the user actually dragged the mouse
			// while holding the button down, in which case they've already selected some text and
			// we'll leave things alone!
			if (bWasFocusedByLastMouseDown)
			{
				if (!bHasDragSelectedSinceFocused || !SelectionStart.IsSet())
				{
					if (OwnerWidget->ShouldSelectAllTextWhenFocused())
					{
						// Move the cursor to the end of the string
						JumpTo(ETextLocation::EndOfDocument, ECursorAction::MoveCursor);

						// User wasn't dragging the mouse, so go ahead and select all of the text now
						// that we've become focused
						SelectAllText();

						// @todo Slate: In this state, the caret should actually stay hidden (until the user interacts again), and we should not move the caret
					}
				}

				bWasFocusedByLastMouseDown = false;
			}

			// Release mouse capture
			Reply = FReply::Handled();
			Reply.ReleaseMouseCapture();
		}
		else if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (MyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()))
			{
				// Right clicked, so summon a context menu if the cursor is within the widget
				FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();

				TSharedPtr<SWidget> MenuContentWidget = OwnerWidget->BuildContextMenuContent();
				if (MenuContentWidget.IsValid())
				{
					ActiveContextMenu.PrepareToSummon();

					static const bool bFocusImmediately = true;
					TSharedPtr<IMenu> ContextMenu = FSlateApplication::Get().PushMenu(
						InMouseEvent.GetWindow(), 
						WidgetPath, 
						MenuContentWidget.ToSharedRef(), 
						InMouseEvent.GetScreenSpacePosition(), 
						FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu), 
						bFocusImmediately
						);

					// Make sure the window is valid. It's possible for the parent to already be in the destroy queue, for example if the editable text was configured to dismiss it's window during OnTextCommitted.
					if (ContextMenu.IsValid())
					{
						ContextMenu->GetOnMenuDismissed().AddRaw(this, &FSlateEditableTextLayout::OnContextMenuClosed);
						ActiveContextMenu.SummonSucceeded(ContextMenu.ToSharedRef());
					}
					else
					{
						ActiveContextMenu.SummonFailed();
					}
				}
			}

			// Release mouse capture
			Reply = FReply::Handled();
			Reply.ReleaseMouseCapture();
		}
	}

	return Reply;
}

FReply FSlateEditableTextLayout::HandleMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (bIsDragSelecting && OwnerWidget->GetSlateWidget()->HasMouseCapture() && InMouseEvent.GetCursorDelta() != FVector2f::ZeroVector)
	{
		MoveCursor(FMoveCursor::ViaScreenPointer(InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition()), InMyGeometry.Scale, ECursorAction::SelectText));
		bHasDragSelectedSinceFocused = true;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FSlateEditableTextLayout::HandleMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (OwnerWidget->ShouldSelectWordOnMouseDoubleClick() && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		SelectWordAt(InMyGeometry, InMouseEvent.GetScreenSpacePosition());
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool FSlateEditableTextLayout::HandleEscape()
{
	if (!SearchText.IsEmpty())
	{
		// Clear search
		SearchText = FText::GetEmpty();
		UpdateCursorHighlight();
		return true;
	}

	if (AnyTextSelected())
	{
		// Clear selection
		ClearSelection();
		UpdateCursorHighlight();
		return true;
	}

	if (!OwnerWidget->IsTextReadOnly())
	{
		// Restore the text if the revert flag is set
		if (OwnerWidget->ShouldRevertTextOnEscape() && HasTextChangedFromOriginal())
		{
			RestoreOriginalText();
			return true;
		}
	}

	return false;
}

bool FSlateEditableTextLayout::HandleBackspace()
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return false;
	}

	if (AnyTextSelected())
	{
		// Delete selected text
		DeleteSelectedText();
	}
	else
	{
		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		FTextLocation FinalCursorLocation = CursorInteractionPosition;

		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

		// If we are at the very beginning of the line...
		if (CursorInteractionPosition.GetOffset() == 0)
		{
			//And if the current line isn't the very first line then...
			if (CursorInteractionPosition.GetLineIndex() > 0)
			{
				const int32 PreviousLineIndex = CursorInteractionPosition.GetLineIndex() - 1;
				const int32 CachePreviousLinesCurrentLength = Lines[PreviousLineIndex].Text->Len();
				if (TextLayout->JoinLineWithNextLine(PreviousLineIndex))
				{
					// Update the cursor so it appears at the end of the previous line,
					// as we're going to delete the imaginary \n separating them
					FinalCursorLocation = FTextLocation(PreviousLineIndex, CachePreviousLinesCurrentLength);
				}
			}
			// else do nothing as the FinalCursorLocation is already correct
		}
		else
		{
			// Delete grapheme to the left of the caret
			const FTextSelection DeleteSelection = TextLayout->GetGraphemeAt(FTextLocation(CursorInteractionPosition, -1));
			const int32 GraphemeSize = DeleteSelection.GetEnd().GetOffset() - DeleteSelection.GetBeginning().GetOffset();
			if (TextLayout->RemoveAt(DeleteSelection.GetBeginning(), GraphemeSize))
			{
				// Adjust caret to the left
				FinalCursorLocation = FTextLocation(CursorInteractionPosition, -GraphemeSize);
			}
		}

		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, FinalCursorLocation);

		ClearSelection();
		UpdateCursorHighlight();
	}

	return true;
}

bool FSlateEditableTextLayout::HandleDelete()
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return false;
	}

	if (AnyTextSelected())
	{
		// Delete selected text
		DeleteSelectedText();
	}
	else
	{
		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		FTextLocation FinalCursorLocation = CursorInteractionPosition;

		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
		const FTextLayout::FLineModel& Line = Lines[CursorInteractionPosition.GetLineIndex()];

		//If we are at the very beginning of the line...
		if (Line.Text->Len() == 0)
		{
			// And if the current line isn't the very last line then...
			if (Lines.IsValidIndex(CursorInteractionPosition.GetLineIndex() + 1))
			{
				TextLayout->RemoveLine(CursorInteractionPosition.GetLineIndex());
			}
			// else do nothing as the FinalCursorLocation is already correct
		}
		else if (CursorInteractionPosition.GetOffset() >= Line.Text->Len())
		{
			// And if the current line isn't the very last line then...
			if (Lines.IsValidIndex(CursorInteractionPosition.GetLineIndex() + 1))
			{
				if (TextLayout->JoinLineWithNextLine(CursorInteractionPosition.GetLineIndex()))
				{
					//else do nothing as the FinalCursorLocation is already correct
				}
			}
			// else do nothing as the FinalCursorLocation is already correct
		}
		else
		{
			// Delete grapheme to the right of the caret
			const FTextSelection DeleteSelection = TextLayout->GetGraphemeAt(CursorInteractionPosition);
			const int32 GraphemeSize = DeleteSelection.GetEnd().GetOffset() - DeleteSelection.GetBeginning().GetOffset();
			TextLayout->RemoveAt(DeleteSelection.GetBeginning(), GraphemeSize);
			// do nothing to the cursor as the FinalCursorLocation is already correct
		}

		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, FinalCursorLocation);

		ClearSelection();
		UpdateCursorHighlight();
	}

	return true;
}

bool FSlateEditableTextLayout::HandleTypeChar(const TCHAR InChar)
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return false;
	}

	// Certain characters are not allowed
	const bool bIsCharAllowed = IsCharAllowed(InChar);
	if (bIsCharAllowed)
	{
		if (AnyTextSelected())
		{
			// Delete selected text only if an allowed char is received
			DeleteSelectedText();
		}
		
		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
		const FTextLayout::FLineModel& Line = Lines[CursorInteractionPosition.GetLineIndex()];

		// Insert character at caret position
		TextLayout->InsertAt(CursorInteractionPosition, InChar);

		// Advance caret position
		ClearSelection();
		const FTextLocation FinalCursorLocation = FTextLocation(CursorInteractionPosition.GetLineIndex(), FMath::Min(CursorInteractionPosition.GetOffset() + 1, Line.Text->Len()));

		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, FinalCursorLocation);
		UpdateCursorHighlight();

		return true;
	}

	return false;
}

bool FSlateEditableTextLayout::HandleCarriageReturn(bool isRepeat, int32 UserIndex)
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return false;
	}

	if (OwnerWidget->IsMultiLineTextEdit() && OwnerWidget->CanInsertCarriageReturn())
	{
		InsertNewLineAtCursorImpl();
	}
	else if (isRepeat)
	{
		// Skip committing text on a repeat enter key
		return false;
	}
	else
	{
		// Always clear the local undo chain on commit.
		ClearUndoStates();

		// Make sure to update the text if there is any VK change that was not processed yet
		UpdateTextChangedByVirtualKeyboard();

		const FText EditedText = GetEditableText();

		// When enter is pressed text is committed.  Let anyone interested know about it.
		OwnerWidget->OnTextCommitted(EditedText, ETextCommit::OnEnter);

		// Reload underlying value now it is committed  (commit may alter the value) 
		// so it can be re-displayed in the edit box if it retains focus
		LoadText();

		// Select all text?
		if (OwnerWidget->ShouldSelectAllTextOnCommit())
		{
			SelectAllText();
		}

		// Release input focus?
		if (OwnerWidget->ShouldClearKeyboardFocusOnCommit())
		{
			TSharedPtr<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidgetPtr();
			if (OwnerSlateWidget && OwnerSlateWidget->HasUserFocus(UserIndex))
			{
				FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
			}
		}
	}

	return true;
}

bool FSlateEditableTextLayout::CanExecuteDelete() const
{
	bool bCanExecute = true;

	// Can't execute if this is a read-only control
	if (OwnerWidget->IsTextReadOnly())
	{
		bCanExecute = false;
	}

	// Can't execute unless there is some text selected
	if (!AnyTextSelected())
	{
		bCanExecute = false;
	}

	return bCanExecute;
}

void FSlateEditableTextLayout::DeleteSelectedText()
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return;
	}

	if (AnyTextSelected())
	{
		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
		FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		int32 SelectionBeginningLineIndex = Selection.GetBeginning().GetLineIndex();
		int32 SelectionBeginningLineOffset = Selection.GetBeginning().GetOffset();

		int32 SelectionEndLineIndex = Selection.GetEnd().GetLineIndex();
		int32 SelectionEndLineOffset = Selection.GetEnd().GetOffset();

		if (SelectionBeginningLineIndex == SelectionEndLineIndex)
		{
			TextLayout->RemoveAt(FTextLocation(SelectionBeginningLineIndex, SelectionBeginningLineOffset), SelectionEndLineOffset - SelectionBeginningLineOffset);
			// Do nothing to the cursor as it is already in the correct location
		}
		else
		{
			const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
			const FTextLayout::FLineModel& EndLine = Lines[SelectionEndLineIndex];

			if (EndLine.Text->Len() == SelectionEndLineOffset)
			{
				TextLayout->RemoveLine(SelectionEndLineIndex);
			}
			else
			{
				TextLayout->RemoveAt(FTextLocation(SelectionEndLineIndex, 0), SelectionEndLineOffset);
			}

			for (int32 LineIndex = SelectionEndLineIndex - 1; LineIndex > SelectionBeginningLineIndex; LineIndex--)
			{
				TextLayout->RemoveLine(LineIndex);
			}

			const FTextLayout::FLineModel& BeginLine = Lines[SelectionBeginningLineIndex];
			TextLayout->RemoveAt(FTextLocation(SelectionBeginningLineIndex, SelectionBeginningLineOffset), BeginLine.Text->Len() - SelectionBeginningLineOffset);

			TextLayout->JoinLineWithNextLine(SelectionBeginningLineIndex);

			if (Lines.Num() == 0)
			{
				TSharedRef< FString > EmptyText = MakeShared<FString>();
				TArray< TSharedRef< IRun > > Runs;
				Runs.Add(CreateTextOrPasswordRun(FRunInfo(), EmptyText, TextStyle));

				TextLayout->AddLine(FTextLayout::FNewLineData(MoveTemp(EmptyText), MoveTemp(Runs)));
			}
		}

		// Clear selection
		ClearSelection();
		const FTextLocation FinalCursorLocation = FTextLocation(SelectionBeginningLineIndex, SelectionBeginningLineOffset);
		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, FinalCursorLocation);
		UpdateCursorHighlight();
	}
}

bool FSlateEditableTextLayout::AnyTextSelected() const
{
	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	const FTextLocation SelectionPosition = SelectionStart.Get(CursorInteractionPosition);
	return SelectionPosition != CursorInteractionPosition;
}

bool FSlateEditableTextLayout::IsTextSelectedAt(const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& ScreenSpacePosition) const
{
	const FVector2f LocalPosition = MyGeometry.AbsoluteToLocal(ScreenSpacePosition);
	return IsTextSelectedAt(LocalPosition * MyGeometry.Scale);
}

bool FSlateEditableTextLayout::IsTextSelectedAt(const UE::Slate::FDeprecateVector2DParameter& InLocalPosition) const
{
	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	const FTextLocation SelectionPosition = SelectionStart.Get(CursorInteractionPosition);

	if (SelectionPosition == CursorInteractionPosition)
	{
		return false;
	}

	const FTextLocation ClickedPosition = TextLayout->GetTextLocationAt(FVector2D(InLocalPosition));

	FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
	FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

	int32 SelectionBeginningLineIndex = Selection.GetBeginning().GetLineIndex();
	int32 SelectionBeginningLineOffset = Selection.GetBeginning().GetOffset();

	int32 SelectionEndLineIndex = Selection.GetEnd().GetLineIndex();
	int32 SelectionEndLineOffset = Selection.GetEnd().GetOffset();

	if (SelectionBeginningLineIndex == SelectionEndLineIndex)
	{
		return ClickedPosition.GetLineIndex() == SelectionBeginningLineIndex
			&& SelectionBeginningLineOffset <= ClickedPosition.GetOffset()
			&& SelectionEndLineOffset >= ClickedPosition.GetOffset();
	}

	if (SelectionBeginningLineIndex == ClickedPosition.GetLineIndex())
	{
		return SelectionBeginningLineOffset <= ClickedPosition.GetOffset();
	}

	if (SelectionEndLineIndex == ClickedPosition.GetLineIndex())
	{
		return SelectionEndLineOffset >= ClickedPosition.GetOffset();
	}

	return SelectionBeginningLineIndex < ClickedPosition.GetLineIndex()
		&& SelectionEndLineIndex > ClickedPosition.GetLineIndex();
}

bool FSlateEditableTextLayout::CanExecuteSelectAll() const
{
	return true;
}

void FSlateEditableTextLayout::SelectAllText()
{
	if (TextLayout->IsEmpty())
	{
		return;
	}

	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	const int32 NumberOfLines = Lines.Num();

	SelectionStart = FTextLocation(0, 0);
	const FTextLocation NewCursorPosition = FTextLocation(NumberOfLines - 1, Lines[NumberOfLines - 1].Text->Len());
	CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
	UpdateCursorHighlight();
	OwnerWidget->GetSlateWidget()->Invalidate(EInvalidateWidget::Layout);
}

void FSlateEditableTextLayout::SelectWordAt(const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& ScreenSpacePosition)
{
	const FVector2f LocalPosition = MyGeometry.AbsoluteToLocal(ScreenSpacePosition);
	SelectWordAt(LocalPosition * MyGeometry.Scale);
}

void FSlateEditableTextLayout::SelectWordAt(const UE::Slate::FDeprecateVector2DParameter& InLocalPosition)
{
	FTextLocation InitialLocation = TextLayout->GetTextLocationAt(FVector2d(InLocalPosition));
	FTextSelection WordSelection = TextLayout->GetWordAt(InitialLocation);

	FTextLocation WordStart = WordSelection.GetBeginning();
	FTextLocation WordEnd = WordSelection.GetEnd();

	if (WordStart.IsValid() && WordEnd.IsValid())
	{
		// Deselect any text that was selected
		ClearSelection();

		if (WordStart != WordEnd)
		{
			SelectionStart = WordStart;
		}

		const FTextLocation NewCursorPosition = WordEnd;
		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
		UpdateCursorHighlight();
	}
}

void FSlateEditableTextLayout::SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation)
{
	if (TextLayout->IsEmpty())
	{
		return;
	}

	const FTextLocation NewCursorPosition = InCursorLocation;
	CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);

	if (InSelectionStart != InCursorLocation)
	{
		SelectionStart = InSelectionStart;
	}
	else
	{
		SelectionStart.Reset();
	}
	
	UpdateCursorHighlight();	
}

void FSlateEditableTextLayout::ClearSelection()
{
	SelectionStart = TOptional<FTextLocation>();
}

bool FSlateEditableTextLayout::CanExecuteCut() const
{
	bool bCanExecute = true;

	// Can't execute if this is a read-only control
	if (OwnerWidget->IsTextReadOnly())
	{
		bCanExecute = false;
	}

	// Can't execute if this control contains a password
	if (OwnerWidget->IsTextPassword())
	{
		bCanExecute = false;
	}

	// Can't execute if there is no text selected
	if (!AnyTextSelected())
	{
		bCanExecute = false;
	}

	return bCanExecute;
}

void FSlateEditableTextLayout::CutSelectedTextToClipboard()
{
	if (OwnerWidget->IsTextReadOnly() || OwnerWidget->IsTextPassword())
	{
		return;
	}

	if (AnyTextSelected())
	{
		FScopedEditableTextTransaction TextTransaction(*this);

		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
		FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		// Grab the selected substring
		FString SelectedText;
		TextLayout->GetSelectionAsText(SelectedText, Selection);

		// Copy text to clipboard
		FPlatformApplicationMisc::ClipboardCopy(*SelectedText);

		DeleteSelectedText();

		UpdateCursorHighlight();
	}
}

bool FSlateEditableTextLayout::CanExecuteCopy() const
{
	bool bCanExecute = true;

	// Can't execute if this control contains a password
	if (OwnerWidget->IsTextPassword())
	{
		bCanExecute = false;
	}

	// Can't execute if there is no text selected
	if (!AnyTextSelected())
	{
		bCanExecute = false;
	}

	return bCanExecute;
}

void FSlateEditableTextLayout::CopySelectedTextToClipboard()
{
	if (OwnerWidget->IsTextPassword())
	{
		return;
	}

	if (AnyTextSelected())
	{
		const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
		FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
		FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		// Grab the selected substring
		FString SelectedText;
		TextLayout->GetSelectionAsText(SelectedText, Selection);

		// Copy text to clipboard
		FPlatformApplicationMisc::ClipboardCopy(*SelectedText);
	}
}

bool FSlateEditableTextLayout::CanExecutePaste() const
{
	bool bCanExecute = true;

	// Can't execute if this is a read-only control
	if (OwnerWidget->IsTextReadOnly())
	{
		bCanExecute = false;
	}

	// Can't paste unless the clipboard has a string in it
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (ClipboardContent.IsEmpty())
	{
		bCanExecute = false;
	}

	return bCanExecute;
}

void FSlateEditableTextLayout::PasteTextFromClipboard()
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return;
	}

	FScopedEditableTextTransaction TextTransaction(*this);

	DeleteSelectedText();

	// Paste from the clipboard
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	if (PastedText.Len() > 0)
	{
		InsertTextAtCursorImpl(PastedText);
		TextLayout->UpdateIfNeeded();
	}
}

void FSlateEditableTextLayout::InsertTextAtCursor(const FString& InString)
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return;
	}

	FScopedEditableTextTransaction TextTransaction(*this);

	DeleteSelectedText();

	if (InString.Len() > 0)
	{
		InsertTextAtCursorImpl(InString);
		TextLayout->UpdateIfNeeded();
	}
}

void FSlateEditableTextLayout::InsertTextAtCursorImpl(const FString& InString)
{
	if (OwnerWidget->IsTextReadOnly() || InString.Len() == 0)
	{
		return;
	}

	// Sanitize out any invalid characters
	FString SanitizedString = InString;
	{
		const bool bIsMultiLine = OwnerWidget->IsMultiLineTextEdit();
		SanitizedString.GetCharArray().RemoveAll([&](const TCHAR InChar) -> bool
		{
			if (InChar != 0)
			{
				const bool bIsCharAllowed = IsCharAllowed(InChar) || (bIsMultiLine && FChar::IsLinebreak(InChar));
				return !bIsCharAllowed;
			}
			return false;
		});
	}

	// Split into lines
	TArray<FTextRange> LineRanges;
	FTextRange::CalculateLineRangesFromString(SanitizedString, LineRanges);

	if (AnyTextSelected())
	{
		// Delete selected text
		DeleteSelectedText();
	}

	// Insert each line
	{
		bool bIsFirstLine = true;
		for (const FTextRange& LineRange : LineRanges)
		{
			if (!bIsFirstLine)
			{
				const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
				if (TextLayout->SplitLineAt(CursorInteractionPosition))
				{
					// Adjust the cursor position to be at the beginning of the new line
					const FTextLocation NewCursorPosition = FTextLocation(CursorInteractionPosition.GetLineIndex() + 1, 0);
					CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
				}
			}
			bIsFirstLine = false;

			const FString NewLineText = FString(SanitizedString.Mid(LineRange.BeginIndex, LineRange.Len()));

			const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
			const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
			const FTextLayout::FLineModel& Line = Lines[CursorInteractionPosition.GetLineIndex()];

			// Insert character at caret position
			TextLayout->InsertAt(CursorInteractionPosition, NewLineText);

			// Advance caret position
			ClearSelection();
			const FTextLocation NewCursorPosition = FTextLocation(CursorInteractionPosition.GetLineIndex(), FMath::Min(CursorInteractionPosition.GetOffset() + NewLineText.Len(), Line.Text->Len()));
			CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
		}

		UpdateCursorHighlight();
	}
}

void FSlateEditableTextLayout::InsertNewLineAtCursorImpl()
{
	check(OwnerWidget->IsMultiLineTextEdit());

	if (AnyTextSelected())
	{
		// Delete selected text
		DeleteSelectedText();
	}

	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	if (TextLayout->SplitLineAt(CursorInteractionPosition))
	{
		// Adjust the cursor position to be at the beginning of the new line
		const FTextLocation NewCursorPosition = FTextLocation(CursorInteractionPosition.GetLineIndex() + 1, 0);
		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
	}

	ClearSelection();
	UpdateCursorHighlight();
}

TSharedRef<IRun> FSlateEditableTextLayout::CreateTextOrPasswordRun(const FRunInfo& InRunInfo, const TSharedRef<const FString>& InText, const FTextBlockStyle& InStyle)
{
	if (OwnerWidget->IsTextPassword())
	{
		return FSlatePasswordRun::Create(InRunInfo, InText, InStyle);
	}
	return FSlateTextRun::Create(InRunInfo, InText, InStyle);
}

void FSlateEditableTextLayout::OnContextMenuClosed(TSharedRef<IMenu> Menu)
{
	// Note: We don't reset the ActiveContextMenu here, as Slate hasn't yet finished processing window focus events, and we need 
	// to know that the window is still available for OnFocusReceived and OnFocusLost even though it's about to be destroyed

	// Give our owner widget focus when the context menu has been dismissed
	TSharedPtr<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidgetPtr();
	if (OwnerSlateWidget.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(OwnerSlateWidget, EFocusCause::OtherWidgetLostFocus);
	}
}

void FSlateEditableTextLayout::InsertRunAtCursor(TSharedRef<IRun> InRun)
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return;
	}

	FScopedEditableTextTransaction TextTransaction(*this);

	DeleteSelectedText();

	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	TextLayout->InsertAt(CursorInteractionPosition, InRun, true); // true to preserve the run after the insertion point, even if it's empty - this preserves the text formatting correctly

	// Move the cursor along since we've inserted some new text
	FString InRunText;
	InRun->AppendTextTo(InRunText);

	const TArray<FTextLayout::FLineModel>& Lines = TextLayout->GetLineModels();
	const FTextLayout::FLineModel& Line = Lines[CursorInteractionPosition.GetLineIndex()];
	const FTextLocation FinalCursorLocation = FTextLocation(CursorInteractionPosition.GetLineIndex(), FMath::Min(CursorInteractionPosition.GetOffset() + InRunText.Len(), Line.Text->Len()));

	CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, FinalCursorLocation);
	UpdateCursorHighlight();
}

bool FSlateEditableTextLayout::MoveCursor(const FMoveCursor& InArgs)
{
	// We can't use the keyboard to move the cursor while composing, as the IME has control over it
	if (!FSlateApplication::Get().AllowMoveCursor() ||
		(TextInputMethodContext->IsComposing() && InArgs.GetMoveMethod() != ECursorMoveMethod::ScreenPosition))
	{
		// Claim we handled this
		return true;
	}

	bool bAllowMoveCursor = true;
	FTextLocation NewCursorPosition;
	FTextLocation CursorPosition = CursorInfo.GetCursorInteractionLocation();

	// If we have selected text, the cursor needs to:
	// a) Jump to the start of the selection if moving the cursor Left or Up
	// b) Jump to the end of the selection if moving the cursor Right or Down
	// This is done regardless of whether the selection was made left-to-right, or right-to-left
	// This also needs to be done *before* moving to word boundaries, or moving vertically, as the 
	// start point needs to be the start or end of the selection depending on the above rules
	if (InArgs.GetAction() == ECursorAction::MoveCursor && InArgs.GetMoveMethod() != ECursorMoveMethod::ScreenPosition && AnyTextSelected())
	{
		if (InArgs.IsHorizontalMovement())
		{
			// If we're moving the cursor horizontally, we just snap to the start or end of the selection rather than 
			// move the cursor by the normal movement rules
			bAllowMoveCursor = false;
		}

		// Work out which edge of the selection we need to start at
		bool bSnapToSelectionStart =
			InArgs.GetMoveMethod() == ECursorMoveMethod::Cardinal &&
			(InArgs.GetMoveDirection().X < 0.0f || InArgs.GetMoveDirection().Y < 0.0f);


		// Adjust the current cursor position - also set the new cursor position so that the bAllowMoveCursor == false case is handled
		const FTextSelection Selection(SelectionStart.GetValue(), CursorPosition);
		CursorPosition = bSnapToSelectionStart ? Selection.GetBeginning() : Selection.GetEnd();
		NewCursorPosition = CursorPosition;

		// If we're snapping to a word boundary, but the selection was already at a word boundary, don't let the cursor move any more
		if (InArgs.GetGranularity() == ECursorMoveGranularity::Word && IsAtWordStart(NewCursorPosition))
		{
			bAllowMoveCursor = false;
		}
	}

	TOptional<SlateEditableTextTypes::ECursorAlignment> NewCursorAlignment;
	bool bUpdatePreferredCursorScreenOffsetInLine = false;
	if (bAllowMoveCursor)
	{
		if (InArgs.GetMoveMethod() == ECursorMoveMethod::Cardinal)
		{
			if (InArgs.GetGranularity() == ECursorMoveGranularity::Character)
			{
				if (InArgs.IsHorizontalMovement())
				{
					NewCursorPosition = TranslatedLocation(CursorPosition, InArgs.GetMoveDirection().X);
					bUpdatePreferredCursorScreenOffsetInLine = true;
				}
				else if (OwnerWidget->IsMultiLineTextEdit())
				{
					TranslateLocationVertical(CursorPosition, InArgs.GetMoveDirection().Y, InArgs.GetGeometryScale(), NewCursorPosition, NewCursorAlignment);
				}
				else
				{
					// Vertical movement not supported on single-line editable text controls - return false so we fallback to generic widget navigation
					return false;
				}
			}
			else
			{
				checkSlow(InArgs.IsHorizontalMovement());
				checkSlow(InArgs.GetGranularity() == ECursorMoveGranularity::Word);
				checkSlow(InArgs.GetMoveDirection().X != 0);
				NewCursorPosition = ScanForWordBoundary(CursorPosition, InArgs.GetMoveDirection().X);
				bUpdatePreferredCursorScreenOffsetInLine = true;
			}
		}
		else if (InArgs.GetMoveMethod() == ECursorMoveMethod::ScreenPosition)
		{
			ETextHitPoint HitPoint = ETextHitPoint::WithinText;
			NewCursorPosition = TextLayout->GetTextLocationAt(InArgs.GetLocalPosition() * InArgs.GetGeometryScale(), &HitPoint);
			bUpdatePreferredCursorScreenOffsetInLine = true;

			// Moving with the mouse behaves a bit differently to moving with the keyboard, as clicking at the end of a wrapped line needs to place the cursor there
			// rather than at the start of the next line (which is tricky since they have the same index according to GetTextLocationAt!).
			// We use the HitPoint to work this out and then adjust the cursor position accordingly
			if (HitPoint == ETextHitPoint::RightGutter)
			{
				NewCursorPosition = FTextLocation(NewCursorPosition, -1);
				NewCursorAlignment = SlateEditableTextTypes::ECursorAlignment::Right;
			}
		}
		else
		{
			checkfSlow(false, TEXT("Unknown ECursorMoveMethod value"));
		}
	}

	if (InArgs.GetAction() == ECursorAction::SelectText)
	{
		// We are selecting text. Just remember where the selection started.
		// The cursor is implicitly the other endpoint.
		if (!SelectionStart.IsSet())
		{
			SelectionStart = CursorPosition;
		}
	}
	else
	{
		// No longer selection text; clear the selection!
		ClearSelection();
	}

	if (NewCursorAlignment.IsSet())
	{
		CursorInfo.SetCursorLocationAndAlignment(*TextLayout, NewCursorPosition, NewCursorAlignment.GetValue());
	}
	else
	{
		CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
	}

	OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());

	if (bUpdatePreferredCursorScreenOffsetInLine)
	{
		UpdatePreferredCursorScreenOffsetInLine();
	}

	UpdateCursorHighlight();

	// If we've moved the cursor while composing, we need to end the current composition session
	// Note: You should only be able to do this via the mouse due to the check at the top of this function
	if (TextInputMethodContext->IsComposing())
	{
		ITextInputMethodSystem* const TextInputMethodSystem = FSlateApplication::Get().GetTextInputMethodSystem();
		if (TextInputMethodSystem && bHasRegisteredTextInputMethodContext)
		{
			TextInputMethodSystem->DeactivateContext(TextInputMethodContext.ToSharedRef());
			TextInputMethodSystem->ActivateContext(TextInputMethodContext.ToSharedRef());
		}
	}

	return true;
}

void FSlateEditableTextLayout::GoTo(const FTextLocation& NewLocation)
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	if (Lines.IsValidIndex(NewLocation.GetLineIndex()))
	{
		const FTextLayout::FLineModel& Line = Lines[NewLocation.GetLineIndex()];
		if (NewLocation.GetOffset() <= Line.Text->Len())
		{
			ClearSelection();

			CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewLocation);
			OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
			UpdatePreferredCursorScreenOffsetInLine();
			UpdateCursorHighlight();
		}
	}
}

void FSlateEditableTextLayout::GoTo(ETextLocation NewLocation)
{
	JumpTo(NewLocation, ECursorAction::MoveCursor);
}

void FSlateEditableTextLayout::JumpTo(ETextLocation JumpLocation, ECursorAction Action)
{
	// Utility function to count the number of fully visible lines (vertically)
	// We consider this to be the number of lines on the current page
	auto CountVisibleLines = [](const TArray<FTextLayout::FLineView>& LineViews, const float VisibleHeight) -> int32
	{
		int32 LinesInView = 0;
		for (const auto& LineView : LineViews)
		{
			// The line view is scrolled such that lines above the top of the text area have negative offsets
			if (LineView.Offset.Y >= 0.0f)
			{
				const float EndOffsetY = LineView.Offset.Y + LineView.Size.Y;
				if (EndOffsetY <= VisibleHeight)
				{
					// Line is completely in view
					++LinesInView;
				}
				else
				{
					// Line extends beyond the bottom of the text area - we've finished finding visible lines
					break;
				}
			}
		}
		return LinesInView;
	};

	switch (JumpLocation)
	{
	case ETextLocation::BeginningOfLine:
		{
			const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
			const TArray< FTextLayout::FLineView >& LineViews = TextLayout->GetLineViews();
			const int32 CurrentLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, CursorInteractionPosition, CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);

			if (LineViews.IsValidIndex(CurrentLineViewIndex))
			{
				const FTextLayout::FLineView& CurrentLineView = LineViews[CurrentLineViewIndex];

				const FTextLocation OldCursorPosition = CursorInteractionPosition;
				const FTextLocation NewCursorPosition = FTextLocation(OldCursorPosition.GetLineIndex(), CurrentLineView.Range.BeginIndex);

				if (Action == ECursorAction::SelectText)
				{
					if (!SelectionStart.IsSet())
					{
						this->SelectionStart = OldCursorPosition;
					}
				}
				else
				{
					ClearSelection();
				}

				CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
				UpdatePreferredCursorScreenOffsetInLine();
				UpdateCursorHighlight();
			}
		}
		break;

	case ETextLocation::BeginningOfDocument:
		{
			const FTextLocation OldCursorPosition = CursorInfo.GetCursorInteractionLocation();
			const FTextLocation NewCursorPosition = FTextLocation(0, 0);

			if (Action == ECursorAction::SelectText)
			{
				if (!SelectionStart.IsSet())
				{
					this->SelectionStart = OldCursorPosition;
				}
			}
			else
			{
				ClearSelection();
			}

			CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
			OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
			UpdatePreferredCursorScreenOffsetInLine();
			UpdateCursorHighlight();
		}
		break;

	case ETextLocation::EndOfLine:
		{
			const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
			const TArray< FTextLayout::FLineView >& LineViews = TextLayout->GetLineViews();
			const int32 CurrentLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, CursorInteractionPosition, CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);

			if (LineViews.IsValidIndex(CurrentLineViewIndex))
			{
				const FTextLayout::FLineView& CurrentLineView = LineViews[CurrentLineViewIndex];

				const FTextLocation OldCursorPosition = CursorInteractionPosition;
				const FTextLocation NewCursorPosition = FTextLocation(OldCursorPosition.GetLineIndex(), FMath::Max(0, CurrentLineView.Range.EndIndex - 1));

				if (Action == ECursorAction::SelectText)
				{
					if (!SelectionStart.IsSet())
					{
						this->SelectionStart = OldCursorPosition;
					}
				}
				else
				{
					ClearSelection();
				}

				CursorInfo.SetCursorLocationAndAlignment(*TextLayout, NewCursorPosition, SlateEditableTextTypes::ECursorAlignment::Right);
				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
				UpdatePreferredCursorScreenOffsetInLine();
				UpdateCursorHighlight();
			}
		}
		break;

	case ETextLocation::EndOfDocument:
		{
			if (!TextLayout->IsEmpty())
			{
				const FTextLocation OldCursorPosition = CursorInfo.GetCursorInteractionLocation();
				const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

				const int32 LastLineIndex = Lines.Num() - 1;
				const FTextLocation NewCursorPosition = FTextLocation(LastLineIndex, Lines[LastLineIndex].Text->Len());

				if (Action == ECursorAction::SelectText)
				{
					if (!SelectionStart.IsSet())
					{
						this->SelectionStart = OldCursorPosition;
					}
				}
				else
				{
					ClearSelection();
				}

				CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
				UpdatePreferredCursorScreenOffsetInLine();
				UpdateCursorHighlight();
			}
		}
		break;

	case ETextLocation::PreviousPage:
		{
			const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
			const TArray< FTextLayout::FLineView >& LineViews = TextLayout->GetLineViews();
			const int32 CurrentLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, CursorInteractionPosition, CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);

			if (LineViews.IsValidIndex(CurrentLineViewIndex))
			{
				const FTextLayout::FLineView& CurrentLineView = LineViews[CurrentLineViewIndex];

				const FTextLocation OldCursorPosition = CursorInteractionPosition;

				FTextLocation NewCursorPosition;
				TOptional<SlateEditableTextTypes::ECursorAlignment> NewCursorAlignment;
				const int32 NumLinesToMove = FMath::Max(1, CountVisibleLines(LineViews, CachedSize.Y));
				TranslateLocationVertical(OldCursorPosition, -NumLinesToMove, TextLayout->GetScale(), NewCursorPosition, NewCursorAlignment);

				if (Action == ECursorAction::SelectText)
				{
					if (!SelectionStart.IsSet())
					{
						this->SelectionStart = OldCursorPosition;
					}
				}
				else
				{
					ClearSelection();
				}

				if (NewCursorAlignment.IsSet())
				{
					CursorInfo.SetCursorLocationAndAlignment(*TextLayout, NewCursorPosition, NewCursorAlignment.GetValue());
				}
				else
				{
					CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
				}
				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
				UpdatePreferredCursorScreenOffsetInLine();
				UpdateCursorHighlight();

				// We need to scroll by the delta vertical offset value of the old line and the new line
				// This will (try to) keep the cursor in the same relative location after the page jump
				const int32 NewLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, CursorInfo.GetCursorInteractionLocation(), CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);
				if (LineViews.IsValidIndex(NewLineViewIndex))
				{
					const FTextLayout::FLineView& NewLineView = LineViews[NewLineViewIndex];
					const float DeltaScrollY = (NewLineView.Offset.Y - CurrentLineView.Offset.Y) / TextLayout->GetScale();
					ScrollOffset.Y = FMath::Max(0.0f, ScrollOffset.Y + DeltaScrollY);

					// Disable the normal cursor scrolling that UpdateCursorHighlight triggers
					PositionToScrollIntoView.Reset();
				}
			}
		}
		break;

	case ETextLocation::NextPage:
		{
			const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
			const TArray< FTextLayout::FLineView >& LineViews = TextLayout->GetLineViews();
			const int32 CurrentLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, CursorInteractionPosition, CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);

			if (LineViews.IsValidIndex(CurrentLineViewIndex))
			{
				const FTextLayout::FLineView& CurrentLineView = LineViews[CurrentLineViewIndex];

				const FTextLocation OldCursorPosition = CursorInteractionPosition;

				FTextLocation NewCursorPosition;
				TOptional<SlateEditableTextTypes::ECursorAlignment> NewCursorAlignment;
				const int32 NumLinesToMove = FMath::Max(1, CountVisibleLines(LineViews, CachedSize.Y));
				TranslateLocationVertical(OldCursorPosition, NumLinesToMove, TextLayout->GetScale(), NewCursorPosition, NewCursorAlignment);

				if (Action == ECursorAction::SelectText)
				{
					if (!SelectionStart.IsSet())
					{
						this->SelectionStart = OldCursorPosition;
					}
				}
				else
				{
					ClearSelection();
				}

				if (NewCursorAlignment.IsSet())
				{
					CursorInfo.SetCursorLocationAndAlignment(*TextLayout, NewCursorPosition, NewCursorAlignment.GetValue());
				}
				else
				{
					CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, NewCursorPosition);
				}
				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());
				UpdatePreferredCursorScreenOffsetInLine();
				UpdateCursorHighlight();

				// We need to scroll by the delta vertical offset value of the old line and the new line
				// This will (try to) keep the cursor in the same relative location after the page jump
				const int32 NewLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, CursorInfo.GetCursorInteractionLocation(), CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);
				if (LineViews.IsValidIndex(NewLineViewIndex))
				{
					const FTextLayout::FLineView& NewLineView = LineViews[NewLineViewIndex];
					const float DeltaScrollY = (NewLineView.Offset.Y - CurrentLineView.Offset.Y) / TextLayout->GetScale();
					ScrollOffset.Y = FMath::Min(TextLayout->GetSize().Y - CachedSize.Y, ScrollOffset.Y + DeltaScrollY);

					// Disable the normal cursor scrolling that UpdateCursorHighlight triggers
					PositionToScrollIntoView.Reset();
				}
			}
		}
		break;
	}
}

void FSlateEditableTextLayout::ScrollTo(const FTextLocation& NewLocation)
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	if (Lines.IsValidIndex(NewLocation.GetLineIndex()))
	{
		const FTextLayout::FLineModel& Line = Lines[NewLocation.GetLineIndex()];
		if (NewLocation.GetOffset() <= Line.Text->Len())
		{
			PositionToScrollIntoView = SlateEditableTextTypes::FScrollInfo(NewLocation, SlateEditableTextTypes::ECursorAlignment::Left);
			OwnerWidget->EnsureActiveTick();
		}
	}
}

void FSlateEditableTextLayout::ScrollTo(const ETextLocation NewLocation)
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

	FTextLocation ResolvedTextLocation;
	switch (NewLocation)
	{
	case ETextLocation::BeginningOfDocument:
		ResolvedTextLocation = FTextLocation(0);
		break;

	case ETextLocation::EndOfDocument:
		ResolvedTextLocation = FTextLocation(FMath::Max(0, Lines.Num() - 1));
		break;

	default:
		checkf(false, TEXT("Unsupported ETextLocation mode passed to ScrollTo!"));
		break;
	}

	ScrollTo(ResolvedTextLocation);
}

void FSlateEditableTextLayout::UpdateCursorHighlight()
{
	PositionToScrollIntoView = SlateEditableTextTypes::FScrollInfo(CursorInfo.GetCursorInteractionLocation(), CursorInfo.GetCursorAlignment());
	OwnerWidget->EnsureActiveTick();

	RemoveCursorHighlight();

	static const int32 SelectionHighlightZOrder = -10; // draw below the text
	static const int32 SearchHighlightZOrder = -9; // draw above the base highlight as this is partially transparent
	static const int32 CompositionRangeZOrder = 10; // draw above the text
	static const int32 CursorZOrder = 11; // draw above the text and the composition

	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	const FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);

	const bool bHasKeyboardFocus = OwnerWidget->GetSlateWidget()->HasAnyUserFocus().IsSet();
	const bool bIsComposing = TextInputMethodContext->IsComposing();
	const bool bHasSelection = SelectionLocation != CursorInteractionPosition;
	const bool bHasSearch = !SearchText.IsEmpty();

	if (bHasSearch)
	{
		const FString& SearchTextString = SearchText.ToString();
		const int32 SearchTextLength = SearchTextString.Len();

		int32 SearchResultIndex = 0;
		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
		for (int32 LineIndex = 0; LineIndex < Lines.Num(); ++LineIndex)
		{
			const FTextLayout::FLineModel& Line = Lines[LineIndex];

			int32 FindBegin = 0;
			int32 CurrentSearchBegin = 0;
			const int32 TextLength = Line.Text->Len();
			while (FindBegin < TextLength && (CurrentSearchBegin = Line.Text->Find(SearchTextString, SearchCase, ESearchDir::FromStart, FindBegin)) != INDEX_NONE)
			{
				FindBegin = CurrentSearchBegin + SearchTextLength;
				ActiveLineHighlights.Add(FTextLineHighlight(LineIndex, FTextRange(CurrentSearchBegin, FindBegin), SearchHighlightZOrder, SearchSelectionHighlighter.ToSharedRef()));

				// SearchResultIndex starts from 1
				// for example, if it is used to display stats about search results
				// it would appear as "1 of 5" for the first match among five matches.
				SearchResultIndex++;
				SearchResultToIndexMap.Add(FTextLocation(LineIndex, CurrentSearchBegin), SearchResultIndex);
			}
		}

		SearchSelectionHighlighter->SetHasKeyboardFocus(bHasKeyboardFocus);
	}

	if (bIsComposing)
	{
		FTextLayout::FTextOffsetLocations OffsetLocations;
		// CompositionRange comes from TSF in the \n ACP space, so resolve it back to a text location using
		// the same single-character line terminator, otherwise the composition underline lands on the wrong glyph.
		TextLayout->GetAsTextAndOffsets(nullptr, &OffsetLocations, GIMETSFLineTerminator);

		const FTextRange CompositionRange = TextInputMethodContext->GetCompositionRange();
		const FTextLocation CompositionBeginLocation = OffsetLocations.OffsetToTextLocation(CompositionRange.BeginIndex);
		const FTextLocation CompositionEndLocation = OffsetLocations.OffsetToTextLocation(CompositionRange.EndIndex);

		// Composition should never span more than one (hard) line
		if (CompositionBeginLocation.GetLineIndex() == CompositionEndLocation.GetLineIndex())
		{
			const FTextRange Range(CompositionBeginLocation.GetOffset(), CompositionEndLocation.GetOffset());

			// We only draw the composition highlight if the cursor is within the composition range
			const bool bCursorInRange = (CompositionBeginLocation.GetLineIndex() == CursorInteractionPosition.GetLineIndex() && Range.InclusiveContains(CursorInteractionPosition.GetOffset()));
			if (!Range.IsEmpty() && bCursorInRange)
			{
				ActiveLineHighlights.Add(FTextLineHighlight(CompositionBeginLocation.GetLineIndex(), Range, CompositionRangeZOrder, TextCompositionHighlighter.ToSharedRef()));
			}
		}
	}
	
	if (bHasSelection)
	{
		const FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		const int32 SelectionBeginningLineIndex = Selection.GetBeginning().GetLineIndex();
		const int32 SelectionBeginningLineOffset = Selection.GetBeginning().GetOffset();

		const int32 SelectionEndLineIndex = Selection.GetEnd().GetLineIndex();
		const int32 SelectionEndLineOffset = Selection.GetEnd().GetOffset();

		TextSelectionHighlighter->SetHasKeyboardFocus(bHasKeyboardFocus);

		if (SelectionBeginningLineIndex == SelectionEndLineIndex)
		{
			const FTextRange Range(SelectionBeginningLineOffset, SelectionEndLineOffset);
			ActiveLineHighlights.Add(FTextLineHighlight(SelectionBeginningLineIndex, Range, SelectionHighlightZOrder, TextSelectionHighlighter.ToSharedRef()));
		}
		else
		{
			const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

			for (int32 LineIndex = SelectionBeginningLineIndex; LineIndex <= SelectionEndLineIndex; ++LineIndex)
			{
				if (LineIndex == SelectionBeginningLineIndex)
				{
					const FTextRange Range(SelectionBeginningLineOffset, Lines[LineIndex].Text->Len());
					ActiveLineHighlights.Add(FTextLineHighlight(LineIndex, Range, SelectionHighlightZOrder, TextSelectionHighlighter.ToSharedRef()));
				}
				else if (LineIndex == SelectionEndLineIndex)
				{
					const FTextRange Range(0, SelectionEndLineOffset);
					ActiveLineHighlights.Add(FTextLineHighlight(LineIndex, Range, SelectionHighlightZOrder, TextSelectionHighlighter.ToSharedRef()));
				}
				else
				{
					const FTextRange Range(0, Lines[LineIndex].Text->Len());
					ActiveLineHighlights.Add(FTextLineHighlight(LineIndex, Range, SelectionHighlightZOrder, TextSelectionHighlighter.ToSharedRef()));
				}
			}
		}
	}

	if (bHasKeyboardFocus)
	{
		// The cursor mode uses the literal position rather than the interaction position
		const FTextLocation CursorPosition = CursorInfo.GetCursorLocation();

		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
		if (Lines.IsValidIndex(CursorPosition.GetLineIndex()))
		{
			// Ensure the cursor is sitting on a valid grapheme
			// Right-aligned cursors naively subtract 1 from their real position, which may leave us in the middle of a codepoint or grapheme and break measuring
			const FTextSelection CursorSelection = TextLayout->GetGraphemeAt(CursorPosition);
			ActiveLineHighlights.Add(FTextLineHighlight(CursorPosition.GetLineIndex(), FTextRange(CursorSelection.GetBeginning().GetOffset(), CursorSelection.GetEnd().GetOffset()), CursorZOrder, CursorLineHighlighter.ToSharedRef()));
		}
	}

	// We don't use SetLineHighlights here as we don't want to remove any line highlights that other code might have added (eg, underlines)
	for (const FTextLineHighlight& LineHighlight : ActiveLineHighlights)
	{
		TextLayout->AddLineHighlight(LineHighlight);
	}
	
	VirtualKeyboardEntry->OnSelectionChanged.ExecuteIfBound();
}

void FSlateEditableTextLayout::RemoveCursorHighlight()
{
	const TArray<FTextLayout::FLineModel>& Lines = TextLayout->GetLineModels();

	for (const FTextLineHighlight& LineHighlight : ActiveLineHighlights)
	{
		if (Lines.IsValidIndex(LineHighlight.LineIndex))
		{
			TextLayout->RemoveLineHighlight(LineHighlight);
		}
	}

	ActiveLineHighlights.Empty();
	SearchResultToIndexMap.Reset();
}

void FSlateEditableTextLayout::UpdatePreferredCursorScreenOffsetInLine()
{
	PreferredCursorScreenOffsetInLine = TextLayout->GetLocationAt(CursorInfo.GetCursorInteractionLocation(), CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right).X;
}

void FSlateEditableTextLayout::ApplyToSelection(const FRunInfo& InRunInfo, const FTextBlockStyle& InStyle)
{
	if (OwnerWidget->IsTextReadOnly())
	{
		return;
	}

	FScopedEditableTextTransaction TextTransaction(*this);

	const FTextLocation CursorInteractionPosition = CursorInfo.GetCursorInteractionLocation();
	const FTextLocation SelectionLocation = SelectionStart.Get(CursorInteractionPosition);
	const FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

	const int32 SelectionBeginningLineIndex = Selection.GetBeginning().GetLineIndex();
	const int32 SelectionBeginningLineOffset = Selection.GetBeginning().GetOffset();

	const int32 SelectionEndLineIndex = Selection.GetEnd().GetLineIndex();
	const int32 SelectionEndLineOffset = Selection.GetEnd().GetOffset();

	if (SelectionBeginningLineIndex == SelectionEndLineIndex)
	{
		TSharedRef<FString> SelectedText = MakeShareable(new FString);
		TextLayout->GetSelectionAsText(*SelectedText, Selection);

		TextLayout->RemoveAt(Selection.GetBeginning(), SelectionEndLineOffset - SelectionBeginningLineOffset);

		TSharedRef<IRun> StyledRun = CreateTextOrPasswordRun(InRunInfo, SelectedText, InStyle);
		TextLayout->InsertAt(Selection.GetBeginning(), StyledRun);
	}
	else
	{
		const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

		{
			const FTextLayout::FLineModel& Line = Lines[SelectionBeginningLineIndex];

			const FTextLocation LineStartLocation(SelectionBeginningLineIndex, SelectionBeginningLineOffset);
			const FTextLocation LineEndLocation(SelectionBeginningLineIndex, Line.Text->Len());

			TSharedRef<FString> SelectedText = MakeShareable(new FString);
			TextLayout->GetSelectionAsText(*SelectedText, FTextSelection(LineStartLocation, LineEndLocation));

			TextLayout->RemoveAt(LineStartLocation, LineEndLocation.GetOffset() - LineStartLocation.GetOffset());

			TSharedRef<IRun> StyledRun = CreateTextOrPasswordRun(InRunInfo, SelectedText, InStyle);
			TextLayout->InsertAt(LineStartLocation, StyledRun);
		}

		for (int32 LineIndex = SelectionBeginningLineIndex + 1; LineIndex < SelectionEndLineIndex; ++LineIndex)
		{
			const FTextLayout::FLineModel& Line = Lines[LineIndex];

			const FTextLocation LineStartLocation(LineIndex, 0);
			const FTextLocation LineEndLocation(LineIndex, Line.Text->Len());

			TSharedRef<FString> SelectedText = MakeShareable(new FString);
			TextLayout->GetSelectionAsText(*SelectedText, FTextSelection(LineStartLocation, LineEndLocation));

			TextLayout->RemoveAt(LineStartLocation, LineEndLocation.GetOffset() - LineStartLocation.GetOffset());

			TSharedRef<IRun> StyledRun = CreateTextOrPasswordRun(InRunInfo, SelectedText, InStyle);
			TextLayout->InsertAt(LineStartLocation, StyledRun);
		}

		{
			const FTextLayout::FLineModel& Line = Lines[SelectionEndLineIndex];

			const FTextLocation LineStartLocation(SelectionEndLineIndex, 0);
			const FTextLocation LineEndLocation(SelectionEndLineIndex, SelectionEndLineOffset);

			TSharedRef<FString> SelectedText = MakeShareable(new FString);
			TextLayout->GetSelectionAsText(*SelectedText, FTextSelection(LineStartLocation, LineEndLocation));

			TextLayout->RemoveAt(LineStartLocation, LineEndLocation.GetOffset() - LineStartLocation.GetOffset());

			TSharedRef<IRun> StyledRun = CreateTextOrPasswordRun(InRunInfo, SelectedText, InStyle);
			TextLayout->InsertAt(LineStartLocation, StyledRun);
		}
	}

	SelectionStart = SelectionLocation;
	CursorInfo.SetCursorLocationAndCalculateAlignment(*TextLayout, CursorInteractionPosition);

	UpdatePreferredCursorScreenOffsetInLine();
	UpdateCursorHighlight();
}

TSharedPtr<const IRun> FSlateEditableTextLayout::GetRunUnderCursor() const
{
	const TArray<FTextLayout::FLineModel>& Lines = TextLayout->GetLineModels();

	const FTextLocation CursorInteractionLocation = CursorInfo.GetCursorInteractionLocation();
	if (Lines.IsValidIndex(CursorInteractionLocation.GetLineIndex()))
	{
		const FTextLayout::FLineModel& LineModel = Lines[CursorInteractionLocation.GetLineIndex()];
		for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); ++RunIndex)
		{
			const FTextLayout::FRunModel& RunModel = LineModel.Runs[RunIndex];
			const FTextRange RunRange = RunModel.GetTextRange();

			const bool bIsLastRun = RunIndex == LineModel.Runs.Num() - 1;
			if (RunRange.Contains(CursorInteractionLocation.GetOffset()) || bIsLastRun)
			{
				return RunModel.GetRun();
			}
		}
	}

	return nullptr;
}

TArray<TSharedRef<const IRun>> FSlateEditableTextLayout::GetSelectedRuns() const
{
	TArray<TSharedRef<const IRun>> Runs;

	if (AnyTextSelected())
	{
		const TArray<FTextLayout::FLineModel>& Lines = TextLayout->GetLineModels();
		const FTextLocation CursorInteractionLocation = CursorInfo.GetCursorInteractionLocation();
		if (Lines.IsValidIndex(SelectionStart.GetValue().GetLineIndex()) && Lines.IsValidIndex(CursorInteractionLocation.GetLineIndex()))
		{
			const FTextSelection Selection(SelectionStart.GetValue(), CursorInteractionLocation);
			const int32 StartLine = Selection.GetBeginning().GetLineIndex();
			const int32 EndLine = Selection.GetEnd().GetLineIndex();

			// iterate over lines
			for (int32 LineIndex = StartLine; LineIndex <= EndLine; LineIndex++)
			{
				const bool bIsFirstLine = LineIndex == StartLine;
				const bool bIsLastLine = LineIndex == EndLine;

				const FTextLayout::FLineModel& LineModel = Lines[LineIndex];
				for (int32 RunIndex = 0; RunIndex < LineModel.Runs.Num(); ++RunIndex)
				{
					const FTextLayout::FRunModel& RunModel = LineModel.Runs[RunIndex];

					// check what we should be intersecting with
					if (!bIsFirstLine && !bIsLastLine)
					{
						// whole line is inside the range, so just add the run
						Runs.Add(RunModel.GetRun());
					}
					else
					{
						const FTextRange RunRange = RunModel.GetTextRange();
						if (bIsFirstLine && !bIsLastLine)
						{
							// on first line of multi-line selection
							const FTextRange IntersectedRange = RunRange.Intersect(FTextRange(Selection.GetBeginning().GetOffset(), LineModel.Text->Len()));
							if (!IntersectedRange.IsEmpty())
							{
								Runs.Add(RunModel.GetRun());
							}
						}
						else if (!bIsFirstLine && bIsLastLine)
						{
							// on last line of multi-line selection
							const FTextRange IntersectedRange = RunRange.Intersect(FTextRange(0, Selection.GetEnd().GetOffset()));
							if (!IntersectedRange.IsEmpty())
							{
								Runs.Add(RunModel.GetRun());
							}
						}
						else
						{
							// single line selection
							const FTextRange IntersectedRange = RunRange.Intersect(FTextRange(Selection.GetBeginning().GetOffset(), Selection.GetEnd().GetOffset()));
							if (!IntersectedRange.IsEmpty())
							{
								Runs.Add(RunModel.GetRun());
							}
						}
					}
				}
			}
		}
	}

	return Runs;
}

FTextLocation FSlateEditableTextLayout::GetCursorLocation() const
{
	return CursorInfo.GetCursorInteractionLocation();
}

FTextLocation FSlateEditableTextLayout::TranslatedLocation(const FTextLocation& Location, int8 Direction) const
{
	check(Direction != 0);

	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

	// Move to the previous or next grapheme based upon the requested direction and current position
	GraphemeBreakIterator->SetStringRef(&Lines[Location.GetLineIndex()].Text.Get());
	const int32 NewOffsetInLine = (Direction > 0) ? GraphemeBreakIterator->MoveToCandidateAfter(Location.GetOffset()) : GraphemeBreakIterator->MoveToCandidateBefore(Location.GetOffset());
	GraphemeBreakIterator->ClearString();

	// If our new offset is still invalid then there was no valid grapheme to move to (end or start of line, or an empty line)
	if (NewOffsetInLine == INDEX_NONE)
	{
		if (Direction > 0)
		{
			// Overflow to the start of the next line if we're not the last line
			if (Location.GetLineIndex() < Lines.Num() - 1)
			{
				return FTextLocation(Location.GetLineIndex() + 1, 0);
			}
		}
		else if (Location.GetLineIndex() > 0)
		{
			// Underflow to the end of the previous line if we're not the first line
			const int32 NewLineIndex = Location.GetLineIndex() - 1;
			return FTextLocation(NewLineIndex, Lines[NewLineIndex].Text->Len());
		}

		// Could not move onto a new line, just return the same offset we were passed
		return Location;
	}

	// Return the new offset within the current line
	check(NewOffsetInLine >= 0 && NewOffsetInLine <= Lines[Location.GetLineIndex()].Text->Len());
	return FTextLocation(Location.GetLineIndex(), NewOffsetInLine);
}

void FSlateEditableTextLayout::TranslateLocationVertical(const FTextLocation& Location, int32 NumLinesToMove, float GeometryScale, FTextLocation& OutCursorPosition, TOptional<SlateEditableTextTypes::ECursorAlignment>& OutCursorAlignment) const
{
	const TArray< FTextLayout::FLineView >& LineViews = TextLayout->GetLineViews();
	const int32 NumberOfLineViews = LineViews.Num();

	const int32 CurrentLineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, Location, CursorInfo.GetCursorAlignment() == SlateEditableTextTypes::ECursorAlignment::Right);
	ensure(CurrentLineViewIndex != INDEX_NONE);
	const FTextLayout::FLineView& CurrentLineView = LineViews[CurrentLineViewIndex];

	const int32 NewLineViewIndex = FMath::Clamp(CurrentLineViewIndex + NumLinesToMove, 0, NumberOfLineViews - 1);
	const FTextLayout::FLineView& NewLineView = LineViews[NewLineViewIndex];

	// Our horizontal position is the clamped version of whatever the user explicitly set with horizontal movement.
	ETextHitPoint HitPoint = ETextHitPoint::WithinText;
	OutCursorPosition = TextLayout->GetTextLocationAt(NewLineView, FVector2D(PreferredCursorScreenOffsetInLine, NewLineView.Offset.Y) * GeometryScale, &HitPoint);

	// PreferredCursorScreenOffsetInLine can cause the cursor to move to the right hand gutter, and it needs to be placed there
	// rather than at the start of the next line (which is tricky since they have the same index according to GetTextLocationAt!).
	// We use the HitPoint to work this out and then adjust the cursor position accordingly
	if (HitPoint == ETextHitPoint::RightGutter)
	{
		OutCursorPosition = FTextLocation(OutCursorPosition, -1);
		OutCursorAlignment = SlateEditableTextTypes::ECursorAlignment::Right;
	}
}

FTextLocation FSlateEditableTextLayout::ScanForWordBoundary(const FTextLocation& CurrentLocation, int8 Direction) const
{
	FTextLocation Location = TranslatedLocation(CurrentLocation, Direction);

	while (!IsAtBeginningOfDocument(Location) && !IsAtBeginningOfLine(Location) && !IsAtEndOfDocument(Location) && !IsAtEndOfLine(Location) && !IsAtWordStart(Location))
	{
		Location = TranslatedLocation(Location, Direction);
	}

	return Location;
}

TCHAR FSlateEditableTextLayout::GetCharacterAt(const FTextLocation& Location) const
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();

	const bool bIsLineEmpty = Lines[Location.GetLineIndex()].Text->IsEmpty();
	return (bIsLineEmpty)
		? '\n'
		: (*Lines[Location.GetLineIndex()].Text)[Location.GetOffset()];
}

bool FSlateEditableTextLayout::IsAtBeginningOfDocument(const FTextLocation& Location) const
{
	return Location.GetLineIndex() == 0 && Location.GetOffset() == 0;
}

bool FSlateEditableTextLayout::IsAtEndOfDocument(const FTextLocation& Location) const
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	const int32 NumberOfLines = Lines.Num();

	return NumberOfLines == 0 || (NumberOfLines - 1 == Location.GetLineIndex() && Lines[Location.GetLineIndex()].Text->Len() == Location.GetOffset());
}

bool FSlateEditableTextLayout::IsAtBeginningOfLine(const FTextLocation& Location) const
{
	return Location.GetOffset() == 0;
}

bool FSlateEditableTextLayout::IsAtEndOfLine(const FTextLocation& Location) const
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	return Lines[Location.GetLineIndex()].Text->Len() == Location.GetOffset();
}

bool FSlateEditableTextLayout::IsAtWordStart(const FTextLocation& Location) const
{
	const FTextSelection WordUnderCursor = TextLayout->GetWordAt(Location);
	const FTextLocation WordStart = WordUnderCursor.GetBeginning();

	return WordStart.IsValid() && WordStart == Location;
}

void FSlateEditableTextLayout::RestoreOriginalText()
{
	if (HasTextChangedFromOriginal())
	{
		SetEditableText(OriginalText.Text);
		TextLayout->UpdateIfNeeded();

		// Let outsiders know that the text content has been changed
		OwnerWidget->OnTextCommitted(OriginalText.Text, ETextCommit::OnCleared);
	}
}

bool FSlateEditableTextLayout::HasTextChangedFromOriginal() const
{
	bool bHasChanged = false;
	if (!OwnerWidget->IsTextReadOnly())
	{
		const FText EditedText = GetEditableText();
		bHasChanged = !EditedText.ToString().Equals(OriginalText.Text.ToString(), ESearchCase::CaseSensitive);
	}
	return bHasChanged;
}

void FSlateEditableTextLayout::BeginEditTransaction()
{
	NumTransactionsOpened += 1;
	
	if (NumTransactionsOpened > 1 || OwnerWidget->IsTextReadOnly())
	{
		// Already within a translation - don't open another
		// Or never change text on read only controls.
		// The TextReadOnly is an attribute, the return value may have changed since the last time it was checked.
		return;
	}

	// We're starting to (potentially) change text
	// Save off an undo state in case we actually change the text
	StateBeforeChangingText = SlateEditableTextTypes::FUndoState();
	MakeUndoState(StateBeforeChangingText.GetValue());
}

void FSlateEditableTextLayout::EndEditTransaction()
{
	NumTransactionsOpened -= 1;

	check(NumTransactionsOpened >= 0);

	if (NumTransactionsOpened > 0)
	{
		// Don't close transaction if there are more opened
		// Caller of the first opened transaction should be
		// responsible for actually closing the transaction
		return;
	}

	if (!StateBeforeChangingText.IsSet())
	{
		// if OwnerWidget->IsTextReadOnly happens on the first transaction in BeginEditTransaction, the count is increased but the State is never being set so we should skip on the work
		// if EndEditTransaction is getting called after the UndoState has been reset, probably by getting deleted
		// before the call happens on another thread, there's no point in trying to close the transaction.
		return;
	}

	// We're no longer changing text
	const FText EditedText = GetEditableText();

	// Has the text changed?
	const bool bHasTextChanged = !EditedText.ToString().Equals(StateBeforeChangingText.GetValue().Text.ToString(), ESearchCase::CaseSensitive);
	if (bHasTextChanged)
	{
		// Save text state
		SaveText(EditedText);

		if (TextInputMethodChangeNotifier.IsValid() && TextInputMethodContext.IsValid())
		{
			// Don't notify TSF while the IME is composing: TSF already drives (and knows about) its own
			// composition edits, and reporting them back as an "external" change double-counts indices and
			// corrupts the position model. EndComposition() calls EndEditTransaction() while bIsComposing is
			// still true, so the final commit is suppressed here too - which is correct, TSF already has it.
			if (!TextInputMethodContext->IsComposing())
			{
				// Compute the changed range in the SAME single-character (\n) ACP space that every other
				// IME/TSF method reports, so the notified indices line up 1:1 with TSF's model. Using
				// GetEditableText() here (CRLF on Windows) would put the notification in a different space
				// than GetTextLength()/GetSelectionRange()/GetTextInRange(), reintroducing the desync.
				FString OldString = StateBeforeChangingText.GetValue().Text.ToString();
				OldString.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);
				FString NewString;
				TextLayout->GetAsTextAndOffsets(&NewString, nullptr, GIMETSFLineTerminator);

				// Report only the actually-changed span instead of the whole document. ComputeMinimalChangedRange
				// returns false when the two \n-space strings are identical, in which case there is nothing for
				// TSF to reconcile and we skip the (degenerate, zero-length) notification entirely.
				int32 ChangedBeginIndex = 0;
				int32 ChangedOldLength = 0;
				int32 ChangedNewLength = 0;
				if (SlateEditableTextLayoutIME::ComputeMinimalChangedRange(OldString, NewString, ChangedBeginIndex, ChangedOldLength, ChangedNewLength))
				{
					TextInputMethodChangeNotifier->NotifyTextChanged(ChangedBeginIndex, ChangedOldLength, ChangedNewLength);
				}
			}
		}

		// Text was actually changed, so push the undo state we previously saved off
		PushUndoState(StateBeforeChangingText.GetValue());

		TextLayout->UpdateIfNeeded();

		// Let outsiders know that the text content has been changed
		BroadcastOnTextChanged(EditedText);

		OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());

		// Update the desired cursor position, since typing will have moved it
		UpdatePreferredCursorScreenOffsetInLine();

		// If the marshaller we're using requires live text updates (eg, because it injects formatting into the source text)
		// then we need to force a SetEditableText here so that it can update the new editable text with any extra formatting
		if (Marshaller->RequiresLiveUpdate())
		{
			ForceRefreshTextLayout(EditedText);
		}
	}

	// We're done with this state data now. Clear out any old data.
	StateBeforeChangingText.Reset();
}

void FSlateEditableTextLayout::PushUndoState(const SlateEditableTextTypes::FUndoState& InUndoState)
{
	// If we've already undone some state, then we'll remove any undo state beyond the level that
	// we've already undone up to.
	if (CurrentUndoLevel != INDEX_NONE)
	{
		UndoStates.RemoveAt(CurrentUndoLevel, UndoStates.Num() - CurrentUndoLevel);

		// Reset undo level as we haven't undone anything since this latest undo
		CurrentUndoLevel = INDEX_NONE;
	}

	// Cache new undo state
	UndoStates.Add(InUndoState);

	// If we've reached the maximum number of undo levels, then trim our array
	if (UndoStates.Num() > EditableTextDefs::MaxUndoLevels)
	{
		UndoStates.RemoveAt(0);
	}
}

void FSlateEditableTextLayout::ClearUndoStates()
{
	CurrentUndoLevel = INDEX_NONE;
	UndoStates.Empty();
}

void FSlateEditableTextLayout::MakeUndoState(SlateEditableTextTypes::FUndoState& OutUndoState)
{
	//@todo save and restoring the whole document is not ideal [3/31/2014 justin.sargent]
	const FText EditedText = GetEditableText();

	OutUndoState.Text = EditedText;
	OutUndoState.CursorInfo = CursorInfo;
	OutUndoState.SelectionStart = SelectionStart;
}

bool FSlateEditableTextLayout::CanExecuteUndo() const
{
	// Previously, if UndoStates was empty then the event would bubble up back to the Editor and trigger an undo.
	// Now, the editable text always catches the undo event so that it never bubbles up. This prevents bugs such
	// as undo-ing in a search box triggering an undo, and various issues related to undo-ing property changes
	// while focused in to the property widget being undone.
	// Note that these cases are all still checked in the actual Undo method.
	return !OwnerWidget->IsTextReadOnly()/* && UndoStates.Num() > 0*/ && !TextInputMethodContext->IsComposing();
}

void FSlateEditableTextLayout::Undo()
{
	if (!OwnerWidget->IsTextReadOnly() && UndoStates.Num() > 0 && !TextInputMethodContext->IsComposing())
	{
		// Restore from undo state
		int32 UndoStateIndex;
		if (CurrentUndoLevel == INDEX_NONE)
		{
			// We haven't undone anything since the last time a new undo state was added
			UndoStateIndex = UndoStates.Num() - 1;

			// Store an undo state for the current state (before undo was pressed)
			SlateEditableTextTypes::FUndoState NewUndoState;
			MakeUndoState(NewUndoState);
			PushUndoState(NewUndoState);
		}
		else
		{
			// Move down to the next undo level
			UndoStateIndex = CurrentUndoLevel - 1;
		}

		// Is there anything else to undo?
		if (UndoStateIndex >= 0)
		{
			{
				// NOTE: It's important the no code called here creates or destroys undo states!
				const SlateEditableTextTypes::FUndoState& UndoState = UndoStates[UndoStateIndex];

				SaveText(UndoState.Text);

				if (SetEditableText(UndoState.Text))
				{
					// Let outsiders know that the text content has been changed
					BroadcastOnTextChanged(UndoState.Text);
				}

				CursorInfo = UndoState.CursorInfo.CreateUndo();
				SelectionStart = UndoState.SelectionStart;

				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());

				UpdateCursorHighlight();
			}

			CurrentUndoLevel = UndoStateIndex;
		}
	}
}

bool FSlateEditableTextLayout::CanExecuteRedo() const
{
	// See comment in CanExecuteUndo
	return !OwnerWidget->IsTextReadOnly() /*&& CurrentUndoLevel != INDEX_NONE*/ && !TextInputMethodContext->IsComposing();
}

void FSlateEditableTextLayout::Redo()
{
	// Is there anything to redo?  If we've haven't tried to undo since the last time new
	// undo state was added, then CurrentUndoLevel will be INDEX_NONE
	if (!OwnerWidget->IsTextReadOnly() && CurrentUndoLevel != INDEX_NONE && !TextInputMethodContext->IsComposing())
	{
		const int32 NextUndoLevel = CurrentUndoLevel + 1;
		if (UndoStates.Num() > NextUndoLevel)
		{
			// Restore from undo state
			{
				// NOTE: It's important the no code called here creates or destroys undo states!
				const SlateEditableTextTypes::FUndoState& UndoState = UndoStates[NextUndoLevel];

				SaveText(UndoState.Text);

				if (SetEditableText(UndoState.Text))
				{
					// Let outsiders know that the text content has been changed
					BroadcastOnTextChanged(UndoState.Text);
				}

				CursorInfo.RestoreFromUndo(UndoState.CursorInfo);
				SelectionStart = UndoState.SelectionStart;

				OwnerWidget->OnCursorMoved(CursorInfo.GetCursorInteractionLocation());

				UpdateCursorHighlight();
			}

			CurrentUndoLevel = NextUndoLevel;

			if (UndoStates.Num() <= CurrentUndoLevel + 1)
			{
				// We've redone all available undo states
				CurrentUndoLevel = INDEX_NONE;

				// Pop last undo state that we created on initial undo
				UndoStates.RemoveAt(UndoStates.Num() - 1);
			}
		}
	}
}

void FSlateEditableTextLayout::SaveText(const FText& TextToSave)
{
	const TSharedRef<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidget();
	// Don't set text if the text attribute has a 'getter' binding on it, otherwise we'd blow away
	// that binding.  If there is a getter binding, then we'll assume it will provide us with
	// updated text after we've fired our 'text changed' callbacks
	if (!BoundText.IsBound(OwnerSlateWidget.Get()))
	{
		BoundText.Assign(OwnerSlateWidget.Get(), TextToSave);
	}
}

void FSlateEditableTextLayout::LoadText()
{
	const TSharedRef<SWidget> OwnerSlateWidget = OwnerWidget->GetSlateWidget();
	// We only need to do this if we're bound to a delegate, otherwise the text layout will already be up-to-date
	// either from Construct, or a call to SetText
	if (BoundText.IsBound(OwnerSlateWidget.Get()))
	{
		bForceBoundTextReview = true;
		BoundText.UpdateNow(OwnerSlateWidget.Get());
		if (bForceBoundTextReview)
		{
			// We would only arrive here if OnBoundTextChanged did not trigger (because the BoundText technically did not change),
			// but we still want to make sure the EditableText is aligning so we call it manually
			OnBoundTextChanged();
		}
		TextLayout->UpdateIfNeeded();
	}
}

bool FSlateEditableTextLayout::ComputeVolatility() const
{
	return OwnerWidget->GetSlateWidget()->HasAnyUserFocus().IsSet() && CursorLineHighlighter->GetDrawCursor();
}

void FSlateEditableTextLayout::UpdateTextChangedByVirtualKeyboard()
{
	if (bTextChangedByVirtualKeyboard)
	{
		SetEditableText(VirtualKeyboardText);
		TextLayout->UpdateIfNeeded();
		// Let outsiders know that the text content has been changed
		BroadcastOnTextChanged(GetEditableText());
		bTextChangedByVirtualKeyboard = false;
	}
}

void FSlateEditableTextLayout::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	check(IsInGameThread());

	const bool bTextChangedByVKThisTick = bTextChangedByVirtualKeyboard;
	
	UpdateTextChangedByVirtualKeyboard();

	if (bTextCommittedByVirtualKeyboard)
	{
		// Let outsiders know that the text content has been changed
		OwnerWidget->OnTextCommitted(GetEditableText(), VirtualKeyboardTextCommitType);
		bTextCommittedByVirtualKeyboard = false;
	}

	if (TextInputMethodChangeNotifier.IsValid() && TextInputMethodContext.IsValid() && TextInputMethodContext->UpdateCachedGeometry(AllottedGeometry))
	{
		TextInputMethodChangeNotifier->NotifyLayoutChanged(ITextInputMethodChangeNotifier::ELayoutChangeType::Changed);
	}

	//#jira UE - 49301 Text in UMG controls flickers during update from Virtual Keyboard
	const bool bShouldAppearFocused = FSlateApplication::Get().AllowMoveCursor() &&
		(OwnerWidget->GetSlateWidget()->HasAnyUserFocus().IsSet() || HasActiveContextMenu());
	if (bShouldAppearFocused)
	{
		// When focused the user is editing or selecting text. Never allow ellipsis to replace text
		TextLayout->SetTextOverflowPolicy(ETextOverflowPolicy::Clip);
		// If we have focus then we don't allow the editable text itself to update, but we do still need to refresh the password and marshaller state
		RefreshImpl(nullptr);
	}
	else
	{
		TextLayout->SetTextOverflowPolicy(OverflowPolicyOverride);

		// We don't have focus, so we can perform a full refresh
		Refresh();
	}

	// Prevent changes to selection when text did not change this tick as this cause the caret to disappear and only appear when new text is entered due to the SelectionChanged being received before the new text.
	if (bSelectionChangedExternally && bTextChangedByVKThisTick)
	{
		bSelectionChangedExternally = false;
		if (TextInputMethodContext.IsValid())
		{
			if (ExternalSelectionStart <= ExternalSelectionEnd)
			{
				TextInputMethodContext->SetSelectionRange(ExternalSelectionStart, ExternalSelectionEnd - ExternalSelectionStart, ITextInputMethodContext::ECaretPosition::Beginning);
			}
			else
			{
				TextInputMethodContext->SetSelectionRange(ExternalSelectionEnd, ExternalSelectionStart - ExternalSelectionEnd, ITextInputMethodContext::ECaretPosition::Ending);
			}
		}
	}

	// Update the search before we process the next PositionToScrollIntoView
	{
		const FText& SearchTextToSet = BoundSearchText.Get();
		if (!BoundSearchTextLastTick.IdenticalTo(SearchTextToSet))
		{
			// The pointer used by the bound text has changed, however the text may still be the same - check that now
			if (!BoundSearchTextLastTick.IsDisplayStringEqualTo(SearchTextToSet))
			{
				BeginSearch(SearchTextToSet);
			}

			// Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next Tick
			BoundSearchTextLastTick = FTextSnapshot(SearchTextToSet);
		}
	}

	const float FontMaxCharHeight = FTextEditHelper::GetFontHeight(TextStyle.Font);
	const float CaretWidth = FTextEditHelper::CalculateCaretWidth(FontMaxCharHeight);

	// If we're auto-wrapping, we need to hide the scrollbars until the first valid auto-wrap has been performed
	// If we don't do this, then we can get some nasty layout shuffling as the scrollbars appear for one frame and then vanish again
	// We also hide the scrollbars for non-multi-line text widgets
	const EVisibility ScrollBarVisibilityOverride = ((AutoWrapText.Get() && CachedSize.IsZero()) || !OwnerWidget->IsMultiLineTextEdit()) ? EVisibility::Collapsed : EVisibility::Visible;

	// Try and make sure that the line containing the cursor is in view
	// Prevent scrolling if Virtual Keyboard changed text this tick to prevent using stale AllottedGeometry. Scroll will be applied properly on the next tick.
	if (PositionToScrollIntoView.IsSet() && !bTextChangedByVKThisTick)
	{
		const SlateEditableTextTypes::FScrollInfo& ScrollInfo = PositionToScrollIntoView.GetValue();

		const TArray<FTextLayout::FLineView>& LineViews = TextLayout->GetLineViews();
		const int32 LineViewIndex = TextLayout->GetLineViewIndexForTextLocation(LineViews, ScrollInfo.Position, ScrollInfo.Alignment == SlateEditableTextTypes::ECursorAlignment::Right);
		if (LineViews.IsValidIndex(LineViewIndex))
		{
			const FTextLayout::FLineView& LineView = LineViews[LineViewIndex];
			const FSlateRect LocalLineViewRect(LineView.Offset / TextLayout->GetScale(), (LineView.Offset + LineView.Size) / TextLayout->GetScale());

			const FVector2D LocalCursorLocation = TextLayout->GetLocationAt(ScrollInfo.Position, ScrollInfo.Alignment == SlateEditableTextTypes::ECursorAlignment::Right) / TextLayout->GetScale();
			const FSlateRect LocalCursorRect(LocalCursorLocation, FVector2f(LocalCursorLocation.X + CaretWidth, LocalCursorLocation.Y + FontMaxCharHeight));

			if (LocalCursorRect.Left < 0.0f)
			{
				ScrollOffset.X += LocalCursorRect.Left;
			}
			else if (LocalCursorRect.Right > AllottedGeometry.GetLocalSize().X)
			{
				ScrollOffset.X += (LocalCursorRect.Right - AllottedGeometry.GetLocalSize().X);
			}

			if (LocalLineViewRect.Top < 0.0f)
			{
				ScrollOffset.Y += LocalLineViewRect.Top;
			}
			else if (LocalLineViewRect.Bottom > AllottedGeometry.GetLocalSize().Y)
			{
				ScrollOffset.Y += (LocalLineViewRect.Bottom - AllottedGeometry.GetLocalSize().Y);
			}
		}

		PositionToScrollIntoView.Reset();
	}

	{
		// The caret width is included in the margin
		const float ContentSize = TextLayout->GetSize().X;
		const float VisibleSize = AllottedGeometry.GetLocalSize().X;

		// If this text box has no size, do not compute a view fraction because it will be wrong and causes pop in when the size is available
		const float ViewFraction = (VisibleSize > 0.0f && ContentSize > 0.0f) ? VisibleSize / ContentSize : 1;
		const float ViewOffset = (ContentSize > 0.0f && ViewFraction < 1.0f) ? FMath::Clamp<float>(ScrollOffset.X / ContentSize, 0.0f, 1.0f - ViewFraction) : 0.0f;

		// Update the scrollbar with the clamped version of the offset
		ScrollOffset.X = ViewOffset * ContentSize;
		ScrollOffset.X = OwnerWidget->UpdateAndClampHorizontalScrollBar(ViewOffset, ViewFraction, ScrollBarVisibilityOverride);
	}

	{
		const float ContentSize = TextLayout->GetSize().Y;
		const float VisibleSize = AllottedGeometry.GetLocalSize().Y;

		// If this text box has no size, do not compute a view fraction because it will be wrong and causes pop in when the size is available
		const float ViewFraction = (VisibleSize > 0.0f && ContentSize > 0.0f) ? VisibleSize / ContentSize : 1;
		const float ViewOffset = (ContentSize > 0.0f && ViewFraction < 1.0f) ? FMath::Clamp<float>(ScrollOffset.Y / ContentSize, 0.0f, 1.0f - ViewFraction) : 0.0f;

		// Update the scrollbar with the clamped version of the offset
		ScrollOffset.Y = ViewOffset * ContentSize;
		ScrollOffset.Y = OwnerWidget->UpdateAndClampVerticalScrollBar(ViewOffset, ViewFraction, ScrollBarVisibilityOverride);
	}

	TextLayout->SetVisibleRegion(AllottedGeometry.Size, FVector2D(ScrollOffset) * TextLayout->GetScale());
}

int32 FSlateEditableTextLayout::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Update the auto-wrap size now that we have computed paint geometry; won't take effect until text frame
	// Note: This is done here rather than in Tick(), because Tick() doesn't get called while resizing windows, but OnPaint() does
	const FVector2f PreviousCachedSize = CachedSize;
	CachedSize = FVector2f(AllottedGeometry.GetLocalSize());

	// If CacheSize is set for the first time, we relayout to avoid a one frame jump for non left justifications.
	if (PreviousCachedSize == FVector2f::Zero())
	{
		TextLayout->SetVisibleRegion(FVector2D(CachedSize), FVector2D(ScrollOffset) * TextLayout->GetScale());
		TextLayout->UpdateIfNeeded();
	}

	// Only paint the hint text layout if we don't have any text set
	if (TextLayout->IsEmpty() && HintTextLayout.IsValid())
	{
		// Make sure the hint text is the correct color before we paint it
		FLinearColor ThisColorAndOpacity = TextStyle.ColorAndOpacity.GetColor(InWidgetStyle);
		ThisColorAndOpacity.A = GetHintTextOpacity();
		HintTextStyle.ColorAndOpacity = ThisColorAndOpacity;
		HintTextLayout->OverrideTextStyle(HintTextStyle);

		LayerId = HintTextLayout->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

	LayerId = TextLayout->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return LayerId;
}

void FSlateEditableTextLayout::SetCachedSize(const FGeometry& AllottedGeometry)
{
	CachedSize = FVector2f(AllottedGeometry.GetLocalSize());
}

void FSlateEditableTextLayout::CacheDesiredSize(float LayoutScaleMultiplier)
{
	const float FontMaxCharHeight = FTextEditHelper::GetFontHeight(TextStyle.Font);
	const float CaretWidth = FTextEditHelper::CalculateCaretWidth(FontMaxCharHeight);

	// Get the wrapping width and font to see if they have changed
	float WrappingWidth = WrapTextAt.Get();

	// Text wrapping can either be used defined (WrapTextAt), automatic (AutoWrapText), or a mixture of both
	// Take whichever has the smallest value (>1)
	if (AutoWrapText.Get() && CachedSize.X >= 1.0f)
	{
		WrappingWidth = (WrappingWidth >= 1.0f) ? FMath::Min(WrappingWidth, CachedSize.X) : CachedSize.X;
	}

	// Append the caret width to the margin to make sure it doesn't get clipped
	FMargin MarginValue = Margin.Get();
	MarginValue.Left += CaretWidth;
	MarginValue.Right += CaretWidth;

	TextLayout->SetScale(LayoutScaleMultiplier);
	TextLayout->SetWrappingWidth(WrappingWidth);
	TextLayout->SetWrappingPolicy(WrappingPolicy.Get());
	TextLayout->SetMargin(MarginValue);
	TextLayout->SetLineHeightPercentage(LineHeightPercentage.Get());
	TextLayout->SetApplyLineHeightToBottomLine(ApplyLineHeightToBottomLine.Get());
	TextLayout->SetJustification(Justification.Get());
	TextLayout->SetVisibleRegion(FVector2D(CachedSize), FVector2D(ScrollOffset) * TextLayout->GetScale());
	TextLayout->UpdateIfNeeded();
}

FVector2D FSlateEditableTextLayout::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	check(IsInGameThread());

	const float FontMaxCharHeight = FTextEditHelper::GetFontHeight(TextStyle.Font);
	const float CaretWidth = FTextEditHelper::CalculateCaretWidth(FontMaxCharHeight);

	const float WrappingWidth = WrapTextAt.Get();
	float DesiredWidth = 0.0f;
	float DesiredHeight = 0.0f;

	// If we have hint text, make sure we include that in any size calculations
	if (TextLayout->IsEmpty() && HintTextLayout.IsValid())
	{
		// Append the caret width to the margin to mimic what happens to the main text layout
		FMargin MarginValue = Margin.Get();
		MarginValue.Left += CaretWidth;
		MarginValue.Right += CaretWidth;

		const FVector2D HintTextSize = HintTextLayout->ComputeDesiredSize(
			FSlateTextBlockLayout::FWidgetDesiredSizeArgs(HintText.Get(), FText::GetEmpty(), WrappingWidth, AutoWrapText.Get(), WrappingPolicy.Get(), ETextTransformPolicy::None, MarginValue, LineHeightPercentage.Get(), ApplyLineHeightToBottomLine.Get(), Justification.Get()),
			LayoutScaleMultiplier, HintTextStyle
			);

		// If a wrapping width has been provided, then we need to report that as the desired width
		DesiredWidth = WrappingWidth > 0 ? WrappingWidth : HintTextSize.X;
		DesiredHeight = HintTextSize.Y;
	}
	else
	{
		// If an explicit wrapping width has been provided, then we need to report the wrapped size as the desired width if it has lines that extend beyond the fixed wrapping width
		// Note: We don't do this when auto-wrapping with a non-explicit width as it would cause a feedback loop in the Slate sizing logic
		FVector2D TextLayoutSize = TextLayout->GetSize();
		if (WrappingWidth > 0 && TextLayoutSize.X > WrappingWidth)
		{
			TextLayoutSize = TextLayout->GetWrappedSize();
		}

		DesiredWidth = TextLayoutSize.X;
		DesiredHeight = TextLayoutSize.Y;
	}

	// The layouts current margin size. We should not report a size smaller then the margins.
	const FMargin TextLayoutMargin = TextLayout->GetMargin();
	DesiredWidth = FMath::Max(TextLayoutMargin.GetTotalSpaceAlong<Orient_Horizontal>(), DesiredWidth);
	DesiredHeight = FMath::Max(TextLayoutMargin.GetTotalSpaceAlong<Orient_Vertical>(), DesiredHeight);
	DesiredHeight = FMath::Max(FontMaxCharHeight, DesiredHeight);

	return FVector2D(DesiredWidth, DesiredHeight);
}

FChildren* FSlateEditableTextLayout::GetChildren()
{
	// Only use the hint text layout if we don't have any text set
	return (TextLayout->IsEmpty() && HintTextLayout.IsValid())
		? HintTextLayout->GetChildren()
		: TextLayout->GetChildren();
}

void FSlateEditableTextLayout::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// Only arrange the hint text layout if we don't have any text set
	if (TextLayout->IsEmpty() && HintTextLayout.IsValid())
	{
		HintTextLayout->ArrangeChildren(AllottedGeometry, ArrangedChildren);
	}
	else
	{
		TextLayout->ArrangeChildren(AllottedGeometry, ArrangedChildren);
	}
}

UE::Slate::FDeprecateVector2DResult FSlateEditableTextLayout::GetSize() const
{
	return UE::Slate::CastToVector2f(TextLayout->GetSize());
}

TSharedRef<SWidget> FSlateEditableTextLayout::BuildDefaultContextMenu(const TSharedPtr<FExtender>& InMenuExtender) const
{
#define LOCTEXT_NAMESPACE "EditableTextContextMenu"
	// Set the menu to automatically close when the user commits to a choice
	const bool bShouldCloseWindowAfterMenuSelection = true;

	// This is a context menu which could be summoned from within another menu if this text block is in a menu
	// it should not close the menu it is inside
	bool bCloseSelfOnly = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList, InMenuExtender, bCloseSelfOnly, &FCoreStyle::Get());
	{
		MenuBuilder.BeginSection("EditText", LOCTEXT("Heading", "Modify Text"));
		{
			// Undo
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Undo);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("EditableTextModify2");
		{
			// Cut
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);

			// Copy
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);

			// Paste
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);

			// Delete
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("EditableTextModify3");
		{
			// Select All
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().SelectAll);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
#undef LOCTEXT_NAMESPACE
}

bool FSlateEditableTextLayout::HasActiveContextMenu() const
{
	return ActiveContextMenu.IsValid();
}

void FSlateEditableTextLayout::GetCurrentTextLine(FString& OutTextLine) const
{	
	// grab line of text
	const int32 LineIdx = CursorInfo.GetCursorLocation().GetLineIndex();
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	if (Lines.IsValidIndex(LineIdx))
	{
		OutTextLine = *Lines[LineIdx].Text;
	}
}

void FSlateEditableTextLayout::GetTextLine(const int32 InLineIndex, FString& OutTextLine) const
{
	const TArray< FTextLayout::FLineModel >& Lines = TextLayout->GetLineModels();
	if (Lines.IsValidIndex(InLineIndex))
	{
		OutTextLine = *Lines[InLineIndex].Text;
	}
}

TSharedRef<FSlateEditableTextLayout::FVirtualKeyboardEntry> FSlateEditableTextLayout::FVirtualKeyboardEntry::Create(FSlateEditableTextLayout& InOwnerLayout)
{
	return MakeShareable(new FVirtualKeyboardEntry(InOwnerLayout));
}

FSlateEditableTextLayout::FVirtualKeyboardEntry::FVirtualKeyboardEntry(FSlateEditableTextLayout& InOwnerLayout)
	: OwnerLayout(&InOwnerLayout)
{
}

void FSlateEditableTextLayout::FVirtualKeyboardEntry::SetTextFromVirtualKeyboard(const FText& InNewText, ETextEntryType TextEntryType)
{
	check(IsInGameThread());

	const TSharedRef<SWidget> OwnerSlateWidget = OwnerLayout->OwnerWidget->GetSlateWidget();

	// Only set the text if the text attribute doesn't have a getter binding (otherwise it would be blown away).
	// If it is bound, we'll assume that OnTextCommitted will handle the update.
	if (!OwnerLayout->BoundText.IsBound(OwnerSlateWidget.Get()))
	{
		OwnerLayout->BoundText.Assign(OwnerSlateWidget.Get(), InNewText);
	}

	// Update the internal editable text
	// This method is called from the main thread (i.e. not the game thread) of the device with the virtual keyboard
	// This causes the app to crash on those devices, so we're using polling here to ensure delegates are
	// fired on the game thread in Tick.		
	OwnerLayout->VirtualKeyboardText = InNewText;
	OwnerLayout->bTextChangedByVirtualKeyboard = true;
	if (TextEntryType == ETextEntryType::TextEntryAccepted)
	{
		if (OwnerLayout->OwnerWidget->GetVirtualKeyboardDismissAction() == EVirtualKeyboardDismissAction::TextCommitOnAccept ||
			OwnerLayout->OwnerWidget->GetVirtualKeyboardDismissAction() == EVirtualKeyboardDismissAction::TextCommitOnDismiss)
		{
			OwnerLayout->VirtualKeyboardTextCommitType = ETextCommit::OnEnter;
			OwnerLayout->bTextCommittedByVirtualKeyboard = true;
		}
	}
	else if (TextEntryType == ETextEntryType::TextEntryCanceled)
	{
		if (OwnerLayout->OwnerWidget->GetVirtualKeyboardDismissAction() == EVirtualKeyboardDismissAction::TextCommitOnDismiss)
		{
			OwnerLayout->VirtualKeyboardTextCommitType = ETextCommit::Default;
			OwnerLayout->bTextCommittedByVirtualKeyboard = true;
		}
	}
}

void FSlateEditableTextLayout::FVirtualKeyboardEntry::SetSelectionFromVirtualKeyboard(int InSelStart, int InSelEnd)
{
	check(IsInGameThread());

	// Update the text selection and the cursor position
	// This method is called externally (eg. on Android from the native virtual keyboard implementation) 
	// The text may also change on the same frame, so the external selection must happen in Tick after the text update

	OwnerLayout->bSelectionChangedExternally = true;
	OwnerLayout->ExternalSelectionStart = InSelStart;
	OwnerLayout->ExternalSelectionEnd = InSelEnd;
}

FText FSlateEditableTextLayout::FVirtualKeyboardEntry::GetText() const
{
	check(IsInGameThread());

	return OwnerLayout->GetText();
}

bool FSlateEditableTextLayout::FVirtualKeyboardEntry::GetSelection(int& OutSelStart, int& OutSelEnd)
{
	check(IsInGameThread());

	const FTextLocation CursorInteractionPosition = OwnerLayout->CursorInfo.GetCursorInteractionLocation();
	FTextLocation SelectionLocation = OwnerLayout->SelectionStart.Get(CursorInteractionPosition);
	FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

	OutSelStart = Selection.GetBeginning().GetOffset();
	OutSelEnd = Selection.GetEnd().GetOffset();
	return true;
}

FText FSlateEditableTextLayout::FVirtualKeyboardEntry::GetHintText() const
{
	check(IsInGameThread());

	return OwnerLayout->GetHintText();
}

EKeyboardType FSlateEditableTextLayout::FVirtualKeyboardEntry::GetVirtualKeyboardType() const
{
	check(IsInGameThread());

	return (OwnerLayout->OwnerWidget->IsTextPassword()) ? Keyboard_Password : OwnerLayout->OwnerWidget->GetVirtualKeyboardType();
}

FVirtualKeyboardOptions FSlateEditableTextLayout::FVirtualKeyboardEntry::GetVirtualKeyboardOptions() const
{
	check(IsInGameThread());

	return OwnerLayout->OwnerWidget->GetVirtualKeyboardOptions();
}

bool FSlateEditableTextLayout::FVirtualKeyboardEntry::IsMultilineEntry() const
{
	check(IsInGameThread());

	return OwnerLayout->OwnerWidget->IsMultiLineTextEdit();
}

bool FSlateEditableTextLayout::FVirtualKeyboardEntry::IsIntegratedKeyboardEnabled() const
{
	check(IsInGameThread());
	
	return OwnerLayout->OwnerWidget->IsIntegratedKeyboardEnabled();
}


TSharedRef<FSlateEditableTextLayout::FTextInputMethodContext> FSlateEditableTextLayout::FTextInputMethodContext::Create(FSlateEditableTextLayout& InOwnerLayout)
{
	return MakeShareable(new FTextInputMethodContext(InOwnerLayout));
}

FSlateEditableTextLayout::FTextInputMethodContext::FTextInputMethodContext(FSlateEditableTextLayout& InOwnerLayout)
	: OwnerLayout(&InOwnerLayout)
	, bIsComposing(false)
	, CompositionBeginIndex(INDEX_NONE)
	, CompositionLength(0)
{
}

bool FSlateEditableTextLayout::FTextInputMethodContext::IsComposing()
{
	return OwnerLayout && bIsComposing;
}

bool FSlateEditableTextLayout::FTextInputMethodContext::IsReadOnly()
{
	return !OwnerLayout || OwnerLayout->OwnerWidget->IsTextReadOnly();
}

uint32 FSlateEditableTextLayout::FTextInputMethodContext::GetTextLength()
{
	if (!OwnerLayout)
	{
		return 0;
	}

	FTextLayout::FTextOffsetLocations OffsetLocations;
	OwnerLayout->TextLayout->GetAsTextAndOffsets(nullptr, &OffsetLocations, GIMETSFLineTerminator);
	return OffsetLocations.GetTextLength();
}

void FSlateEditableTextLayout::FTextInputMethodContext::GetSelectionRange(uint32& BeginIndex, uint32& Length, ECaretPosition& OutCaretPosition)
{
	if (!OwnerLayout)
	{
		BeginIndex = 0;
		Length = 0;
		OutCaretPosition = ITextInputMethodContext::ECaretPosition::Beginning;
		return;
	}

	const FTextLocation CursorInteractionPosition = OwnerLayout->CursorInfo.GetCursorInteractionLocation();
	const FTextLocation SelectionLocation = OwnerLayout->SelectionStart.Get(CursorInteractionPosition);

	FTextLayout::FTextOffsetLocations OffsetLocations;
	OwnerLayout->TextLayout->GetAsTextAndOffsets(nullptr, &OffsetLocations, GIMETSFLineTerminator);

	const bool bHasSelection = SelectionLocation != CursorInteractionPosition;
	if (bHasSelection)
	{
		// We need to translate the selection into "editable text" space
		const FTextSelection Selection(SelectionLocation, CursorInteractionPosition);

		const FTextLocation& BeginningOfSelectionInDocumentSpace = Selection.GetBeginning();
		const int32 BeginningOfSelectionInEditableTextSpace = OffsetLocations.TextLocationToOffset(BeginningOfSelectionInDocumentSpace);

		const FTextLocation& EndOfSelectionInDocumentSpace = Selection.GetEnd();
		const int32 EndOfSelectionInEditableTextSpace = OffsetLocations.TextLocationToOffset(EndOfSelectionInDocumentSpace);

		BeginIndex = BeginningOfSelectionInEditableTextSpace;
		Length = EndOfSelectionInEditableTextSpace - BeginningOfSelectionInEditableTextSpace;

		const bool bCursorIsBeforeSelection = CursorInteractionPosition < SelectionLocation;
		OutCaretPosition = (bCursorIsBeforeSelection) ? ITextInputMethodContext::ECaretPosition::Beginning : ITextInputMethodContext::ECaretPosition::Ending;
	}
	else
	{
		// We need to translate the cursor position into "editable text" space
		const int32 CursorInteractionPositionInEditableTextSpace = OffsetLocations.TextLocationToOffset(CursorInteractionPosition);

		BeginIndex = CursorInteractionPositionInEditableTextSpace;
		Length = 0;

		OutCaretPosition = ITextInputMethodContext::ECaretPosition::Beginning;
	}
}

void FSlateEditableTextLayout::FTextInputMethodContext::SetSelectionRange(const uint32 BeginIndex, const uint32 Length, const ECaretPosition InCaretPosition)
{
	if (!OwnerLayout)
	{
		return;
	}

	const uint32 TextLength = GetTextLength();

	const uint32 MinIndex = FMath::Min(BeginIndex, TextLength);
	const uint32 MaxIndex = FMath::Min(MinIndex + Length, TextLength);

	FTextLayout::FTextOffsetLocations OffsetLocations;
	OwnerLayout->TextLayout->GetAsTextAndOffsets(nullptr, &OffsetLocations, GIMETSFLineTerminator);

	// We need to translate the indices into document space
	const FTextLocation MinTextLocation = OffsetLocations.OffsetToTextLocation(MinIndex);
	const FTextLocation MaxTextLocation = OffsetLocations.OffsetToTextLocation(MaxIndex);

	OwnerLayout->ClearSelection();

	switch (InCaretPosition)
	{
	case ITextInputMethodContext::ECaretPosition::Beginning:
		{
			OwnerLayout->CursorInfo.SetCursorLocationAndCalculateAlignment(*OwnerLayout->TextLayout, MinTextLocation);
			OwnerLayout->SelectionStart = MaxTextLocation;
		}
		break;

	case ITextInputMethodContext::ECaretPosition::Ending:
		{
			OwnerLayout->SelectionStart = MinTextLocation;
			OwnerLayout->CursorInfo.SetCursorLocationAndCalculateAlignment(*OwnerLayout->TextLayout, MaxTextLocation);
		}
		break;
	}

	OwnerLayout->OwnerWidget->OnCursorMoved(OwnerLayout->CursorInfo.GetCursorInteractionLocation());
	OwnerLayout->UpdateCursorHighlight();
}

void FSlateEditableTextLayout::FTextInputMethodContext::GetTextInRange(const uint32 BeginIndex, const uint32 Length, FString& OutString)
{
	if (!OwnerLayout)
	{
		OutString.Reset();
		return;
	}

	// Use the same single-character (\n) line-terminator ACP space the rest of the IME path uses,
	// so BeginIndex/Length coming from TSF index into the correct characters across line breaks.
	FString FullText;
	OwnerLayout->TextLayout->GetAsTextAndOffsets(&FullText, nullptr, GIMETSFLineTerminator);
	OutString = FullText.Mid(BeginIndex, Length);
}

void FSlateEditableTextLayout::FTextInputMethodContext::SetTextInRange(const uint32 BeginIndex, const uint32 Length, const FString& InString)
{
	if (!OwnerLayout)
	{
		return;
	}

	// We don't use Start/FinishEditing text here because the whole IME operation handles that.
	// Also, we don't want to support undo for individual characters added during an IME context
	const FText OldEditedText = OwnerLayout->GetEditableText();

	// We do this as a select, delete, and insert as it's the simplest way to keep the text layout correct
	SetSelectionRange(BeginIndex, Length, ITextInputMethodContext::ECaretPosition::Beginning);
	OwnerLayout->DeleteSelectedText();
	OwnerLayout->InsertTextAtCursorImpl(InString);

	// Has the text changed?
	const FText EditedText = OwnerLayout->GetEditableText();
	const bool HasTextChanged = !EditedText.ToString().Equals(OldEditedText.ToString(), ESearchCase::CaseSensitive);
	if (HasTextChanged)
	{
		OwnerLayout->SaveText(EditedText);
		OwnerLayout->TextLayout->UpdateIfNeeded();
		OwnerLayout->BroadcastOnTextChanged(EditedText);
	}
}

int32 FSlateEditableTextLayout::FTextInputMethodContext::GetCharacterIndexFromPoint(const FVector2D& Point)
{
	if (!OwnerLayout)
	{
		return INDEX_NONE;
	}

	const FTextLocation CharacterPosition = OwnerLayout->TextLayout->GetTextLocationAt(Point * OwnerLayout->TextLayout->GetScale());

	FTextLayout::FTextOffsetLocations OffsetLocations;
	OwnerLayout->TextLayout->GetAsTextAndOffsets(nullptr, &OffsetLocations, GIMETSFLineTerminator);

	return OffsetLocations.TextLocationToOffset(CharacterPosition);
}

bool FSlateEditableTextLayout::FTextInputMethodContext::GetTextBounds(const uint32 BeginIndex, const uint32 Length, FVector2D& Position, FVector2D& Size)
{
	if (!OwnerLayout)
	{
		Position = FVector2D::ZeroVector;
		Size = FVector2D::ZeroVector;
		return false;
	}

	FTextLayout::FTextOffsetLocations OffsetLocations;
	OwnerLayout->TextLayout->GetAsTextAndOffsets(nullptr, &OffsetLocations, GIMETSFLineTerminator);

	const FTextLocation BeginLocation = OffsetLocations.OffsetToTextLocation(BeginIndex);
	const FTextLocation EndLocation = OffsetLocations.OffsetToTextLocation(BeginIndex + Length);

	const FVector2D BeginPosition = OwnerLayout->TextLayout->GetLocationAt(BeginLocation, false);
	const FVector2D EndPosition = OwnerLayout->TextLayout->GetLocationAt(EndLocation, false);

	const TArray< FTextLayout::FLineView >& LineViews = OwnerLayout->TextLayout->GetLineViews();
	int32 LineViewIndex = OwnerLayout->TextLayout->GetLineViewIndexForTextLocation(LineViews, BeginLocation, false);
	double BeginLineHeight{};
	if (LineViewIndex != INDEX_NONE)
	{
		BeginLineHeight = LineViews[LineViewIndex].Size.Y;
	}

	if (BeginPosition.Y == EndPosition.Y)
	{
		// The text range is contained within a single line
		Position = FVector2D(BeginPosition.X, BeginPosition.Y - BeginLineHeight);
		Size = EndPosition - Position;
	}
	else
	{
		// If the two positions aren't on the same line, then we assume the worst case scenario, and make the size as wide as the text area itself
		Position = FVector2D(0.0f, BeginPosition.Y - BeginLineHeight);
		Size = FVector2D(OwnerLayout->TextLayout->GetDrawSize().X, EndPosition.Y - Position.Y);
	}

	// Translate the position (which is in local space) into screen (absolute) space
	// Note: The local positions are pre-scaled, so we don't scale them again here
	Position += FVector2D(CachedGeometry.AbsolutePosition);
	
	return false; // false means "not clipped"
}

void FSlateEditableTextLayout::FTextInputMethodContext::GetScreenBounds(FVector2D& Position, FVector2D& Size)
{
	if (!OwnerLayout)
	{
		Position = FVector2D::ZeroVector;
		Size = FVector2D::ZeroVector;
		return;
	}

	Position = FVector2D(CachedGeometry.AbsolutePosition);
	Size = CachedGeometry.GetDrawSize();
}

void FSlateEditableTextLayout::FTextInputMethodContext::CacheWindow()
{
	if (!OwnerLayout)
	{
		return;
	}

	const TSharedRef<const SWidget> OwningSlateWidgetPtr = OwnerLayout->OwnerWidget->GetSlateWidget();
	CachedParentWindow = FSlateApplication::Get().FindWidgetWindow(OwningSlateWidgetPtr);
}

TSharedPtr<FGenericWindow> FSlateEditableTextLayout::FTextInputMethodContext::GetWindow()
{
	if (!OwnerLayout)
	{
		return nullptr;
	}

	const TSharedPtr<SWindow> SlateWindow = CachedParentWindow.Pin();
	return SlateWindow.IsValid() ? SlateWindow->GetNativeWindow() : nullptr;
}

void FSlateEditableTextLayout::FTextInputMethodContext::BeginComposition()
{
	if (!OwnerLayout)
	{
		return;
	}

	if (!bIsComposing)
	{
		bIsComposing = true;

		OwnerLayout->BeginEditTransaction();
		OwnerLayout->UpdateCursorHighlight();
	}
}

void FSlateEditableTextLayout::FTextInputMethodContext::UpdateCompositionRange(const int32 InBeginIndex, const uint32 InLength)
{
	if (!OwnerLayout)
	{
		return;
	}

	if (bIsComposing)
	{
		CompositionBeginIndex = InBeginIndex;
		CompositionLength = InLength;

		OwnerLayout->UpdateCursorHighlight();
	}
}

void FSlateEditableTextLayout::FTextInputMethodContext::EndComposition()
{
	if (!OwnerLayout)
	{
		return;
	}

	if (bIsComposing)
	{
		OwnerLayout->EndEditTransaction();
		OwnerLayout->UpdateCursorHighlight();

		bIsComposing = false;
	}
}

void FSlateEditableTextLayout::ToggleVirtualKeyboard(bool bShow, const int32 UserIndex)
{
	if (FSlateApplication::IsInitialized() && FPlatformApplicationMisc::RequiresVirtualKeyboard())
	{
		FSlateApplication::Get().ShowVirtualKeyboard(bShow, UserIndex, VirtualKeyboardEntry);
	}
}

void FSlateEditableTextLayout::SetOnAllFontFacesFinishLoadingDelegate(const FSimpleDelegate& InAllFontFacesFinishLoadingDelegate)
{
	if (TextLayout)
	{
		TextLayout->SetOnAllFontFacesFinishLoading(InAllFontFacesFinishLoadingDelegate);
	}
}

void FSlateEditableTextLayout::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	if (TextLayout)
	{
		TextLayout->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
	}
}

void FSlateEditableTextLayout::BroadcastOnTextChanged(const FText& InText)
{
	// We update the PreviousText so future text change events will trigger as expected.
	// This is needed because this OnTextChanged event can be broadcasted from a few places
	PreviousText = InText;
	OwnerWidget->OnTextChanged(InText);
}

void FSlateEditableTextLayout::OnBoundTextChanged()
{
	const TSharedRef<SWidget> Widget = OwnerWidget->GetSlateWidget();

	// During construction, we do not want to trigger events.
	// If we have focus, the bound text might have changed but the EditableText prevails and we don't want to replace it
	if (!OwnerWidget->IsWidgetConstructed() || (Widget->HasAnyUserFocus().IsSet() && !bForceBoundTextReview))
	{
		bForceBoundTextReview = false;
		return;
	}

	const FText NewBoundText = BoundText.Get();

	// We need to force an update if the text doesn't match the editable text, as the editable
	// text may not match the current bound text since it may have been changed by the user
	const FText EditableText = GetEditableText();
	const bool bForceRefresh = !EditableText.ToString().Equals(NewBoundText.ToString(), ESearchCase::CaseSensitive);

	// Only emit the "text changed" event if the text has actually been changed
	const bool bHasTextChanged = Widget->HasAnyUserFocus().IsSet()
		? !NewBoundText.ToString().Equals(EditableText.ToString(), ESearchCase::CaseSensitive)
		: !NewBoundText.ToString().Equals(PreviousText.ToString(), ESearchCase::CaseSensitive);

	if (RefreshImpl(&NewBoundText, bForceRefresh))
	{
		// Make sure we move the cursor to the end of the new text if we had keyboard focus, but only if we set it via SetText or LoadText
		if (bForceBoundTextReview && Widget->HasAnyUserFocus().IsSet() && !bWasFocusedByLastMouseDown)
		{
			JumpTo(ETextLocation::EndOfDocument, ECursorAction::MoveCursor);
		}

		// Let outsiders know that the text content has been changed
		if (bHasTextChanged)
		{
			BroadcastOnTextChanged(NewBoundText);
		}
	}

	bForceBoundTextReview = false;
}

void FSlateEditableTextLayout::OnBoundSearchTextChanged(SWidget& Widget, FSlateEditableTextLayout& SlateTextLayout)
{
	const FText& SearchTextToSet = SlateTextLayout.BoundSearchText.Get();
	SlateTextLayout.BoundSearchTextLastTick = FTextSnapshot(SearchTextToSet);
	SlateTextLayout.BeginSearch(SearchTextToSet);
}

void FSlateEditableTextLayout::OnIsReadOnlyChanged(SWidget& Widget, FSlateEditableTextLayout& SlateTextLayout)
{
	SlateTextLayout.CursorLineHighlighter->SetDrawCursor(!SlateTextLayout.bIsReadOnlyAttribute.Get());
}
