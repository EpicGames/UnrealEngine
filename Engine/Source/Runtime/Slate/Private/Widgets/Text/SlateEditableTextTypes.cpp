// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/SlateEditableTextTypes.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/TextEditHelper.h"

namespace SlateEditableTextTypes
{

void FCursorInfo::UpdateLastCursorInteractionTime()
{
	LastCursorInteractionTime = FSlateApplication::Get().GetCurrentTime();
}

void FCursorInfo::SetCursorLocationAndCalculateAlignment(const FTextLayout& InTextLayout, const FTextLocation& InCursorPosition)
{
	FTextLocation NewCursorPosition = InCursorPosition;
	ECursorAlignment NewAlignment = ECursorAlignment::Left;

	{
		const int32 CursorLineIndex = InCursorPosition.GetLineIndex();
		const int32 CursorOffset = InCursorPosition.GetOffset();

		// A CursorOffset of zero could mark the end of an empty line, but we don't need to adjust the cursor for an empty line
		if (CursorOffset > 0)
		{
			const TArray<FTextLayout::FLineModel>& Lines = InTextLayout.GetLineModels();
			const FTextLayout::FLineModel& Line = Lines[CursorLineIndex];
			if (Line.Text->Len() == CursorOffset)
			{
				// We need to move the cursor back one from where it currently is; this keeps the interaction point the same 
				// (since the cursor is aligned to the right), but visually draws the cursor in the correct place
				NewCursorPosition = FTextLocation(NewCursorPosition, -1);
				NewAlignment = ECursorAlignment::Right;
			}
		}
	}

	SetCursorLocationAndAlignment(InTextLayout, NewCursorPosition, NewAlignment);
}

void FCursorInfo::SetCursorLocationAndAlignment(const FTextLayout& InTextLayout, const FTextLocation& InCursorPosition, const ECursorAlignment InCursorAlignment)
{
	CursorPosition = InCursorPosition;
	CursorAlignment = InCursorAlignment;
	CursorTextDirection = TextBiDi::ETextDirection::LeftToRight;

	// Get the text direction for the block under the cursor
	{
		const TArray<FTextLayout::FLineView>& LineViews = InTextLayout.GetLineViews();
		const int32 LineViewIndex = InTextLayout.GetLineViewIndexForTextLocation(LineViews, InCursorPosition, InCursorAlignment == ECursorAlignment::Right);
		if (LineViews.IsValidIndex(LineViewIndex))
		{
			const FTextLayout::FLineView& LineView = LineViews[LineViewIndex];
			const int32 CursorOffset = InCursorPosition.GetOffset();

			for (const auto& Block : LineView.Blocks)
			{
				const FTextRange BlockTextRange = Block->GetTextRange();
				if (BlockTextRange.Contains(CursorOffset))
				{
					CursorTextDirection = Block->GetTextContext().TextDirection;
					break;
				}
			}
		}
	}

	UpdateLastCursorInteractionTime();
}

FCursorInfo FCursorInfo::CreateUndo() const
{
	FCursorInfo UndoData;
	UndoData.CursorPosition = CursorPosition;
	UndoData.CursorAlignment = CursorAlignment;
	UndoData.CursorTextDirection = CursorTextDirection;
	return UndoData;
}

void FCursorInfo::RestoreFromUndo(const FCursorInfo& UndoData)
{
	CursorPosition = UndoData.CursorPosition;
	CursorAlignment = UndoData.CursorAlignment;
	CursorTextDirection = UndoData.CursorTextDirection;

	UpdateLastCursorInteractionTime();
}

FCursorLineHighlighter::FCursorLineHighlighter(const FCursorInfo* InCursorInfo)
	: CursorInfo(InCursorInfo)
{
	check(CursorInfo);
	CursorBrush = FCoreStyle::Get().GetBrush("EditableText.SelectionBackground");
}

void FCursorLineHighlighter::SetCursorBrush(const TAttribute<const FSlateBrush*>& InCursorBrush)
{
	CursorBrush = InCursorBrush;
}

int32 FCursorLineHighlighter::OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (!bDrawCursor)
	{
		return LayerId;
	}

	const FVector2D Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y);
	const FVector2D Size(Width, Line.TextHeight);

	FLinearColor CursorColorAndOpacity = InWidgetStyle.GetForegroundColor();

	const float FontMaxCharHeight = FTextEditHelper::GetFontHeight(DefaultStyle.Font);
	const float CursorWidth = FTextEditHelper::CalculateCaretWidth(FontMaxCharHeight);
	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	// The cursor is always visible (i.e. not blinking) when we're interacting with it; otherwise it might get lost.
	const double TimeSinceLastInteraction = CurrentTime - CursorInfo->GetLastCursorInteractionTime();
	const bool bForceCursorVisible = TimeSinceLastInteraction < EditableTextDefs::CaretBlinkPauseTime;
	float CursorOpacity = (bForceCursorVisible)
		? 1.0f
		: FMath::RoundToFloat(FMath::MakePulsatingValue(TimeSinceLastInteraction, EditableTextDefs::BlinksPerSecond));

	CursorOpacity *= CursorOpacity;	// Squared falloff, because it looks more interesting
	CursorColorAndOpacity.A = CursorOpacity;

	ECursorAlignment VisualCursorAlignment = CursorInfo->GetCursorAlignment();
	if (CursorInfo->GetCursorTextDirection() == TextBiDi::ETextDirection::RightToLeft)
	{
		// We need to flip the visual alignment for right-to-left text, as the start of the glyph is one the right hand side of the highlight
		VisualCursorAlignment = VisualCursorAlignment == ECursorAlignment::Left ? ECursorAlignment::Right : ECursorAlignment::Left;
	}
	const FVector2D OptionalWidth = VisualCursorAlignment == ECursorAlignment::Right ? FVector2D(Size.X, 0) : FVector2D::ZeroVector;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, FVector2D(FMath::Max(CursorWidth * AllottedGeometry.Scale, 1.0f), Size.Y)), FSlateLayoutTransform(TransformPoint(InverseScale, Location + OptionalWidth))),
		CursorBrush.Get(),
		bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		CursorColorAndOpacity*InWidgetStyle.GetColorAndOpacityTint()
		);

	return LayerId;
}

TSharedRef<FCursorLineHighlighter> FCursorLineHighlighter::Create(const FCursorInfo* InCursorInfo)
{
	return MakeShareable(new FCursorLineHighlighter(InCursorInfo));
}

FTextCompositionHighlighter::FTextCompositionHighlighter()
{
	CompositionBrush = FCoreStyle::Get().GetBrush("EditableText.CompositionBackground");
}

void FTextCompositionHighlighter::SetCompositionBrush(const TAttribute<const FSlateBrush*>& InCompositionBrush)
{
	CompositionBrush = InCompositionBrush;
}

int32 FTextCompositionHighlighter::OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector2D Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y);
	const FVector2D Size(Width, Line.TextHeight);

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	if (Size.X)
	{
		const FLinearColor LineColorAndOpacity = InWidgetStyle.GetForegroundColor();

		// Draw composition background
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, Size), FSlateLayoutTransform(TransformPoint(InverseScale, Location))),
			CompositionBrush.Get(),
			bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			LineColorAndOpacity * InWidgetStyle.GetColorAndOpacityTint()
			);
	}

	return LayerId;
}

TSharedRef<FTextCompositionHighlighter> FTextCompositionHighlighter::Create()
{
	return MakeShareable(new FTextCompositionHighlighter());
}

FTextSelectionHighlighter::FTextSelectionHighlighter()
{
}

int32 FTextSelectionHighlighter::OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector2D Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y);

	// If we've not been set to an explicit color, calculate a suitable one from the linked color
	const FLinearColor SelectionBackgroundColorAndOpacity = DefaultStyle.SelectedBackgroundColor.IsColorSpecified()
		? DefaultStyle.SelectedBackgroundColor.GetSpecifiedColor() * InWidgetStyle.GetColorAndOpacityTint()
		: ((FLinearColor::White - DefaultStyle.SelectedBackgroundColor.GetColor(InWidgetStyle))*0.5f + FLinearColor(-0.2f, -0.05f, 0.15f)) * InWidgetStyle.GetColorAndOpacityTint();

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	// We still want to show a small selection outline on empty lines to make it clear that the line itself is selected despite being empty
	const float MinHighlightWidth = (Line.Range.IsEmpty()) ? 4.0f * AllottedGeometry.Scale : 0.0f;
	const float HighlightWidth = FMath::Max(Width, MinHighlightWidth);
	if (HighlightWidth > 0.0f)
	{
		// Draw the actual highlight rectangle
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, FVector2D(HighlightWidth, FMath::Max(Line.Size.Y, Line.TextHeight))), FSlateLayoutTransform(TransformPoint(InverseScale, Location))),
			&DefaultStyle.HighlightShape,
			bParentEnabled && bHasKeyboardFocus ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			SelectionBackgroundColorAndOpacity
			);
	}

	return LayerId;
}

TSharedRef<FTextSelectionHighlighter> FTextSelectionHighlighter::Create()
{
	return MakeShareable(new FTextSelectionHighlighter());
}

FTextSearchHighlighter::FTextSearchHighlighter()
{
}

int32 FTextSearchHighlighter::OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector2D Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y);

	// If we've not been set to an explicit color, calculate a suitable one from the linked color
	FLinearColor SelectionBackgroundColorAndOpacity = DefaultStyle.HighlightColor.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
	SelectionBackgroundColorAndOpacity.A *= 0.2f;

	// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
	const float InverseScale = Inverse(AllottedGeometry.Scale);

	if (Width > 0.0f)
	{
		// Draw the actual highlight rectangle
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, FVector2D(Width, FMath::Max(Line.Size.Y, Line.TextHeight))), FSlateLayoutTransform(TransformPoint(InverseScale, Location))),
			&DefaultStyle.HighlightShape,
			bParentEnabled && bHasKeyboardFocus ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			SelectionBackgroundColorAndOpacity
			);
	}

	return LayerId;
}

TSharedRef<FTextSearchHighlighter> FTextSearchHighlighter::Create()
{
	return MakeShareable(new FTextSearchHighlighter());
}

} // namespace SlateEditableTextTypes
