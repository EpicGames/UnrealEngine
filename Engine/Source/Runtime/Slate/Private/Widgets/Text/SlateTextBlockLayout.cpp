// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Text/SlateTextBlockLayout.h"
#include "Fonts/FontCache.h"
#include "Framework/Text/SlateTextHighlightRunRenderer.h"

FSlateTextBlockLayout::FSlateTextBlockLayout(SWidget* InOwner, FTextBlockStyle InDefaultTextStyle, const TOptional<ETextShapingMethod> InTextShapingMethod, const TOptional<ETextFlowDirection> InTextFlowDirection, const FCreateSlateTextLayout& InCreateSlateTextLayout, TSharedRef<ITextLayoutMarshaller> InMarshaller, TSharedPtr<IBreakIterator> InLineBreakPolicy)
	: TextLayout((InCreateSlateTextLayout.IsBound()) ? InCreateSlateTextLayout.Execute(InOwner, MoveTemp(InDefaultTextStyle)) : FSlateTextLayout::Create(InOwner, MoveTemp(InDefaultTextStyle)))
	, Marshaller(MoveTemp(InMarshaller))
	, TextHighlighter(FSlateTextHighlightRunRenderer::Create())
	, CachedSize(ForceInitToZero)
	, CachedWrapTextAt(0)
	, bCachedAutoWrapText(false)
{
	if (InTextShapingMethod.IsSet())
	{
		TextLayout->SetTextShapingMethod(InTextShapingMethod.GetValue());
	}

	if (InTextFlowDirection.IsSet())
	{
		TextLayout->SetTextFlowDirection(InTextFlowDirection.GetValue());
	}

	TextLayout->SetLineBreakIterator(MoveTemp(InLineBreakPolicy));
}

void FSlateTextBlockLayout::ConditionallyUpdateTextStyle(const FTextBlockStyle& InTextStyle)
{
	// Has the style used for this text block changed?
	if (!IsStyleUpToDate(InTextStyle))
	{
		UpdateTextStyle(InTextStyle);
	}
}

void FSlateTextBlockLayout::ConditionallyUpdateTextStyle(const FTextBlockStyle::CompareParams& InNewStyleParams)
{
	// Has the style used for this text block changed?
	if (!IsStyleUpToDate(InNewStyleParams))
	{
		UpdateTextStyle(InNewStyleParams);
	}
}

void FSlateTextBlockLayout::UpdateTextStyle(const FTextBlockStyle& InTextStyle)
{
	TextLayout->SetDefaultTextStyle(InTextStyle);
	Marshaller->MakeDirty(); // will regenerate the text in ComputeDesiredSize()
}

void FSlateTextBlockLayout::UpdateTextStyle(const FTextBlockStyle::CompareParams& InNewStyleParams)
{
	FTextBlockStyle ComputedStyle = InNewStyleParams.StyleBase;
	ComputedStyle.SetFont(InNewStyleParams.Font);
	if (InNewStyleParams.StrikeBrush)
	{
		ComputedStyle.SetStrikeBrush(*InNewStyleParams.StrikeBrush);
	}
	ComputedStyle.SetColorAndOpacity(InNewStyleParams.ColorAndOpacity);
	ComputedStyle.SetShadowOffset(InNewStyleParams.ShadowOffset);
	ComputedStyle.SetShadowColorAndOpacity(InNewStyleParams.ShadowColorAndOpacity);
	ComputedStyle.SetHighlightColor(InNewStyleParams.HighlightColor);
	if (InNewStyleParams.HighlightShape)
	{
		ComputedStyle.SetHighlightShape(*InNewStyleParams.HighlightShape);
	}

	UpdateTextStyle(ComputedStyle);
}


FVector2D FSlateTextBlockLayout::ComputeDesiredSize(const FWidgetDesiredSizeArgs& InWidgetArgs, const float InScale)
{
	// Cache the wrapping rules so that we can recompute the wrap at width in paint.
	CachedWrapTextAt = InWidgetArgs.WrapTextAt;
	bCachedAutoWrapText = InWidgetArgs.AutoWrapText;

	const ETextTransformPolicy PreviousTransformPolicy = TextLayout->GetTransformPolicy();

	// Set the text layout information
	TextLayout->SetScale(InScale);
	TextLayout->SetWrappingWidth(CalculateWrappingWidth());
	TextLayout->SetWrappingPolicy(InWidgetArgs.WrappingPolicy);
	TextLayout->SetTransformPolicy(InWidgetArgs.TransformPolicy);
	TextLayout->SetMargin(InWidgetArgs.Margin);
	TextLayout->SetJustification(InWidgetArgs.Justification);
	TextLayout->SetLineHeightPercentage(InWidgetArgs.LineHeightPercentage);
	TextLayout->SetApplyLineHeightToBottomLine(InWidgetArgs.ApplyLineHeightToBottomLine);

	// Has the transform policy changed? If so we need a full refresh as that is destructive to the model text
	if (PreviousTransformPolicy != TextLayout->GetTransformPolicy())
	{
		Marshaller->MakeDirty();
	}

	{
		bool bRequiresTextUpdate = false;
		const FText& TextToSet = InWidgetArgs.Text;
		if (!TextLastUpdate.IdenticalTo(TextToSet))
		{
			// The pointer used by the bound text has changed, however the text may still be the same - check that now
			if (!TextLastUpdate.IsDisplayStringEqualTo(TextToSet))
			{
				// The source text has changed, so update the internal text
				bRequiresTextUpdate = true;
			}

			// Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next Tick
			TextLastUpdate = FTextSnapshot(TextToSet);
		}

		if (bRequiresTextUpdate || Marshaller->IsDirty())
		{
			UpdateTextLayout(TextToSet);
		}
	}

	{
		const FText& HighlightTextToSet = InWidgetArgs.HighlightText;
		if (!HighlightTextLastUpdate.IdenticalTo(HighlightTextToSet))
		{
			// The pointer used by the bound text has changed, however the text may still be the same - check that now
			if (!HighlightTextLastUpdate.IsDisplayStringEqualTo(HighlightTextToSet))
			{
				UpdateTextHighlights(HighlightTextToSet);
			}

			// Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next Tick
			HighlightTextLastUpdate = FTextSnapshot(HighlightTextToSet);
		}
	}

	// We need to update our size if the text layout has become dirty
	TextLayout->UpdateIfNeeded();

	return TextLayout->GetSize();
}

FVector2D FSlateTextBlockLayout::ComputeDesiredSize(const FWidgetDesiredSizeArgs& InWidgetArgs, const float InScale, const FTextBlockStyle& InTextStyle)
{
	ConditionallyUpdateTextStyle(InTextStyle);
	return ComputeDesiredSize(InWidgetArgs, InScale);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVector2D FSlateTextBlockLayout::ComputeDesiredSize(const FWidgetArgs& InWidgetArgs, const float InScale, const FTextBlockStyle& InTextStyle)
{
	// Cache the wrapping rules so that we can recompute the wrap at width in paint.
	CachedWrapTextAt = InWidgetArgs.WrapTextAt.Get(0.0f);
	bCachedAutoWrapText = InWidgetArgs.AutoWrapText.Get(false);

	const ETextTransformPolicy PreviousTransformPolicy = TextLayout->GetTransformPolicy();

	// Set the text layout information
	TextLayout->SetScale(InScale);
	TextLayout->SetWrappingWidth(CalculateWrappingWidth());
	TextLayout->SetWrappingPolicy(InWidgetArgs.WrappingPolicy.Get());
	TextLayout->SetTransformPolicy(InWidgetArgs.TransformPolicy.Get());
	TextLayout->SetMargin(InWidgetArgs.Margin.Get());
	TextLayout->SetJustification(InWidgetArgs.Justification.Get());
	TextLayout->SetLineHeightPercentage(InWidgetArgs.LineHeightPercentage.Get());
	TextLayout->SetApplyLineHeightToBottomLine(true);

	// Has the transform policy changed? If so we need a full refresh as that is destructive to the model text
	if (PreviousTransformPolicy != TextLayout->GetTransformPolicy())
	{
		Marshaller->MakeDirty();
	}

	// Has the style used for this text block changed?
	if(!IsStyleUpToDate(InTextStyle))
	{
		TextLayout->SetDefaultTextStyle(InTextStyle);
		Marshaller->MakeDirty(); // will regenerate the text using the new default style
	}

	{
		bool bRequiresTextUpdate = false;
		const FText& TextToSet = InWidgetArgs.Text.Get(FText::GetEmpty());
		if(!TextLastUpdate.IdenticalTo(TextToSet))
		{
			// The pointer used by the bound text has changed, however the text may still be the same - check that now
			if(!TextLastUpdate.IsDisplayStringEqualTo(TextToSet))
			{
				// The source text has changed, so update the internal text
				bRequiresTextUpdate = true;
			}

			// Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next Tick
			TextLastUpdate = FTextSnapshot(TextToSet);
		}

		if(bRequiresTextUpdate || Marshaller->IsDirty())
		{
			UpdateTextLayout(TextToSet);
		}
	}

	{
		const FText& HighlightTextToSet = InWidgetArgs.HighlightText.Get(FText::GetEmpty());
		if(!HighlightTextLastUpdate.IdenticalTo(HighlightTextToSet))
		{
			// The pointer used by the bound text has changed, however the text may still be the same - check that now
			if(!HighlightTextLastUpdate.IsDisplayStringEqualTo(HighlightTextToSet))
			{
				UpdateTextHighlights(HighlightTextToSet);
			}

			// Update this even if the text is lexically identical, as it will update the pointer compared by IdenticalTo for the next Tick
			HighlightTextLastUpdate = FTextSnapshot(HighlightTextToSet);
		}
	}

	// We need to update our size if the text layout has become dirty
	TextLayout->UpdateIfNeeded();

	return TextLayout->GetSize();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FVector2D FSlateTextBlockLayout::GetDesiredSize() const
{
	return TextLayout->GetSize();
}

float FSlateTextBlockLayout::GetLayoutScale() const
{
	return TextLayout->GetScale();
}

int32 FSlateTextBlockLayout::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled)
{
	if (!TextLayout || TextLayout->IsEmpty())
	{
		return LayerId;
	}

	// Store a new cached size with the scale
	CachedSize = FVector2f(InAllottedGeometry.GetLocalSize());

	// Recompute wrapping in case the cached size changed.
	TextLayout->SetWrappingWidth(CalculateWrappingWidth());

	// Text blocks don't have scroll bars, so when the visible region is smaller than the desired size, 
	// we attempt to auto-scroll to keep the view of the text aligned with the current justification method
	const ETextJustify::Type VisualJustification = TextLayout->GetVisualJustification();
	FVector2D AutoScrollValue = FVector2D::ZeroVector; // Scroll to the left
	if(VisualJustification != ETextJustify::Left)
	{
		const float ActualWidth = TextLayout->GetSize().X;
		const float VisibleWidth = CachedSize.X;
		if(VisibleWidth < ActualWidth)
		{
			switch(VisualJustification)
			{
			case ETextJustify::Center:
				AutoScrollValue.X = (ActualWidth - VisibleWidth) * 0.5f; // Scroll to the center
				break;

			case ETextJustify::Right:
				AutoScrollValue.X = (ActualWidth - VisibleWidth); // Scroll to the right
				break;

			default:
				break;
			}
		}
	}

	TextLayout->SetVisibleRegion(FVector2D(CachedSize), AutoScrollValue * TextLayout->GetScale());
	TextLayout->UpdateIfNeeded();

	return TextLayout->OnPaint(InPaintArgs, InAllottedGeometry, InClippingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

void FSlateTextBlockLayout::DirtyLayout()
{
	TextLayout->DirtyLayout();
}

void FSlateTextBlockLayout::DirtyContent()
{
	DirtyLayout();
	Marshaller->MakeDirty();
}

void FSlateTextBlockLayout::OverrideTextStyle(const FTextBlockStyle& InTextStyle)
{
	// Has the style used for this text block changed?
	if(!IsStyleUpToDate(InTextStyle))
	{
		TextLayout->SetDefaultTextStyle(InTextStyle);
		
		FString CurrentText;
		Marshaller->GetText(CurrentText, *TextLayout);
		UpdateTextLayout(CurrentText);
	}
}

void FSlateTextBlockLayout::SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod)
{
	TextLayout->SetTextShapingMethod((InTextShapingMethod.IsSet()) ? InTextShapingMethod.GetValue() : GetDefaultTextShapingMethod());
}

void FSlateTextBlockLayout::SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection)
{
	TextLayout->SetTextFlowDirection((InTextFlowDirection.IsSet()) ? InTextFlowDirection.GetValue() : GetDefaultTextFlowDirection());
}

void FSlateTextBlockLayout::SetTextOverflowPolicy(const TOptional<ETextOverflowPolicy> InTextOverflowPolicy)
{
	TextLayout->SetTextOverflowPolicy(InTextOverflowPolicy);
}

void FSlateTextBlockLayout::SetDebugSourceInfo(const TAttribute<FString>& InDebugSourceInfo)
{
	TextLayout->SetDebugSourceInfo(InDebugSourceInfo);
}

FChildren* FSlateTextBlockLayout::GetChildren()
{
	return TextLayout->GetChildren();
}

void FSlateTextBlockLayout::ArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	TextLayout->ArrangeChildren(AllottedGeometry, ArrangedChildren);
}

void FSlateTextBlockLayout::UpdateTextLayout(const FText& InText)
{
	UpdateTextLayout(InText.ToString());
}

void FSlateTextBlockLayout::UpdateTextLayout(const FString& InText)
{
	Marshaller->ClearDirty();
	TextLayout->ClearLines();

	TextLayout->ClearLineHighlights();
	TextLayout->ClearRunRenderers();

	Marshaller->SetText(InText, *TextLayout);

	HighlightTextLastUpdate = FTextSnapshot();
}

void FSlateTextBlockLayout::UpdateTextHighlights(const FText& InHighlightText)
{
	const FString& HighlightTextString = InHighlightText.ToString();
	const int32 HighlightTextLength = HighlightTextString.Len();

	const TArray< FTextLayout::FLineModel >& LineModels = TextLayout->GetLineModels();

	TArray<FTextRunRenderer> TextHighlights;
	if (HighlightTextString.Len() > 0)
	{
		for(int32 LineIndex = 0; LineIndex < LineModels.Num(); ++LineIndex)
		{
			const FTextLayout::FLineModel& LineModel = LineModels[LineIndex];

			int32 FindBegin = 0;
			int32 CurrentHighlightBegin;
			const int32 TextLength = LineModel.Text->Len();
			while(FindBegin < TextLength && (CurrentHighlightBegin = LineModel.Text->Find(HighlightTextString, ESearchCase::IgnoreCase, ESearchDir::FromStart, FindBegin)) != INDEX_NONE)
			{
				FindBegin = CurrentHighlightBegin + HighlightTextLength;

				if(TextHighlights.Num() > 0 && TextHighlights.Last().LineIndex == LineIndex && TextHighlights.Last().Range.EndIndex == CurrentHighlightBegin)
				{
					TextHighlights[TextHighlights.Num() - 1] = FTextRunRenderer(LineIndex, FTextRange(TextHighlights.Last().Range.BeginIndex, FindBegin), TextHighlighter.ToSharedRef());
				}
				else
				{
					TextHighlights.Add(FTextRunRenderer(LineIndex, FTextRange(CurrentHighlightBegin, FindBegin), TextHighlighter.ToSharedRef()));
				}
			}
		}
	}

	TextLayout->SetRunRenderers(TextHighlights);
}

bool FSlateTextBlockLayout::IsStyleUpToDate(const FTextBlockStyle& NewStyle) const
{
	const FTextBlockStyle& CurrentStyle = TextLayout->GetDefaultTextStyle();
	return CurrentStyle.IsIdenticalTo(NewStyle);
}

bool FSlateTextBlockLayout::IsStyleUpToDate(const FTextBlockStyle::CompareParams& InNewStyleParams) const
{
	const FTextBlockStyle& CurrentStyle = TextLayout->GetDefaultTextStyle();
	return CurrentStyle.IsIdenticalTo(InNewStyleParams);
}

float FSlateTextBlockLayout::CalculateWrappingWidth() const
{
	// Text wrapping can either be used defined (WrapTextAt), automatic (bAutoWrapText and CachedSize), 
	// or a mixture of both. Take whichever has the smallest value (>1)
	float WrappingWidth = CachedWrapTextAt;
	if (bCachedAutoWrapText && CachedSize.X >= 1.0f)
	{
		WrappingWidth = (WrappingWidth >= 1.0f) ? FMath::Min(WrappingWidth, CachedSize.X) : CachedSize.X;
	}

	return FMath::Max(0.0f, WrappingWidth);
}

void FSlateTextBlockLayout::SetOnAllFontFacesFinishLoading(const FSimpleDelegate& InAllFontFacesFinishLoading)
{
	if (TextLayout)
	{
		TextLayout->SetOnAllFontFacesFinishLoading(InAllFontFacesFinishLoading);
	}
}

void FSlateTextBlockLayout::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	if (TextLayout)
	{
		TextLayout->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
	}
}
