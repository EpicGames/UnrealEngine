// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Views/SExpanderArrow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableRow.h"

SLATE_IMPLEMENT_WIDGET(SExpanderArrow)
void SExpanderArrow::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShouldDrawWires, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BaseIndentLevelAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IndentAmountAttribute, EInvalidateWidgetReason::Layout);
}

SExpanderArrow::SExpanderArrow()
	: StyleSet(nullptr)
	, ShouldDrawWires(*this, false)
	, IndentAmountAttribute(*this, 10.0f)
	, BaseIndentLevelAttribute(*this, 0)
{

}

SExpanderArrow::~SExpanderArrow() = default;

void SExpanderArrow::Construct( const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow  )
{
	OwnerRowPtr = TableRow;
	StyleSet = InArgs._StyleSet;
	SetIndentAmount(InArgs._IndentAmount);
	SetBaseIndentLevel(InArgs._BaseIndentLevel);
	ShouldDrawWires.Assign(*this, InArgs._ShouldDrawWires);

	this->ChildSlot
	.Padding( TAttribute<FMargin>( this, &SExpanderArrow::GetExpanderPadding ) )
	[
		SAssignNew(ExpanderArrow, SButton)
		.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Visibility( this, &SExpanderArrow::GetExpanderVisibility )
		.ClickMethod( EButtonClickMethod::MouseDown )
		.OnClicked( this, &SExpanderArrow::OnArrowClicked )
		.ContentPadding(0.f)
		.ForegroundColor( FSlateColor::UseForeground() )
		.IsFocusable( false )
		[
			SNew(SImage)
			.Image( this, &SExpanderArrow::GetExpanderImage )
			.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
		]
	];
}


int32 SExpanderArrow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{

	static const float WireThickness = 2.0f;
	static const float HalfWireThickness = WireThickness / 2.0f;

	// We want to support drawing wires for the tree
	//                 Needs Wire Array
	//   v-[A]         {}
	//   |-v[B]        {1}
	//   | '-v[B]      {1,1}
	//   |   |--[C]    {1,0,1}
	//   |   |--[D]    {1,0,1}
	//   |   '--[E]    {1,0,1} 
	//   |>-[F]        {}
	//   '--[G]        {}
	//   
	//

	static const FName NAME_VerticalBarBrush = "WhiteBrush";
	const float Indent = IndentAmountAttribute.Get();
	const FSlateBrush* VerticalBarBrush = (StyleSet == nullptr) ? nullptr : StyleSet->GetBrush(NAME_VerticalBarBrush);

	if (ShouldDrawWires.Get() == true && VerticalBarBrush != nullptr)
	{
		const TSharedPtr<ITableRow> OwnerRow = OwnerRowPtr.Pin();
		FLinearColor WireTint = InWidgetStyle.GetForegroundColor();
		WireTint.A = 0.15f;

		// Draw vertical wires to indicate paths to parent nodes.
		const TBitArray<>& NeedsWireByLevel = OwnerRow->GetWiresNeededByDepth();
		const int32 NumLevels = NeedsWireByLevel.Num();
		for (int32 Level = 0; Level < NumLevels; ++Level)
		{
			const float CurrentIndent = Indent * Level;

			if (NeedsWireByLevel[Level])
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, 0))),
					VerticalBarBrush,
					ESlateDrawEffect::None,
					WireTint
				);
			}
		}

		const float HalfCellHeight = 0.5f * AllottedGeometry.Size.Y;

		// For items that are the last expanded child in a list, we need to draw a special angle connector wire.
		if (const bool bIsLastChild = OwnerRow->IsLastChild())
		{
			const float CurrentIndent = Indent * (NumLevels-1);
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, HalfCellHeight + HalfWireThickness), FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, 0))),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				WireTint
			);
		}

		// If this item is expanded, we need to draw a 1/2-height the line down to its first child cell.
		if ( const bool bItemAppearsExpanded = OwnerRow->IsItemExpanded() && OwnerRow->DoesItemHaveChildren())
		{
			const float CurrentIndent = Indent * NumLevels;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, HalfCellHeight+ HalfWireThickness), FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, HalfCellHeight- HalfWireThickness))),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				WireTint
			);
		}

		// Draw horizontal connector from parent wire to child.
		if (NumLevels > 1)
		{
			float LeafDepth = OwnerRow->DoesItemHaveChildren() ? 10.f : 0.0f;
			const float HorizontalWireStart = (NumLevels - 1)*Indent;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(
					FVector2D(AllottedGeometry.Size.X - HorizontalWireStart - WireThickness - LeafDepth, WireThickness),
					FSlateLayoutTransform(FVector2D(HorizontalWireStart + WireThickness - 3.f, 0.5f*(AllottedGeometry.Size.Y - WireThickness)))
				),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				WireTint
			);
		}	
	}

	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	return LayerId;
}

/** Invoked when the expanded button is clicked (toggle item expansion) */
FReply SExpanderArrow::OnArrowClicked()
{
	// Recurse the expansion if "shift" is being pressed
	const FModifierKeysState ModKeyState = FSlateApplication::Get().GetModifierKeys();
	if(ModKeyState.IsShiftDown())
	{
		OwnerRowPtr.Pin()->Private_OnExpanderArrowShiftClicked();
	}
	else
	{
		OwnerRowPtr.Pin()->ToggleExpansion();
	}

	return FReply::Handled();
}

void SExpanderArrow::SetIndentAmount(TAttribute<float> InIndentAmount)
{
	IndentAmountAttribute.Assign(*this, MoveTemp(InIndentAmount));
}

void SExpanderArrow::SetBaseIndentLevel(TAttribute<int32> InBaseIndentLevel)
{
	BaseIndentLevelAttribute.Assign(*this,  MoveTemp(InBaseIndentLevel));
}

/** @return Visible when has children; invisible otherwise */
EVisibility SExpanderArrow::GetExpanderVisibility() const
{
	return OwnerRowPtr.Pin()->DoesItemHaveChildren() ? EVisibility::Visible : EVisibility::Hidden;
}

/** @return the margin corresponding to how far this item is indented */
FMargin SExpanderArrow::GetExpanderPadding() const
{
	const int32 NestingDepth = FMath::Max(0, OwnerRowPtr.Pin()->GetIndentLevel() - BaseIndentLevelAttribute.Get());
	const float Indent = IndentAmountAttribute.Get();
	return FMargin( NestingDepth * Indent, 0,0,0 );
}

/** @return the name of an image that should be shown as the expander arrow */
const FSlateBrush* SExpanderArrow::GetExpanderImage() const
{
	const bool bIsItemExpanded = OwnerRowPtr.Pin()->IsItemExpanded();

	FName ResourceName;
	if (bIsItemExpanded)
	{
		if ( ExpanderArrow->IsHovered() )
		{
			static FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
			ResourceName = ExpandedHoveredName;
		}
		else
		{
			static FName ExpandedName = "TreeArrow_Expanded";
			ResourceName = ExpandedName;
		}
	}
	else
	{
		if ( ExpanderArrow->IsHovered() )
		{
			static FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
			ResourceName = CollapsedHoveredName;
		}
		else
		{
			static FName CollapsedName = "TreeArrow_Collapsed";
			ResourceName = CollapsedName;
		}
	}

	return StyleSet->GetBrush(ResourceName);
}

