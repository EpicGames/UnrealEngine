// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Layout/SSplitter.h"
#include "Types/SlateStructs.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SHeaderRow"

template <>
struct TWidgetTypeTraits<class STableColumnHeader>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class STableColumnHeader : public SCompoundWidget
{
public:
	 
	SLATE_BEGIN_ARGS(STableColumnHeader)
		: _Style( &FAppStyle::Get().GetWidgetStyle<FTableColumnHeaderStyle>("TableView.Header.Column") )
		{}
		SLATE_STYLE_ARGUMENT( FTableColumnHeaderStyle, Style )

	SLATE_END_ARGS()

	STableColumnHeader()
		: InitialSortMode( EColumnSortMode::Ascending )
		, SortMode( EColumnSortMode::None )
		, SortPriority( EColumnSortPriority::Primary )
		, IsSorting( false )
		, OnSortModeChanged()
		, CanManuallySort( true )
		, ContextMenuContent( SNullWidget::NullWidget )
		, ColumnId( NAME_None )
		, Style( nullptr )
	{

	}

	/**
	 * Construct the widget
	 * 
	 * @param InDeclaration   A declaration from which to construct the widget
	 */
	void Construct( const STableColumnHeader::FArguments& InArgs, const SHeaderRow::FColumn& Column, const TSharedRef<SWidget>& OverrideHeaderMenuContent, const FMargin DefaultHeaderContentPadding )
	{
		check(InArgs._Style);

		Style = InArgs._Style;
		ColumnId = Column.ColumnId;
		InitialSortMode = Column.InitialSortMode;
		SortMode = Column.SortMode;
		SortPriority = Column.SortPriority;
		IsSorting = Column.IsSorting;
		CanManuallySort = Column.CanManuallySort;

		OnSortModeChanged = Column.OnSortModeChanged;
		ContextMenuContent = OverrideHeaderMenuContent;

		ComboVisibility = Column.HeaderComboVisibility;

		bHasBeenManuallySorted = false;

		TAttribute< FText > LabelText = Column.DefaultText;
		if (Column.HeaderContent.Widget == SNullWidget::NullWidget)
		{
			if (!Column.DefaultText.IsSet())
			{
				LabelText = FText::FromString( Column.ColumnId.ToString() + TEXT("[LabelMissing]") );
			}
		}

		TSharedPtr< SHorizontalBox > Box;
		TSharedRef< SOverlay > Overlay = SNew( SOverlay );

		Overlay->AddSlot( 0 )
			[
				SAssignNew( Box, SHorizontalBox )
			];

		TSharedRef< SWidget > PrimaryContent = Column.HeaderContent.Widget;
		if ( PrimaryContent == SNullWidget::NullWidget )
		{
			PrimaryContent = 
				SNew( SBox )
				.HeightOverride( 24.0f )
				.Padding( 0.0f )
				.VAlign( VAlign_Center )
				[
					SNew( STextBlock )
					.TextStyle( FAppStyle::Get(), "NormalText" )
					.Text( LabelText )
					.OverflowPolicy(Column.OverflowPolicy)
				];
		}

		if ( OnSortModeChanged.IsBound() )
		{
			TSharedRef<SWidget> SortedHeaderContent = SNew(SBox)
				.HAlign(Column.HeaderHAlignment)
				.VAlign(Column.HeaderVAlignment)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0.0f))
						[
							PrimaryContent
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
						[
							SNew(SOverlay)
								+ SOverlay::Slot()
								[
									SNew(SImage)
										.ColorAndOpacity(FSlateColor::UseForeground())
										.Image(this, &STableColumnHeader::GetSortingBrush)
										.Visibility(this, &STableColumnHeader::GetSortModeVisibility)
								]
								+ SOverlay::Slot()
								[
									SNew(SCircularThrobber)
										.Radius(10.0f)
										.PieceImage(FAppStyle::Get().GetBrush("Throbber.CircleChunk.Small"))
										.Visibility(this, &STableColumnHeader::GetSortProcessingVisibility)
								]
						]
				];

			if (CanManuallySort.Get())
			{
				//optional main button with the column's title. Used to toggle sorting modes.
				Box->AddSlot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.ForegroundColor(FSlateColor::UseForeground())
							.OnClicked(this, &STableColumnHeader::OnTitleClicked)
							.ContentPadding(0.0f)
							[
								SortedHeaderContent
							]
					];
			}
			else
			{
				// Variation that shows sorting direction without allowing user controls over sorting
				Box->AddSlot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SortedHeaderContent							
					];
                   ApplySortMode();
			}

			
		}
		else
		{
			Box->AddSlot()
			.FillWidth( 1.0f )
			.HAlign( Column.HeaderHAlignment )
			.VAlign( Column.HeaderVAlignment )
			[
				PrimaryContent
			];
		}

		if( ComboVisibility != EHeaderComboVisibility::Never &&
			(Column.HeaderMenuContent.Widget != SNullWidget::NullWidget ||
			Column.OnGetMenuContent.IsBound()))
		{
			// Add Drop down menu button (only if menu content has been specified)
			Box->AddSlot()
			.AutoWidth()
			[
				SAssignNew( MenuOverlay, SOverlay )
				.Visibility( this, &STableColumnHeader::GetMenuOverlayVisibility )

				+SOverlay::Slot()
				[
					SNew( SBorder )
					.Padding( FMargin( 0.0f ) )
					.BorderImage( this, &STableColumnHeader::GetComboButtonBorderBrush )
					[
						SAssignNew( ComboButton, SComboButton )
						.HasDownArrow(false)
						.ButtonStyle( FAppStyle::Get(), "NoBorder" )
						.ContentPadding( FMargin(0) )
						.ButtonContent()
						[
							SNew( SSpacer )
							.Size( FVector2D( 14.0f, 0 ) )
						]
					]
				]

				+SOverlay::Slot()
				.HAlign( HAlign_Center )
				.VAlign( VAlign_Center )
				[
					SNew( SBox )
					.HeightOverride( 18.0f )
					.Padding( FMargin( 0.0f, -2.0f ) )
					[
						SNew( SImage )
						.Image( &Style->MenuDropdownImage )
						.ColorAndOpacity( this, &STableColumnHeader::GetComboButtonTint )
						.Visibility( EVisibility::HitTestInvisible )
					]
				]
			];

			if (Column.HeaderMenuContent.Widget != SNullWidget::NullWidget)
			{
				ComboButton->SetMenuContent( ContextMenuContent );
			}
			else if (Column.OnGetMenuContent.IsBound())
			{
				ComboButton->SetOnGetMenuContent( Column.OnGetMenuContent );
			}
		}

		this->ChildSlot
		[
			SNew( SBorder )
			.BorderImage( this, &STableColumnHeader::GetHeaderBackgroundBrush )
			.HAlign( HAlign_Fill )
			.VAlign( VAlign_Fill )
			.ToolTip( Column.ToolTip )
			.ToolTipText( Column.ToolTip.IsSet()                                 ? TAttribute<FText>() :
			              Column.DefaultTooltip.IsSet()                          ? Column.DefaultTooltip :
			              Column.HeaderContent.Widget == SNullWidget::NullWidget ? LabelText :
			                                                                       TAttribute<FText>() )
			.Padding( Column.HeaderContentPadding.Get( DefaultHeaderContentPadding ) )
			.Clipping( EWidgetClipping::ClipToBounds )
			[
				Overlay
			]
		];
	}

	/** Gets initial sorting mode */
	EColumnSortMode::Type GetInitialSortMode() const
	{
		return InitialSortMode.Get();
	}

	/** Sets initial sorting mode */
	void SetInitialSortMode(EColumnSortMode::Type NewMode)
	{
		InitialSortMode = NewMode;
	}

	/** Gets sorting mode */
	EColumnSortMode::Type GetSortMode() const
	{
		if (SortMode.Get() == EColumnSortMode::None && !CanManuallySort.Get() )
		{
			return GetInitialSortMode();
		}
		else
		{
			return SortMode.Get();
		}
	}

	/** Sets sorting mode */
	void SetSortMode( EColumnSortMode::Type NewMode )
	{
		SortMode = NewMode;
	}

	/** Gets sorting order */
	EColumnSortPriority::Type GetSortPriority() const
	{
		return SortPriority.Get();
	}

	/** Sets sorting order */
	void SetSortPriority(EColumnSortPriority::Type NewPriority)
	{
		SortPriority = NewPriority;
	}

	/** Get column id that generated this header */
	FName GetColumnId() const
	{
		return ColumnId;
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && ContextMenuContent != SNullWidget::NullWidget )
		{
			OpenContextMenu( MouseEvent );
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	EVisibility GetMenuOverlayVisibility() const
	{
		if (ComboVisibility == EHeaderComboVisibility::OnHover)
		{
			if (!ComboButton.IsValid() || !(IsHovered() || ComboButton->IsOpen()))
			{
				return EVisibility::Collapsed;
			}
		}

		if (ComboVisibility == EHeaderComboVisibility::Never)
		{
			return EVisibility::Collapsed;
		}

		return EVisibility::Visible;
	}

	UE::Slate::FDeprecateVector2DResult GetMenuOverlaySize() const 
	{ 
		return MenuOverlay.IsValid() ? MenuOverlay->GetDesiredSize() : FVector2f::ZeroVector; 
	}

private:


	const FSlateBrush* GetHeaderBackgroundBrush() const
	{
		if ( IsHovered() && SortMode.IsBound() )
		{
			return &Style->HoveredBrush;
		}

		return &Style->NormalBrush;
	}
	
	const FSlateBrush* GetComboButtonBorderBrush() const
	{
		if (ComboVisibility != EHeaderComboVisibility::Never)
		{
			if ( ComboButton.IsValid() && ( ComboButton->IsHovered() || ComboButton->IsOpen() ) )
			{
				return &Style->MenuDropdownHoveredBorderBrush;
			}

			if ( IsHovered() || ComboVisibility == EHeaderComboVisibility::Always )
			{
				return &Style->MenuDropdownNormalBorderBrush;
			}
		}

		return FStyleDefaults::GetNoBrush();
	}

	FSlateColor GetComboButtonTint() const
	{
		if (!ComboButton.IsValid())
		{
			return FLinearColor::White;
		}

		switch (ComboVisibility)
		{
		case EHeaderComboVisibility::Always:
			{
				return FLinearColor::White;
			}

		case EHeaderComboVisibility::Ghosted:
			if ( ComboButton->IsHovered() || ComboButton->IsOpen() )
			{
				return FLinearColor::White;
			}
			else
			{
				return FLinearColor::White.CopyWithNewOpacity(0.5f);
			}

		case EHeaderComboVisibility::OnHover:
			if ( IsHovered() || ComboButton->IsHovered() || ComboButton->IsOpen() )
			{
				return FLinearColor::White;
			}
			break;

		case EHeaderComboVisibility::Never:
			{
				return FLinearColor::White;
			}

		default:
			break;
		}
		
		return FLinearColor::Transparent;
	}

	/** Gets the icon associated with the current sorting mode */
	const FSlateBrush* GetSortingBrush() const
	{
		EColumnSortPriority::Type ColumnSortPriority = SortPriority.Get();

		return (GetSortMode() == EColumnSortMode::Ascending ?
			(ColumnSortPriority == EColumnSortPriority::Secondary ? &Style->SortSecondaryAscendingImage : &Style->SortPrimaryAscendingImage) :
			(ColumnSortPriority == EColumnSortPriority::Secondary ? &Style->SortSecondaryDescendingImage : &Style->SortPrimaryDescendingImage) );
	}

	/** Checks if sorting mode has been selected */
	EVisibility GetSortModeVisibility() const
	{
		EColumnSortMode::Type ColumnSortMode = GetSortMode();
		const bool bShow = ColumnSortMode != EColumnSortMode::None && bHasBeenManuallySorted && !IsSorting.Get();
		return bShow ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
	}

	/** Checks if sorting is in progress */
	EVisibility GetSortProcessingVisibility() const
	{
		return IsSorting.Get() ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
	}
	
	/** Called when the column title has been clicked to change sorting mode */
	FReply OnTitleClicked()
	{
		bHasBeenManuallySorted = true;
		ApplySortMode();
		return FReply::Handled();
	}

	void ApplySortMode()
	{
		if (OnSortModeChanged.IsBound())
		{
			FSlateApplication::Get().CloseToolTip();

			const bool bIsShiftClicked = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
			EColumnSortPriority::Type ColumnSortPriority = SortPriority.Get();
			EColumnSortMode::Type ColumnSortMode = GetSortMode();
			if (ColumnSortMode == EColumnSortMode::None)
			{
				if (bIsShiftClicked && SortPriority.IsBound())
				{
					ColumnSortPriority = EColumnSortPriority::Secondary;
				}
				else
				{
					ColumnSortPriority = EColumnSortPriority::Primary;
				}

				ColumnSortMode = InitialSortMode.Get();
			}
			else
			{
				if (!bIsShiftClicked && ColumnSortPriority == EColumnSortPriority::Secondary)
				{
					ColumnSortPriority = EColumnSortPriority::Primary;
				}

				if (ColumnSortMode == EColumnSortMode::Descending)
				{
					ColumnSortMode = EColumnSortMode::Ascending;
				}
				else
				{
					ColumnSortMode = EColumnSortMode::Descending;
				}
			}

			OnSortModeChanged.Execute(ColumnSortPriority, ColumnId, ColumnSortMode);
		}
	}

	void OpenContextMenu(const FPointerEvent& MouseEvent)
	{
		if ( ContextMenuContent != SNullWidget::NullWidget )
		{
			FVector2f SummonLocation = MouseEvent.GetScreenSpacePosition();
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

			FSlateApplication::Get().CloseToolTip();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, ContextMenuContent, SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		}
	}


private:

	/** Initial sorting mode */
	TAttribute< EColumnSortMode::Type > InitialSortMode;

	/** Current sorting mode */
	TAttribute< EColumnSortMode::Type > SortMode;

	/** Current sorting order */
	TAttribute< EColumnSortPriority::Type > SortPriority;
	
	/** Whether or not the list under the header is being sorted. */
	TAttribute< bool > IsSorting;

	/** Callback triggered when sorting mode changes */
	FOnSortModeChanged OnSortModeChanged;

	/** Whether or not the sorting mode should be under manual user control. */
	TAttribute< bool > CanManuallySort;

	TSharedRef< SWidget > ContextMenuContent;

	TSharedPtr< SComboButton > ComboButton;
	
	/** The visibility method of the combo button */
	EHeaderComboVisibility ComboVisibility;

	TSharedPtr< SOverlay > MenuOverlay;

	FName ColumnId;

	const FTableColumnHeaderStyle* Style;

	bool bHasBeenManuallySorted = false;
};

void SHeaderRow::Construct( const FArguments& InArgs )
{
	check(InArgs._Style);

	ScrollBarThickness = FVector2f::ZeroVector;
	ScrollBarVisibility = EVisibility::Collapsed;
	Style = InArgs._Style;
	OnGetMaxRowSizeForColumn = InArgs._OnGetMaxRowSizeForColumn;
	ResizeMode = InArgs._ResizeMode;

	SplitterHandleSize =   Style->SplitterHandleSize;
	if (InArgs._SplitterHandleSize.IsSet())
	{
		SplitterHandleSize = InArgs._SplitterHandleSize.GetValue();
	}
	bCanSelectGeneratedColumn = InArgs._CanSelectGeneratedColumn;
	OnHiddenColumnsListChanged = InArgs._OnHiddenColumnsListChanged;

	if ( InArgs._OnColumnsChanged.IsBound() )
	{
		ColumnsChanged.Add( InArgs._OnColumnsChanged );
	}

	SBorder::Construct( SBorder::FArguments()
		.Padding( 0.f )
		.BorderImage( &Style->BackgroundBrush )
		.ForegroundColor( Style->ForegroundColor )
	);

	// Copy all the column info from the declaration
	for ( FColumn* const Column : InArgs.Slots )
	{
		Columns.Add( Column );
		Column->bIsVisible = !InArgs._HiddenColumnsList.Contains(Column->ColumnId);
	}

	// Generate widgets for all columns
	RegenerateWidgets();
}

void SHeaderRow::ResetColumnWidths()
{
	for ( int32 iColumn = 0; iColumn < Columns.Num(); iColumn++ )
	{
		FColumn& Column = Columns[iColumn];
		Column.SetWidth( Column.DefaultWidth );
	}
}

const TIndirectArray<SHeaderRow::FColumn>& SHeaderRow::GetColumns() const
{
	return Columns;
}

int32 SHeaderRow::FindColumnIndex(const FName& InColumnId) const
{
	for (int32 iColumn = 0; iColumn < Columns.Num(); iColumn++ )
	{
		const FColumn& Column = Columns[iColumn];
		if (Column.ColumnId == InColumnId)
		{
			return iColumn;
		}
	}
	return INDEX_NONE;
}

void SHeaderRow::AddColumn( const FColumn::FArguments& NewColumnArgs )
{
	SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn( NewColumnArgs );
	AddColumn( *NewColumn );
}

void SHeaderRow::AddColumns( TConstArrayView<FColumn::FArguments> NewColumnArgsArray )
{
	Columns.Reserve(Columns.Num() + NewColumnArgsArray.Num());

	const bool bHasFillerColumn = (Columns.Num() > 0 && Columns[Columns.Num() - 1].ColumnId == NAME_None);

	for (FColumn::FArguments const& NewColumnArgs : NewColumnArgsArray)
	{
		SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn(NewColumnArgs);
		check(NewColumn->ColumnId != NAME_None);

		const int32 InsertIndex = bHasFillerColumn ? Columns.Num() - 1 : Columns.Num();
		Columns.Insert(NewColumn, InsertIndex);
	}

	ColumnsChanged.Broadcast(SharedThis(this));

	RegenerateWidgets();
}

void SHeaderRow::AddColumn( SHeaderRow::FColumn& NewColumn )
{
	int32 InsertIdx = Columns.Num();
	InsertColumn( NewColumn, InsertIdx );
}

void SHeaderRow::InsertColumn( const FColumn::FArguments& NewColumnArgs, int32 InsertIdx )
{
	SHeaderRow::FColumn* NewColumn = new SHeaderRow::FColumn( NewColumnArgs );
	InsertColumn( *NewColumn, InsertIdx );
}

void SHeaderRow::InsertColumn( FColumn& NewColumn, int32 InsertIdx )
{
	check(NewColumn.ColumnId != NAME_None);

	if ( Columns.Num() > 0 && Columns[Columns.Num() - 1].ColumnId == NAME_None )
	{
		// Insert before the filler column, or where the filler column used to be if we replaced it.
		InsertIdx--;
	}

	Columns.Insert( &NewColumn, InsertIdx );
	ColumnsChanged.Broadcast( SharedThis( this ) );

	RegenerateWidgets();
}

void SHeaderRow::RemoveColumn( const FName& InColumnId )
{
	check(InColumnId != NAME_None);

	for ( int32 SlotIndex=Columns.Num() - 1; SlotIndex >= 0; --SlotIndex )
	{
		FColumn& Column = Columns[SlotIndex];
		if ( Column.ColumnId == InColumnId )
		{
			Columns.RemoveAt(SlotIndex);
		}
	}

	ColumnsChanged.Broadcast( SharedThis( this ) );
	RegenerateWidgets();
}

void SHeaderRow::RemoveColumns(const int32 StartIdx, const int32 Count)
{
	Columns.RemoveAt(StartIdx, Count);

	ColumnsChanged.Broadcast( SharedThis( this ) );
	RegenerateWidgets();
}

void SHeaderRow::SwapColumns(const int32 FirstColumnIdx, const int32 SecondColumnIdx)
{
	if (Columns.IsValidIndex(FirstColumnIdx) && Columns.IsValidIndex(SecondColumnIdx) && FirstColumnIdx != SecondColumnIdx)
	{
		Columns.Swap(FirstColumnIdx, SecondColumnIdx);

		ColumnsChanged.Broadcast( SharedThis( this ) );
		RegenerateWidgets();
	}
}

void SHeaderRow::RefreshColumns()
{
	RegenerateWidgets();
}

void SHeaderRow::ClearColumns()
{
	const bool bHadColumnsItems = Columns.Num() > 0;
	Columns.Empty();
	if (bHadColumnsItems)
	{
		ColumnsChanged.Broadcast( SharedThis( this ) );
	}

	RegenerateWidgets();
}

void SHeaderRow::SetAssociatedVerticalScrollBar( const TSharedRef< SScrollBar >& ScrollBar, const float ScrollBarSize )
{
	ScrollBarThickness.X = ScrollBarSize;
	ScrollBarVisibility.Bind( TAttribute< EVisibility >::FGetter::CreateSP( ScrollBar, &SScrollBar::ShouldBeVisible ) );
	RegenerateWidgets();
}

void SHeaderRow::SetColumnWidth( const FName& InColumnId, float InWidth )
{
	check(InColumnId != NAME_None);

	for ( int32 SlotIndex=Columns.Num() - 1; SlotIndex >= 0; --SlotIndex )
	{
		FColumn& Column = Columns[SlotIndex];
		if ( Column.ColumnId == InColumnId )
		{
			Column.SetWidth( InWidth );
		}
	}
}

FVector2D SHeaderRow::GetRowSizeForSlotIndex(int32 SlotIndex) const
{
	if (Columns.IsValidIndex(SlotIndex))
	{
		const TSharedPtr<STableColumnHeader>& HeaderWidget = HeaderWidgets[SlotIndex];
		const FColumn& Column = Columns[SlotIndex];

		FVector2D HeaderSize = FVector2D(HeaderWidget->GetDesiredSize());

		if (Column.HeaderMenuContent.Widget != SNullWidget::NullWidget && HeaderWidget->GetMenuOverlayVisibility() != EVisibility::Visible)
		{
			HeaderSize += HeaderWidget->GetMenuOverlaySize();
		}

		if (OnGetMaxRowSizeForColumn.IsBound())
		{
			// It's assume that a header is at the top, so the sizing is for the width
			FVector2D MaxChildColumnSize = OnGetMaxRowSizeForColumn.Execute(Column.ColumnId, EOrientation::Orient_Horizontal);

			return MaxChildColumnSize.Component(EOrientation::Orient_Horizontal) < HeaderSize.Component(EOrientation::Orient_Horizontal) ? HeaderSize : MaxChildColumnSize;
		}
	}

	return FVector2D::ZeroVector;
}

TArray<FName> SHeaderRow::GetHiddenColumnIds() const
{
	TArray<FName> Result;
	Result.Reserve(Columns.Num());
	for (const FColumn& SomeColumn : Columns)
	{
		if (!SomeColumn.bIsVisible)
		{
			Result.Add(SomeColumn.ColumnId);
		}
	}
	return Result;
}

bool SHeaderRow::ShouldGeneratedColumn(const FName& InColumnId) const
{
	for (const FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ColumnId == InColumnId)
		{
			return SomeColumn.ShouldGenerateWidget.Get(true) && SomeColumn.bIsVisible;
		}
	}
	return false;
}

bool SHeaderRow::ShouldGenerateEmptyColumnForSpacing(const FName& InColumnId) const
{
	for (const FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ColumnId == InColumnId)
		{
			return SomeColumn.ShouldGenerateWidget.Get(true) && SomeColumn.ShouldGenerateEmptyWidgetForSpacing.Get(true) && SomeColumn.bIsVisible;
		}
	}
	return false;
}

bool SHeaderRow::IsColumnGenerated(const FName& InColumnId) const
{
	for (const TSharedPtr<STableColumnHeader>& ColumnHeader : HeaderWidgets)
	{
		if (ColumnHeader->GetColumnId() == InColumnId)
		{
			return true;
		}
	}
	return false;
}

bool SHeaderRow::IsColumnVisible(const FName& InColumnId) const
{
	for (const FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ColumnId == InColumnId)
		{
			return SomeColumn.bIsVisible;
		}
	}
	return false;
}

FReply SHeaderRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bCanSelectGeneratedColumn && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FVector2f SummonLocation = MouseEvent.GetScreenSpacePosition();
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder(CloseAfterSelection, nullptr);
		OnGenerateSelectColumnsSubMenu(MenuBuilder);

		FSlateApplication::Get().CloseToolTip();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SHeaderRow::RegenerateWidgets()
{
	const float SplitterHandleDetectionSize = SplitterHandleSize > 0.0f ? SplitterHandleSize + 4.0f : 5.0f;
	HeaderWidgets.Empty();

	TSharedPtr<SSplitter> Splitter;

	TSharedRef< SHorizontalBox > HeaderContent = 
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth( 1.0f )
		[
			SAssignNew(Splitter, SSplitter)
			.Style( &Style->ColumnSplitterStyle )
			.ResizeMode(ResizeMode)
			.PhysicalSplitterHandleSize( SplitterHandleSize )
			.HitDetectionSplitterHandleSize( SplitterHandleDetectionSize )
			.OnGetMaxSlotSize(this, &SHeaderRow::GetRowSizeForSlotIndex)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding( 0 )
		[
			SNew( SSpacer )
			.Size( ScrollBarThickness )
			.Visibility( ScrollBarVisibility )
		];

	// Construct widgets for all columns
	{
		const float HalfSplitterDetectionSize = ( SplitterHandleDetectionSize + 2 ) / 2;

		int32 LastSlotIndex = Columns.Num() - 1;
		for ( ; LastSlotIndex >= 0; --LastSlotIndex )
		{
			const FColumn& SomeColumn = Columns[LastSlotIndex];
			if ( SomeColumn.ShouldGenerateWidget.Get(true) && SomeColumn.bIsVisible )
			{
				break;
			}
		}

		// Populate the slot with widgets that represent the columns.
		TSharedPtr<STableColumnHeader> NewlyMadeHeader;
		for ( int32 SlotIndex=0; SlotIndex < Columns.Num(); ++SlotIndex )
		{
			FColumn& SomeColumn = Columns[SlotIndex];

			if ( SomeColumn.ShouldGenerateWidget.Get(true) && SomeColumn.bIsVisible )
			{
				// Keep track of the last header we created.
				TSharedPtr<STableColumnHeader> PrecedingHeader = NewlyMadeHeader;
				NewlyMadeHeader.Reset();

				FMargin DefaultPadding = FMargin(HalfSplitterDetectionSize, 0, HalfSplitterDetectionSize, 0);

				TSharedRef<SWidget> HeaderMenuContent = SomeColumn.HeaderMenuContent.Widget;
				if ( bCanSelectGeneratedColumn && HeaderMenuContent != SNullWidget::NullWidget )
				{
					static constexpr bool bCloseAfterSelection = true;
					static constexpr bool bNoIndent = true;

					FMenuEntryStyleParams StyleParams;
					StyleParams.bNoIndent = bNoIndent;

					FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);
					MenuBuilder.AddWidget(SomeColumn.HeaderMenuContent.Widget, FText::GetEmpty(), StyleParams);
					MenuBuilder.AddMenuSeparator();
					MenuBuilder.AddSubMenu(
						LOCTEXT("SelectColumns", "Select Columns"),
						LOCTEXT("SelectColumns_Tooltip", "Show/hide columns"),
						FNewMenuDelegate::CreateSP(this, &SHeaderRow::OnGenerateSelectColumnsSubMenu)
						);
					HeaderMenuContent = MenuBuilder.MakeWidget();
				}

				TSharedRef<STableColumnHeader> NewHeader =
					SAssignNew(NewlyMadeHeader, STableColumnHeader, SomeColumn, HeaderMenuContent, DefaultPadding)
					.Style((SlotIndex == LastSlotIndex) ? &Style->LastColumnStyle : &Style->ColumnStyle);

				HeaderWidgets.Add(NewlyMadeHeader);

				switch (SomeColumn.SizeRule)
				{
				case EColumnSizeMode::Fill:
				{
					TAttribute<float> WidthBinding;
					WidthBinding.BindRaw(&SomeColumn, &FColumn::GetWidth);

					// Add resizable cell
					Splitter->AddSlot()
						.Value(WidthBinding)
						.SizeRule(SSplitter::FractionOfParent)
						.OnSlotResized(SSplitter::FOnSlotResized::CreateRaw(&SomeColumn, &FColumn::SetWidth))
						.MinSize(SomeColumn.MinSize)
						[
							NewHeader
						];
				}
				break;

				case EColumnSizeMode::Fixed:
				{
					// Add fixed size cell
					Splitter->AddSlot()
						.SizeRule(SSplitter::SizeToContent)
						[
							SNew(SBox)
							.WidthOverride(SomeColumn.GetWidth() - SplitterHandleSize)
							[
								NewHeader
							]
						];
				}
				break;

				case EColumnSizeMode::Manual:
				{
					// Sizing grip to put at the end of the column - we can't use a SSplitter here as it doesn't have the resizing behavior we need
					const float GripSize = SplitterHandleSize > 0.0f ? SplitterHandleSize + 4.0f : 5.0f;
					TSharedRef<SBorder> SizingGrip = SNew(SBorder)
						.Padding(0.0f)
						.BorderImage( nullptr )
						.Cursor(EMouseCursor::ResizeLeftRight)
						.Content()
						[
							SNew(SSpacer)
							.Size(FVector2D(GripSize, GripSize))
						];

					TWeakPtr<SBorder> WeakSizingGrip = SizingGrip;
					auto SizingGrip_OnMouseButtonDown = [&SomeColumn, WeakSizingGrip](const FGeometry&, const FPointerEvent&) -> FReply
					{
						TSharedPtr<SBorder> SizingGripPtr = WeakSizingGrip.Pin();
						if (SizingGripPtr.IsValid())
						{
							return FReply::Handled().CaptureMouse(SizingGripPtr.ToSharedRef());
						}
						return FReply::Unhandled();
					};

					auto SizingGrip_OnMouseButtonUp = [&SomeColumn, WeakSizingGrip](const FGeometry&, const FPointerEvent&) -> FReply
					{
						TSharedPtr<SBorder> SizingGripPtr = WeakSizingGrip.Pin();
						if (SizingGripPtr.IsValid() && SizingGripPtr->HasMouseCapture())
						{
							return FReply::Handled().ReleaseMouseCapture();
						}
						return FReply::Unhandled();
					};

					auto SizingGrip_OnMouseMove = [&SomeColumn, WeakSizingGrip](const FGeometry&, const FPointerEvent& InPointerEvent) -> FReply
					{
						TSharedPtr<SBorder> SizingGripPtr = WeakSizingGrip.Pin();
						if (SizingGripPtr.IsValid() && SizingGripPtr->HasMouseCapture())
						{
							// The sizing grip has been moved, so update our columns size from the movement delta
							const float NewWidth = SomeColumn.GetWidth() + InPointerEvent.GetCursorDelta().X;
							SomeColumn.SetWidth(FMath::Max(20.0f, NewWidth));
							return FReply::Handled();
						}
						return FReply::Unhandled();
					};

					// Bind the events to handle the drag sizing
					SizingGrip->SetOnMouseButtonDown(FPointerEventHandler::CreateLambda(SizingGrip_OnMouseButtonDown));
					SizingGrip->SetOnMouseButtonUp(FPointerEventHandler::CreateLambda(SizingGrip_OnMouseButtonUp));
					SizingGrip->SetOnMouseMove(FPointerEventHandler::CreateLambda(SizingGrip_OnMouseMove));
					SizingGrip->SetOnMouseDoubleClick(SomeColumn.OnColumnSplitterDoubleClick);

					auto GetColumnWidthAsOptionalSize = [&SomeColumn, SplitterHandleSizeCopy = SplitterHandleSize]() -> FOptionalSize
					{
						// Subtract SplitterHandleSize to compensate for SSplitter adding a handle between items.
						const float DesiredWidth = SomeColumn.GetWidth() - SplitterHandleSizeCopy;
						return FOptionalSize(DesiredWidth);
					};

					TAttribute<FOptionalSize> WidthBinding;
					WidthBinding.Bind(TAttribute<FOptionalSize>::FGetter::CreateLambda(GetColumnWidthAsOptionalSize));

					// Add resizable cell
					Splitter->AddSlot()
						.SizeRule(SSplitter::SizeToContent)
						.Resizable(false)
						.MinSize(SomeColumn.MinSize)
						[
							SNew(SBox)
							.WidthOverride(WidthBinding)
							[
								SNew(SOverlay)
								+ SOverlay::Slot()
								[
									NewHeader
								]
								+ SOverlay::Slot()
								.HAlign(HAlign_Right)
								[
									SizingGrip
								]
							]
						];
				}
				break;

				case EColumnSizeMode::FillSized:
				{

					auto GetColumnWidthAsOptionalSize = [&SomeColumn]() -> FOptionalSize
					{
						const float DesiredWidth = SomeColumn.GetWidth();
						return FOptionalSize(DesiredWidth);
					};

					TAttribute<FOptionalSize> WidthBinding;
					WidthBinding.Bind(TAttribute<FOptionalSize>::FGetter::CreateLambda(GetColumnWidthAsOptionalSize));

					Splitter->AddSlot()
						.SizeRule(SSplitter::SizeToContent)
						.Resizable(true)
						.OnSlotResized(SSplitter::FOnSlotResized::CreateRaw(&SomeColumn, &FColumn::SetWidth))
						.MinSize(SomeColumn.MinSize)
						[
							SNew(SBox)
							.WidthOverride(WidthBinding)
							[
								NewHeader
							]
						];
				}
				break;

				default:
					ensure(false);
					break;
				}
			}			
		}
	}

	if(Style->HorizontalSeparatorBrush.GetDrawType() != ESlateBrushDrawType::NoDrawType && Style->HorizontalSeparatorThickness > 0)
	{
		// Create a box to contain widgets for each column
		SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				HeaderContent
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.Thickness(Style->HorizontalSeparatorThickness)
				.SeparatorImage(&Style->HorizontalSeparatorBrush)
			]);
	}
	else
	{
		SetContent(HeaderContent);
	}
}

void SHeaderRow::ToggleAllColumns()
{
	ECheckBoxState CurrentState = GetSelectAllColumnsCheckState();
	bool bShouldSelectAll = CurrentState == ECheckBoxState::Unchecked || CurrentState == ECheckBoxState::Undetermined;

	for (FColumn& SomeColumn : Columns)
	{
		if (!SomeColumn.ShouldGenerateWidget.IsSet())
		{
			SomeColumn.bIsVisible = bShouldSelectAll;
		}
	}
	
	RefreshColumns();
	ColumnsChanged.Broadcast(SharedThis(this));
	OnHiddenColumnsListChanged.ExecuteIfBound();
}

bool SHeaderRow::CanToggleAllColumns() const
{
	return true;
}

ECheckBoxState SHeaderRow::GetSelectAllColumnsCheckState() const
{
	bool bAnyColumnsVisible = false;
	bool bAnyColumnsInvisible = false;
	for (const FColumn& SomeColumn : Columns)
	{
		if (!SomeColumn.ShouldGenerateWidget.IsSet())
		{
			if(SomeColumn.bIsVisible)
			{
				bAnyColumnsVisible = true;
			}
			else
			{
				bAnyColumnsInvisible = true;
			}			
		}
	}

	if(bAnyColumnsVisible && bAnyColumnsInvisible)
	{
		return ECheckBoxState::Undetermined;
	}

	if(bAnyColumnsVisible)
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

FText SHeaderRow::GetToggleAllColumnsText() const
{
	ECheckBoxState CurrentState = GetSelectAllColumnsCheckState();
	if(CurrentState == ECheckBoxState::Checked)
	{
		return LOCTEXT("DeselectAllColumns", "Select: None");
	}
	else if(CurrentState == ECheckBoxState::Unchecked || CurrentState == ECheckBoxState::Undetermined)
	{
		return LOCTEXT("SelectAllColumns", "Select: All");
	}

	return FText::GetEmpty();
}

void SHeaderRow::OnGenerateSelectColumnsSubMenu(FMenuBuilder& InSubMenuBuilder)
{
	InSubMenuBuilder.AddMenuEntry(
		TAttribute<FText>::CreateSP(this, &SHeaderRow::GetToggleAllColumnsText),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SHeaderRow::ToggleAllColumns),
			FCanExecuteAction::CreateSP(this, &SHeaderRow::CanToggleAllColumns),
			FGetActionCheckState::CreateSP(this, &SHeaderRow::GetSelectAllColumnsCheckState)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);
	
	for (const FColumn& SomeColumn : Columns)
	{
		const bool bCanExecuteAction = !SomeColumn.ShouldGenerateWidget.IsSet();
		const FName ColumnId = SomeColumn.ColumnId;

		InSubMenuBuilder.AddMenuEntry(
			SomeColumn.DefaultText,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SHeaderRow::ToggleGeneratedColumn, ColumnId),
				FCanExecuteAction::CreateLambda([bCanExecuteAction] () { return bCanExecuteAction; }),
				FGetActionCheckState::CreateSP(this, &SHeaderRow::GetGeneratedColumnCheckedState, ColumnId)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SHeaderRow::ToggleGeneratedColumn(FName ColumnId)
{
	// Only column that doesn't have a ShouldGenerateWidget, can be toggled
	for (FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ColumnId == ColumnId)
		{
			if (!SomeColumn.ShouldGenerateWidget.IsSet())
			{
				SomeColumn.bIsVisible = !SomeColumn.bIsVisible;

				RefreshColumns();
				ColumnsChanged.Broadcast(SharedThis(this));
				OnHiddenColumnsListChanged.ExecuteIfBound();
			}
			break;
		}
	}
}

ECheckBoxState SHeaderRow::GetGeneratedColumnCheckedState(FName ColumnId) const
{
	return IsColumnGenerated(ColumnId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SHeaderRow::SetShowGeneratedColumn(const FName& ColumnId, bool InShow)
{
	// Only column that doesn't have a ShouldGenerateWidget, can be toggled
	for (FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ColumnId == ColumnId)
		{
			if (!SomeColumn.ShouldGenerateWidget.IsSet())
			{
				if (SomeColumn.bIsVisible != InShow)
				{
					SomeColumn.bIsVisible = !SomeColumn.bIsVisible;

					RefreshColumns();
					ColumnsChanged.Broadcast(SharedThis(this));
					OnHiddenColumnsListChanged.ExecuteIfBound();
				}
			}
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
