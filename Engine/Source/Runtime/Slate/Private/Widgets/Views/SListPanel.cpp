// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Widgets/Views/SListPanel.h"
#include "Layout/ArrangedChildren.h"


// Used to subtract a tiny amount from the available dimension to avoid floating point precision problems when arranging children
static const float FloatingPointPrecisionOffset = 0.001f;

SLATE_IMPLEMENT_WIDGET(SListPanel)
void SListPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FSlot::RegisterAttributes(Initializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ItemWidth", ItemWidthAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ItemHeight", ItemHeightAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "NumDesiredItems", NumDesiredItemsAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ItemAlignment", ItemAlignmentAttribute, EInvalidateWidgetReason::Layout);
}

SListPanel::SListPanel()
	: Children(this)
	, ItemAlignmentAttribute(*this)
	, ItemWidthAttribute(*this, 16.0f)
	, ItemHeightAttribute(*this, 0.0f)
	, NumDesiredItemsAttribute(*this, 0)
{
}

SListPanel::~SListPanel() = default;

void SListPanel::Construct( const FArguments& InArgs )
{
	ItemWidthAttribute.Assign(*this, InArgs._ItemWidth);
	ItemHeightAttribute.Assign(*this, InArgs._ItemHeight);
	NumDesiredItemsAttribute.Assign(*this, InArgs._NumDesiredItems);
	ItemAlignmentAttribute.Assign(*this, InArgs._ItemAlignment);
	Orientation = InArgs._ListOrientation;
	Children.AddSlots(MoveTemp(const_cast<TArray<FSlot::FSlotArguments>&>(InArgs._Slots)));
}

SListPanel::FSlot::FSlotArguments SListPanel::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}
	
SListPanel::FScopedWidgetSlotArguments SListPanel::AddSlot(int32 InsertAtIndex)
{
	return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Children, InsertAtIndex };
}

void SListPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	if (Children.Num() > 0)
	{
		const FTableViewDimensions AllottedDimensions(Orientation, AllottedGeometry.GetLocalSize());
		FTableViewDimensions DimensionsSoFar(Orientation);

		if (ShouldArrangeAsTiles())
		{
			// This is a tile view list - arrange items side by side along the line axis until there is no more room, then create a new line along the scroll axis.
			const EListItemAlignment ListItemAlignment = ItemAlignmentAttribute.Get();
			const float ItemPadding = GetItemPadding(AllottedGeometry, ListItemAlignment);
			const float HalfItemPadding = ItemPadding * 0.5f;

			const FTableViewDimensions LocalItemSize = GetItemSize(AllottedGeometry, ListItemAlignment);
			DimensionsSoFar.ScrollAxis = -FMath::FloorToInt(FirstLineScrollOffset * LocalItemSize.ScrollAxis) - OverscrollAmount;

			bool bIsNewLine = true;
			for (int32 ItemIndex = 0; ItemIndex < Children.Num(); ++ItemIndex)
			{
				if (bIsNewLine)
				{
					if (ListItemAlignment == EListItemAlignment::RightAligned || ListItemAlignment == EListItemAlignment::CenterAligned)
					{
						const float LinePadding = GetLinePadding(AllottedGeometry, ItemIndex);
						const bool IsVisible = Children[ItemIndex].GetWidget()->GetVisibility().IsVisible();
						if (ListItemAlignment == EListItemAlignment::RightAligned)
						{
							DimensionsSoFar.LineAxis += IsVisible ? LinePadding : 0;
						}
						else
						{
							const float HalfLinePadding = LinePadding * 0.5f;
							DimensionsSoFar.LineAxis += IsVisible ? HalfLinePadding : 0;
						}
					}

					bIsNewLine = false;
				}

				FTableViewDimensions ChildOffset = DimensionsSoFar;
				ChildOffset.LineAxis += HalfItemPadding;
				ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(Children[ItemIndex].GetWidget(), ChildOffset.ToVector2D(), LocalItemSize.ToVector2D()));

				DimensionsSoFar.LineAxis += LocalItemSize.LineAxis + ItemPadding;
				if (DimensionsSoFar.LineAxis + LocalItemSize.LineAxis + ItemPadding > AllottedDimensions.LineAxis)
				{
					DimensionsSoFar.LineAxis = 0;
					DimensionsSoFar.ScrollAxis += LocalItemSize.ScrollAxis;
					bIsNewLine = true;
				}
			}
		}
		else
		{
			// This is a normal list, arrange items in a line along the scroll axis
			float ScrollAxisOffset = 0.0f;
			const int32 NumWholeWidgetsOffset = FMath::Min(FMath::Floor(FirstLineScrollOffset), Children.Num());
			for (int32 ItemIndex = 0; ItemIndex < NumWholeWidgetsOffset; ++ItemIndex)
			{
				const FTableViewDimensions CurrentChildDimensions(Orientation, Children[ItemIndex].GetWidget()->GetDesiredSize());
				ScrollAxisOffset += CurrentChildDimensions.ScrollAxis;
			}

			const int32 FractionalWidgetIndex = NumWholeWidgetsOffset;
			if (Children.IsValidIndex(FractionalWidgetIndex))
			{
				const FTableViewDimensions OffsetFractionChildDimensions(Orientation, Children[FractionalWidgetIndex].GetWidget()->GetDesiredSize());
				ScrollAxisOffset += FMath::Frac(FirstLineScrollOffset) * OffsetFractionChildDimensions.ScrollAxis;
			}

			DimensionsSoFar.ScrollAxis = -FMath::FloorToInt(ScrollAxisOffset) - OverscrollAmount;

			for (int32 ItemIndex = 0; ItemIndex < Children.Num(); ++ItemIndex)
			{
				const FTableViewDimensions WidgetDesiredDimensions(Orientation, Children[ItemIndex].GetWidget()->GetDesiredSize());
				const bool bIsVisible = Children[ItemIndex].GetWidget()->GetVisibility().IsVisible();
				
				FTableViewDimensions FinalWidgetDimensions(Orientation);
				FinalWidgetDimensions.ScrollAxis = bIsVisible ? WidgetDesiredDimensions.ScrollAxis : 0.f;
				FinalWidgetDimensions.LineAxis = AllottedDimensions.LineAxis;

				ArrangedChildren.AddWidget(
					AllottedGeometry.MakeChild(Children[ItemIndex].GetWidget(), DimensionsSoFar.ToVector2D(), FinalWidgetDimensions.ToVector2D())
				);

				DimensionsSoFar.ScrollAxis += FinalWidgetDimensions.ScrollAxis;
			}
		}
	}
}
	
void SListPanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	int32 OldPreferredNumLines = PreferredNumLines;
	
	if (ShouldArrangeAsTiles())
	{
		const FTableViewDimensions AllottedDimensions(Orientation, AllottedGeometry.GetLocalSize());

		const EListItemAlignment ListItemAlignment = ItemAlignmentAttribute.Get();
		const float ItemPadding = GetItemPadding(AllottedGeometry, ListItemAlignment);
		FTableViewDimensions ItemDimensions = GetItemSize(AllottedGeometry, ListItemAlignment);
		ItemDimensions.LineAxis += ItemPadding;

		const int32 NumChildren = NumDesiredItemsAttribute.Get();

		if (ItemDimensions.LineAxis > 0.0f && NumChildren > 0)
		{
			const int32 NumItemsPerLine = FMath::Clamp(FMath::CeilToInt(AllottedDimensions.LineAxis / ItemDimensions.LineAxis) - 1, 1, NumChildren);
			PreferredNumLines = FMath::CeilToInt(NumChildren / (float)NumItemsPerLine);
		}
		else
		{
			PreferredNumLines = 1;
		}
	}
	else
	{
		PreferredNumLines = NumDesiredItemsAttribute.Get();
	}

	if (OldPreferredNumLines != PreferredNumLines)
	{
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

FVector2D SListPanel::ComputeDesiredSize( float ) const
{
	FTableViewDimensions DesiredListPanelDimensions(Orientation);

	if (ShouldArrangeAsTiles())
	{
		FTableViewDimensions ItemDimensions = GetDesiredItemDimensions();

		DesiredListPanelDimensions.LineAxis = ItemDimensions.LineAxis;
		DesiredListPanelDimensions.ScrollAxis = ItemDimensions.ScrollAxis;
	}
	else if (Children.Num() > 0)
	{
		// Simply the sum of all the children along the scroll axis and the largest width along the line axis.
		for (int32 ItemIndex = 0; ItemIndex < Children.Num(); ++ItemIndex)
		{
			// Notice that we do not respect child Visibility - it's not useful for ListPanels.
			FTableViewDimensions ChildDimensions(Orientation, Children[ItemIndex].GetWidget()->GetDesiredSize());
			DesiredListPanelDimensions.ScrollAxis += ChildDimensions.ScrollAxis;
			DesiredListPanelDimensions.LineAxis = FMath::Max(DesiredListPanelDimensions.LineAxis, ChildDimensions.LineAxis);
		}

		DesiredListPanelDimensions.ScrollAxis /= Children.Num();
	}

	DesiredListPanelDimensions.ScrollAxis *= PreferredNumLines;
	return DesiredListPanelDimensions.ToVector2D();
}

FChildren* SListPanel::GetAllChildren()
{
	return &Children;
}

FChildren* SListPanel::GetChildren()
{
	if (bIsRefreshPending)
	{
		// When a refresh is pending it is unsafe to cache the desired sizes of our children because
		// they may be representing unsound data structures. Any delegates/attributes accessing unsound
		// data will cause a crash.
		return &FNoChildren::NoChildrenInstance;
	}
	else
	{
		return &Children;
	}
	
}

void SListPanel::SetFirstLineScrollOffset(float InFirstLineScrollOffset)
{
	if (FirstLineScrollOffset != InFirstLineScrollOffset)
	{
		FirstLineScrollOffset = InFirstLineScrollOffset;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SListPanel::SetOverscrollAmount( float InOverscrollAmount )
{
	if (OverscrollAmount != InOverscrollAmount)
	{
		OverscrollAmount = InOverscrollAmount;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SListPanel::ClearItems()
{
	Children.Empty();
}

FTableViewDimensions SListPanel::GetDesiredItemDimensions() const
{
	return FTableViewDimensions(Orientation, ItemWidthAttribute.Get(), ItemHeightAttribute.Get());
}

float SListPanel::GetItemPadding(const FGeometry& AllottedGeometry) const
{
	return GetItemPadding(AllottedGeometry, ItemAlignmentAttribute.Get());
}

float SListPanel::GetItemPadding(const FGeometry& AllottedGeometry, const EListItemAlignment ListItemAlignment) const
{
	if (ListItemAlignment == EListItemAlignment::EvenlyDistributed)
	{
		FTableViewDimensions AllottedDimensions(Orientation, AllottedGeometry.GetLocalSize());
		FTableViewDimensions DesiredDimensions = GetDesiredItemDimensions();

		// Only add padding between items if we have more total items that we can fit on a single line.  Otherwise,
		// the padding around items would continue to change proportionately with the (ample) free space along the line axis
		const int32 NumItemsPerLine = DesiredDimensions.LineAxis > 0.f ? FMath::FloorToInt(AllottedDimensions.LineAxis / DesiredDimensions.LineAxis) : 0;
		
		if (NumItemsPerLine > 0 && Children.Num() >= NumItemsPerLine)
		{
			return (AllottedDimensions.LineAxis - FloatingPointPrecisionOffset - (NumItemsPerLine * DesiredDimensions.LineAxis)) / NumItemsPerLine;
		}
	}

	return 0.f;
}

FTableViewDimensions SListPanel::GetItemSize(const FGeometry& AllottedGeometry) const
{
	return GetItemSize(AllottedGeometry, ItemAlignmentAttribute.Get());
}

FTableViewDimensions SListPanel::GetItemSize(const FGeometry& AllottedGeometry, const EListItemAlignment ListItemAlignment) const
{
	const FTableViewDimensions AllottedDimensions(Orientation, AllottedGeometry.GetLocalSize());
	const FTableViewDimensions ItemDimensions = GetDesiredItemDimensions();

	// Only add padding between items if we have more total items that we can fit on a single line.  Otherwise,
	// the padding around items would continue to change proportionately with the (ample) free space in our minor axis
	const int32 NumItemsPerLine = ItemDimensions.LineAxis > 0 ? FMath::FloorToInt(AllottedDimensions.LineAxis / ItemDimensions.LineAxis) : 0;

	FTableViewDimensions ExtraDimensions(Orientation);
	if (NumItemsPerLine > 0)
	{
		if (ListItemAlignment == EListItemAlignment::Fill || 
			ListItemAlignment == EListItemAlignment::EvenlyWide || 
			ListItemAlignment == EListItemAlignment::EvenlySize)
		{
			ExtraDimensions.LineAxis = (AllottedDimensions.LineAxis - FloatingPointPrecisionOffset - NumItemsPerLine * ItemDimensions.LineAxis) / NumItemsPerLine;
			
			if (ListItemAlignment == EListItemAlignment::EvenlySize)
			{
				ExtraDimensions.ScrollAxis = ItemDimensions.ScrollAxis * (ExtraDimensions.LineAxis / (ItemDimensions.LineAxis + ExtraDimensions.LineAxis));
			}
		}
	}
	
	return ItemDimensions + ExtraDimensions;
}

float SListPanel::GetLinePadding(const FGeometry& AllottedGeometry, const int32 LineStartIndex) const
{
	const int32 NumItemsLeft = Children.Num() - LineStartIndex;
	if(NumItemsLeft <= 0)
	{
		return 0.0f;
	}

	const FTableViewDimensions AllottedDimensions(Orientation, AllottedGeometry.GetLocalSize());
	const FTableViewDimensions ItemSize = GetItemSize(AllottedGeometry);

	const int32 MaxNumItemsOnLine = ItemSize.LineAxis > 0 ? FMath::FloorToInt(AllottedDimensions.LineAxis / ItemSize.LineAxis) : 0;
	const int32 NumItemsOnLine = FMath::Min(NumItemsLeft, MaxNumItemsOnLine);

	return AllottedDimensions.LineAxis - FloatingPointPrecisionOffset - (NumItemsOnLine * ItemSize.LineAxis);
}

void SListPanel::SetRefreshPending( bool IsPendingRefresh )
{
	bIsRefreshPending = IsPendingRefresh;
}

bool SListPanel::IsRefreshPending() const
{
	return bIsRefreshPending;
}

bool SListPanel::ShouldArrangeAsTiles() const
{
	FTableViewDimensions DesiredItemDimensions = GetDesiredItemDimensions();
	return DesiredItemDimensions.LineAxis > 0 && DesiredItemDimensions.ScrollAxis > 0;
}

void SListPanel::SetItemHeight(TAttribute<float> Height)
{
	ItemHeightAttribute.Assign(*this, MoveTemp(Height));
}

void SListPanel::SetItemWidth(TAttribute<float> Width)
{
	ItemWidthAttribute.Assign(*this, MoveTemp(Width));
}
