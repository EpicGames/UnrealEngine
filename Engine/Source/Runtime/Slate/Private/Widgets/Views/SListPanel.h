// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Widgets/Views/STableViewBase.h"

class FArrangedChildren;

template <>
struct TWidgetTypeTraits<class SListPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A really simple panel that arranges its children in a vertical list with no spacing.
 * Items in this panel have a uniform height.
 * Also supports offsetting its items vertically.
 */
class SListPanel : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SListPanel, SPanel, SLATE_API)

public:
	/** A ListPanel slot is very simple - it just stores a widget. */
	class FSlot : public TWidgetSlotWithAttributeSupport<FSlot>
	{
	public:
		FSlot()
		: TWidgetSlotWithAttributeSupport<FSlot>()
		{}
	};
	
	/** Make a new ListPanel::Slot  */
	static SLATE_API FSlot::FSlotArguments Slot();
	
	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/** Add a slot to the ListPanel */
	SLATE_API FScopedWidgetSlotArguments AddSlot(int32 InsertAtIndex = INDEX_NONE);
	
	SLATE_BEGIN_ARGS( SListPanel )
		: _ItemWidth(0)
		, _ItemHeight(16)
		, _NumDesiredItems(0)
		, _ItemAlignment(EListItemAlignment::EvenlyDistributed)
		, _ListOrientation(Orient_Vertical)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
			_Clipping = EWidgetClipping::ClipToBounds;
		}
	
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
		SLATE_ATTRIBUTE( float, ItemWidth )
		SLATE_ATTRIBUTE( float, ItemHeight )
		SLATE_ATTRIBUTE( int32, NumDesiredItems )
		SLATE_ATTRIBUTE( EListItemAlignment, ItemAlignment )
		SLATE_ARGUMENT( EOrientation, ListOrientation )

	SLATE_END_ARGS()
	
	SLATE_API SListPanel();

	SLATE_API virtual ~SListPanel();

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

public:

	// SWidget interface
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FChildren* GetAllChildren() override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/** Fraction of the first line that we should offset by to account for the current scroll amount. */
	SLATE_API void SetFirstLineScrollOffset(float InFirstLineScrollOffset);

	/** Set how much we should appear to have scrolled past the beginning/end of the list. */
	SLATE_API void SetOverscrollAmount( float InOverscrollAmount );
	
	/** Remove all the children from this panel */
	SLATE_API void ClearItems();

	/** @return the uniform desired item dimensions used when arranging children. */
	SLATE_API FTableViewDimensions GetDesiredItemDimensions() const;

	/** @return the uniform item width used when arranging children. */
	SLATE_API FTableViewDimensions GetItemSize(const FGeometry& AllottedGeometry) const;

	/** @return the uniform item width used when arranging children. */
	SLATE_API FTableViewDimensions GetItemSize(const FGeometry& AllottedGeometry, const EListItemAlignment ListItemAlignment) const;

	/** @return the horizontal padding applied to each tile item */
	SLATE_API float GetItemPadding(const FGeometry& AllottedGeometry) const;

	/** @return the horizontal padding applied to each tile item */
	SLATE_API float GetItemPadding(const FGeometry& AllottedGeometry, const EListItemAlignment ListItemAlignment) const;

	/** @return the horizontal padding applied to all the items on a line */
	SLATE_API float GetLinePadding(const FGeometry& AllottedGeometry, const int32 LineStartIndex) const;

	/** Tells the list panel whether items in the list are pending a refresh */
	SLATE_API void SetRefreshPending( bool IsPendingRefresh );

	/** Returns true if this list panel is pending a refresh, false otherwise */
	SLATE_API bool IsRefreshPending() const;

	/** See ItemHeight attribute */
	SLATE_API void SetItemHeight(TAttribute<float> Height);

	/** See ItemWidth attribute */
	SLATE_API void SetItemWidth(TAttribute<float> Width);

protected:

	/** @return true if this panel should arrange items as tiles placed alongside one another in each line */
	SLATE_API bool ShouldArrangeAsTiles() const;

protected:
	/** The children being arranged by this panel */
	TPanelChildren<FSlot> Children;
	
	/**
	 * The offset of the view area from the top of the list in item heights.
	 * Translate to physical units based on first line in list.
	 */
	float FirstLineScrollOffset = 0.f;

	/** Amount scrolled past beginning/end of list in Slate Units. */
	float OverscrollAmount = 0.f;

	/** The preferred number of lines that this widget should have orthogonal to the scroll axis. Only relevant for tile views. */
	int32 PreferredNumLines = 1;

	/**
	 * When true, a refresh of the table view control that is using this panel is pending.
	 * Some of the widgets in this panel are associated with items that may no longer be sound data.
	 */
	bool bIsRefreshPending = false;

	/** Overall orientation of the list for layout and scrolling. Only relevant for tile views. */
	EOrientation Orientation;

private:
	/** How should be horizontally aligned? Only relevant for tile views. */
	TSlateAttribute<EListItemAlignment> ItemAlignmentAttribute;

	/** The uniform item width used to arrange the children. Only relevant for tile views. */
	TSlateAttribute<float> ItemWidthAttribute;

	/** The uniform item height used to arrange the children */
	TSlateAttribute<float> ItemHeightAttribute;

	/** Total number of items that the tree wants to visualize */
	TSlateAttribute<int32> NumDesiredItemsAttribute;
};
