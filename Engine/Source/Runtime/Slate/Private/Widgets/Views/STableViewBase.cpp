// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Views/STableViewBase.h"
#include "Rendering/DrawElements.h"
#include "Types/SlateConstants.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListPanel.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(STableViewBase)

int32 MaxNumberOfDeltaTimes = 20;
FAutoConsoleVariableRef CVarMaxNumberOfDeltaTimes(
	TEXT("Slate.InertialScrolling.MaxNumberOfDeltaTimes"),
	MaxNumberOfDeltaTimes,
	TEXT("The number of DeltaTimes used to calculate the smooth DeltaTime used for animated scrolling."),
	ECVF_Default);

float RatioOfOutliers = 0.4f;
FAutoConsoleVariableRef CVarRatioOfOutliers(
	TEXT("Slate.InertialScrolling.RatioOfOutliers"),
	RatioOfOutliers,
	TEXT("The ratio of outliers to remove from DeltaTimeQueue for the smooth DeltaTime calculation."),
	ECVF_Default);

namespace ListConstants
{
	static const float OvershootMax = 150.0f;
	static const float OvershootBounceRate = 250.0f;
}


FTableViewDimensions::FTableViewDimensions(EOrientation InOrientation)
	: Orientation(InOrientation)
{
}


FTableViewDimensions::FTableViewDimensions(EOrientation InOrientation, float X, float Y)
	: FTableViewDimensions(InOrientation, FVector2f(X, Y))
{
}

FTableViewDimensions::FTableViewDimensions(EOrientation InOrientation, const UE::Slate::FDeprecateVector2DParameter& Size)
	: FTableViewDimensions(InOrientation)
{
	if (InOrientation == Orient_Vertical)
	{
		LineAxis = Size.X;
		ScrollAxis = Size.Y;
	}
	else
	{
		ScrollAxis = Size.X;
		LineAxis = Size.Y;
	}
}

SLATE_IMPLEMENT_WIDGET(STableViewBase)
void STableViewBase::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, SelectionModeAttribute, EInvalidateWidgetReason::None)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			STableViewBase* TableView = static_cast<STableViewBase*>(&Widget);
			const ESelectionMode::Type OldSelectionMode = TableView->PreviousSelectionMode;
			TableView->PreviousSelectionMode = TableView->GetSelectionMode();
			TableView->OnSelectionModeChanged(OldSelectionMode, TableView->GetSelectionMode());
		}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BackgroundBrushAttribute, EInvalidateWidgetReason::Layout);
}

TSharedRef<SWidget> STableViewBase::ConstructVerticalShadowBox(TSharedRef<SWidget> Content)
{
	return SNew(SOverlay)
		+ SOverlay::Slot()
		.Padding(0.0)
		[
			Content
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			// Shadow: Hint to scroll up
			SAssignNew(TopLeftShadowImage, SImage)
			.Visibility(ShadowBoxStyle ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
			.ColorAndOpacity(this, &STableViewBase::GetStartShadowOpacity)
			.Image(ShadowBoxStyle ? &ShadowBoxStyle->TopShadowBrush : nullptr)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Bottom)
		[
			// Shadow: a hint to scroll down
			SAssignNew(BottomRightShadowImage, SImage)
			.Visibility(ShadowBoxStyle ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
			.ColorAndOpacity(this, &STableViewBase::GetEndShadowOpacity)
			.Image(ShadowBoxStyle ? &ShadowBoxStyle->BottomShadowBrush : nullptr)
		];
}

TSharedRef<SWidget> STableViewBase::ConstructHorizontalShadowBox(TSharedRef<SWidget> Content)
{
	return SNew(SOverlay)
		+ SOverlay::Slot()
		.Padding(0.0)
		[
			Content
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		[
			// Shadow: Hint to left
			SAssignNew(TopLeftShadowImage, SImage)
			.Visibility(ShadowBoxStyle ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
			.ColorAndOpacity(this, &STableViewBase::GetStartShadowOpacity)
			.Image(ShadowBoxStyle ? &ShadowBoxStyle->LeftShadowBrush : nullptr)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		[
			// Shadow: a hint to scroll right
			SAssignNew(BottomRightShadowImage, SImage)
			.Visibility(ShadowBoxStyle ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
			.ColorAndOpacity(this, &STableViewBase::GetEndShadowOpacity)
			.Image(ShadowBoxStyle ? &ShadowBoxStyle->RightShadowBrush : nullptr)
		];
}

void STableViewBase::ConstructChildren( const TAttribute<float>& InItemWidth, const TAttribute<float>& InItemHeight, const TAttribute<EListItemAlignment>& InItemAlignment, const TSharedPtr<SHeaderRow>& InHeaderRow, const TSharedPtr<SScrollBar>& InScrollBar, EOrientation InScrollOrientation, const FOnTableViewScrolled& InOnTableViewScrolled, const FScrollBarStyle* InScrollBarStyle, const bool bInPreventThrottling )
{
	bItemsNeedRefresh = true;
	
	HeaderRow = InHeaderRow;

	OnTableViewScrolled = InOnTableViewScrolled;

	Orientation = InHeaderRow ? Orient_Vertical : InScrollOrientation;
	UE_CLOGF(InScrollOrientation != Orientation, LogSlate, Error, "STableViewBase does not support horizontal scrolling when displaying a header row");

	ItemsPanel = SNew(SListPanel)
		.Clipping(GetClipping())
		.ItemWidth(InItemWidth)
		.ItemHeight(InItemHeight)
		.NumDesiredItems(this, &STableViewBase::GetNumItemsBeingObserved)
		.ItemAlignment(InItemAlignment)
		.ListOrientation(Orientation);

	PinnedItemsPanel = SNew(SListPanel)
		.Clipping(GetClipping())
		.ItemWidth(InItemWidth)
		.ItemHeight(InItemHeight)
		.NumDesiredItems(this, &STableViewBase::GetNumPinnedItems)
		.ItemAlignment(InItemAlignment)
		.ListOrientation(Orientation)
		.Visibility(EVisibility::Collapsed);

	TSharedPtr<SWidget> ListAndScrollbar;
	if (InScrollBar)
	{
		// The user provided us with a scrollbar; we will rely on it.
		ScrollBar = InScrollBar;
		ScrollBar->SetOnUserScrolled(FOnUserScrolled::CreateSP(this, &STableViewBase::ScrollBar_OnUserScrolled));
		
		ListAndScrollbar = ItemsPanel;
	}
	else
	{
		ScrollBar = SNew(SScrollBar)
			.OnUserScrolled(this, &STableViewBase::ScrollBar_OnUserScrolled)
			.Orientation(Orientation)
			.Style(InScrollBarStyle ? InScrollBarStyle : &FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
			.PreventThrottling(bInPreventThrottling);

		const FOptionalSize ScrollBarSize(InScrollBarStyle ? InScrollBarStyle->Thickness + (SScrollBar::DefaultUniformPadding * 2.0f) : 16.f);

		// We create the itemsPanel container
		TSharedRef<SWidget> PanelContainer = SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				PinnedItemsPanel.ToSharedRef()
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				ItemsPanel.ToSharedRef()
			];		

		// If we use Shadow Box Style, wrap the Container in a Shadow Box.
		if (bShouldUseShadowBoxStyle)
		{
			if (Orientation == Orient_Vertical)
			{
				PanelContainer = ConstructVerticalShadowBox(PanelContainer);
			}
			else if (Orientation == Orient_Horizontal)
			{
				PanelContainer = ConstructHorizontalShadowBox(PanelContainer);
			}
		}
		
		if (Orientation == Orient_Vertical)
		{
			VerticalScrollBarSlot = nullptr;
			ListAndScrollbar = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					PanelContainer
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ScrollBarSlotPadding)
				.Expose(VerticalScrollBarSlot)
				[
					SNew(SBox)
					.WidthOverride(ScrollBarSize)
					[
						ScrollBar.ToSharedRef()
					]
				];
		}
		else
		{
			HorizontalScrollBarSlot = nullptr;
			ListAndScrollbar = SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1)
				[
					PanelContainer
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(ScrollBarSlotPadding)
				.Expose(HorizontalScrollBarSlot)
				[
					SNew(SBox)
					.HeightOverride(ScrollBarSize)
					[
						ScrollBar.ToSharedRef()
					]
				];
		}
	}

	if (InHeaderRow)
	{
		// Only associate the scrollbar if we created it.
		// If the scrollbar was passed in from outside then it won't appear under our header row so doesn't need compensating for.
		if (!InScrollBar)
		{
			InHeaderRow->SetAssociatedVerticalScrollBar(ScrollBar.ToSharedRef(), 16.f);
		}

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InHeaderRow.ToSharedRef()
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				ListAndScrollbar.ToSharedRef()
			]
		];
	}
	else
	{
		this->ChildSlot
		[
			ListAndScrollbar.ToSharedRef()
		];
	}
	
}

bool STableViewBase::SupportsKeyboardFocus() const
{
	// The ListView is focusable.
	return true;
}

void STableViewBase::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	bShowSoftwareCursor = false;
}

void STableViewBase::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SCompoundWidget::OnMouseCaptureLost(CaptureLostEvent);

	bShowSoftwareCursor = false;
}

EActiveTimerReturnType STableViewBase::UpdateInertialScroll(double InCurrentTime, float InDeltaTime)
{
	bool bKeepTicking = false;
	if (ItemsPanel.IsValid())
	{
		if (IsRightClickScrolling())
		{
			bKeepTicking = true;

			// We sample for the inertial scroll on tick rather than on mouse/touch move so
			// that we still get samples even if the mouse has not moved.
			if (CanUseInertialScroll(TickScrollDelta))
			{
				InertialScrollManager.AddScrollSample(TickScrollDelta, InCurrentTime);
				bIsInertialScroll = true;
			}
		}
		else
		{
			const float CurrentSmoothDeltaTime = GetSmoothDeltaTime(InDeltaTime);
			InertialScrollManager.UpdateScrollVelocity(CurrentSmoothDeltaTime);
			const float ScrollVelocity = InertialScrollManager.GetScrollVelocity();

			if (ScrollVelocity != 0.f)
			{
				if (CanUseInertialScroll(ScrollVelocity))
				{
					bKeepTicking = true;
					ScrollBy(GetTickSpaceGeometry(), ScrollVelocity * CurrentSmoothDeltaTime, AllowOverscroll);
				}
				else
				{
					EndInertialScrolling();
				}
			}

			if (AllowOverscroll == EAllowOverscroll::Yes)
			{
				// If we are currently in overscroll, the list will need refreshing.
				// Do this before UpdateOverscroll, as that could cause GetOverscroll() to be 0
				if (Overscroll.GetOverscroll(GetTickSpaceGeometry()) != 0.0f)
				{
					bKeepTicking = true;
					RequestLayoutRefresh();
				}

				Overscroll.UpdateOverscroll(InDeltaTime);
			}
		}

		TickScrollDelta = 0.f;
	}

	bIsScrollingActiveTimerRegistered = bKeepTicking;
	return bKeepTicking ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}

void STableViewBase::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STableViewBase::Tick);

	if (ItemsPanel.IsValid())
	{
		FGeometry PanelGeometry = FindChildGeometry( AllottedGeometry, ItemsPanel.ToSharedRef() );

		bool bPanelGeometryChanged = PanelGeometryLastTick.GetLocalSize() != PanelGeometry.GetLocalSize();
		
		if ( bItemsNeedRefresh || bPanelGeometryChanged)
		{
			UpdateSmoothDeltaTime(InDeltaTime);

			PanelGeometryLastTick = PanelGeometry;
			
			const int32 NumItemsPerLine = GetNumItemsPerLine();
			const EScrollIntoViewResult ScrollIntoViewResult = ScrollIntoView(PanelGeometry);

			double TargetScrollOffset = GetTargetScrollOffset();
			
			if (InertialScrollManager.GetShouldStopScrollNow())
			{
				TargetScrollOffset = DesiredScrollOffset = CurrentScrollOffset;
				InertialScrollManager.ResetShouldStopScrollNow();
			}

			bool bEndScroll = true;
			if ((bStartedTouchInteraction && bEnableTouchAnimatedScrolling) || (!bStartedTouchInteraction && bEnableAnimatedScrolling))
			{
				CurrentScrollOffset = FMath::FInterpTo(CurrentScrollOffset, TargetScrollOffset, (double)GetSmoothDeltaTime(InDeltaTime), ScrollingAnimationInterpolationSpeed);
				// We may need to wait until a new target scroll offset is recalculated at 0 velocity before we can end the scroll
				const bool bWaitForZeroVelocityAutoScroll = ShouldScrollToFixedLineOffsetAtZeroVelocity() && InertialScrollManager.GetScrollVelocity() != 0.f;
				bEndScroll = !bWaitForZeroVelocityAutoScroll && FMath::IsNearlyEqual(CurrentScrollOffset, TargetScrollOffset, 0.01);
			}

			if (bEndScroll)
			{
				CurrentScrollOffset = TargetScrollOffset;
			}

			const FReGenerateResults ReGenerateResults = ReGenerateItems( PanelGeometry );
			LastGenerateResults = ReGenerateResults;

			const int32 NumItemsBeingObserved = GetNumItemsBeingObserved();
			const int32 NumItemLines = NumItemsBeingObserved / NumItemsPerLine;

			const double InitialDesiredOffset = DesiredScrollOffset;
			const bool bEnoughRoomForAllItems = ReGenerateResults.ExactNumLinesOnScreen >= NumItemLines;
			if (bEnoughRoomForAllItems)
			{
				// We can show all the items, so make sure there is no scrolling.
				SetScrollOffset(0.0);
				CurrentScrollOffset = TargetScrollOffset = DesiredScrollOffset;
			}
			else if (ReGenerateResults.bGeneratedPastLastItem)
			{
				SetScrollOffset(FMath::Max(0.0, ReGenerateResults.NewScrollOffset));
				CurrentScrollOffset = TargetScrollOffset = DesiredScrollOffset;
			}
			
			ItemsPanel->SetFirstLineScrollOffset(GetFirstLineScrollOffset());

			if (AllowOverscroll == EAllowOverscroll::Yes)
			{
				const float OverscrollAmount = Overscroll.GetOverscroll(GetTickSpaceGeometry());
				ItemsPanel->SetOverscrollAmount( OverscrollAmount );
			}

			UpdateSelectionSet();

			// Update scrollbar
			if (NumItemsBeingObserved > 0)
			{
				if (ReGenerateResults.ExactNumLinesOnScreen < 1.0f)
				{
					// We are observing a single row which is larger than the available visible area, so we should calculate thumb size based on that
					const double VisibleSizeFraction = (Orientation == Orient_Vertical ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X) / ReGenerateResults.LengthOfGeneratedItems / NumItemLines;
					const double ThumbSizeFraction = FMath::Min(VisibleSizeFraction, 1.0);
					const double OffsetFraction = CurrentScrollOffset / NumItemsBeingObserved;
					ScrollBar->SetState( OffsetFraction, ThumbSizeFraction );
				}
				else
				{
					// The thumb size is whatever fraction of the items we are currently seeing (including partially seen items).
					// e.g. if we are seeing 0.5 of the first generated widget and 0.75 of the last widget, that's 1.25 widgets.
					const double ThumbSizeFraction = ReGenerateResults.ExactNumLinesOnScreen / NumItemLines;
					const double OffsetFraction = CurrentScrollOffset / NumItemsBeingObserved;
					ScrollBar->SetState( OffsetFraction, ThumbSizeFraction );
				}
			}
			else
			{
				const double ThumbSizeFraction = 1;
				const double OffsetFraction = 0;
				ScrollBar->SetState( OffsetFraction, ThumbSizeFraction );
			}

			bWasAtEndOfList = (ScrollBar->DistanceFromBottom() < SMALL_NUMBER);

			bItemsNeedRefresh = false;
			ItemsPanel->SetRefreshPending(false);

			Invalidate(EInvalidateWidget::ChildOrder);
			
			if (ScrollIntoViewResult == EScrollIntoViewResult::Success)
			{
				// Notify as soon as we've made a widget for the item, even if we still have scrolling to do
				NotifyItemScrolledIntoView();
			}
			if (ScrollIntoViewResult == EScrollIntoViewResult::Deferred || CurrentScrollOffset != TargetScrollOffset)
			{
				// Either we haven't made the item yet or we still have scrolling to do, so we'll need another refresh next frame
				RequestLayoutRefresh();
			}
			else if (CurrentScrollOffset == TargetScrollOffset
					&& !IsUserScrolling()
					&& (AllowOverscroll == EAllowOverscroll::No || Overscroll.GetOverscroll(GetTickSpaceGeometry()) == 0.0f))
			{
				// Sync the desired scroll offset with the target scroll offset when the scroll ends.
				// This prevents teleporting the table upon the next scroll. 
				DesiredScrollOffset = TargetScrollOffset;
				
				NotifyFinishedScrolling();
			}

			OnItemsRebuilt.ExecuteIfBound();
		}
	}
}


void STableViewBase::ScrollBar_OnUserScrolled( float InScrollOffsetFraction )
{
	const double ClampedScrollOffsetInItems = FMath::Clamp<double>( InScrollOffsetFraction, 0.0, 1.0 )* GetNumItemsBeingObserved();
	ScrollTo( ClampedScrollOffsetInItems );
}


FReply STableViewBase::OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bEnableTouchScrolling && MouseEvent.IsTouchEvent())
	{
		// Clear any inertia
		EndInertialScrolling(true);
		// We have started a new interaction; track how far the user has moved since they put their finger down.
		AmountScrolledWhileRightMouseDown = 0;

		PressedScreenSpacePosition = MouseEvent.GetScreenSpacePosition();

		// Someone put their finger down in this list, so they probably want to drag the list.
		bStartedTouchInteraction = true;
		return FReply::Unhandled();
	}
	else
	{
		return FReply::Unhandled();
	}
}


FReply STableViewBase::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Zero the scroll velocity so the list stops immediately on mouse down, even if the user does not drag
	EndInertialScrolling(true);

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		OnRightMouseButtonDown(MouseEvent);
	}

	if (bEnableRightClickScrolling)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && ScrollBar->IsNeeded())
		{
			AmountScrolledWhileRightMouseDown = 0;

			// NOTE: We don't bother capturing the mouse, unless the user starts dragging a few pixels (see the
			// mouse move handling here.)  This is important so that the item row has a chance to select
			// items when the right mouse button is released.  Just keep in mind that you might not get
			// an OnMouseButtonUp event for the right mouse button if the user moves off of the table before
			// they reach our scroll threshold
			return FReply::Handled();
		}
		else if (this->HasMouseCapture())
		{
			// Consume all mouse buttons while we are RMB-dragging.
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();			
}

FReply STableViewBase::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if (bEnableRightClickScrolling)
	{
		if ( this->HasMouseCapture() )
		{
			// Consume all other mouse buttons while we are RMB-dragging.
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();			

}


FReply STableViewBase::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{

		OnRightMouseButtonUp( MouseEvent );

		FReply Reply = FReply::Handled().ReleaseMouseCapture();
		bShowSoftwareCursor = false;

		// If we have mouse capture, snap the mouse back to the closest location that is within the list's bounds
		if ( HasMouseCapture() )
		{
			FSlateRect ListScreenSpaceRect = MyGeometry.GetLayoutBoundingRect();
			FVector2f CursorPosition = MyGeometry.LocalToAbsolute( SoftwareCursorPosition );

			FIntPoint BestPositionInList(
				FMath::RoundToInt( FMath::Clamp( CursorPosition.X, ListScreenSpaceRect.Left, ListScreenSpaceRect.Right ) ),
				FMath::RoundToInt( FMath::Clamp( CursorPosition.Y, ListScreenSpaceRect.Top, ListScreenSpaceRect.Bottom ) )
				);

			Reply.SetMousePos(BestPositionInList);
		}

		return Reply;
	}
	return FReply::Unhandled();
}


FReply STableViewBase::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{	
	if( bEnableRightClickScrolling && MouseEvent.IsMouseButtonDown( EKeys::RightMouseButton ) && !MouseEvent.IsTouchEvent() && bIsPointerScrollingEnabled )
	{
		// We only care about deltas along the scroll axis
		FTableViewDimensions CursorDeltaDimensions(Orientation, MouseEvent.GetCursorDelta());
		CursorDeltaDimensions.LineAxis = 0.f;
		
		const float ScrollByAmount = CursorDeltaDimensions.ScrollAxis / MyGeometry.Scale;

		// If scrolling with the right mouse button, we need to remember how much we scrolled.
		// If we did not scroll at all, we will bring up the context menu when the mouse is released.
		AmountScrolledWhileRightMouseDown += FMath::Abs( ScrollByAmount );

		// Has the mouse moved far enough with the right mouse button held down to start capturing
		// the mouse and dragging the view?
		if( IsRightClickScrolling() )
		{
			// Make sure the active timer is registered to update the inertial scroll
			if (!bIsScrollingActiveTimerRegistered)
			{
				bIsScrollingActiveTimerRegistered = true;
				RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &STableViewBase::UpdateInertialScroll));
			}

			TickScrollDelta -= ScrollByAmount;

			const float AmountScrolled = this->ScrollBy( MyGeometry, -ScrollByAmount, AllowOverscroll );

			FReply Reply = FReply::Handled();

			// The mouse moved enough that we're now dragging the view. Capture the mouse
			// so the user does not have to stay within the bounds of the list while dragging.
			if(this->HasMouseCapture() == false)
			{
				Reply.CaptureMouse( AsShared() ).UseHighPrecisionMouseMovement( AsShared() );
				SoftwareCursorPosition = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
				bShowSoftwareCursor = true;
			}

			// Check if the mouse has moved.
			if( AmountScrolled != 0 )
			{
				SoftwareCursorPosition += CursorDeltaDimensions.ToVector2D();
			}

			return Reply;
		}
	}

	return FReply::Unhandled();
}


void STableViewBase::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bEnableTouchScrolling && MouseEvent.IsTouchEvent())
	{
		if ( !bStartedTouchInteraction )
		{
			// If we don't have touch capture, see if a touch event entered from a child widget.
			// If it did, begin scrolling
			if ( MyGeometry.IsUnderLocation(MouseEvent.GetLastScreenSpacePosition()) )
			{
				bStartedTouchInteraction = true;
			}
		}
	}
}


void STableViewBase::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	bStartedTouchInteraction = false;
	if(this->HasMouseCapture() == false)
	{
		// No longer scrolling (unless we have mouse capture)
		AmountScrolledWhileRightMouseDown = 0;
	}
}


FReply STableViewBase::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (WheelScrollMultiplier == 0.0f)
    {
        return FReply::Unhandled();
    }

	if( bIsPointerScrollingEnabled && !MouseEvent.IsControlDown() )
	{
		// Make sure scroll velocity is cleared so it doesn't fight with the mouse wheel input
		EndInertialScrolling();

		float AmountScrolledInItems = 0.f;
		if (FixedLineScrollOffset.IsSet())
		{
			// When we need to maintain a fixed offset, we scroll by items. This prevents the list not moving or jumping unexpectedly far on an individual scroll wheel motion.
			const double AdditionalOffset = (MouseEvent.GetWheelDelta() >= 0.f ? -1.f : 1.f) * GetNumItemsPerLine();
			const double NewScrollOffset = FMath::Max(0., DesiredScrollOffset + AdditionalOffset);
			AmountScrolledInItems = this->ScrollTo(NewScrollOffset);
		}
		else
		{
			// No required offset to maintain, so we scroll by units
			AmountScrolledInItems = this->ScrollBy(MyGeometry, -MouseEvent.GetWheelDelta() * WheelScrollMultiplier, EAllowOverscroll::No);
		}
		
		if (ConsumeMouseWheel == EConsumeMouseWheel::Always || (FMath::Abs(AmountScrolledInItems) > 0.0f && ConsumeMouseWheel != EConsumeMouseWheel::Never))
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}


FReply STableViewBase::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( InKeyEvent.IsControlDown() && InKeyEvent.GetKey() == EKeys::End )
	{
		ScrollToBottom();
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent); 
}

FCursorReply STableViewBase::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if ( IsRightClickScrolling() && CursorEvent.IsMouseButtonDown(EKeys::RightMouseButton) )
	{
		// We hide the native cursor as we'll be drawing the software EMouseCursor::GrabHandClosed cursor
		return FCursorReply::Cursor( EMouseCursor::None );
	}
	else
	{
		return FCursorReply::Unhandled();
	}
}

FReply STableViewBase::OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	// See OnPreviewMouseButtonDown()
	//     if (MouseEvent.IsTouchEvent())

	NotifyTouchStart();

	return FReply::Unhandled();
}

FReply STableViewBase::OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	if (bIsPointerScrollingEnabled && bEnableTouchScrolling && bStartedTouchInteraction)
	{
		// We only care about deltas along the scroll axis
		FTableViewDimensions CursorDeltaDimensions(Orientation, InTouchEvent.GetCursorDelta());
		CursorDeltaDimensions.LineAxis = 0.f;

		const float ScrollByAmount = CursorDeltaDimensions.ScrollAxis / MyGeometry.Scale;

		AmountScrolledWhileRightMouseDown += FMath::Abs( ScrollByAmount );
		
		TickScrollDelta -= ScrollByAmount;

		if (FSlateApplication::Get().HasTraveledFarEnoughToTriggerDrag(InTouchEvent, PressedScreenSpacePosition, Orientation))
		{
			// Make sure the active timer is registered to update the inertial scroll
			if ( !bIsScrollingActiveTimerRegistered )
			{
				bIsScrollingActiveTimerRegistered = true;
				RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &STableViewBase::UpdateInertialScroll));
			}

			const float AmountScrolled = this->ScrollBy( MyGeometry, -ScrollByAmount, AllowOverscroll );
			if (AmountScrolled != 0)
			{
				ScrollBar->BeginScrolling();

				NotifyTouchMove();

				// The user has moved the list some amount; they are probably
				// trying to scroll. From now on, the list assumes the user is scrolling
				// until they lift their finger.
				return HasMouseCapture() ? FReply::Handled() : FReply::Handled().CaptureMouse(AsShared());
			}
		}

		return FReply::Unhandled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply STableViewBase::OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	AmountScrolledWhileRightMouseDown = 0;
	bStartedTouchInteraction = false;

	ScrollBar->EndScrolling();

	NotifyTouchEnd();

	if (HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}
	else
	{
		return FReply::Unhandled();
	}
}


int32 STableViewBase::GetNumGeneratedChildren() const
{
	return (ItemsPanel.IsValid())
		? ItemsPanel->GetChildren()->Num()
		: 0;
}

TSharedPtr<SWidget> STableViewBase::GetGeneratedChildAt(const int32 Index) const
{
	return GetNumGeneratedChildren() > Index ? ItemsPanel->GetChildren()->GetChildAt(Index).ToSharedPtr() : nullptr;
}

TSharedPtr<SHeaderRow> STableViewBase::GetHeaderRow() const
{
	return HeaderRow;
}

bool STableViewBase::IsRightClickScrolling() const
{
	return AmountScrolledWhileRightMouseDown >= FSlateApplication::Get().GetDragTriggerDistance() &&
		(this->ScrollBar->IsNeeded() || AllowOverscroll == EAllowOverscroll::Yes);
}

bool STableViewBase::IsUserScrolling() const
{
	bool bUserScroll = this->ScrollBar->IsNeeded() && this->ScrollBar->IsScrolling();
	return bUserScroll || IsRightClickScrolling();
}

void STableViewBase::RequestListRefresh()
{
	RequestLayoutRefresh();
}

bool STableViewBase::IsPendingRefresh() const
{
	return bItemsNeedRefresh || ItemsPanel->IsRefreshPending();
}

int32 STableViewBase::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayerId = LayerId;

	const FSlateBrush* BackgroundBrushResource = GetBackgroundBrush();
	if ( BackgroundBrushResource && BackgroundBrushResource->DrawAs != ESlateBrushDrawType::NoDrawType )
	{

		const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
		const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		FSlateDrawElement::MakeBox(
				OutDrawElements,
				++NewLayerId,
				AllottedGeometry.ToPaintGeometry(),
				BackgroundBrushResource,
				DrawEffects,
				BackgroundBrushResource->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint() 
		);
	}

	NewLayerId = SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayerId, InWidgetStyle, bParentEnabled );

	if( !bShowSoftwareCursor )
	{
		return NewLayerId;
	}

	const FSlateBrush* Brush = FCoreStyle::Get().GetBrush(TEXT("SoftwareCursor_Grab"));
	const FVector2f CursorSize = Brush->ImageSize / AllottedGeometry.Scale;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		++NewLayerId,
		AllottedGeometry.ToPaintGeometry(CursorSize, FSlateLayoutTransform(SoftwareCursorPosition - (CursorSize / .5f ))),
		Brush
		);

	return NewLayerId;
}

STableViewBase::STableViewBase( ETableViewMode::Type InTableViewMode )
	: TableViewMode( InTableViewMode )
	, bStartedTouchInteraction( false )
	, AmountScrolledWhileRightMouseDown( 0 )
	, TickScrollDelta( 0 )
	, LastGenerateResults( 0,0,0,false )
	, bWasAtEndOfList(false)
	, SoftwareCursorPosition( ForceInitToZero )
	, bShowSoftwareCursor( false )
	, WheelScrollMultiplier(GetGlobalScrollAmount())
	, bShouldUseShadowBoxStyle(false)
	, bIsScrollingActiveTimerRegistered( false )
	, Overscroll()
	, AllowOverscroll(EAllowOverscroll::Yes)
	, ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
	, bItemsNeedRefresh( false )	
	, ShadowBoxStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBox"))
	, SelectionModeAttribute(*this, ESelectionMode::Multi)
	, PreviousSelectionMode(ESelectionMode::Multi)
	, BackgroundBrushAttribute(*this, FStyleDefaults::GetNoBrush())
{
	FixedLineScrollOffset = 0.25f;
}

STableViewBase::~STableViewBase() = default;

double STableViewBase::GetTargetScrollOffset() const
{
	if (ShouldScrollToFixedLineOffsetAtZeroVelocity() && InertialScrollManager.GetScrollVelocity() == 0.f)
	{
		// The scroll has ended, we calculate the appropriate the scroll offset to snap on a line.
		// We don't do that if we are at the end of the list because in this case showing the last line correctly is more important.
		const int32 NumItemsPerLine = GetNumItemsPerLine();
		double DesiredLineOffset = DesiredScrollOffset / NumItemsPerLine;
		const double RoundedDesiredLineOffset = FMath::RoundToDouble(DesiredLineOffset);

		// If DesiredLineOffset is almost equal to RoundedDesiredLineOffset:
		// There is no need to round it and it is better not to floor it in case it is slightly less that the rounded value
		if (!FMath::IsNearlyEqual(RoundedDesiredLineOffset, DesiredLineOffset, UE_DOUBLE_KINDA_SMALL_NUMBER))
		{
			if (bIsInertialScroll)
			{
				// Scroll to the item most in view
				DesiredLineOffset = RoundedDesiredLineOffset;
			}
			else
			{
				// Scroll to the topmost item.
				DesiredLineOffset = FMath::FloorToDouble(DesiredLineOffset);
			}
		}
		DesiredLineOffset -= FixedLineScrollOffset.GetValue();

		return FMath::Max(0.0, DesiredLineOffset * NumItemsPerLine);
	}
	return DesiredScrollOffset;
}

bool STableViewBase::ShouldScrollToFixedLineOffsetAtZeroVelocity() const
{
	return !bWasAtEndOfList && FixedLineScrollOffset.IsSet() && !IsRightClickScrolling();
}

float STableViewBase::GetSmoothDeltaTime(float InDeltaTime)
{
	// This is to be sure we have an initial value.
	return SmoothDeltaTime >= 0 ? SmoothDeltaTime : InDeltaTime;
}

void STableViewBase::UpdateSmoothDeltaTime(float InDeltaTime)
{
	if (DeltaTimeQueue.IsValidIndex(DeltaTimeCircularIndex))
	{
		DeltaTimeQueue[DeltaTimeCircularIndex] = InDeltaTime;
	}
	else
	{
		DeltaTimeQueue.Add(InDeltaTime);
	}
	DeltaTimeCircularIndex = (DeltaTimeCircularIndex + 1) % MaxNumberOfDeltaTimes;
	
	SortedDeltaTimes.Empty(MaxNumberOfDeltaTimes);
	SortedDeltaTimes.Append(DeltaTimeQueue);
	SortedDeltaTimes.Sort();

	float SumOfDeltaTimes = 0.0f;

	const int32 NumberOfOutliers = FMath::RoundFromZero(SortedDeltaTimes.Num() * FMath::Abs(RatioOfOutliers));
	const int32 HalfNumberOfOutliers = NumberOfOutliers / 2;
	
	const int32 StartIndex = HalfNumberOfOutliers;
	const int32 EndIndex = SortedDeltaTimes.Num() - HalfNumberOfOutliers;

	ensure(EndIndex - StartIndex > 0);

	for (int32 i = StartIndex; i < EndIndex; ++i)
	{
		SumOfDeltaTimes += SortedDeltaTimes[i];
	}

	SmoothDeltaTime = SumOfDeltaTimes / (EndIndex - StartIndex);
}

float STableViewBase::ScrollBy(const FGeometry& MyGeometry, float ScrollByAmountInSlateUnits, EAllowOverscroll InAllowOverscroll)
{
	const int32 NumItemsBeingObserved = GetNumItemsBeingObserved();
	const float FractionalScrollOffsetInItems = NumItemsBeingObserved == 0 ? 0.f : (DesiredScrollOffset + GetScrollRateInItems() * ScrollByAmountInSlateUnits) / NumItemsBeingObserved;
	const double ClampedScrollOffsetInItems = FMath::Clamp<double>( FractionalScrollOffsetInItems*NumItemsBeingObserved, -10.0f, NumItemsBeingObserved+10.0f ) * NumItemsBeingObserved;
	if (InAllowOverscroll == EAllowOverscroll::Yes)
	{
		Overscroll.ScrollBy(MyGeometry, ClampedScrollOffsetInItems - ScrollByAmountInSlateUnits );
	}
	return ScrollTo( ClampedScrollOffsetInItems );
}

float STableViewBase::ScrollTo( float InScrollOffset)
{
	const float NewScrollOffset = FMath::Clamp( InScrollOffset, -10.0f, GetNumItemsBeingObserved()+10.0f );
	float AmountScrolled = FMath::Abs( DesiredScrollOffset - NewScrollOffset );

	if (bWasAtEndOfList && NewScrollOffset >= DesiredScrollOffset)
	{
		AmountScrolled = 0;
	}

	SetScrollOffset( NewScrollOffset );

	return AmountScrolled;
}

float STableViewBase::GetScrollOffset() const
{
	return DesiredScrollOffset;
}

void STableViewBase::SetScrollOffset( const float InScrollOffset )
{
	const float InValidatedOffset = FMath::Max(0.0f, InScrollOffset);
	if (DesiredScrollOffset != InValidatedOffset)
	{
		DesiredScrollOffset = InValidatedOffset;
		OnTableViewScrolled.ExecuteIfBound(DesiredScrollOffset);
		RequestLayoutRefresh();
	}
}

void STableViewBase::EndInertialScrolling(const bool bInShouldStopScrollNow)
{
	InertialScrollManager.ClearScrollVelocity(bInShouldStopScrollNow);
	bIsInertialScroll = false;
}

void STableViewBase::AddScrollOffset(const float InScrollOffsetDelta, bool RefreshList)
{
	if (FMath::IsNearlyEqual(InScrollOffsetDelta, 0.0f) == false)
	{
		DesiredScrollOffset += InScrollOffsetDelta;
		if (RefreshList)
		{
			OnTableViewScrolled.ExecuteIfBound(DesiredScrollOffset);
			RequestLayoutRefresh();
		}
	}
}

void STableViewBase::SetScrollbarVisibility(const EVisibility InVisibility)
{
	if (ScrollBar)
	{
		ScrollBar->SetVisibility(InVisibility);
	}
}

void STableViewBase::SetScrollbarPadding(const FMargin& InScrollbarPadding)
{
	ScrollBarSlotPadding = InScrollbarPadding;

	if (Orientation == Orient_Vertical)
	{
		if (VerticalScrollBarSlot)
		{
			VerticalScrollBarSlot->SetPadding(ScrollBarSlotPadding);
		}
	}
	else
	{
		if (HorizontalScrollBarSlot)
		{
			HorizontalScrollBarSlot->SetPadding(ScrollBarSlotPadding);
		}
	}
}

EVisibility STableViewBase::GetScrollbarVisibility() const
{
	return ScrollBar ? ScrollBar->ShouldBeVisible() : EVisibility::Collapsed;
}

bool STableViewBase::IsScrollbarNeeded() const
{
	if (ScrollBar)
	{
		return ScrollBar->IsNeeded();
	}

	return false;
}

void STableViewBase::SetFixedLineScrollOffset(TOptional<double> InFixedLineScrollOffset)
{
	if (FixedLineScrollOffset != InFixedLineScrollOffset)
	{
		FixedLineScrollOffset = InFixedLineScrollOffset;
		RequestLayoutRefresh();
	}
}

void STableViewBase::SetIsScrollAnimationEnabled(bool bInEnableScrollAnimation)
{
	bEnableAnimatedScrolling = bInEnableScrollAnimation;
}

void STableViewBase::SetScrollingAnimationInterpolationSpeed(float InScrollingAnimationInterpolationSpeed)
{
	ScrollingAnimationInterpolationSpeed = InScrollingAnimationInterpolationSpeed;
}

void STableViewBase::SetEnableTouchAnimatedScrolling(bool bInEnableTouchAnimatedScrolling)
{
	bEnableTouchAnimatedScrolling = bInEnableTouchAnimatedScrolling;
}

void STableViewBase::SetAllowOverscroll(EAllowOverscroll InAllowOverscroll)
{
	AllowOverscroll = InAllowOverscroll;
}

void STableViewBase::SetIsRightClickScrollingEnabled(const bool bInEnableRightClickScrolling)
{
	bEnableRightClickScrolling = bInEnableRightClickScrolling;
}

void STableViewBase::SetIsTouchScrollingEnabled(const bool bInEnableTouchScrolling)
{
	bEnableTouchScrolling = bInEnableTouchScrolling;
	ensureMsgf(!bStartedTouchInteraction, TEXT("TouchScrollingEnabled flag should not be changed while scrolling."));
}

void STableViewBase::SetSelectItemOnNavigation(const bool bInSelectItemOnNavigation)
{
	bSelectItemOnNavigation = bInSelectItemOnNavigation;
}

void STableViewBase::SetWheelScrollMultiplier(float NewWheelScrollMultiplier)
{
	WheelScrollMultiplier = NewWheelScrollMultiplier;
}

void STableViewBase::SetIsPointerScrollingEnabled(bool bInIsPointerScrollingEnabled)
{
	bIsPointerScrollingEnabled = bInIsPointerScrollingEnabled;
}

void STableViewBase::SetIsGamepadScrollingEnabled(bool bInIsGamepadScrollingEnabled)
{
	bIsGamepadScrollingEnabled = bInIsGamepadScrollingEnabled;
}

void STableViewBase::SetBackgroundBrush(TAttribute<const FSlateBrush*> InBackgroundBrush)
{
	BackgroundBrushAttribute.Assign(*this, TAttribute<FSlateBrushCached>::CreateLambda(
		[BackgroundBrush = MoveTemp(InBackgroundBrush)]()
		{
			return FSlateBrushCached(BackgroundBrush.Get(nullptr));
		}));
}

void STableViewBase::InsertWidget( const TSharedRef<ITableRow> & WidgetToInset )
{
	ItemsPanel->AddSlot(0)
	[
		WidgetToInset->AsWidget()
	];
}

void STableViewBase::AppendWidget( const TSharedRef<ITableRow>& WidgetToAppend )
{
	ItemsPanel->AddSlot()
	[
		WidgetToAppend->AsWidget()
	];
}

void STableViewBase::ClearWidgets()
{
	ItemsPanel->ClearItems();
}

const FChildren* STableViewBase::GetConstructedTableItems() const
{
	return ItemsPanel->GetChildren();
}

void STableViewBase::InsertPinnedWidget( const TSharedRef<SWidget> & WidgetToInset )
{
	PinnedItemsPanel->AddSlot(0)
	[
		WidgetToInset
	];
	PinnedItemsPanel->SetVisibility(EVisibility::Visible);
}

void STableViewBase::AppendPinnedWidget( const TSharedRef<SWidget>& WidgetToAppend )
{
	PinnedItemsPanel->AddSlot()
	[
		WidgetToAppend
	];
	PinnedItemsPanel->SetVisibility(EVisibility::Visible);
}

void STableViewBase::ClearPinnedWidgets()
{
	PinnedItemsPanel->SetVisibility(EVisibility::Collapsed);
	PinnedItemsPanel->ClearItems();
}

float STableViewBase::GetItemWidth() const
{
	return GetItemSize().X;
}

float STableViewBase::GetItemHeight() const
{
	return GetItemSize().Y;
}

UE::Slate::FDeprecateVector2DResult STableViewBase::GetItemSize() const
{
	FTableViewDimensions ItemDimensions = ItemsPanel->GetItemSize(PanelGeometryLastTick);
	ItemDimensions.LineAxis += ItemsPanel->GetItemPadding(PanelGeometryLastTick);
	return ItemDimensions.ToVector2D();
}

void STableViewBase::SetItemHeight(TAttribute<float> Height)
{
	ItemsPanel->SetItemHeight(Height);
}

void STableViewBase::SetItemWidth(TAttribute<float> Width)
{
	ItemsPanel->SetItemWidth(Width);
}

float STableViewBase::GetNumLiveWidgets() const
{
	return ItemsPanel->GetChildren()->Num();
}

int32 STableViewBase::GetNumItemsPerLine() const
{
	return 1;
}

float STableViewBase::GetFirstLineScrollOffset() const
{
	// FMath::Fractional() is insufficient here as it casts to int32 (too small for the integer part of a float when
	// the scroll offset is enormous), so we do a double/int64 version here.
	const double FirstLineScrollOffset = CurrentScrollOffset / GetNumItemsPerLine();
	return FirstLineScrollOffset - (int64)FirstLineScrollOffset;
}

void STableViewBase::NavigateToWidget(const uint32 UserIndex, const TSharedPtr<SWidget>& NavigationDestination, ENavigationSource NavigationSource, EUINavigation NavigationType) const
{
	FSlateApplication::Get().NavigateToWidget(UserIndex, NavigationDestination, NavigationSource, NavigationType);
}

int32 STableViewBase::FindChildUnderPosition(FArrangedChildren& ArrangedChildren, const FVector2D& ArrangedSpacePosition) const
{
	if (ItemsPanel.IsValid())
	{
		const FGeometry MyGeometry = ItemsPanel->GetCachedGeometry();
		ItemsPanel->ArrangeChildren(MyGeometry, ArrangedChildren, true);

		return ItemsPanel->FindChildUnderPosition(ArrangedChildren, ArrangedSpacePosition);
	}

	return INDEX_NONE;
}

void STableViewBase::OnRightMouseButtonUp(const FPointerEvent& MouseEvent)
{
	FVector2f SummonLocation = MouseEvent.GetScreenSpacePosition();
	const bool bShouldOpenContextMenu = !IsRightClickScrolling();
	const bool bContextMenuOpeningBound = OnContextMenuOpening.IsBound();

	if (bShouldOpenContextMenu && bContextMenuOpeningBound)
	{
		// Get the context menu content. If NULL, don't open a menu.
		TSharedPtr<SWidget> MenuContent = OnContextMenuOpening.Execute();

		if (MenuContent.IsValid())
		{
			bShowSoftwareCursor = false;

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		}
	}

	AmountScrolledWhileRightMouseDown = 0;
}

float STableViewBase::GetScrollRateInItems() const
{
	return (LastGenerateResults.LengthOfGeneratedItems != 0 && LastGenerateResults.ExactNumLinesOnScreen != 0)
		// Approximate a consistent scrolling rate based on the average item height.
		? LastGenerateResults.ExactNumLinesOnScreen / LastGenerateResults.LengthOfGeneratedItems
		// Scroll 1/2 an item at a time as a default.
		: 0.5f;
}

void STableViewBase::RequestLayoutRefresh()
{
	bItemsNeedRefresh = true;

	if (ItemsPanel.IsValid())
	{
		ItemsPanel->SetRefreshPending(true);
	}

	Invalidate(EInvalidateWidget::Layout);
}

void STableViewBase::ScrollToTop()
{
	EndInertialScrolling();
	SetScrollOffset(0);
	RequestLayoutRefresh();
}

void STableViewBase::ScrollToBottom()
{
	EndInertialScrolling();
	SetScrollOffset(GetNumItemsBeingObserved());
	RequestLayoutRefresh();
}

bool STableViewBase::IsScrolling() const
{
	return ScrollBar->IsScrolling();
}

FVector2D STableViewBase::GetScrollDistance()
{
	return FVector2D( 0, ScrollBar->DistanceFromTop() );
}

FVector2D STableViewBase::GetScrollDistanceRemaining()
{
	return FVector2D( 0, ScrollBar->DistanceFromBottom() );
}

void STableViewBase::SetShadowBoxStyle(const FScrollBoxStyle* InShadowBoxStyle)
{
	if (ShadowBoxStyle != InShadowBoxStyle)
	{
		ShadowBoxStyle = InShadowBoxStyle;
		if (TopLeftShadowImage)
		{
			const FSlateBrush* Image = nullptr;
			if (ShadowBoxStyle)
			{
				Image = Orientation == Orient_Vertical ? &ShadowBoxStyle->TopShadowBrush : &ShadowBoxStyle->LeftShadowBrush;
			}
			TopLeftShadowImage->SetImage(Image);
			TopLeftShadowImage->SetVisibility(ShadowBoxStyle ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
		}
		if (BottomRightShadowImage)
		{
			const FSlateBrush* Image = nullptr;
			if (ShadowBoxStyle)
			{
				Image = Orientation == Orient_Vertical ? &ShadowBoxStyle->BottomShadowBrush : &ShadowBoxStyle->RightShadowBrush;
			}
			BottomRightShadowImage->SetImage(Image);
			BottomRightShadowImage->SetVisibility(ShadowBoxStyle ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
		}
	}
}

TSharedRef<class SWidget> STableViewBase::GetScrollWidget()
{
	return SharedThis(this);
}

void STableViewBase::OnClippingChanged()
{
	ItemsPanel->SetClipping(GetClipping());
}

FSlateColor STableViewBase::GetStartShadowOpacity() const
{
	// The shadow should only be visible when the user needs a hint that they can scroll up.
	const float ShadowOpacity = FMath::Clamp( CurrentScrollOffset/ShadowFadeDistance, 0.0f, 1.0f);
	
	return FLinearColor(1.0f, 1.0f, 1.0f, ShadowOpacity);
}

FSlateColor STableViewBase::GetEndShadowOpacity() const
{	
	// The shadow should only be visible when the user needs a hint that they can scroll down.
	const float ShadowOpacity = (ScrollBar->DistanceFromBottom() * GetScrollComponentFromVector(ItemsPanel->GetDesiredSize()) / ShadowFadeDistance);
	
	return FLinearColor(1.0f, 1.0f, 1.0f, ShadowOpacity);
}

bool STableViewBase::CanUseInertialScroll( float ScrollAmount ) const
{
	const auto CurrentOverscroll = Overscroll.GetOverscroll(GetTickSpaceGeometry());

	// We allow sampling for the inertial scroll if we are not in the overscroll region,
	// Or if we are scrolling outwards of the overscroll region
	return CurrentOverscroll == 0.f || FMath::Sign(CurrentOverscroll) != FMath::Sign(ScrollAmount);
}

int32 STableViewBase::GetNumPinnedItems() const
{
	return PinnedItemsPanel->GetChildren()->Num();
}

EVisibility STableViewBase::GetPinnedItemsVisiblity() const
{
	return PinnedItemsPanel->GetChildren()->Num() != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

static const TBitArray<> EmptyBitArray = TBitArray<>();

const TBitArray<>& TableViewHelpers::GetEmptyBitArray()
{
	return EmptyBitArray;
}
