// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "AnimatedAttributeManager.h"
#include "AttributeInterpolator.h"

/**
 * Animated Attribute object
 */
template< typename NumericType >
class TAnimatedAttribute : public TAnimatedAttributeBase
{
public:

	/**
	 * Attribute 'getter' delegate
	 *
	 * NumericType GetValue() const
	 *
	 * @return  The attribute's value
	 */
	using FGetter = TDelegate<NumericType()>;

protected:

	struct FPrivateToken { explicit FPrivateToken() = default; };
	
	/** Default constructor. */
	TAnimatedAttribute() = delete;

public:

	/**
	 * Construct only from interpolator settings
	 * This is used only for the Create methods and only accessible from there
	 * due to the use of a protected FPrivateToken. Please rely on the Create
	 * methods if you are seeing a compiler error related to this. 
	 *
	 * @param  InSettings		The settings for the interpolator
	 */
	template< typename InterpolatorSettings >
	TAnimatedAttribute( FPrivateToken, const InterpolatorSettings& InSettings )
		: Interpolator( MakeUnique<typename InterpolatorSettings::InterpolatorType>(InSettings) )
		, Getter()
		, DesiredValue() 
	{
	}

	/**
	 * Static: Creates an animated attribute implicitly from an initial value
	 *
	 * @param  InSettings		The settings for the interpolator
	 * @param  InInitialValue	The value for this attribute
	 */
	template< typename InterpolatorSettings, typename OtherType >
	[[nodiscard]] static TSharedRef< TAnimatedAttribute > Create( const InterpolatorSettings& InSettings, OtherType&& InInitialValue )
	{
		TSharedRef< TAnimatedAttribute< NumericType > > Attribute = MakeShared< TAnimatedAttribute< NumericType > >(FPrivateToken{}, InSettings);
		Attribute->Set(Forward<OtherType>(InInitialValue));
		Attribute->Register();
		return Attribute;
	}
	
	/**
	 * Static: Creates an animated attribute that's pre-bound to the specified 'getter' delegate
	 *
	 * @param  InSettings		The settings for the interpolator
	 * @param  InGetter			Delegate to bind
	 * @param  InDefaultValue   The optional default value. If not specified the getter will be used to determine the default
	 */
	template< typename InterpolatorSettings >
	[[nodiscard]] static TSharedRef< TAnimatedAttribute > CreateWithGetter( const InterpolatorSettings& InSettings, const FGetter& InGetter, const TOptional<NumericType>& InDefaultValue = TOptional<NumericType>() )
	{
		TSharedRef< TAnimatedAttribute< NumericType > > Attribute = MakeShared< TAnimatedAttribute< NumericType > >(FPrivateToken{}, InSettings);
		Attribute->Set(InDefaultValue.IsSet() ? InDefaultValue.GetValue() : InGetter.Execute());
		Attribute->Getter = InGetter;
		Attribute->Register();
		return Attribute;
	}
	
	/**
	 * Static: Creates an animated attribute that's pre-bound to the specified 'getter' delegate
	 *
	 * @param  InSettings		The settings for the interpolator
	 * @param  InGetter			Delegate to bind
	 * @param  InDefaultValue   The optional default value. If not specified the getter will be used to determine the default
	 */
	template< typename InterpolatorSettings >
	[[nodiscard]] static TSharedRef< TAnimatedAttribute > CreateWithGetter( const InterpolatorSettings& InSettings, FGetter&& InGetter, const TOptional<NumericType>& InDefaultValue = TOptional<NumericType>() )
	{
		TSharedRef< TAnimatedAttribute< NumericType > > Attribute = MakeShared< TAnimatedAttribute< NumericType > >(FPrivateToken{}, InSettings);
		Attribute->Set(InDefaultValue.IsSet() ? InDefaultValue.GetValue() : InGetter.Execute());
		Attribute->Getter = MoveTemp(InGetter);
		Attribute->Register();
		return Attribute;
	}
	
	/**
	 * Sets the attribute's value while keeping a running interpolation going.
	 * The value will be become the new desired value and the interpolation will
	 * continue trying to reach it. If the interpolation is currently stopped the
	 * animation / interpolation will restart towards the new provided value.
	 *
	 * @param  InNewValue  The value to set the attribute to
	 */
	template< typename OtherType >
	void Set( const OtherType& InNewValue )
	{
		Getter.Unbind();
		if(DesiredValue.IsSet() && !Interpolator->DesiredValue.IsSet())
		{
			Interpolator->SetValue(DesiredValue.GetValue());
		}
		DesiredValue = InNewValue;
		Interpolator->SetValue(DesiredValue.GetValue());
	}

	/**
	 * Sets the attribute's value while keeping a running interpolation going.
	 * The value will be become the new desired value and the interpolation will
	 * continue trying to reach it. If the interpolation is currently stopped the
	 * animation / interpolation will restart towards the new provided value.
	 * 
	 * @param InNewValue  The value to set the attribute to
	 */
	void Set( NumericType&& InNewValue )
	{
		Getter.Unbind();
		if(DesiredValue.IsSet() && !Interpolator->DesiredValue.IsSet())
		{
			Interpolator->SetValue(DesiredValue.GetValue());
		}
		DesiredValue = InNewValue;
		Interpolator->SetValue(DesiredValue.GetValue());
	}

	/**
	 * Sets the attribute's value and disables animation. The interpolator
	 * will be stopped and the value will become the new current value.
	 * 
	 * @param InNewValue  The value to set the attribute to
     */
	void SetValueAndStop( const NumericType& InNewValue )
	{
		Set(InNewValue);
		Interpolator->Reset();
	}
	
	/**
	 * Sets the attribute's value and disables animation. The interpolator
	 * will be stopped and the value will become the new current value.
	 * 
	 * @param InNewValue  The value to set the attribute to
	 */
	void SetValueAndStop( NumericType&& InNewValue )
	{
		Set(Forward<NumericType>(InNewValue));
		Interpolator->Reset();
	}

	/** Was this TAnimatedAttribute ever assigned? */
	bool IsSet() const
	{
		return DesiredValue.IsSet() || IsBound();
	}

	/**
	 * Gets the attribute's current value.
	 * Assumes that the attribute is set.
	 *
	 * @return  The attribute's value
	 */
	const NumericType& Get() const
	{
		// If we have a getter delegate, then we'll call that to generate the value
		if( Getter.IsBound() )
		{
			// Call the delegate to get the value.  Note that this will assert if the delegate is not currently
			// safe to call (e.g. object was deleted and we could detect that)

			// NOTE: We purposely overwrite our value copy here so that we can return the value by address in
			// the most common case, which is an attribute that doesn't have a delegate bound to it at all.
			DesiredValue = Getter.Execute();

			// Update the interpolator
			if(DesiredValue.IsSet())
			{
				Interpolator->SetValue(DesiredValue.GetValue());
			}
		}

		// first see if the interpolator has a value
		if(Interpolator->IsSet())
		{
			return Interpolator->Get();
		}
		
		if(DesiredValue.IsSet())
		{
			// Return the stored value
			return DesiredValue.GetValue();
		}

		static const NumericType EmptyResult = NumericType();
		return EmptyResult;
	}

	/**
	 * Gets the attribute's current value. The attribute may not be set, in which case use the default value provided.
	 * Shorthand for the boilerplate code: MyAttribute.IsSet() ? MyAttribute.Get() : DefaultValue
	 */
	const NumericType& Get( const NumericType& DefaultValue ) const
	{
		return IsSet() ? Get() : DefaultValue;
	}

	/**
	 * Returns the desired value this attribute is trying to reach.
	 */
	const NumericType& GetDesiredValue() const
	{
		if(DesiredValue.IsSet())
		{
			return DesiredValue.GetValue();
		}
		static const NumericType EmptyResult = NumericType();
		return EmptyResult;
	}

	/**
	 * Returns the overall deltatime of the interpolator
	 */
	double GetOverAllDeltaTime() const
	{
		return Interpolator->GetOverAllDeltaTime();
	}

	/**
	 * Returns the currently set delay on the interpolator
	 */
	TOptional<NumericType> GetDelay() const
	{
		return Interpolator->Delay;
	}

	/**
	 * Sets the attribute's delay as a one shot
	 */
	void SetDelayOneShot( double InDelay )
	{
		Interpolator->SetDelayOneShot(InDelay);
	}

	/**
	 * Sets the tolerance of this interpolator
	 */
	void SetTolerance( double Tolerance )
	{
		Interpolator->SetTolerance(Tolerance);
	}

	/**
	 * Returns true if the attribute is currently animating
	 */
	bool IsPlaying() const
	{
		return Interpolator->IsPlaying();
	}

	/**
	 * Enables (or disables) the interpolator and returns values in interpolated or immediate mode
	 */
	void EnableInterpolation(bool bEnabled = true)
	{
		Interpolator->SetEnabled(bEnabled);
	}

	/**
	 * Disables the interpolator and returns values in immediate mode
	 */
	void DisableInterpolation()
	{
		EnableInterpolation(false);
	}

	/**
	 * Checks to see if this attribute has a 'getter' function bound
	 *
	 * @return  True if attribute is bound to a getter function
	 */
	bool IsBound() const
	{
		return Getter.IsBound();
	}

	/**
	 * Is this attribute identical to another TAnimationAttribute.
	 *
	 * @param InOther The other attribute to compare with.
	 * @return true if the attributes are identical, false otherwise.
	 */
	bool IdenticalTo(const TAnimatedAttribute& InOther) const
	{
		if(!Interpolator->IdenticalTo(InOther.Interpolator.Get()))
		{
			return false;
		}

		const bool bIsBound = IsBound();
		if ( bIsBound == InOther.IsBound() )
		{
			if ( bIsBound )
			{
				return Getter.GetHandle() == InOther.Getter.GetHandle();
			}
			return IsSet() == InOther.IsSet() && DesiredValue == InOther.DesiredValue;
		}

		return false;
	}

	/*
	 * Returns the delegate to react to the interpolator starting to interpolate
	 */
	typename TAttributeInterpolator< NumericType >::FInterpolatorEvent& OnInterpolationStarted()
	{
		return Interpolator->OnInterpolationStarted();
	}

	/*
	 * Returns the delegate to react to the interpolator ending to interpolate
	 */
	typename TAttributeInterpolator< NumericType >::FInterpolatorEvent& OnInterpolationStopped()
	{
		return Interpolator->OnInterpolationStopped();
	}

private:

	/**
	 * Updates the interpolator / ticks the animation
	 */
	virtual void Tick(float InDeltaTime) override
	{
		if( Getter.IsBound() )
		{
			Interpolator->SetValue(Getter.Execute());
		}
		if(DesiredValue.IsSet())
		{
			Interpolator->SetValue(DesiredValue.GetValue());
		}
		Interpolator->Tick(InDeltaTime);
	}

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
	template< class OtherType > friend class TAnimatedAttribute;

	/** The attribute's interpolator */
	TUniquePtr< TAttributeInterpolator< NumericType > > Interpolator;

	/** Bound member function for this attribute (may be NULL if no function is bound.)  When set, all attempts
		to read the attribute's value will instead call this delegate to generate the value. */
	/** Our attribute's 'getter' delegate */
	FGetter Getter;
	
	/** Current value.  Mutable so that we can cache the value locally when using a bound Getter (allows const ref return value.) */
	mutable TOptional< NumericType > DesiredValue;
};
