// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Application/SlateApplication.h"

/**
 * Animated Attribute Base Layer - used for registering the attribute centrally
 */
class TAnimatedAttributeBase : public TSharedFromThis<TAnimatedAttributeBase>
{
protected:
	
	SLATE_API TAnimatedAttributeBase();
	SLATE_API virtual ~TAnimatedAttributeBase();

	SLATE_API void Register();
	SLATE_API void Unregister();

	virtual void Tick(float InDeltaTime) = 0;

private:

	TAnimatedAttributeBase(const TAnimatedAttributeBase& InOther) = delete;
	TAnimatedAttributeBase& operator =(const TAnimatedAttributeBase& Other) = delete;

	bool bIsRegistered;

	friend class FAnimatedAttributeManager;
};

/**
 * A central manager for animated attributes
 */
class FAnimatedAttributeManager
{

public:

	FAnimatedAttributeManager();
	~FAnimatedAttributeManager();

	SLATE_API static FAnimatedAttributeManager& Get();
	SLATE_API void Tick(float InDeltaTime);

	SLATE_API void SetupTick();
	SLATE_API void TeardownTick();

private:

	void RegisterAttribute(const TSharedRef<TAnimatedAttributeBase>& InAttribute);
	void UnregisterAttribute(const TSharedRef<TAnimatedAttributeBase>& InAttribute);

	TArray<TWeakPtr<TAnimatedAttributeBase>> Attributes;
	FDelegateHandle TickHandle;

	friend class TAnimatedAttributeBase;
};
