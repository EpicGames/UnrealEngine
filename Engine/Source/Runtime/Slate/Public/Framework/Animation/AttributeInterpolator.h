// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/UnrealMathUtility.h"

enum class EAttributeInterpolatorType : uint8
{
	Easing,
	Arrive,
	Verlet
};

/**
 * Attribute Interpolator Base Class
 */
template< typename NumericType >
class TAttributeInterpolator
{

public:

	DECLARE_MULTICAST_DELEGATE_OneParam(FInterpolatorEvent, TOptional<NumericType>);

	/*
	 * Default constructor
	 */
	TAttributeInterpolator()
	: bEnabled(true)
	, Tolerance(0.001f)
	{
	}

	/*
	 * Default destructor
	 */
	virtual ~TAttributeInterpolator() = default;

	/**
	 * Helper function to compare to values - to allow to specialize per type as needed
	 *
	 * @param InA The first value to compare
	 * @param InB The second value to compare
	 * @return True if the two values are considered "equal"
	 */
	static bool Equals(const NumericType& InA, const NumericType& InB, double InTolerance = 0.001)
	{
		return InA.Equals(InB, InTolerance);
	}

	/**
	 * Returns true if the interpolator is of a given type
	 *
	 * @param InType The type to compare agains
	 * @return True if the interpolator matches the type
	 */
	virtual bool IsTypeOf(EAttributeInterpolatorType InType) const = 0;

	/**
	 * Compares this interpolator to another one
	 *
	 * @param InOther The interpolator to compare this one to
	 * @return True if the this interpolator is considered identical to the passed one
	 */
	virtual bool IdenticalTo(const TAttributeInterpolator* InOther) const = 0;

	/**
	 * Returns true if this interpolator's value has even been set
	 * @return True if this interpolator's value has even been set
	 */
	bool IsSet() const
	{
		return DesiredValue.IsSet();
	}

	/**
	 * Resets the interpolator
	 */
	void Reset() const
	{
		const bool bFireStopEvent = Stop();
		InterpolatedValue.Reset();
		LastValue.Reset();
		DesiredValue.Reset();
		Delay.Reset();
		if(bFireStopEvent)
		{
			InterpolationStopped.Broadcast(TOptional<NumericType>());
		}
	}

	/*
	 * Returns true if the interpolator is enabled
	 */
	bool IsEnabled() const
	{
		return bEnabled;
	}

	/*
	 * Enables (or disables) the interpolator
	 */
	void SetEnabled(bool InEnabled = true)
	{
		if(bEnabled != InEnabled)
		{
			bEnabled = InEnabled;

			// if we disabled or re-enabled the interpolator set
			// the interpolated values based on the desired value
			if(DesiredValue.IsSet())
			{
				SetValueAndStop(DesiredValue.GetValue());
			}
		}
	}

	/**
 	 * Set the internal state of the interpolator
 	 * 
 	 * @param InValue The value to be set on the interpolator
 	 */
	void SetValue(const NumericType& InValue) const
	{
		if(DesiredValue.IsSet())
		{
			if(Equals(DesiredValue.GetValue(), InValue, Tolerance))
			{
				return;
			}

			if(!LastValue.IsSet())
			{
				LastValue = InterpolatedValue.IsSet() ? InterpolatedValue : DesiredValue;
			}
			DesiredValue = InValue;
		}
		else
		{
			SetValueAndStop(InValue);
		}
	}

	/**
	 * Set the internal state of the interpolator and disables animation
	 * 
	 * @param InValue The value to be set on the interpolator
	 */
	void SetValueAndStop(const NumericType& InValue) const
	{
		const bool bFireStopEvent = Stop();
		LastValue = DesiredValue = InValue;
		if(bFireStopEvent)
		{
			InterpolationStopped.Broadcast(InValue);
		}
	}

	/**
	 * Returns the interpolated value
	 *
	 * @return The interpolated value
	 */
	const NumericType& Get() const
	{
		check(IsSet());
		if(!IsEnabled())
		{
			return DesiredValue.GetValue();
		}
		if(InterpolatedValue.IsSet())
		{
			return InterpolatedValue.GetValue();
		}
		if(LastValue.IsSet())
		{
			return LastValue.GetValue();
		}
		return DesiredValue.GetValue();
	}

	/**
	 * Updates the interpolator
	 */
	void Tick(float InDeltaTime)
	{
		if(!IsPlaying() && LastValue.IsSet())
		{
			const NumericType& CurrentValue = InterpolatedValue.IsSet() ? InterpolatedValue.GetValue() : DesiredValue.GetValue();
			if(Equals(CurrentValue, LastValue.GetValue(), Tolerance))
			{
				SetValueAndStop(LastValue.GetValue());
				return;
			}
		}

		if(Delay.IsSet())
		{
			const float DelayLeft = Delay.GetValue() - InDeltaTime;
			if(DelayLeft > SMALL_NUMBER)
			{
				Delay = DelayLeft;
				return;
			}
			Delay.Reset();
			InDeltaTime = -DelayLeft;
		}

		(void)PlayIfStopped();
		Interpolate();
		
		if(OverallDeltaTime.IsSet())
		{
			OverallDeltaTime = OverallDeltaTime.GetValue() + InDeltaTime;
		}
	}

	/**
	 * Returns the delta time between the start of the animation and now
	 */
	double GetOverAllDeltaTime() const
	{
		if(IsPlaying())
		{
			if(OverallDeltaTime.IsSet())
			{
				return OverallDeltaTime.GetValue();
			}
		}
		return 0.0f; 
	}

	/**
	 * Sets the delay for the next shot - delays are consumed and only used once
	 */
	void SetDelayOneShot(float InDelay) const
	{
		Delay = InDelay;
	}

	/**
	 * Sets the tolerance on this interpolator
	 */
	void SetTolerance(float InTolerance)
	{
		Tolerance = InTolerance;
	}

	/*
	 * Returns the delegate to react to the interpolator starting to interpolate
	 */
	FInterpolatorEvent& OnInterpolationStarted() { return InterpolationStarted; }

	/*
	 * Returns the delegate to react to the interpolator ending to interpolate
	 */
	FInterpolatorEvent& OnInterpolationStopped() { return InterpolationStopped; }

protected:

	// Interpolates the internal value against the new value
	// This method is expected to set the Value member
	virtual void Interpolate() const = 0;

	// Returns true if this interpolator is currently playing
	bool IsPlaying() const
	{
		return InterpolatedValue.IsSet();
	}

	// Starts the animation and records time
	virtual bool Start() const
	{
		InterpolatedValue = LastValue;
		OverallDeltaTime = 0.f;
		return true;
	}

	virtual bool PlayIfStopped() const
	{
		if(!IsPlaying())
		{
			return Start();
		}
		return false;
	}

	// Stops the animation and stops recording time
	virtual bool Stop() const
	{
		Delay.Reset();
		OverallDeltaTime.Reset();
		if(IsPlaying())
		{
			InterpolatedValue.Reset();
			return true;
		}
		return false;
	}

	// restart the animation
	virtual bool Restart() const
	{
		return Start();
	}

	bool bEnabled;
	float Tolerance;
	mutable TOptional<float> Delay;
	mutable TOptional<float> OverallDeltaTime;
	mutable TOptional<NumericType> InterpolatedValue;
	mutable TOptional<NumericType> LastValue;
	mutable TOptional<NumericType> DesiredValue;
	
	FInterpolatorEvent InterpolationStarted;
	FInterpolatorEvent InterpolationStopped;

	template<typename> friend class TAnimatedAttribute;
};

template<>
inline bool TAttributeInterpolator<float>::Equals(const float& InA, const float& InB, double InTolerance)
{
	return FMath::IsNearlyEqual(InA, InB, (float)InTolerance);
}

template<>
inline bool TAttributeInterpolator<double>::Equals(const double& InA, const double& InB, double InTolerance)
{
	return FMath::IsNearlyEqual(InA, InB, InTolerance);
}

enum class EEasingInterpolatorType : uint8
{
	Linear,
	QuadraticEaseIn,
	QuadraticEaseOut,
	QuadraticEaseInOut,
	CubicEaseIn,
	CubicEaseOut,
	CubicEaseInOut,
	QuarticEaseIn,
	QuarticEaseOut,
	QuarticEaseInOut,
	QuinticEaseIn,
	QuinticEaseOut,
	QuinticEaseInOut,
	SineEaseIn,
	SineEaseOut,
	SineEaseInOut,
	CircularEaseIn,
	CircularEaseOut,
	CircularEaseInOut,
	ExponentialEaseIn,
	ExponentialEaseOut,
	ExponentialEaseInOut,
	ElasticEaseIn,
	ElasticEaseOut,
	ElasticEaseInOut,
	BackEaseIn,
	BackEaseOut,
	BackEaseInOut,
	BounceEaseIn,
	BounceEaseOut,
	BounceEaseInOut
};

SLATE_API float EaseInterpolatorRatio(EEasingInterpolatorType InEasingType, float InRatio);

template< typename NumericType >
class TEasingAttributeInterpolator;

/**
 * An easing attribute interpolator.
 * This interpolator is to implement default easing functions such as Cubic etc
 * probably to rely on FCurveSequence.
 */
template< typename NumericType >
class TEasingAttributeInterpolator : public TAttributeInterpolator<NumericType>
{
private:

	typedef TAttributeInterpolator<NumericType> Super;

public:

	struct FSettings
	{
		typedef TEasingAttributeInterpolator<NumericType> InterpolatorType;
	
		FSettings(EEasingInterpolatorType InEasingType, float InDuration)
			: EasingType(InEasingType)
			, Duration(InDuration)
		{
		}
		
		EEasingInterpolatorType EasingType;
		float Duration;
	};

	explicit TEasingAttributeInterpolator(const FSettings& InSettings)
		: Super()
		, Settings(InSettings)
	{
	}

	//////////////////////////////////////////////////////////
	// TAttributeInterpolator interface
	
	virtual bool IsTypeOf(EAttributeInterpolatorType InType) const override
	{
		return InType == EAttributeInterpolatorType::Easing;
	}

	virtual bool IdenticalTo(const TAttributeInterpolator<NumericType>* InOther) const override
	{
		if(!InOther->IsTypeOf(EAttributeInterpolatorType::Easing))
		{
			return false; 
		}

		// C++ cast due to polymorphism dynamic_cast is not available
		const TEasingAttributeInterpolator<NumericType>* CastInterpolator =
			(const TEasingAttributeInterpolator<NumericType>*)InOther;

		return FMath::IsNearlyEqual(Settings.Duration, CastInterpolator->Settings.Duration);
	}

protected:

	virtual void Interpolate() const override
	{
		check(Super::LastValue.IsSet());
		check(Super::DesiredValue.IsSet());

		const float DeltaTime = (float)Super::GetOverAllDeltaTime();
		if(Settings.Duration <= 0.f || DeltaTime > Settings.Duration)
		{
			Super::SetValueAndStop(Super::DesiredValue.GetValue());
			return;
		}

		const float Ratio = FMath::Clamp<float>(DeltaTime / Settings.Duration, 0.f, 1.f);
		const float EasedRatio = EaseInterpolatorRatio(Settings.EasingType, Ratio);
		Super::InterpolatedValue = FMath::Lerp<NumericType>(Super::LastValue.GetValue(), Super::DesiredValue.GetValue(), EasedRatio);
	}

	FSettings Settings;
};

/**
 * An arrive attribute interpolator. Arrives smoothly at the desired value.
 */
template< typename NumericType >
class TArriveAttributeInterpolator : public TAttributeInterpolator<NumericType>
{
private:

	typedef TAttributeInterpolator<NumericType> Super;

public:

	struct FSettings
	{
		typedef TArriveAttributeInterpolator<NumericType> InterpolatorType;

		FSettings(int32 InIterations, float InStrength)
		: Iterations(InIterations)
		, Strength(InStrength)
		{
		}
		
		int32 Iterations;
		float Strength;
	};

	explicit TArriveAttributeInterpolator(const FSettings& InSettings)
		: Super()
		, Settings(InSettings)
		, LastDeltaTime(0)
	{
	}

	//////////////////////////////////////////////////////////
	// TAttributeInterpolator interface
	
	virtual bool IsTypeOf(EAttributeInterpolatorType InType) const override
	{
		return InType == EAttributeInterpolatorType::Arrive;
	}

	virtual bool IdenticalTo(const TAttributeInterpolator<NumericType>* InOther) const override
	{
		if(!InOther->IsTypeOf(EAttributeInterpolatorType::Arrive))
		{
			return false; 
		}

		// C++ cast due to polymorphism dynamic_cast is not available
		const TArriveAttributeInterpolator<NumericType>* CastInterpolator =
			(const TArriveAttributeInterpolator<NumericType>*)InOther;

		return (Settings.Iterations == CastInterpolator->Settings.Iterations) && 
			FMath::IsNearlyEqual(Settings.Strength, CastInterpolator->Settings.Strength);
	}

protected:

	virtual bool Start() const override
	{
		LastDeltaTime = 0.0;
		return Super::Start();
	}

	virtual bool Restart() const override
	{
		const bool bWasPlaying = Super::IsPlaying();
		TOptional<NumericType> PreviousValue = Super::InterpolatedValue;
		
		const bool bResult = Super::Restart();
		if(bWasPlaying)
		{
			if(PreviousValue.IsSet())
			{
				Super::LastValue = Super::InterpolatedValue = PreviousValue;
			}
		}
		return bResult;
	}
	
	virtual void Interpolate() const override
	{
		check(Super::LastValue.IsSet());
		check(Super::DesiredValue.IsSet());
		
		const float OverallDeltaTime = (float)Super::GetOverAllDeltaTime();
		const float LocalDeltaTime = OverallDeltaTime - LastDeltaTime;

		const float Blend =  FMath::Max<float>(Settings.Strength, 0) * 8.f * LocalDeltaTime;

		for(int32 Iteration = 0; Iteration < Settings.Iterations; Iteration++)
		{
			Super::InterpolatedValue = FMath::Lerp<NumericType>(
				Super::InterpolatedValue.IsSet() ? Super::InterpolatedValue.GetValue() : Super::LastValue.GetValue(),
				Super::DesiredValue.GetValue(),
				Blend);
		}

		LastDeltaTime = OverallDeltaTime;

		if(Super::Equals(Super::DesiredValue.GetValue(), Super::InterpolatedValue.GetValue(), Super::Tolerance))
		{
			Super::SetValueAndStop(Super::DesiredValue.GetValue());
		}
	}
	
	FSettings Settings;
	mutable float LastDeltaTime;
};

/**
 * A verlet attribute interpolator. Desired values are simulated relying a simple (semi-fake) verlet integration.
 */
template< typename NumericType >
class TVerletAttributeInterpolator : public TAttributeInterpolator<NumericType>
{
private:

	typedef TAttributeInterpolator<NumericType> Super;

public:

	struct FSettings
	{
		typedef TVerletAttributeInterpolator<NumericType> InterpolatorType;

		FSettings(float InBlend, float InStrength, float InDamping)
		: Blend(InBlend)
		, Strength(InStrength)
		, Damping(InDamping)
		{
		}
		
		float Blend;
		float Strength;
		float Damping;
	};

	explicit TVerletAttributeInterpolator(const FSettings& InSettings)
		: Super()
		, Settings(InSettings)
		, LastDeltaTime(0)
	{
	}

	//////////////////////////////////////////////////////////
	// TAttributeInterpolator interface
	
	virtual bool IsTypeOf(EAttributeInterpolatorType InType) const override
	{
		return InType == EAttributeInterpolatorType::Verlet;
	}

	virtual bool IdenticalTo(const TAttributeInterpolator<NumericType>* InOther) const override
	{
		if(!InOther->IsTypeOf(EAttributeInterpolatorType::Verlet))
		{
			return false; 
		}

		// C++ cast due to polymorphism dynamic_cast is not available
		const TVerletAttributeInterpolator<NumericType>* CastInterpolator =
			(const TVerletAttributeInterpolator<NumericType>*)InOther;

		return FMath::IsNearlyEqual(Settings.Blend, CastInterpolator->Settings.Blend) &&
				FMath::IsNearlyEqual(Settings.Strength, CastInterpolator->Settings.Strength) && 
			FMath::IsNearlyEqual(Settings.Damping, CastInterpolator->Settings.Damping);
	}

protected:

	virtual bool Start() const override
	{
		LastDeltaTime = 0.0;
		Velocity = NumericType(0);
		return Super::Start();
	}

	virtual bool Restart() const override
	{
		const bool bWasPlaying = Super::IsPlaying();
		TOptional<NumericType> PreviousValue = Super::InterpolatedValue;
		TOptional<NumericType> PreviousVelocity = Velocity;
		
		const bool bResult = Super::Restart();
		if(bWasPlaying)
		{
			if(PreviousValue.IsSet())
			{
				Super::LastValue = Super::InterpolatedValue = PreviousValue;
			}
			if(PreviousVelocity.IsSet())
			{
				Velocity = PreviousVelocity;
			}
		}
		return bResult;
	}

	virtual void Interpolate() const override
	{
		check(Super::LastValue.IsSet());
		check(Super::DesiredValue.IsSet());
		
		const float OverallDeltaTime = (float)Super::GetOverAllDeltaTime();
		const float LocalDeltaTime = OverallDeltaTime - LastDeltaTime;

		const float U = FMath::Clamp<float>(FMath::Max<float>(Settings.Blend, 0) * 8.f * LocalDeltaTime, 0, 1);
		const float ScaleDown = FMath::Clamp<float>(1.f - Settings.Damping, 0.f, 1.f);

		const NumericType PreviousValue = Super::InterpolatedValue.IsSet() ? Super::InterpolatedValue.GetValue() : Super::LastValue.GetValue();
		const NumericType Force = (Super::DesiredValue.GetValue() - PreviousValue) * Settings.Strength;
		const NumericType PreviousVelocity = Velocity.GetValue();

		Velocity = FMath::Lerp<NumericType>(PreviousVelocity, Force, U) * ScaleDown;
		Super::InterpolatedValue = PreviousValue + Velocity.GetValue() * LocalDeltaTime;

		LastDeltaTime = OverallDeltaTime;

		if(Super::Equals(Super::DesiredValue.GetValue(), Super::InterpolatedValue.GetValue(), Super::Tolerance) &&
			Super::Equals(Velocity.GetValue(), NumericType(0), Super::Tolerance))
		{
			Super::SetValueAndStop(Super::DesiredValue.GetValue());
		}
	}

	FSettings Settings;
	mutable float LastDeltaTime;
	mutable TOptional<NumericType> Velocity;
};
