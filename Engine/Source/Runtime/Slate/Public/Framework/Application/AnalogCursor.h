// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ICursor.h"
#include "Framework/Application/IInputProcessor.h"

class FSlateApplication;
class FSlateUser;
struct FInputEvent;
struct FAnalogInputEvent;
struct FKeyEvent;
struct FPointerEvent;

namespace AnalogCursorMode
{
	enum Type
	{
		Accelerated,
		Direct,
	};
}

enum class EAnalogStick : uint8
{
	Left,
	Right,
	Max,
};

/**
 * A class that simulates a cursor driven by an analog stick.
 */
class FAnalogCursor : public IInputProcessor, public TSharedFromThis<FAnalogCursor>
{
public:
	SLATE_API FAnalogCursor();
	SLATE_API FAnalogCursor(const FAnalogCursor&);

	/** Dtor */
	SLATE_API virtual ~FAnalogCursor();

	SLATE_API virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override;

	SLATE_API virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
	SLATE_API virtual bool HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
	SLATE_API virtual bool HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent) override;
	SLATE_API virtual bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;
	virtual const TCHAR* GetDebugName() const override { return TEXT("AnalogCursor"); }


	virtual int32 GetOwnerUserIndex() const { return 0; };

	SLATE_API void SetAcceleration(float NewAcceleration);
	SLATE_API void SetMaxSpeed(float NewMaxSpeed);
	SLATE_API void SetStickySlowdown(float NewStickySlowdown);
	SLATE_API void SetDeadZone(float NewDeadZone);
	SLATE_API void SetMode(AnalogCursorMode::Type NewMode);

protected:

	SLATE_API virtual bool IsRelevantInput(const FInputEvent& InputEvent) const;
	SLATE_API virtual bool IsRelevantInput(const FKeyEvent& KeyEvent) const;
	SLATE_API virtual bool IsRelevantInput(const FAnalogInputEvent& AnalogInputEvent) const;
	SLATE_API virtual bool IsRelevantInput(const FPointerEvent& MouseEvent) const;

	/** Getter */
	inline const FVector2D& GetAnalogValues( EAnalogStick Stick = EAnalogStick::Left ) const
	{
		return AnalogValues[ static_cast< uint8 >( Stick ) ];
	}

	/** Set the cached analog stick declinations to 0 */
	SLATE_API void ClearAnalogValues();

	/** Handles updating the cursor position and processing a Mouse Move Event */
	UE_DEPRECATED(4.24, "FAnalogCursor now updates cursor position based on user, not the hardware cursor specifically.")
	SLATE_API virtual void UpdateCursorPosition(FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor, const FVector2D& NewPosition, bool bForce = false);

	SLATE_API virtual void UpdateCursorPosition(FSlateApplication& SlateApp, TSharedRef<FSlateUser> SlateUser, const FVector2D& NewPosition, bool bForce = false);

	SLATE_API virtual FVector2D CalculateTickedCursorPosition(const float DeltaTime, FSlateApplication& SlateApp, TSharedPtr<FSlateUser> SlateUser);

	/** Current speed of the cursor */
	FVector2D CurrentSpeed;

	/** Current sub-pixel offset */
	FVector2D CurrentOffset;

	/** Current settings */
	float Acceleration;
	float MaxSpeed;
	float StickySlowdown;
	float DeadZone;
	AnalogCursorMode::Type Mode;

private:

	inline FVector2D& GetAnalogValue( EAnalogStick Stick )
	{
		return AnalogValues[ static_cast< uint8 >( Stick ) ];
	}

	/** Input from the gamepad */
	FVector2D AnalogValues[ static_cast<uint8>( EAnalogStick::Max ) ];
};

