// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/BitArray.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"

class FDefaultBitArrayAllocator;

/**
 * The gesture detector can simulate the detection of certain kinds of gestures that may or may not
 * be available to be detected at the platform level.
 */
class FGestureDetector
{
public:
	/** The amount of time in seconds you hold a finger down before it's detected as a long press. */
	static SLATE_API double LongPressSeconds;

	/**
	 * The amount of movement allowed before the finger is no longer considered valid for a long press, 
	 * until it's removed and re-pressed.
	 */
	static float LongPressAllowedMovement;

public:
	/**
	 * Call to determine if the gesture is supported by the gesture detector.
	 */
	static bool IsGestureSupported(EGestureEvent Gesture);

	void OnTouchStarted(int32 TouchIndex, const FVector2D& Location);
	void OnTouchEnded(int32 TouchIndex, const FVector2D& Location);
	void OnTouchMoved(int32 TouchIndex, const FVector2D& Location);

	/**
	 * Generates gesture messages for all enabled gestures.
	 */
	void GenerateGestures(FGenericApplicationMessageHandler& MessageHandler, const TBitArray<FDefaultBitArrayAllocator>& EnabledGestures);

private:
	struct FLongPressData
	{
		TOptional<double> Time;
		FVector2D Location;

		FLongPressData()
			: Time()
			, Location(0, 0)
		{
		}

		void Reset()
		{
			Time.Reset();
			Location = FVector2D(0, 0);
		}
	};

	FLongPressData LongPressTrack[EKeys::NUM_TOUCH_KEYS];
};
