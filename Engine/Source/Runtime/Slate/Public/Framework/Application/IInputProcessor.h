// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ICursor.h"

class FSlateApplication;
struct FAnalogInputEvent;
struct FKeyEvent;
struct FMotionEvent;
struct FPointerEvent;

/**
 * Interface for a Slate Input Handler
 */
class IInputProcessor
{
public:
	IInputProcessor(){};
	virtual ~IInputProcessor()= default;

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) = 0;

	/** Key down input */
	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) { return false; }

	/** Key up input */
	virtual bool HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) { return false; }

	/** Analog axis input */
	virtual bool HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent) { return false; }

	/** Mouse movement input */
	virtual bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) { return false; }

	/** Mouse button press */
	virtual bool HandleMouseButtonDownEvent( FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) { return false; }

	/** Mouse button release */
	virtual bool HandleMouseButtonUpEvent( FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) { return false; }

	/** Mouse button double clicked. */
	virtual bool HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) { return false; }

	/** Mouse wheel input */
	virtual bool HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent) { return false; }

	/** Called when a motion-driven device has new input */
	virtual bool HandleMotionDetectedEvent(FSlateApplication& SlateApp, const FMotionEvent& MotionEvent) { return false; };

	/** Debug name for logging purposes */
	virtual const TCHAR* GetDebugName() const { return TEXT(""); }
};
