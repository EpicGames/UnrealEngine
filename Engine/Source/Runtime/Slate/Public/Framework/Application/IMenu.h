// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWidget;
class SWindow;
enum class EPopupMethod : uint8;

/**
 * Represents a popup menu.
 */
class IMenu
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMenuDismissed, TSharedRef<IMenu> /*DismissedMenu*/);

	virtual ~IMenu() = default;
	virtual EPopupMethod GetPopupMethod() const = 0;
	virtual TSharedPtr<SWindow> GetParentWindow() const = 0;
	virtual TSharedPtr<SWindow> GetOwnedWindow() const = 0;
	virtual TSharedPtr<SWidget> GetContent() const = 0;
	virtual FOnMenuDismissed& GetOnMenuDismissed() = 0;
	virtual bool UsingApplicationMenuStack() const = 0;
	virtual void Dismiss() = 0;
};

class IMenuHost
{
public:
	virtual TSharedPtr<SWindow> GetMenuWindow() const = 0;
	virtual void OnMenuDismissed() = 0;
	virtual bool UsingApplicationMenuStack() const = 0;

	bool bShowMenuBackground = true;
};
