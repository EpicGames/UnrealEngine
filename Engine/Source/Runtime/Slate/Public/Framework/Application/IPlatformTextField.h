// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeature.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"

#define UE_API SLATE_API


class IVirtualKeyboardEntry;

class IPlatformTextField
{
public:
	virtual ~IPlatformTextField() = default;

	virtual void ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) = 0;
	virtual bool AllowMoveCursor() { return true; }

	static bool ShouldUseVirtualKeyboardAutocorrect(TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget);

private:

};


class IPlatformTextFieldFactory : public IModularFeature
{
public:
	UE_API static FName FeatureName;

	UE_API static TUniquePtr<IPlatformTextField> TryCreateInstance();
	
	virtual TUniquePtr<IPlatformTextField> CreateInstance() = 0;


};

#undef UE_API
