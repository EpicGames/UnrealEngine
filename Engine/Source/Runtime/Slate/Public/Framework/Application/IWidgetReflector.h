// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWindow.h"

class FSlateWindowElementList;
class FWidgetPath;

/** A Delegate for passing along a string of a source code location to access */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FAccessSourceCode, const FString& /*FileName*/, int32 /*InLineNumber*/, int32 /*InColumnNumber*/);

/** A Delegate for an asset object to access */
DECLARE_DELEGATE_RetVal_OneParam(bool, FAccessAsset, UObject* /*InAsset*/);

/** A Delegate for an debug object to access */
DECLARE_DELEGATE_RetVal_OneParam(bool, FAccessDebugObject, UObject* /*InObject*/);


/**
 * Interface for widget reflectors.
 */
class IWidgetReflector
{
public:
	UE_DEPRECATED(4.26, "OnEventProcessed is deprecated. Use FSlateDebugging.")
	virtual void OnEventProcessed( const FInputEvent& Event, const FReplyBase& InReply ) {}

public:

	/** Called when the user has picked a widget to observe. */
	virtual void OnWidgetPicked( ) = 0;

	/**
	 * Checks whether we are visualizing the focused widgets.
	 *
	 * @return true if focusing widgets, false otherwise.
	 */
	virtual bool IsShowingFocus( ) const = 0;

	/**
	 * Checks whether user is in the process of selecting a widget.
	 *
	 * @return true if the user is selecting a widget, false otherwise.
	 */
	virtual bool IsInPickingMode( ) const = 0;

	/**
	 * Checks whether we should be inspecting widgets and visualizing their layout.
	 *
	 * @return true if visualizing layout, false otherwise.
	 */
	virtual bool IsVisualizingLayoutUnderCursor( ) const = 0;

	/**
	 * Take a snapshot of the UI pertaining to the widget that the user is hovering and visualize it.
	 * If we are not taking a snapshot, draw the overlay from a previous snapshot, if possible.
	 *
	 * @param InWidgetsToVisualize  WidgetPath that the cursor is currently over; could be null.
	 * @param OutDrawElements       List of draw elements to which we will add a visualization overlay
	 * @param LayerId               The maximum layer id attained in the draw element list so far.
	 *
	 * @return The maximum layer ID that we attained while painting the overlay.
	 */
	virtual int32 Visualize( const FWidgetPath& InWidgetsToVisualize, FSlateWindowElementList& OutDrawElements, int32 LayerId ) = 0;

	/** Visualize the cursor position and any pressed keys for demo-recording purposes. */
	UE_DEPRECATED(4.26, "VisualizeCursorAndKeys is deprecated. Use FSlateDebugging.")
	virtual int32 VisualizeCursorAndKeys(FSlateWindowElementList& OutDrawElements, int32 LayerId) const { return LayerId; }

	/**
	 * Sets the widget that should be visualized.
	 *
	 * @param InWidgetsToVisualize The path to the widget to inspect via the reflector.
	 */
	virtual void SetWidgetsToVisualize( const FWidgetPath& InWidgetsToVisualize ) = 0;

	/**
	 * @param InDelegate A delegate to access source code with.
	 */
	virtual void SetSourceAccessDelegate( FAccessSourceCode InDelegate ) = 0;

	/**
	* @param InDelegate A delegate to access assets with.
	*/
	virtual void SetAssetAccessDelegate(FAccessAsset InDelegate) = 0;

	/**
	* @param InDelegate A delegate to access debug objects with.
	*/
	virtual void SetDebugObjectAccessDelegate(FAccessDebugObject InDelegate) = 0;
	
	/**
	 * @param ThisWindow    Do we want to draw something for this window?
	 *
	 * @return true if we want to draw something for this window; false otherwise
	 */
	virtual bool ReflectorNeedsToDrawIn( TSharedRef<SWindow> ThisWindow ) const = 0;

public:

	/** Virtual destructor. */
	virtual ~IWidgetReflector( ) = default;
};
