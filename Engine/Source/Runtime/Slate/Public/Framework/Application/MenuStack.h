// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/PopupMethodReply.h"
#include "Framework/Application/IMenu.h"
#include "Layout/SlateRect.h"
#include "Widgets/SWidget.h"
#include "Types/SlateStructs.h"
#include "Widgets/SWindow.h"
#include "Application/ThrottleManager.h"
#include "Delegates/DelegateCombinations.h"

class FWidgetPath;
class SMenuPanel;
class FMenuBase;

typedef TArray<TSharedPtr<SWindow>> FMenuWindowList;	// deprecated

#if UE_BUILD_DEBUG
	// Debugging is harder with the inline allocator
	typedef TArray<TSharedPtr<IMenu>> FMenuList;
#else
	typedef TArray<TSharedPtr<IMenu>, TInlineAllocator<4>> FMenuList;
#endif


/** Describes a simple animation for popup window introductions */
struct FPopupTransitionEffect
{
	/** Direction to slide in from */
	enum ESlideDirection
	{
		/** No sliding */
		None,

		/** Sliding direction for a combo button */
		ComboButton,

		/** Sliding direction for a top-level pull-down menu or combo box */
		TopMenu,

		/** Sliding direction for a sub-menu */
		SubMenu,

		/** Sliding direction for a popup that lets the user type in data */
		TypeInPopup,

		/** Sliding direction preferred for context menu popups */
		ContextMenu,

	} SlideDirection;


	/** Constructor */
	FPopupTransitionEffect( const ESlideDirection InitSlideDirection )
		: SlideDirection(InitSlideDirection)
	{ }
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnMenuDestroyed, const TSharedRef<IMenu>& InMenu);

#if WITH_SLATE_DEBUGGING
extern FAutoConsoleVariable CVarSlateDismissMenuStacksOnFocusLost;
#endif

/**
 * Represents a stack of open menus. The last item in the stack is the top most menu
 * Menus are described as IMenus. Implementations of IMenu can control how the menus are created and presented (e.g. in their own window)
 */
class FMenuStack
{
public:
	/** Constructor */
	FMenuStack()
		: bHostWindowGuard(false)
	{}

	/**
	 * Pushes a new menu onto the stack. The widget path will be searched for existing menus and the new menu will be parented appropriately.
	 * Menus are always auto-sized. Use fixed-size content if a fixed size is required.
	 * 
	 * @param InOwnerPath			The widget path for the parent widget of this menu.
	 * @param InContent				The menu's content
	 * @param SummonLocation		Location in screen-space where the menu should appear
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bFocusImmediately		Should the popup steal focus when shown?
	 * @param SummonLocationSize	An optional size around the summon location which describes an area in which the menu may not appear
	 * @param InMethod				An optional popup method that will override the default method for the widgets in InOwnerPath
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 * @param bEnablePerPixelTransparency Does the menu's content require per pixel transparency?
	 * @param FocusUserIndex		The index of the user that should focus the menu. INDEX_NONE for all users.
	 */
	TSharedRef<IMenu> Push(const FWidgetPath& InOwnerPath, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately = true, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize = FVector2f::ZeroVector, TOptional<EPopupMethod> InMethod = TOptional<EPopupMethod>(), const bool bIsCollapsedByParent = true, const bool bEnablePerPixelTransparency = false, const int32 FocusUserIndex = INDEX_NONE);
	
	/**
	 * Pushes a new child menu onto the stack.
	 * Menus are always auto-sized. Use fixed-size content if a fixed size is required.
	 * 
	 * @param InParentMenu			The parent menu for this menu
	 * @param InContent				The menu's content
	 * @param SummonLocation		Location in screen-space where the menu should appear
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bFocusImmediately		Should the popup steal focus when shown?
	 * @param SummonLocationSize	An optional size around the summon location which describes an area in which the menu may not appear
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 * @param bEnablePerPixelTransparency Does the menu's content require per pixel transparency?
	 * @param FocusUserIndex		The index of the user that should focus the menu. INDEX_NONE for all users.
	 */
	TSharedRef<IMenu> Push(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately = true, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize = FVector2f::ZeroVector, const bool bIsCollapsedByParent = true, const bool bEnablePerPixelTransparency = false, const int32 FocusUserIndex = INDEX_NONE);

	/**
	 * Pushes a new menu onto the stack that is drawn by an external host widget.
	 * The widget path will be searched for existing menus and the new menu will be parented appropriately.
	 * Menus are always auto-sized. Use fixed-size content if a fixed size is required.
	 * 
	 * @param InOwnerPath			The widget path for the parent widget of this menu
	 * @param InMenuHost			The host widget that draws the menu's content
	 * @param InContent				The menu's content
	 * @param OutWrappedContent		Returns the InContent wrapped with widgets needed by the menu stack system. This is what should be drawn by the host after this call.
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 */
	TSharedRef<IMenu> PushHosted(const FWidgetPath& InOwnerPath, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent = true);
	
	/**
	 * Pushes a new child menu onto the stack that is drawn by an external host widget.
	 * Menus are always auto-sized. Use fixed-size content if a fixed size is required.
	 * 
	 * @param InParentMenu			The parent menu for this menu
	 * @param InMenuHost			The host widget that draws the menu's content
	 * @param InContent				The menu's content
	 * @param OutWrappedContent		Returns the InContent wrapped with widgets needed by the menu stack system. This is what should be drawn by the host after this call.
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 */
	TSharedRef<IMenu> PushHosted(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent = true);
	
	/**
	 * Dismisses the menu stack including InFromMenu and all its child menus
	 * Dismisses in reverse order (children first)
	 *
	 * @param InFromMenu		The menu to remove along with its children
	 */
	void DismissFrom(const TSharedPtr<IMenu>& InFromMenu);

	/**
	 * Dismisses the entire menu stack
	 */
	void DismissAll();

	/**
	 * Called by the application when any window is destroyed.
	 *
	 * @param InWindow	The window that was destroyed
	 */
	void OnWindowDestroyed(TSharedRef<SWindow> InWindow);

	/**
	 * Notifies the stack that a new window was activated.  The menu stack will be dismissed if the activated window is not a menu in the stack
	 *
	 * @param ActivatedWindow	The window that was just activated
	 */
	void OnWindowActivated( TSharedRef<SWindow> ActivatedWindow );

	/**
	 * Finds a menu in the stack that owns InWindow.
	 * Although this method isn't deprecated, it is intended for private use inside FMenuStack
	 * and its use as a public method should be avoided. Menus should be identified in client code as IMenu interfaces.
	 *
	 * @param	InWindow	The window to look for
	 * @return	The menu in the stack that owns InWindow, or an invalid ptr if not found
	 */
	TSharedPtr<IMenu> FindMenuFromWindow(TSharedRef<SWindow> InWindow) const;

	/**
	 * Searches from bottom to top of the widget path for a menu in the stack.
	 * If the widget at the end of PathToQuery is inside a menu, there will be a menu content wrapper widget in the path.
	 *
	 * @param	PathToQuery		The widget path to search for a menu
	 * @return	The menu in the stack that contains the widget at the end of PathToQuery, or an invalid ptr if not found
	 */
	TSharedPtr<IMenu> FindMenuInWidgetPath(const FWidgetPath& PathToQuery) const;

	/**
	 * Called by the application when showing tooltips. It prevents tooltips from drawing over menu items. If the resulting bool is false, the resulting OutSlateRect will be [0,0,0,0] and should not be used.
	 *
	 * @param InMenu				A menu in the stack, used to generate the force field rect
	 * @param InPathContainingMenu	A widget path containing InMenu. This could be generated but the application has the path so it helps performance to pass it in here.
	 * @param OutSlateRect			The output rectangle enclosing the menu's children (created from the clipping rects in the geometry in PathContainingMenu)
	 * @return						A bool indicating whether a solution was found. If false, the resulting OutSlateRect will be [0,0,0,0] and should not be used.
	 */
	bool GetToolTipForceFieldRect(const TSharedRef<IMenu>& InMenu, const FWidgetPath& InPathContainingMenu, FSlateRect& OutSlateRect) const;

	/**
	 * @return	Returns the window that is the parent of everything in the stack, if any. If the stack is empty, returns an invalid ptr.
	 */
	TSharedPtr<SWindow> GetHostWindow() const;

	/**
	 * @return	Returns the widget that is the parent widget that initially created the menu in the stack
	 */
	TSharedPtr<SWidget> GetHostWidget() const;

	/**
	* @return	True if the stack has one or more menus in it, false if it is empty.
	*/
	bool HasMenus() const;

	/**
	 * @return Returns whether the menu has child menus. If the menu isn't in the stack, returns false.
	 */
	bool HasOpenSubMenus(TSharedPtr<IMenu> InMenu) const;

	/** Delegate called when a menu is dismissed and destroyed */
	FOnMenuDestroyed& OnMenuDestroyedEvent() { return MenuDestroyedEvent; }

private:
	/**
	 * Queries the widgets in the path for a menu popup method.
	 *
	 * @param	PathToQuery	The path to query
	 *
	 * @return	The popup method chosen by a widget in the path. Defaults to EPopupMethod::CreateNewWindow if no widgets in the path have a preference.
	 */
	FPopupMethodReply QueryPopupMethod(const FWidgetPath& PathToQuery);

	/**
	 * Called by public Dismiss methods. Dismisses all menus in the stack from FirstStackIndexToRemove and below.
	 *
	 * @param	FirstStackIndexToRemove	Index into the stack from which menus are dismissed.
	 */
	void DismissInternal(int32 FirstStackIndexToRemove);

	/**
	 * Handles changes to the owner path, potentially locating and establishing the new host. Makes sure overlay slots are added/removed as appropriate.
	 */
	void SetHostPath(const FWidgetPath& InOwnerPath);

	/**
	 * Callback method called by all menus when they are dismissed.  Handles changes to the stack state when a menu is destroyed.
	 *
	 * @param	InMenu	The menu that was destroyed.
	 */
	void OnMenuDestroyed(TSharedRef<IMenu> InMenu);

	/**
	 * Callback method called by all menus when they lose focus.  Handles dismissing menus that have lost focus.
	 *
	 * @param	InFocussedPath	The new focus path.
	 */
	void OnMenuContentLostFocus(const FWidgetPath& InFocussedPath);

	/**
	 * Called by all menus during creation. Handles wrapping content in SMenuContentWrapper that acts as a marker in widget paths and in a box that sets minimum size.
	 *
	 * @param	InContent			The unwrapped content.
	 * @param	OptionalMinWidth	Optional minimum width for the wrapped content.
	 * @param	OptionalMinHeight	Optional minimum height for the wrapped content.
	 * @param	bShouldShowBackground	Whether the menu wrapper should put a background behind the content.
	 */
	TSharedRef<SWidget> WrapContent(TSharedRef<SWidget> InContent, FOptionalSize OptionalMinWidth = FOptionalSize(), FOptionalSize OptionalMinHeight = FOptionalSize(), bool bShouldShowBackground = true);

	/** Contains all the options passed to the pre-push stage of the menu creation process */
	struct FPrePushArgs
	{
		FPrePushArgs() : TransitionEffect(FPopupTransitionEffect::None) {}

		TSharedPtr<SWidget> Content;
		FSlateRect Anchor;
		FPopupTransitionEffect TransitionEffect;
		int32 FocusUserIndex = INDEX_NONE;
		bool bFocusImmediately;
		bool bIsCollapsedByParent;
	};

	/** Contains all the options returned from the pre-push stage of the menu creation process */
	struct FPrePushResults
	{
		TSharedPtr<SWidget> WrappedContent;
		TSharedPtr<SWidget> WidgetToFocus;
		FVector2f AnimStartLocation;
		FVector2f AnimFinalLocation;
		bool bAnchorSetsMinWidth;
		FVector2f ExpectedSize;
		bool bAllowAnimations;
		bool bFocusImmediately;
		bool bIsCollapsedByParent;
	};

	/**
	 * This is the pre-push stage of menu creation.
	 * It is responsible for wrapping content and calculating menu positioning. This stage is not used by "hosted" menus.
	 *
	 * @param	InArgs	Pre-push options.
	 *
	 * @return			Pre-push results.
	 */
	FPrePushResults PrePush(const FPrePushArgs& InArgs);

	/**
	 * This is the common code used during menu creation. This stage is not used by "hosted" menus.
	 * It handles pre-push, actual menu creation and post-push stack updates.
	 *
	 * @param InParentMenu			The parent menu for this menu
	 * @param InContent				The menu's content
	 * @param Anchor				The anchor location for the menu (rect around the parent widget that is summoning the menu)
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bFocusImmediately		Should the popup steal focus when shown?
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 * @param FocusUserIndex		The index of the user that should focus the menu. INDEX_NONE for all users.
	 *
	 * @return			The newly created menu.
	 */
	TSharedRef<IMenu> PushInternal(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<SWidget>& InContent, FSlateRect Anchor, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent, const bool bEnablePerPixelTransparency, const int32 FocusUserIndex = INDEX_NONE);

	/**
	 * This is the actual menu object creation method for FMenuInWindow menus.
	 * It creates a new SWindow and new FMenuInWindow that uses it.
	 *
	 * @param InParentMenu			The parent menu for this menu
	 * @param InPrePushResults		The results of the pre-push stage
	 * @param FocusUserIndex		The index of the user that should focus the window. INDEX_NONE for all users.
	 *
	 * @return			The newly created FMenuInWindow menu.
	 */
	TSharedRef<FMenuBase> PushNewWindow(TSharedPtr<IMenu> InParentMenu, const FPrePushResults& InPrePushResults, const bool bEnablePerPixelTransparency, const int32 FocusUserIndex = INDEX_NONE);

	/**
	 * This is the actual menu object creation method for FMenuInPopup menus.
	 * It creates a new FMenuInWindow and adds it into the overlay layer on the host window.
	 *
	 * @param InParentMenu			The parent menu for this menu
	 * @param InPrePushResults		The results of the pre-push stage
	 * @param FocusUserIndex		The index of the user that should focus the popup. INDEX_NONE for all users.
	 *
	 * @return			The newly created FMenuInPopup menu.
	 */
	TSharedRef<FMenuBase> PushPopup(TSharedPtr<IMenu> InParentMenu, const FPrePushResults& InPrePushResults, const int32 FocusUserIndex = INDEX_NONE);

	/**
	 * This is the post-push stage of menu creation.
	 * It is responsible for updating the stack with the new menu and removing any that it replaces.
	 *
	 * @param InParentMenu         The parent menu for this menu
	 * @param InMenu               A newly created menu
	 * @param ShouldThrottle       Should pushing this menu enable throttling of the engine for a more responsive UI
	 * @param bInInsertAfterDismiss	Avoid unwanted delete due to dismissing another menu causing all menus in stack to be dismissed
	 */
	void PostPush(TSharedPtr<IMenu> InParentMenu, TSharedRef<FMenuBase> InMenu, EShouldThrottle ShouldThrottle, bool bInInsertAfterDismiss=false);

	/** The popup method currently used by the whole stack. It can only use one at a time */
	FPopupMethodReply ActiveMethod;

	/** The parent window of the root menu in the stack. NOT the actual menu window if it's a CreateNewWindow */
	TSharedPtr<SWindow> HostWindow;

	/** The parent widget of the root menu in the stack */
	TWeakPtr<SWidget> HostWidget;

	/** The parent window of the root menu in the stack. NOT the actual menu window if it's a CreateNewWindow */
	TSharedPtr<SMenuPanel> HostWindowPopupPanel;

	/** The popup layer that contains our HostWindowPopupPanel. */
	TSharedPtr<FPopupLayer> HostPopupLayer;

	/** The array of menus in the stack */
	TArray<TSharedPtr<class FMenuBase>> Stack;

	/** Maps top-level content widgets (should always be SMenuContentWrappers) to menus in the stack */
	TMap<TSharedPtr<const SWidget>, TSharedPtr<class FMenuBase>> CachedContentMap;
	
	/** Handle to a throttle request made to ensure the menu is responsive in low FPS situations */
	FThrottleRequest ThrottleHandle;

	/** Temporary ptr to a new window created during the menu creation process. Nulled before the Push() call returns. Stops activation of the new window collapsing the stack. */
	TSharedPtr<SWindow> PendingNewWindow;

	/** Temporary ptr to a new menu created during the menu creation process. Nulled before the Push() call returns. Stops it collapsing the stack when it gets focus. */
	TSharedPtr<class FMenuBase> PendingNewMenu;

	/** Guard to prevent the HostWindow and HostWindowPopupPanel being set reentrantly */
	bool bHostWindowGuard;

	/** Called when a menu is destroyed */
	FOnMenuDestroyed MenuDestroyedEvent;
};
