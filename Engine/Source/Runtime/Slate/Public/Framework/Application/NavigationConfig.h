// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/Events.h"

struct FKeyEvent;
enum class EUINavigation : uint8;

/* Since we now support multiple analog values driving the same navigation axis,
 * we need to key their repeat-state by both FKey and EUINavigation */
struct FAnalogNavigationKey
{
	FKey AnalogKey;
	EUINavigation NavigationDir;

	FAnalogNavigationKey(const FKey& InKey, const EUINavigation InNavDir)
		: AnalogKey(InKey)
		, NavigationDir(InNavDir)
	{}

	bool operator==(const FAnalogNavigationKey& Rhs) const
	{
		return     AnalogKey == Rhs.AnalogKey
		    && NavigationDir == Rhs.NavigationDir;
	}

	friend uint32 GetTypeHash(const FAnalogNavigationKey& InAnalogNavKey)
	{
		const uint32 KeyHash = GetTypeHash(InAnalogNavKey.AnalogKey);
		const uint32 NavDirHash = GetTypeHash(InAnalogNavKey.NavigationDir);
		return HashCombine(KeyHash, NavDirHash);
	}
};

struct FAnalogNavigationState
{
	double LastNavigationTime;
	int32 Repeats;

	FAnalogNavigationState()
		: LastNavigationTime(0)
		, Repeats(0)
	{
	}
};

/**  */
struct FUserNavigationState
{
public:
	TMap<FAnalogNavigationKey, FAnalogNavigationState> AnalogNavigationState;
};

/**
 * This class is used to control which FKeys and analog axis should move focus.
 */
class FNavigationConfig : public TSharedFromThis<FNavigationConfig>
{
public:
	/** ctor */
	SLATE_API FNavigationConfig();
	/** dtor */
	SLATE_API virtual ~FNavigationConfig();

	/** Gets the navigation direction from a given key event. */
	SLATE_API virtual EUINavigation GetNavigationDirectionFromKey(const FKeyEvent& InKeyEvent) const;
	/** Gets the navigation direction from a given analog event. */
	SLATE_API virtual EUINavigation GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent);

	/** Called when the navigation config is registered with Slate Application */
	SLATE_API virtual void OnRegister();
	/** Called when the navigation config is registered with Slate Application */
	SLATE_API virtual void OnUnregister();
	/** Notified when users are removed from the system, good chance to clean up any user specific state. */
	SLATE_API virtual void OnUserRemoved(int32 UserIndex);

	/** Notified when navigation has caused a widget change to occur */
	virtual void OnNavigationChangedFocus(TSharedPtr<SWidget> OldWidget, TSharedPtr<SWidget> NewWidget, FFocusEvent FocusEvent) {}

	/** Returns the navigation action corresponding to a key event. This version will handle multiple users correctly */
	SLATE_API virtual EUINavigationAction GetNavigationActionFromKey(const FKeyEvent& InKeyEvent) const;

	UE_DEPRECATED(4.24, "GetNavigationActionForKey doesn't handle multiple users properly, use GetNavigationActionFromKey instead")
	SLATE_API virtual EUINavigationAction GetNavigationActionForKey(const FKey& InKey) const;

	/** Simplification of config as string */
	SLATE_API virtual FString ToString() const;

	/** Returns whether the analog event is beyond the navigation thresholds set in this config. */
	bool IsAnalogEventBeyondNavigationThreshold(const FAnalogInputEvent& InAnalogEvent) const;

public:
	/** Should the Tab key perform next and previous style navigation. */
	bool bTabNavigation;
	/** Should we respect keys for navigation. */
	bool bKeyNavigation;
	/** Should we respect the analog stick for navigation. */
	bool bAnalogNavigation;
	/** Should we ignore modifier keys when checking for navigation actions. If false, only unmodified keys will be processed. */
	bool bIgnoreModifiersForNavigationActions;

	/**  */
	float AnalogNavigationHorizontalThreshold;
	/**  */
	float AnalogNavigationVerticalThreshold;

	/** Which Axis Key controls horizontal navigation */
	FKey AnalogHorizontalKey;
	/** Which Axis Key controls vertical navigation */
	FKey AnalogVerticalKey;

	/** Digital key navigation rules. */
	TMap<FKey, EUINavigation> KeyEventRules;

	/** Digital key action rules. */
	TMap<FKey, EUINavigationAction> KeyActionRules;

protected:
	/**
	 * Gets the repeat rate of the navigation based on the current pressure being applied.  The idea being
	 * that if the user moves the stick a little, we would navigate slowly, if they move it a lot, we would
	 * repeat the navigation often.
	 */
	SLATE_API virtual float GetRepeatRateForPressure(float InPressure, int32 InRepeats) const;

	/**
	 * Gets the navigation direction from the analog internally.
	 */
	SLATE_API virtual EUINavigation GetNavigationDirectionFromAnalogInternal(const FAnalogInputEvent& InAnalogEvent);

	virtual bool IsAnalogHorizontalKey(const FKey& InKey) const { return InKey == AnalogHorizontalKey; }
	virtual bool IsAnalogVerticalKey  (const FKey& InKey) const { return InKey == AnalogVerticalKey;   }

	/** Navigation state that we store per user. */
	TMap<int, FUserNavigationState> UserNavigationState;
};


/** A navigation config that doesn't do any navigation. */
class FNullNavigationConfig : public FNavigationConfig
{
public:
	FNullNavigationConfig()
	{
		bTabNavigation = false;
		bKeyNavigation = false;
		bAnalogNavigation = false;
	}
};

/** A Navigation config that supports UI Navigation with both analog sticks + D-Pad. */
class FTwinStickNavigationConfig : public FNavigationConfig
{
public:
	SLATE_API FTwinStickNavigationConfig();
	
protected:
	SLATE_API virtual bool IsAnalogHorizontalKey(const FKey& InKey) const override;
	SLATE_API virtual bool IsAnalogVerticalKey(const FKey& InKey) const override;
};
