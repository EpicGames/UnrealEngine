// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Animation/CurveSequence.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWindow.h"
#include "Application/SlateApplicationBase.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Framework/MetaData/DriverMetaData.h"

namespace SWindowTitleBarDefs
{
	/** Window flash rate. Flashes per second */
	static const float WindowFlashFrequency = 5.5f;

	/** Window flash duration. Seconds*/
	static const float WindowFlashDuration = 1.0f;
}


template <>
struct TWidgetTypeTraits<class SAppIconWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


/** Widget that represents the app icon + system menu button, usually drawn in the top left of a Windows app */
class SAppIconWidget
	: public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SAppIconWidget )
		: _IconColorAndOpacity( FLinearColor::White )
	{
		_AccessibleText = NSLOCTEXT("AppIconWidget", "System", "System Menu");
	}

	/** Icon color and opacity */
	SLATE_ATTRIBUTE( FSlateColor, IconColorAndOpacity )

	SLATE_END_ARGS( )
	
	void Construct( const FArguments& Args )
	{
		SetCanTick(false);

		IconImage = FSlateApplicationBase::Get().MakeImage(
			FSlateApplicationBase::Get().GetAppIcon(),
			Args._IconColorAndOpacity,
			EVisibility::HitTestInvisible
		);

		this->ChildSlot
		.Padding(FAppStyle::Get().GetMargin("AppIconPadding"))
		[
			IconImage.ToSharedRef()
		];
	}

	void SetIcon(const FSlateBrush* InIcon, const FMargin& InIconPadding)
	{
		ChildSlot.Padding(InIconPadding);
		IconImage->SetImage(InIcon);
	}

	virtual EWindowZone::Type GetWindowZoneOverride() const override
	{
		// Pretend we are a REAL system menu so the user can click to open a menu, or double-click to close the app on Windows
		return EWindowZone::SysMenu;
	}

private:
	TSharedPtr<SImage> IconImage;
};


/**
 * Implements a window title bar widget.
 */
class SWindowTitleBar
	: public SCompoundWidget
	, public IWindowTitleBar
{
public:

	SLATE_BEGIN_ARGS(SWindowTitleBar)
		: _Style(&FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window"))
		, _ShowAppIcon(true)
	{ }
		SLATE_STYLE_ARGUMENT(FWindowStyle, Style)
		SLATE_ARGUMENT(bool, ShowAppIcon)
		SLATE_ATTRIBUTE(FText, Title)
		SLATE_ATTRIBUTE(FText, CloseButtonToolTipText)
	SLATE_END_ARGS()
	
public:

	/**
	 * Creates and initializes a new window title bar widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InWindow The window to create the title bar for.
	 * @param InCenterContent The content for the title bar's center area.
	 * @param CenterContentAlignment The horizontal alignment of the center content.
	 */
	SLATE_API void Construct( const FArguments& InArgs, const TSharedRef<SWindow>& InWindow, const TSharedPtr<SWidget>& InCenterContent, EHorizontalAlignment InCenterContentAlignment );

	virtual EWindowZone::Type GetWindowZoneOverride() const override
	{
		return EWindowZone::TitleBar;
	}

	//~ Begin IWindowTitleBar Interface
	SLATE_API virtual void Flash() override;
	SLATE_API virtual void UpdateWindowMenu(TSharedPtr<SWidget> MenuContent) override;
	SLATE_API virtual void UpdateBackgroundContent(TSharedPtr<SWidget> BackgroundContent) override;
	SLATE_API virtual void SetAllowMenuBar(bool bInAllowMenuBar) override;
	//~ Begin IWindowTitleBar Interface
	
protected:

	float GetFlashValue( ) const
	{
		if (TitleFlashSequence.IsPlaying())
		{
			float Lerp = TitleFlashSequence.GetLerp();

			const float SinRateMultiplier = 2.0f * PI * SWindowTitleBarDefs::WindowFlashDuration * SWindowTitleBarDefs::WindowFlashFrequency;
			float SinTerm = 0.5f * (FMath::Sin( Lerp * SinRateMultiplier ) + 1.0f);

			float FadeTerm = 1.0f - Lerp;

			return SinTerm * FadeTerm;
		}
		return 0.0f;
	}

	/**
	 * Creates widgets for this window's title bar area.
	 *
	 * This is an advanced method, only for fancy windows that want to
	 * override the look of the title area by arranging those widgets itself.
	 */
	SLATE_API virtual void MakeTitleBarContentWidgets( TSharedPtr< SWidget >& OutLeftContent, TSharedPtr< SWidget >& OutRightContent );

	/**
	 * Creates the title bar's content.
	 *
	 * @param CenterContent The content for the title bar's center area.
	 * @param CenterContentAlignment The horizontal alignment of the center content.
	 *
	 * @return The content widget.
	 */
	SLATE_API TSharedRef<SWidget> MakeTitleBarContent( TSharedPtr<SWidget> CenterContent, EHorizontalAlignment CenterContentAlignment );

	FSlateColor GetWindowTitleContentColor( ) const
	{	
		// Color of the title area contents - modulates the icon and buttons
		float Flash = GetFlashValue();

		return FMath::Lerp(FLinearColor::White, FLinearColor::Black, Flash);
	}

private:

	// Callback for clicking the close button.
	FReply CloseButton_OnClicked();

	// Callback for getting the image of the close button.
	const FSlateBrush* GetCloseImage() const;

	// Callback for clicking the maximize button
	FReply MaximizeRestoreButton_OnClicked( );

	// Callback for getting the image of the maximize/restore button.
	const FSlateBrush* GetMaximizeRestoreImage( ) const;

	// Callback for clicking the minimize button.
	FReply MinimizeButton_OnClicked( );

	// Callback for getting the image of the minimize button.
	const FSlateBrush* GetMinimizeImage( ) const;
	
	/** @return An appropriate resource for the window title background depending on whether the window is active */
	const FSlateBrush* GetWindowTitlebackgroundImage( ) const;
	
	EVisibility GetWindowFlashVisibility( ) const;
	
	FSlateColor GetWindowTitleAreaColor( ) const;

	FText HandleWindowTitleText( ) const;

	FText GetMaximizeButtonToolTip() const;

protected:

	// Holds a weak pointer to the owner window.
	TWeakPtr<SWindow> OwnerWindowPtr;

private:

	// Holds the window style to use (for buttons, text, etc.).
	const FWindowStyle* Style;

	// Holds the content widget of the title area.
	TSharedPtr<SWidget> TitleArea;

	// Holds the curve sequence for the window flash animation.
	FCurveSequence TitleFlashSequence;

	// Holds the minimize button.
	TSharedPtr<SButton> MinimizeButton;

	// Holds the maximize/restore button.
	TSharedPtr<SButton> MaximizeRestoreButton;

	// Holds the close button.
	TSharedPtr<SButton> CloseButton;

	TAttribute<FText> Title;
	TAttribute<FText> CloseButtonToolTipText;

	SHorizontalBox::FSlot* WindowMenuSlot;
	SHorizontalBox::FSlot* RightSideContentSlot;

	TSharedPtr<SAppIconWidget> AppIconWidget;
	bool ShowAppIcon;
	bool bAllowMenuBar = false;
}; 
