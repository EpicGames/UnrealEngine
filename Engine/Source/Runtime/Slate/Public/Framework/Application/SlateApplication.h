// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Application/IMenu.h"
#include "Layout/Visibility.h"
#include "GenericPlatform/GenericWindow.h"
#include "Styling/SlateColor.h"
#include "Layout/SlateRect.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Input/Events.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Application/SlateWindowHelper.h"
#include "Rendering/SlateRenderer.h"
#include "Application/SlateApplicationBase.h"
#include "Application/ThrottleManager.h"
#include "Widgets/IToolTip.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Application/GestureDetector.h"
#include "SlateApplication.generated.h"

class FNavigationConfig;
#if WITH_ACCESSIBILITY
class FSlateAccessibleMessageHandler;
#endif
class IInputInterface;
class IInputProcessor;
class IPlatformTextField;
class ISlateSoundDevice;
class ITextInputMethodSystem;
class IVirtualKeyboardEntry;
class IWidgetReflector;
class SNotificationItem;
class SViewport;
class FSlateUser;
class FSlateVirtualUserHandle;

enum class ESlateDebuggingInputEvent : uint8;

/** A Delegate for querying whether source code access is possible */
DECLARE_DELEGATE_RetVal(bool, FQueryAccessSourceCode);

/** Delegates for when modal windows open or close */
DECLARE_DELEGATE(FModalWindowStackStarted)
DECLARE_DELEGATE(FModalWindowStackEnded)

/** Delegate for when window action occurs (ClickedNonClientArea, Maximize, Restore, WindowMenu). Return true if the OS layer should stop processing the action. */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnWindowAction, const TSharedRef<FGenericWindow>&, EWindowAction::Type);

DECLARE_DELEGATE_RetVal(bool, FDragDropCheckingOverride);


/** Allow widgets to find out when someone clicked outside them. Currently needed by MenuAnchros. */
class FPopupSupport
{
	public:

	/**
	 * Given a WidgetPath that was clicked, send notifications to any subscribers that were not under the mouse.
	 * i.e. Send the "Someone clicked outside me" notifications.
	 */
	SLATE_API void SendNotifications( const FWidgetPath& WidgetsUnderCursor );

	/**
	* Register for a notification when the user clicks outside a specific widget.
	*
	* @param NotifyWhenClickedOutsideMe    When the user clicks outside this widget, fire a notification.
	* @param InNotification                The notification to invoke.
	*/
	SLATE_API FDelegateHandle RegisterClickNotification(const TSharedRef<SWidget>& NotifyWhenClickedOutsideMe, const FOnClickedOutside& InNotification);

	/**
	* NOTE: Only necessary if notification no longer desired.
	*       Stale notifications are cleaned up automatically.
	*
	* Unregister the notification because it is no longer desired.
	*/
	SLATE_API void UnregisterClickNotification(FDelegateHandle InHandle);

	private:

	/** A single subscription about clicks happening outside the widget. */
	struct FClickSubscriber
	{
		FClickSubscriber( const TSharedRef<SWidget>& DetectClicksOutsideThisWidget, const FOnClickedOutside& InNotification )
		: DetectClicksOutsideMe( DetectClicksOutsideThisWidget )
		, Notification( InNotification )
		{}

		bool ShouldKeep() const
		{
			return DetectClicksOutsideMe.IsValid() && Notification.IsBound();
		}

		/** If a click occurs outside this widget, we'll send the notification */
		TWeakPtr<SWidget> DetectClicksOutsideMe;
		/** Notification to send */
		FOnClickedOutside Notification;
	};

	/** List of subscriptions that want to be notified when the user clicks outside a certain widget. */
	TArray<FClickSubscriber> ClickZoneNotifications;
};

/**
 * Interface for a Slate Input Mapping.
 */
class ISlateInputManager
{
public:

	virtual ~ISlateInputManager() = default;
	
	virtual int32 GetUserIndexForMouse() const = 0;
	virtual int32 GetUserIndexForKeyboard() const = 0;

	virtual FInputDeviceId GetInputDeviceIdForMouse() const = 0;
	virtual FInputDeviceId GetInputDeviceIdForKeyboard() const = 0;
	
	virtual TOptional<int32> GetUserIndexForInputDevice(FInputDeviceId InputDeviceId) const
	{
		// There is a 1:1 mapping of the platform user ID to a slate user index.
		return GetUserIndexForPlatformUser(IPlatformInputDeviceMapper::Get().GetUserForInputDevice(InputDeviceId));
	}

	virtual TOptional<int32> GetUserIndexForPlatformUser(FPlatformUserId PlatformUser) const
	{
		// There is a 1:1 mapping of the platform user ID to a slate user index.
		return PlatformUser.GetInternalId();
	}

	virtual int32 GetUserIndexForController(int32 ControllerId) const { return ControllerId; }
	virtual TOptional<int32> GetUserIndexForController(int32 ControllerId, FKey InKey) const = 0;
};

class FSlateDefaultInputMapping : public ISlateInputManager
{
public:
	virtual int32 GetUserIndexForMouse() const override { return 0; }
	virtual int32 GetUserIndexForKeyboard() const override { return 0; }
	
	virtual FInputDeviceId GetInputDeviceIdForMouse() const override
	{
		return IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
	};
	
	virtual FInputDeviceId GetInputDeviceIdForKeyboard() const override
	{
		return IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
	};
	
	virtual int32 GetUserIndexForController(int32 ControllerId) const override { return ControllerId; }
	virtual TOptional<int32> GetUserIndexForController(int32 ControllerId, FKey InKey) const override { return GetUserIndexForController(ControllerId); }
};

enum class ESlateTickType : uint8
{
	/** Tick time only */
	Time = 1 << 0,

	/** Only process input for the platform, and additional input tasks by Slate. */
	PlatformAndInput = 1 << 1,

	/** Only Tick and Paint Widgets */
	Widgets = 1 << 2,

	/** Time and Widgets */
	TimeAndWidgets = Time | Widgets,

	/** Update time, tick and paint widgets, and process input */
	All = Time | PlatformAndInput | Widgets,
};

ENUM_CLASS_FLAGS(ESlateTickType);

/** Priority bucket type for Input Pre-Processors. They will be evaluated in ascending order and may block input from later buckets. */
UENUM()
enum class EInputPreProcessorType : uint8
{
	/** Input from platform overlay events */
	Overlay = 0,
	/** Higher priority engine-related input events */
	PreEngine,
	/** Engine-related input events */
	Engine,
	/** Higher priority editor-related input events, can be used as a middle point between Engine and Game */
	PreEditor,
	/** Editor - related input events, can be used as a middle point between Engine and Game */
	Editor,
	/** Higher priority game-related input events */
	PreGame,
	/** Game-related input events. This is the default and lowest priority */
	Game,

	/** Current total number of types. Not to be used by processors, only for storage purposes */
	Count
};

/** Determines how popup windows are laid out by CalculatePopupWindowPosition
  * when there is no space on the screen for the popup window to fit in its preferred orientation. */
UENUM()
enum class EPopupLayoutMode : uint8
{
	/** The popup window is allowed to flip to the other side of the anchor that opens it. */
	Menu = 0,
	/** The popup window does not flip. Tooltips have their own logic to handle placement for this case. */
	ToolTip
};

USTRUCT()
struct FInputPreprocessorRegistrationKey
{
	GENERATED_BODY()

	UPROPERTY(Config)
	EInputPreProcessorType Type = EInputPreProcessorType::Game;
	
	UPROPERTY(Config)
	int32 Priority = INDEX_NONE;

	bool operator==(const FInputPreprocessorRegistrationKey& Other) const
	{
		return Type == Other.Type && Priority == Other.Priority;
	}
};

struct FInputPreprocessorRegistration
{
	FInputPreprocessorRegistrationKey Info;
	TSharedPtr<class IInputProcessor> InputProcessor;

	bool operator==(const FInputPreprocessorRegistration& Other) const
	{
		return Info == Other.Info && InputProcessor == Other.InputProcessor;
	}
};

class FSlateApplication
	: public FSlateApplicationBase
	, public FGenericApplicationMessageHandler
{
public:

	/** Virtual destructor. */
	SLATE_API virtual ~FSlateApplication();

public:

	/**
	 * Returns the running average delta time (smoothed over several frames)
	 *
	 * @return  The average time delta
	 */
	const float GetAverageDeltaTime() const
	{
		return AverageDeltaTime;
	}

	/**
	 * Returns the real time delta since Slate last ticked widgets
	 *
	 * @return  The time delta since last tick
	 */
	const float GetDeltaTime() const
	{
		return (float)( CurrentTime - LastTickTime );
	}

	/**
	 * Returns the running average delta time (smoothed over several frames)
	 * Unlike GetAverageDeltaTime() it excludes exceptional
	 * situations, such as when throttling mode is active.
	 *
	 * @return  The average time delta
	 */
	float GetAverageDeltaTimeForResponsiveness() const
	{
		return AverageDeltaTimeForResponsiveness;
	}

public:

	static SLATE_API void Create();
	static SLATE_API TSharedRef<FSlateApplication> Create(const TSharedRef<class GenericApplication>& InPlatformApplication);

	static SLATE_API TSharedRef<FSlateApplication> InitializeAsStandaloneApplication(const TSharedRef< class FSlateRenderer >& PlatformRenderer);
	
	static SLATE_API TSharedRef<FSlateApplication> InitializeAsStandaloneApplication(const TSharedRef< class FSlateRenderer >& PlatformRenderer, const TSharedRef<class GenericApplication>& InPlatformApplication);

	static SLATE_API void InitializeCoreStyle();

	/**
	 * Returns true if a Slate application instance is currently initialized and ready
	 *
	 * @return  True if Slate application is initialized
	 */
	static bool IsInitialized()
	{
		return CurrentApplication.IsValid();
	}

	/**
	 * Returns the current instance of the application. The application should have been initialized before
	 * this method is called
	 *
	 * @return  Reference to the application
	 */
	static FSlateApplication& Get()
	{
		check( CurrentApplication.IsValid() );
		check( IsInGameThread() || IsInSlateThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread() );
		return *CurrentApplication;
	}

	static SLATE_API void Shutdown(bool bShutdownPlatform = true);

	/** @return the global tab manager */
	static SLATE_API TSharedRef<class FGlobalTabmanager> GetGlobalTabManager();

	/** Initializes high dpi support for the process */
	static SLATE_API void InitHighDPI(const bool bForceEnable);

	/** @return the root style node, which is the entry point to the style graph representing all the current style rules. */
	SLATE_API const class FStyleNode* GetRootStyle() const;

	/**
	 * Initializes the renderer responsible for drawing all elements in this application
	 *
	 * @param InRenderer The renderer to use.
	 * @param bQuietMode Don't show any message boxes when initialization fails.
	 */
	SLATE_API virtual bool InitializeRenderer( TSharedRef<FSlateRenderer> InRenderer, bool bQuietMode = false );

	/** Set the slate sound provider that the slate app should use. */
	SLATE_API virtual void InitializeSound( const TSharedRef<ISlateSoundDevice>& InSlateSoundDevice );

	/** Play SoundToPlay. Interrupt previous sound if one is playing. */
	SLATE_API void PlaySound( const FSlateSound& SoundToPlay, int32 UserIndex = 0 ) const;

	/** @return The duration of the given sound resource */
	SLATE_API float GetSoundDuration(const FSlateSound& Sound) const;

	IInputInterface* GetInputInterface() const { return PlatformApplication->GetInputInterface(); }

	/** @return Whether or not the current platform supports system help */
	bool SupportsSystemHelp() const { return PlatformApplication->SupportsSystemHelp(); }

	void ShowSystemHelp() { PlatformApplication->ShowSystemHelp(); }

	/** @return The text input method interface for this application */
	ITextInputMethodSystem *GetTextInputMethodSystem() const { return PlatformApplication->GetTextInputMethodSystem(); }

	/** 
	 * Sets the position of the cursor.
	 *
	 * @param MouseCoordinate		The new position.
	 */
	SLATE_API void SetCursorPos( const FVector2D& MouseCoordinate ) override;

	/** 
	 * Replace the IPlatformTextField implementation with a custom one.  The implementation used by default is platform specific.
	 * If you need to change the default behavior for virtual keyboard, this method is for you.
	 *
	 * @param PlatformTextField		The platform specific implementation of the IPlatformTextField.
	 */
	SLATE_API void OverridePlatformTextField(TUniquePtr<IPlatformTextField> PlatformTextField);
	
	/** 
	 *	Updates the cursor user's cursor to either the platform cursor or fake cursor
	 */
	SLATE_API void UsePlatformCursorForCursorUser(bool bUsePlatformCursor);

	/** Changes the cursor type to Default (Visible) or None (Not Visible)*/
	SLATE_API void SetPlatformCursorVisibility(bool bNewVisibility);

	/** Polls game devices for input */
	SLATE_API void PollGameDeviceState();

	/** Occurs before Tick(), after all pointer and keyboard input has been processed. */
	SLATE_API void FinishedInputThisFrame();

	/** Ticks this application */
	SLATE_API void Tick(ESlateTickType TickType = ESlateTickType::All);

	/** Returns true if we are currently ticking the SlateApplication. */
	SLATE_API bool IsTicking() const;

	/** Pumps OS messages when a modal window or intra-frame debugging session exists */
	SLATE_API void PumpMessages();

	/** Returns true if this slate application is ready to open modal windows */
	SLATE_API bool CanAddModalWindow() const;

	/** Returns true if this slate application is ready to display windows. */
	SLATE_API bool CanDisplayWindows() const;

	/** Returns navigation direction matching a key event, this is determined in the FNavigationConfig */
	SLATE_API virtual EUINavigation GetNavigationDirectionFromKey(const FKeyEvent& InKeyEvent) const override;

	/** Returns navigation direction matching an anlog event, this is determined in the FNavigationConfig */
	SLATE_API virtual EUINavigation GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent) override;

	/** Returns the navigation action corresponding to a key event. This version will handle multiple users correctly */
	SLATE_API virtual EUINavigationAction GetNavigationActionFromKey(const FKeyEvent& InKeyEvent) const override;

	UE_DEPRECATED(4.24, "GetNavigationActionForKey doesn't handle multiple users properly, use GetNavigationActionFromKey instead")
	SLATE_API virtual EUINavigationAction GetNavigationActionForKey(const FKey& InKey) const override;

	/**
	 * Adds a modal window to the application.  
	 * In most cases, this function does not return until the modal window is closed (the only exception is a modal window for slow tasks)  
	 *
	 * @param InSlateWindow		A SlateWindow to which to add a native window.
	 * @param InParentWindow	The parent of the modal window.  All modal windows must have a parent.
	 * @param bSlowTaskWindow	true if the window is for a slow task and this function should return before the window is closed
	 */
	SLATE_API void AddModalWindow( TSharedRef<SWindow> InSlateWindow, const TSharedPtr<const SWidget> InParentWidget, bool bSlowTaskWindow = false );

	/** Sets the delegate for when a modal window stack begins */
	SLATE_API void SetModalWindowStackStartedDelegate(FModalWindowStackStarted StackStartedDelegate);

	/** Sets the delegate for when a modal window stack ends */
	SLATE_API void SetModalWindowStackEndedDelegate(FModalWindowStackEnded StackEndedDelegate);

	/**
	 * Associates a top level Slate Window with a native window, and "natively" parents that window to the specified Slate window.
	 * Although the window still a top level window in Slate, it will be considered a child window to the operating system.
	 *
	 * @param	InSlateWindow		A Slate window to which to add a native window.
	 * @param	InParentWindow		Slate window that the window being added should be a native child of
	 * @param	bShowImmediately	True to show the window.  Pass false if you're going to call ShowWindow() yourself later.
	 *
	 * @return a reference to the SWindow that was just added.
	 */
	SLATE_API TSharedRef<SWindow> AddWindowAsNativeChild( TSharedRef<SWindow> InSlateWindow, TSharedRef<SWindow> InParentWindow, const bool bShowImmediately = true );

	/**
	 * Creates a new Menu and adds it to the menu stack.
	 * Menus are always auto-sized. Use fixed-size content if a fixed size is required.
	 *
	 * @param InParentWidget		The parent of the menu. If the stack isn't empty, PushMenu will attempt to determine the stack level for the new menu by looking of an open menu in the parent's path.
	 * @param InOwnerPath			Optional full widget path of the parent if one is available. If an invalid path is given PushMenu will attempt to generate a path to the InParentWidget
	 * @param InContent				The content to be placed inside the new menu
	 * @param SummonLocation		The location where this menu should be summoned
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bFocusImmediately		Should the popup steal focus when shown?
	 * @param SummonLocationSize	An optional rect which describes an area in which the menu may not appear
	 * @param Method				An optional popup method override. If not set, the widgets in the InOwnerPath will be queried for this.
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 * @param FocusUserIndex		The index of the user that should focus the menu. INDEX_NONE for all users.
	 */
	SLATE_API TSharedPtr<IMenu> PushMenu(const TSharedRef<SWidget>& InParentWidget, const FWidgetPath& InOwnerPath, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately = true, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize = FVector2f::ZeroVector, TOptional<EPopupMethod> Method = TOptional<EPopupMethod>(), const bool bIsCollapsedByParent = true, const int32 FocusUserIndex = INDEX_NONE);

	/**
	 * Creates a new Menu and adds it to the menu stack under the specified parent menu.
	 * Menus are always auto-sized. Use fixed-size content if a fixed size is required.
	 *
	 * @param InParentMenu			The parent of the menu. Must be a valid menu in the stack.
	 * @param InContent				The content to be placed inside the new menu
	 * @param SummonLocation		The location where this menu should be summoned
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param bFocusImmediately		Should the popup steal focus when shown?
	 * @param SummonLocationSize	An optional rect which describes an area in which the menu may not appear
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 * @param FocusUserIndex		The index of the user that should focus the menu. INDEX_NONE for all users.
	 */
	SLATE_API TSharedPtr<IMenu> PushMenu(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<SWidget>& InContent, const UE::Slate::FDeprecateVector2DParameter& SummonLocation, const FPopupTransitionEffect& TransitionEffect, const bool bFocusImmediately = true, const UE::Slate::FDeprecateVector2DParameter& SummonLocationSize = FVector2f::ZeroVector, const bool bIsCollapsedByParent = true, const int32 FocusUserIndex = INDEX_NONE);

	/**
	 * Creates a new hosted Menu and adds it to the menu stack.
	 * Hosted menus are drawn by an external host widget.
	 * 
	 * @param InParentMenu			The parent of the menu. Must be a valid menu in the stack.
	 * @param InOwnerPath			Optional full widget path of the parent if one is available. If an invalid path is given PushMenu will attempt to generate a path to the InParentWidget
	 * @param InMenuHost			The host widget that draws the menu's content
	 * @param InContent				The content to be placed inside the new menu
	 * @param OutWrappedContent		Returns the InContent wrapped with widgets needed by the menu stack system. This is what should be drawn by the host after this call.
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param ShouldThrottle		Should we throttle engine ticking to maximize the menu responsiveness
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 */
	SLATE_API TSharedPtr<IMenu> PushHostedMenu(const TSharedRef<SWidget>& InParentWidget, const FWidgetPath& InOwnerPath, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent = true);
	
	/**
	 * Creates a new hosted child Menu and adds it to the menu stack under the specified parent menu.
	 * Hosted menus are drawn by an external host widget.
	 * 
	 * @param InParentMenu			The parent menu for this menu
	 * @param InMenuHost			The host widget that draws the menu's content
	 * @param InContent				The menu's content
	 * @param OutWrappedContent		Returns the InContent wrapped with widgets needed by the menu stack system. This is what should be drawn by the host after this call.
	 * @param TransitionEffect		Animation to use when the popup appears
	 * @param ShouldThrottle		Should we throttle engine ticking to maximize the menu responsiveness
	 * @param bIsCollapsedByParent	Is this menu collapsed when a parent menu receives focus/activation? If false, only focus/activation outside the entire stack will auto collapse it.
	 */	
	SLATE_API TSharedPtr<IMenu> PushHostedMenu(const TSharedPtr<IMenu>& InParentMenu, const TSharedRef<IMenuHost>& InMenuHost, const TSharedRef<SWidget>& InContent, TSharedPtr<SWidget>& OutWrappedContent, const FPopupTransitionEffect& TransitionEffect, EShouldThrottle ShouldThrottle, const bool bIsCollapsedByParent = true);

	/** @return Returns whether the menu has child menus. */
	SLATE_API bool HasOpenSubMenus(TSharedPtr<IMenu> InMenu) const;

	/** @return	Returns true if there are any pop-up menus summoned */
	SLATE_API virtual bool AnyMenusVisible() const override;

	/**
	 * Attempt to locate a menu that contains the specified widget
	 *
	 * @param InWidgetPath Path to the widget to use for the search
	 * @return the menu in which the widget resides, or nullptr
	 */
	SLATE_API TSharedPtr<IMenu> FindMenuInWidgetPath(const FWidgetPath& InWidgetPath) const;

	/** @return	Returns a ptr to the window that is currently the host of the menu stack or null if no menus are visible */
	SLATE_API TSharedPtr<SWindow> GetVisibleMenuWindow() const;

	/** @return	Returns a ptr to the widget that created the opened root menu or null if one is not opened */
	SLATE_API TSharedPtr<SWidget> GetMenuHostWidget() const;

	/** Dismisses all open menus */
	SLATE_API virtual void DismissAllMenus() override;

	/**
	 * Dismisses a menu and all its children
	 *
	 * @param InFromMenu	The menu to dismiss, any children, grandchildren etc will also be dismissed
	 */
	SLATE_API void DismissMenu(const TSharedPtr<IMenu>& InFromMenu);

	/**
	 * Dismisses a menu and all its children. The menu is determined by looking for menus in the parent chain of the widget.
	 *
	 * @param InWidgetInMenu	The widget whose path is search upwards for a menu. That menu will then be dismissed.
	 */
	SLATE_API void DismissMenuByWidget(const TSharedRef<SWidget>& InWidgetInMenu);

	/**
	 * HACK: Don't use this unless shutting down a game viewport
	 * Game viewport windows need to be destroyed instantly or else the viewport could tick and access deleted data
	 *
	 * @param WindowToDestroy		Window for destruction
	 */
	SLATE_API void DestroyWindowImmediately( TSharedRef<SWindow> WindowToDestroy );

	/**
	 * Disable Slate components when an external, non-slate, modal window is brought up.  In the case of multiple
	 * external modal windows, we will only increment our tracking counter.
	 */
	SLATE_API void ExternalModalStart();

	/**
	 * Re-enable disabled Slate components when a non-slate modal window is dismissed.  Slate components
	 * will only be re-enabled when all tracked external modal windows have been dismissed.
	 */
	SLATE_API void ExternalModalStop();

	int GetNumExternalModalWindowsActive() const { return NumExternalModalWindowsActive; }

	/** Event before slate application ticks. */
	DECLARE_EVENT_OneParam(FSlateApplication, FSlateTickEvent, float);
	FSlateTickEvent& OnPreTick()  { return PreTickEvent; }

	/** Event after slate application ticks. */
	FSlateTickEvent& OnPostTick()  { return PostTickEvent; }

	/** Event when the application is about to shutdown. */
	FSimpleMulticastDelegate& OnPreShutdown() { return PreShutdownEvent; }

	/** Delegate for when a new user has been registered. */
	DECLARE_EVENT_OneParam(FSlateApplication, FUserRegisteredEvent, int32);
	FUserRegisteredEvent& OnUserRegistered() { return UserRegisteredEvent; }

	/** Delegate called when a window is about to be destroyed */
	DECLARE_EVENT_OneParam(FSlateApplication, FOnWindowBeingDestroyed, const SWindow&);
	FOnWindowBeingDestroyed& OnWindowBeingDestroyed() { return WindowBeingDestroyedEvent; }

	/* Delegate called when a menu is about to be destroyed*/
	FOnMenuDestroyed& OnMenuBeingDestroyed() { return MenuBeingDestroyedEvent; }

	/** Delegate called just before possible focus change */
	DECLARE_MULTICAST_DELEGATE_FiveParams(FOnFocusChanging, const FFocusEvent&, const FWeakWidgetPath&, const TSharedPtr<SWidget>&, const FWidgetPath&, const TSharedPtr<SWidget>&);
	FOnFocusChanging& OnFocusChanging() { return FocusChangingDelegate; }
	
	/** Delegate called when we are about to take a screenshot of a SWidget. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreScreenshot, TSharedRef<SWidget>)
	FOnPreScreenshot& OnPreScreenshot() { return PreScreenshotDelegate; }

	/**
	 * Registers a game viewport with the Slate application so that specific messages can be routed directly to a viewport
	 * 
	 * @param InViewport	The viewport to register.  Note there is currently only one registered viewport
	 */
	SLATE_API void RegisterGameViewport( TSharedRef<SViewport> InViewport );

	/**
	 * Registers a viewport with the Slate application so that specific messages can be routed directly to a viewport
	 * This is for all viewports, there can be multiple of these as opposed to the singular "Game Viewport"
	 * 
	 * @param InViewport	The viewport to register.  Note there is currently only one registered viewport
	 */
	SLATE_API void RegisterViewport(TSharedRef<SViewport> InViewport);

	/**
	 * Returns the game viewport registered with the slate application
	 *
	 * @return registered game viewport
	 */
	SLATE_API TSharedPtr<SViewport> GetGameViewport() const;

	/** 
	 * Unregisters the current game viewport from Slate.  This method sends a final deactivation message to the viewport
	 * to allow it to do a final cleanup before being closed.
	 */
	SLATE_API void UnregisterGameViewport();

	/**
	 * Register another window that may be visible in a non-top level way that still needs to be able to maintain focus paths.
	 * Generally speaking - this is for Virtual Windows that are created to render in the 3D world slate content.
	 */
	SLATE_API void RegisterVirtualWindow(TSharedRef<SWindow> InWindow);

	/**
	 * Unregister a virtual window.
	 */
	SLATE_API void UnregisterVirtualWindow(TSharedRef<SWindow> InWindow);

	/**
	 * Flushes the render state of slate, releasing accesses and flushing all render commands.
	 */
	SLATE_API void FlushRenderState();

	/**
	 * Sets specified user focus to the SWidget representing the currently active game viewport
	 */
	SLATE_API void SetUserFocusToGameViewport(uint32 UserIndex, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/**
	 * Sets all users focus to the SWidget representing the currently active game viewport
	 */
	SLATE_API void SetAllUserFocusToGameViewport(EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/**
	 * Activates the Game Viewport if it is properly childed under a window
	 */
	SLATE_API void ActivateGameViewport();

	/**
	 * True if transforming mouse input coordinates to account for fullscreen distortions
	 */
	SLATE_API bool GetTransformFullscreenMouseInput() const;

#if WITH_SLATE_DEBUGGING
	/**
	 * Tries to dumps the current navigation config along with callstack used to set it.
	 * Does nothing if Slate.Debug.TraceNavigationConfig is false (By default is false)
	 *
	 * @param InNavigationConfig Navigation config to dump info to log for
	 */
	SLATE_API void TryDumpNavigationConfig(TSharedPtr<FNavigationConfig> InNavigationConfig) const;
#endif // WITH_SLATE_DEBUGGING

	/**
	 * Sets specified user focus to the SWidget passed in.
	 *
	 * @param UserIndex Index of the user to change focus for
	 * @param WidgetToFocus the widget to set focus to
	 * @param ReasonFocusIsChanging the contextual reason for the focus change
	 */
	SLATE_API bool SetUserFocus(uint32 UserIndex, const TSharedPtr<SWidget>& WidgetToFocus, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/**
	 * Sets focus for all users to the SWidget passed in.
	 *
	 * @param WidgetToFocus the widget to set focus to
	 * @param ReasonFocusIsChanging the contextual reason for the focus change
	 */
	SLATE_API void SetAllUserFocus(const TSharedPtr<SWidget>& WidgetToFocus, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/** Releases the users focus from whatever it currently is on. */
	SLATE_API void ClearUserFocus(uint32 UserIndex, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/** Releases the focus for all users from whatever it currently is on. */
	SLATE_API void ClearAllUserFocus(EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/**
	 * Sets a typed context payload for the current navigation event, keyed by user index.
	 * Intended to be written during OnNavigation and read during OnFocusReceived on the
	 * destination widget. Automatically cleared at the end of each navigation event.
	 */
	SLATE_API void SetPendingNavigationContext(uint32 UserIndex, TSharedPtr<ISlateMetaData> InContext);

	/**
	 * Returns the pending navigation context for the given user cast to the requested type.
	 * Returns nullptr if no context was set or the stored type does not match.
	 */
	template <typename T>
	TSharedPtr<T> GetPendingNavigationContext(uint32 UserIndex) const
	{
		if (const TSharedPtr<ISlateMetaData>* Found = PendingNavigationContexts.Find(UserIndex))
		{
			if ((*Found)->IsOfType<T>())
			{
				return StaticCastSharedPtr<T>(*Found);
			}
		}
		return nullptr;
	}

	/** Clears the pending navigation context for the given user. */
	SLATE_API void ClearPendingNavigationContext(uint32 UserIndex);

	/**
	 * Sets the Keyboard focus to the specified SWidget
	 */
	SLATE_API bool SetKeyboardFocus(const TSharedPtr<SWidget>& OptionalWidgetToFocus, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/**
	 * Clears keyboard focus, if any widget is currently focused
	 *
	 * @param ReasonFocusIsChanging The reason that keyboard focus is changing
	 */
	SLATE_API void ClearKeyboardFocus(const EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/** Gets a delegate that is invoked when a regular window is moved or resized */
	DECLARE_EVENT_OneParam(FSlateApplication, FOnRegularWindowMovedOrResized, TSharedRef<SWindow>);
	FOnRegularWindowMovedOrResized& OnRegularWindowMovedOrResized() { return OnRegularWindowMovedOrResizedEvent; }

#if WITH_EDITOR
	/**
	* Gets a delegate that is invoked before the input key get process by slate widgets bubble system.
	* Its read only and you cannot mark the input as handled.
	*/
	DECLARE_EVENT_OneParam(FSlateApplication, FOnApplicationPreInputKeyDownListener, const FKeyEvent&);
	FOnApplicationPreInputKeyDownListener& OnApplicationPreInputKeyDownListener() { return OnApplicationPreInputKeyDownListenerEvent; }

	/**
	* Gets a delegate that is invoked before the mouse input button down get process by slate widgets bubble system.
	* Its read only and you cannot mark the input as handled.
	*/
	DECLARE_EVENT_OneParam(FSlateApplication, FOnApplicationMousePreInputButtonDownListener, const FPointerEvent&);
	FOnApplicationMousePreInputButtonDownListener& OnApplicationMousePreInputButtonDownListener() { return OnApplicationMousePreInputButtonDownListenerEvent; }

	/** Gets a delegate that is invoked in the editor when a windows dpi scale changes or when a widget window may have changed and DPI scale info needs to be checked */
	DECLARE_EVENT_OneParam(FSlateApplication, FOnWindowDPIScaleChanged, TSharedRef<SWindow>);
	FOnWindowDPIScaleChanged& OnWindowDPIScaleChanged() { return OnWindowDPIScaleChangedEvent; }

	/** Event used to signal that a DPI change is about to happen */
	FOnWindowDPIScaleChanged& OnSystemSignalsDPIChanged() { return OnSignalSystemDPIChangedEvent; }
#endif //WITH_EDITOR

	/**
	 * Returns the current modifier keys state
	 *
	 * @return  State of modifier keys
	 */
	SLATE_API FModifierKeysState GetModifierKeys() const;

	/**
	 *	Restores all input settings to their original values
	 * 
	 *  Clears all user focus
	 *  Shows the mouse, clears any locks and captures, turns off high precision input
	 */
	SLATE_API void ResetToDefaultInputSettings();
	
	/**
	 *	Restores all pointer input settings to their original values
	 * 
	 *  Shows the mouse, clears any locks and captures, turns off high precision input
	 */
	SLATE_API void ResetToDefaultPointerInputSettings();

	/**
	 * Mouse capture
	 */

	 /**
	  * @param bAllow If true, mouse pointer capture will be processed even when the application is not active or widget is not a virtual window
	  */
	void SetHandleDeviceInputWhenApplicationNotActive(bool bAllow) { bHandleDeviceInputWhenApplicationNotActive = bAllow; }
	bool GetHandleDeviceInputWhenApplicationNotActive() const { return bHandleDeviceInputWhenApplicationNotActive; }

	/** returning platform-specific value designating window that captures mouse, or nullptr if mouse isn't captured */
	SLATE_API virtual void* GetMouseCaptureWindow() const;

	/** Releases capture for every pointer on every user from whatever it currently is on. */
	SLATE_API void ReleaseAllPointerCapture();
	UE_DEPRECATED(4.23, "ReleaseMouseCapture has been renamed to ReleaseAllPointerCapture()")
	SLATE_API void ReleaseMouseCapture();

	/** Releases capture for every pointer belonging to the given user index particular user. */
	SLATE_API void ReleaseAllPointerCapture(int32 UserIndex);
	UE_DEPRECATED(4.23, "ReleaseMouseCaptureForUser has been renamed to ReleaseAllPointerCapture(int32 UserIndex)")
	SLATE_API void ReleaseMouseCaptureForUser(int32 UserIndex);

	/** @return The active modal window or nullptr if there is no modal window. */
	SLATE_API TSharedPtr<SWindow> GetActiveModalWindow() const;

	/**
	 * Assign a delegate to be called when this application is requesting an exit (e.g. when the last window is closed).
	 *
	 * @param OnExitRequestHandler  Function to execute when the application wants to quit
	 */
	SLATE_API void SetExitRequestedHandler( const FSimpleDelegate& OnExitRequestedHandler );
		
	/**
	 * @todo slate: Remove this method or make it private.
	 * Searches for the specified widget and generates a full path to it.  Note that this is
	 * a relatively slow operation!  It can fail, in which case OutWidgetPath.IsValid() will be false.
	 * 
	 * @param  InWidget       Widget to generate a path to
	 * @param  OutWidgetPath  The generated widget path
	 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
	 */
	SLATE_API bool GeneratePathToWidgetUnchecked( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter = EVisibility::Visible ) const;
	
	/**
	 * @todo slate: Remove this method or make it private.
	 * Searches for the specified widget and generates a full path to it.  Note that this is
	 * a relatively slow operation!  Asserts if the widget isn't found.
	 * 
	 * @param  InWidget       Widget to generate a path to
	 * @param  OutWidgetPath  The generated widget path
	 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
	 */
	SLATE_API void GeneratePathToWidgetChecked( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter = EVisibility::Visible ) const;
	
	/**
	 * Finds the window that the provided widget resides in
	 * 
	 * @param InWidget		The widget to find the window for
	 * @return The window where the widget resides, or null if the widget wasn't found.  Remember, a widget might not be found simply because its parent decided not to report the widget in ArrangeChildren.
	 */
	SLATE_API virtual TSharedPtr<SWindow> FindWidgetWindow( TSharedRef<const SWidget> InWidget ) const override;

	/**
	 * Finds the window that the provided widget resides in
	 * 
	 * @param InWidget		The widget to find the window for
	 * @param OutWidgetPath Full widget path generated 
	 * @return The window where the widget resides, or null if the widget wasn't found.  Remember, a widget might not be found simply because its parent decided not to report the widget in ArrangeChildren.
	 */
	UE_DEPRECATED(4.23, "The FindWidgetWindow method that takes an FWidgetPath has been deprecated.  If you dont need the widget path, use FindWidgetWindow(MyWidget) instead.  If you need the path use GeneratePathToWidget")
	SLATE_API TSharedPtr<SWindow> FindWidgetWindow( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath) const;

	/**
	 * @return True if the application is currently routing high precision mouse movement events (OS specific)
	 * If this value is true, the mouse is captured and hidden by the widget that originally made the request.
	 */
	bool IsUsingHighPrecisionMouseMovment() const { return PlatformApplication.IsValid() ? PlatformApplication->IsUsingHighPrecisionMouseMode() : false; }
	
	/**
	 * @return True if the last mouse event was from a trackpad.
	 */
	bool IsUsingTrackpad() const { return PlatformApplication.IsValid() ? PlatformApplication->IsUsingTrackpad() : false; }

	/**
	 * @return True if there is a mouse device attached
	 */
	bool IsMouseAttached() const { return PlatformApplication.IsValid() ? PlatformApplication->IsMouseAttached() : false; }

	/**
	 * @return True if there is a gamepad attached
	 */
	bool IsGamepadAttached() const { return PlatformApplication.IsValid() ? PlatformApplication->IsGamepadAttached() : false; }

	/**
	 * Sets the widget reflector.
	 *
	 * @param WidgetReflector The widget reflector to set.
	 */
	SLATE_API void SetWidgetReflector( const TSharedRef<IWidgetReflector>& WidgetReflector );

	/** @param AccessDelegate The delegate to pass along to the widget reflector */
	void SetWidgetReflectorSourceAccessDelegate(FAccessSourceCode AccessDelegate)
	{
		SourceCodeAccessDelegate = AccessDelegate;
	}

	/** @param QueryAccessDelegate The delegate to pass along to the widget reflector */
	void SetWidgetReflectorQuerySourceAccessDelegate(FQueryAccessSourceCode QueryAccessDelegate)
	{
		QuerySourceCodeAccessDelegate = QueryAccessDelegate;
	}

	/** @param AccessDelegate The delegate to pass along to the widget reflector */
	void SetWidgetReflectorAssetAccessDelegate(FAccessAsset AccessDelegate)
	{
		AssetAccessDelegate = AccessDelegate;
	}

	/** @param AccessDelegate The delegate to pass along to the widget reflector */
	void SetWidgetReflectorDebugAccessDelegate(FAccessDebugObject AccessDelegate)
	{
		DebugObjectAccessDelegate = AccessDelegate;
	}

	/** @param Scale  Sets the ratio SlateUnit / ScreenPixel */
	void SetApplicationScale( float InScale ){ Scale = InScale; }

	virtual void GetInitialDisplayMetrics( FDisplayMetrics& OutDisplayMetrics ) const { PlatformApplication->GetInitialDisplayMetrics( OutDisplayMetrics ); }

	/** Are we drag-dropping right now? */
	SLATE_API bool IsDragDropping() const;

	/** Are we drag-dropping and are we affected by this pointer event? */
	SLATE_API bool IsDragDroppingAffected(const FPointerEvent& InPointerEvent) const;

	/** Get the current drag-dropping content */
	SLATE_API TSharedPtr<class FDragDropOperation> GetDragDroppingContent() const;

	/** Cancels any in flight drag and drops */
	SLATE_API void CancelDragDrop();

	/**
	 * Returns the attribute that can be used by widgets to check if the application is in normal execution mode
	 * Don't hold a reference to this anywhere that can exist when this application closes
	 */
	const TAttribute<bool>& GetNormalExecutionAttribute() const { return NormalExecutionGetter; }

	/**
	 * @return true if not in debugging mode
	 */
	bool IsNormalExecution() const { return !GIntraFrameDebuggingGameThread; }

	/**
	 * @return true if in debugging mode
	 */
	bool InKismetDebuggingMode() const { return GIntraFrameDebuggingGameThread; }

	/**
	 * Enters debugging mode which is a special state that causes the
	 * Slate application to tick in place which in the middle of a stack frame
	 */
	SLATE_API void EnterDebuggingMode();

	/**
	 * Leaves debugging mode
	 *
	 * @param bLeavingDebugForSingleStep	Whether or not we are leaving debug mode due to single stepping
	 */
	SLATE_API void LeaveDebuggingMode( bool bLeavingDebugForSingleStep = false );

#if WITH_EDITOR
	struct FScopedPreventDebuggingMode
	{
		UE_NODISCARD_CTOR SLATE_API FScopedPreventDebuggingMode(FText Reason);
		SLATE_API ~FScopedPreventDebuggingMode();
	private:
		int32 Id;
	};
#endif
	
	/**
	 * Calculates the popup window position from the passed in window position and size. 
	 * Adjusts position for popup windows which are outside of the work area of the monitor where they reside
	 *
	 * @param InAnchor				 The current(suggested) window position and size of an area which may not be covered by the popup.
	 * @param InSize				 The size of the window
	 * @param bAutoAdjustForDPIScale Whether the method should attempt to correct for the DPI scale of the monitor.
	 * @param InProposedPlacement	 The location on screen where the popup should go if allowed. If zero this will be determined from Orientation and Anchor
	 * @param Orientation			 The direction of the popup.
	 *								 If vertical it will attempt to open below the anchor but will open above if there is no room.
	 *								 If horizontal it will attempt to open below the anchor but will open above if there is no room.
	 * @param LayoutMode			 Determines whether the popup is allowed to flip to the other side of the anchor when there is no room on the screen.
	 * @return The adjusted position
	 */
	SLATE_API virtual UE::Slate::FDeprecateVector2DResult CalculatePopupWindowPosition( const FSlateRect& InAnchor, const UE::Slate::FDeprecateVector2DParameter& InSize, bool bAutoAdjustForDPIScale = true, const UE::Slate::FDeprecateVector2DParameter& InProposedPlacement = FVector2f::ZeroVector, const EOrientation Orientation = Orient_Vertical, const EPopupLayoutMode = EPopupLayoutMode::Menu) const;

	/**
	 * Calculates the tooltip window position.
	 *
	 * @param InAnchorRect The current(suggested) window position and size of an area which may not be covered by the popup.
	 * @param InSize The size of the tooltip window.
	 * @param bAutoAdjustForDPIScale Whether the method should attempt to correct for the DPI scale of the monitor.
	 * @param CursorOverlapMode Whether the tooltip window should be permitted to open under the position of the cursor.
	 * @return The suggested position.
	 */
	SLATE_API virtual UE::Slate::FDeprecateVector2DResult CalculateTooltipWindowPosition( const FSlateRect& InAnchorRect, const UE::Slate::FDeprecateVector2DParameter& InSize, bool bAutoAdjustForDPIScale, EPopupCursorOverlapMode CursorOverlapMode = EPopupCursorOverlapMode::PreventOverlap) const;

	/**
	 * Is the window in the app's destroy queue? If so it will be destroyed next tick.
	 *
	 * @param Window		The window to find in the destroy list
	 * @return				true if Window is in the destroy queue
	 */
	SLATE_API bool IsWindowInDestroyQueue(TSharedRef<SWindow> Window) const;

	/** @return	Returns true if the application's average frame rate is at least as high as our target frame rate that satisfies our requirement for a smooth and responsive UI experience */
	SLATE_API bool IsRunningAtTargetFrameRate() const;

	/** @return Returns true if transition effects for new menus and windows should be played */
	SLATE_API bool AreMenuAnimationsEnabled() const;

	/** 
	 * Sets whether transition effects for new menus and windows should be played.  This can be called at any time.
	 *
	 * @param	bEnableAnimations	True if animations should be used, otherwise false.
	 */
	UE_DEPRECATED(5.0, "Enable Window Animations is no longer used and is a no-op so calling this function is no longer necessary.")
	SLATE_API void EnableMenuAnimations( const bool bEnableAnimations );

	SLATE_API void SetPlatformApplication(const TSharedRef<class GenericApplication>& InPlatformApplication);

	/**
	 * Replace the current platform application with a custom version.
	 * @param InPlatformApplication - The replacement platform application.
	 */
	SLATE_API void OverridePlatformApplication(TSharedPtr<class GenericApplication> InPlatformApplication);

	/** Set the global application icon */
	UE_DEPRECATED(4.26, "SetAppIcon has been deprecated.  Set \"AppIcon\" in your applications style to override the icon")
	SLATE_API void SetAppIcon(const FSlateBrush* const InAppIcon);

	/** Sets the display state of external UI such as Steam. */
	void ExternalUIChange(bool bIsOpening)
	{
		bIsExternalUIOpened = bIsOpening;
	}

	/**
	 * Shows or hides an onscreen keyboard
	 *
	 * @param bShow	true to show the keyboard, false to hide it
	 * @param TextEntryWidget The widget that will receive the input from the virtual keyboard
	 */
	SLATE_API void ShowVirtualKeyboard( bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget = nullptr );

	/** @return true if the current platform allows cursor positioning in editable text boxes */
	SLATE_API bool AllowMoveCursor();

	/** Get the work area that has the largest intersection with the specified rectangle */
	SLATE_API FSlateRect GetWorkArea( const FSlateRect& InRect ) const;

	/**
	 * Shows or hides an onscreen keyboard
	 *
	 * @param bShow	true to show the keyboard, false to hide it
	 */
	virtual void NativeApp_ShowKeyboard( bool bShow, FString InitialString = "", int32 SelectionStart = -1, int32 SelectionEnd = -1 )
	{
		// empty default functionality
	}

	/** @return true if the current platform allows source code access */
	SLATE_API bool SupportsSourceAccess() const;

	/** Opens the current platform's code editing IDE (if necessary) and focuses the specified line in the specified file. Will only work if SupportsSourceAccess() returns true */
	SLATE_API void GotoLineInSource(const FString& FileName, int32 LineNumber) const;

	/** @return Service that supports popups; helps with auto-hiding of popups */
	FPopupSupport& GetPopupSupport() { return PopupSupport; }

	/**
	 * Forces the window to redraw immediately.
	 */
	SLATE_API void ForceRedrawWindow( const TSharedRef<SWindow>& InWindowToDraw );

	/**
	 * Takes a screenshot of the widget writing the results into the color buffer provided.  Note that the format is BGRA.
	 * the size of the resulting image is also output.
	 * 
	 * @return true if taking the screenshot was successful.
	 */
	SLATE_API bool TakeScreenshot(const TSharedRef<SWidget>& Widget, TArray<FColor>& OutColorData, FIntVector& OutSize);
	
	/**
	 * Takes a screenshot of the widget writing the results into the color buffer provided.  This is to be used with HDR buffers
	 * the size of the resulting image is also output.
	 *
	 * @return true if taking the screenshot was successful.
	 */
	SLATE_API bool TakeHDRScreenshot(const TSharedRef<SWidget>& Widget, TArray<FLinearColor>& OutColorData, FIntVector& OutSize);

	/**
	 * Takes a screenshot of the widget writing the results into the color buffer provided, this version allows you to provide 
	 * an inner area to screenshot.  Note that the format is BGRA.  The size of the resulting image is also output.
	 *
	 * @return true if taking the screenshot was successful.
	 */
	SLATE_API bool TakeScreenshot(const TSharedRef<SWidget>& Widget, const FIntRect& InnerWidgetArea, TArray<FColor>& OutColorData, FIntVector& OutSize);

	/**
	  * Takes a screenshot of the widget writing the results into the color buffer provided, this version allows you to provide
	  * an inner area to screenshot.  Note that the format is BGRA.  The size of the resulting image is also output.
	  *
	  * @return true if taking the screenshot was successful.
	*/
	SLATE_API bool TakeHDRScreenshot(const TSharedRef<SWidget>& Widget, const FIntRect& InnerWidgetArea, TArray<FLinearColor>& OutColorData, FIntVector& OutSize);

	/** Gets the user at the given index, null if the user does not exist. */
	inline TSharedPtr<const FSlateUser> GetUser(int32 UserIndex) const
	{
		return Users.IsValidIndex(UserIndex) ? Users[UserIndex] : nullptr;
	}
	inline TSharedPtr<FSlateUser> GetUser(int32 UserIndex)
	{
		return Users.IsValidIndex(UserIndex) ? Users[UserIndex] : nullptr;
	}
	SLATE_API TSharedPtr<FSlateUser> GetUser(FPlatformUserId PlatformUser);
	inline TSharedPtr<const FSlateUser> GetUser(const FInputEvent& InputEvent) const { return GetUser(InputEvent.GetUserIndex()); }
	inline TSharedPtr<FSlateUser> GetUser(const FInputEvent& InputEvent) { return GetUser(InputEvent.GetUserIndex()); }
	
	inline TSharedPtr<FSlateUser> GetUserFromControllerId(int32 ControllerId)
	{
		TOptional<int32> UserIndex = GetUserIndexForController(ControllerId);
		if (UserIndex.IsSet())
		{
			return GetUser(UserIndex.GetValue());
		}
		return nullptr;
	}
	inline TSharedPtr<const FSlateUser> GetUserFromControllerId(int32 ControllerId) const
	{
		TOptional<int32> UserIndex = GetUserIndexForController(ControllerId);
		if (UserIndex.IsSet())
		{
			return GetUser(UserIndex.GetValue());
		}
		return nullptr;
	}
	
	SLATE_API TSharedPtr<FSlateUser> GetUserFromPlatformUser(FPlatformUserId PlatformUser);
	
	SLATE_API TSharedPtr<const FSlateUser> GetUserFromPlatformUser(FPlatformUserId PlatformUser) const;

	/** Get the standard 'default' user (there's always guaranteed to be at least one). */
	inline TSharedPtr<const FSlateUser> GetCursorUser() const
	{
		TSharedPtr<const FSlateUser> SlateUser = GetUser(CursorUserIndex);
		check(SlateUser.IsValid());
		return SlateUser;
	}
	inline TSharedPtr<FSlateUser> GetCursorUser()
	{
		TSharedPtr<FSlateUser> SlateUser = GetUser(CursorUserIndex);
		check(SlateUser.IsValid());
		return SlateUser;
	}

	/**
	 * @return a handle for the existing or newly created virtual slate user.  This is handy when you need to create
	 * virtual hardware users for slate components in the virtual world that may need to be interacted with with virtual hardware.
	 */
	SLATE_API TSharedRef<FSlateVirtualUserHandle> FindOrCreateVirtualUser(int32 VirtualUserIndex);

	//@todo DanH: This is only getting called by the WidgetInteractionComponent. There's some piping to be set up here for this to be fully accurate for splitscreen
	SLATE_API void UnregisterUser(int32 UserIndex);

	/** Allows you do some operations for every registered user. */
	SLATE_API void ForEachUser(TFunctionRef<void(FSlateUser&)> InPredicate, bool bIncludeVirtualUsers = false);
	
	UE_DEPRECATED(4.23, "ForEachUser now provides an FSlateUser& parameter to the lambda instead of an FSlateUser*")
	SLATE_API void ForEachUser(TFunctionRef<void(FSlateUser*)> InPredicate, bool bIncludeVirtualUsers = false);

	/**
	 * Gets time step in seconds if a fixed delta time is wanted.
	 * @return Time step in seconds for fixed delta time.
	 */
	inline static double GetFixedDeltaTime()
	{
		return FixedDeltaTime;
	}

	/**
	 * Sets time step in seconds if a fixed delta time is wanted.
	 * @param seconds Time step in seconds for fixed delta time.
	 */
	static SLATE_API void SetFixedDeltaTime(double InSeconds);

protected:
	/**
	 * Register a new user with Slate.  Normally this is unnecessary as Slate automatically adds
	 * a user entry if it gets input from a controller for that index.  Might happen if the user
	 * allocates the virtual user.
	 */
	SLATE_API TSharedRef<FSlateUser> RegisterNewUser(int32 UserIndex, bool bIsVirtual = false);

	/**
	 * Register a new user with Slate.  Normally this is unnecessary as Slate automatically adds
	 * a user entry if it gets input from a controller for that index.  Might happen if the user
	 * allocates the virtual user.
	 */
	SLATE_API TSharedRef<FSlateUser> RegisterNewUser(FPlatformUserId PlatformUserId, bool bIsVirtual = false);

	/**
	 * Locates the SlateUser object corresponding to the index, creating a new one if it doesn't exist.
	 * Asserts if given an invalid (ie, negative) index.
	 */
	SLATE_API TSharedRef<FSlateUser> GetOrCreateUser(int32 UserIndex);
	
	/**
	 * Locates the SlateUser object corresponding to the index, creating a new one if it doesn't exist.
	 * Asserts if given an invalid (ie, negative) index.
	 */
	SLATE_API TSharedRef<FSlateUser> GetOrCreateUser(FPlatformUserId PlatformUserId);

	/**
	 * Locates the SlateUser object corresponding to the input device id, creating a new one if it doesn't exist.
	 * This will call GetOrCreateUser for the owning Platform User of the given input device.
	 * Asserts if given an invalid (ie, negative) index.
	 */
	SLATE_API TSharedRef<FSlateUser> GetOrCreateUser(FInputDeviceId DeviceId);
	
	inline TSharedRef<FSlateUser> GetOrCreateUser(const FInputEvent& InputEvent) { return GetOrCreateUser(InputEvent.GetUserIndex()); }

	friend class FEventRouter;

	/** Transforms a pointer event to account for non-standard viewport resolutions */
	SLATE_API FPointerEvent TransformPointerEvent(const FPointerEvent& PointerEvent, const TSharedPtr<SWindow>& Window) const;

	SLATE_API virtual bool DoesWidgetHaveMouseCaptureByUser(const TSharedPtr<const SWidget> Widget, int32 UserIndex, TOptional<int32> PointerIndex) const override;
	SLATE_API virtual bool DoesWidgetHaveMouseCapture(const TSharedPtr<const SWidget> Widget) const override;

	SLATE_API virtual TOptional<EFocusCause> HasUserFocus(const TSharedPtr<const SWidget> Widget, int32 UserIndex) const override;
	SLATE_API virtual TOptional<EFocusCause> HasAnyUserFocus(const TSharedPtr<const SWidget> Widget) const override;
	SLATE_API virtual bool IsWidgetDirectlyHovered(const TSharedPtr<const SWidget> Widget) const override;
	SLATE_API virtual bool ShowUserFocus(const TSharedPtr<const SWidget> Widget) const override;

	SLATE_API TSharedRef<FNavigationConfig> GetRelevantNavConfig(int32 UserIndex) const;

	/** Called when the slate application is being shut down. */
	SLATE_API void OnShutdown();

	SLATE_API void DestroyRenderer();

	/** Advances time for the application. */
	SLATE_API void TickTime();

	/**
	 * Pumps and ticks the platform.
	 */
	SLATE_API void TickPlatform(float DeltaTime);

	/**
	 * Ticks and paints the actual Slate portion of the application.
	 */
	SLATE_API void TickAndDrawWidgets(float DeltaTime);

	/** Draws Slate windows. Should only be called by the application's main loop or renderer. */
	SLATE_API void DrawWindows();

	/**
	 * Draws slate windows, optionally only drawing the passed in window
	 */
	SLATE_API void PrivateDrawWindows( TSharedPtr<SWindow> DrawOnlyThisWindow = nullptr );

	/**
	 * Pre-pass step before drawing windows to compute geometry size and reshape autosized windows
	 */
	SLATE_API void DrawPrepass( TSharedPtr<SWindow> DrawOnlyThisWindow );

	/**
	 * Draws a window and its children
	 */
	SLATE_API void DrawWindowAndChildren( const TSharedRef<SWindow>& WindowToDraw, struct FDrawWindowArgs& DrawWindowArgs );

	/**
	 * Gets all visible child windows of a window.
	 */
	SLATE_API void GetAllVisibleChildWindows(TArray< TSharedRef<SWindow> >& OutWindows, TSharedRef<SWindow> CurrentWindow);

	/** Engages or disengages application throttling based on user behavior */
	SLATE_API void ThrottleApplicationBasedOnMouseMovement();

	SLATE_API virtual FWidgetPath LocateWidgetInWindow(UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate, const TSharedRef<SWindow>& Window, bool bIgnoreEnabledStatus, int32 UserIndex) const override;

	/**
	 * Sets up any values that need to be based on the physical dimensions of the device.  
	 * Such as dead zones associated with precise tapping...etc
	 */
	SLATE_API void SetupPhysicalSensitivities();

public:

	/**
	 * Called by the native application in response to a mouse move. Routs the event to Slate Widgets.
	 *
	 * @param  InMouseEvent  Mouse event
	 * @param  bIsSynthetic  True when the even is synthesized by slate.
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessMouseMoveEvent( const FPointerEvent& MouseEvent, bool bIsSynthetic = false );

	/**
	 * Called by the native application in response to a mouse button press. Routs the event to Slate Widgets.
	 *
	 * @param  PlatformWindow  The platform window the event originated from, used to set focus at the platform level. 
	 *                         If Invalid the Mouse event will work but there will be no effect on the platform.
	 * @param  InMouseEvent    Mouse event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessMouseButtonDownEvent(const TSharedPtr< FGenericWindow >& PlatformWindow, const FPointerEvent& InMouseEvent);

	/**
	 * Called by the native application in response to a mouse button release. Routs the event to Slate Widgets.
	 *
	 * @param  InMouseEvent  Mouse event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessMouseButtonUpEvent( const FPointerEvent& MouseEvent );

	/**
	 * Called by the native application in response to a mouse release. Routs the event to Slate Widgets.
	 *
	 * @param  InMouseEvent  Mouse event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessMouseButtonDoubleClickEvent( const TSharedPtr< FGenericWindow >& PlatformWindow, const FPointerEvent& InMouseEvent );
	
	/**
	 * Called by the native application in response to a mouse wheel spin or a touch gesture. Routs the event to Slate Widgets.
	 *
	 * @param  InWheelEvent    Mouse wheel event details
	 * @param  InGestureEvent  Optional gesture event details
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessMouseWheelOrGestureEvent( const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent );

	/**
	 * Called when a character is entered
	 *
	 * @param  InCharacterEvent  Character event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessKeyCharEvent( const FCharacterEvent& InCharacterEvent );

	/**
	 * Called when a key is pressed
	 *
	 * @param  InKeyEvent  Keyb event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessKeyDownEvent( const FKeyEvent& InKeyEvent );

	/**
	 * Called when a key is released
	 *
	 * @param  InKeyEvent  Key event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessKeyUpEvent( const FKeyEvent& InKeyEvent );
	
	/**
	 * Called when a analog input values change
	 *
	 * @param  InAnalogInputEvent Analog input event
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessAnalogInputEvent(const FAnalogInputEvent& InAnalogInputEvent);

	/**
	 * Called when a drag from an external (non-slate) source enters a window
	 *
	 * @param WindowEntered  The window that was entered by the drag and drop
	 * @param DragDropEvent  Describes the mouse state (position, pressed buttons, etc) and associated payload
	 * @return true if the drag enter was handled and can be processed by some widget in this window; false otherwise
	 */
	SLATE_API bool ProcessDragEnterEvent( TSharedRef<SWindow> WindowEntered, const FDragDropEvent& DragDropEvent );

	/**
	 * Called when a touchpad touch is started (finger down) when polling game device state
	 * 
	 * @param ControllerEvent	The touch event generated
	 */
	SLATE_API void ProcessTouchStartedEvent( const TSharedPtr< FGenericWindow >& PlatformWindow, const FPointerEvent& InTouchEvent );

	/**
	 * Called when a touchpad touch is moved  (finger moved) when polling game device state
	 * 
	 * @param ControllerEvent	The touch event generated
	 */
	SLATE_API void ProcessTouchMovedEvent( const FPointerEvent& InTouchEvent );

	/**
	 * Called when a touchpad touch is ended (finger lifted) when polling game device state
	 * 
	 * @param ControllerEvent	The touch event generated
	 */
	SLATE_API void ProcessTouchEndedEvent( const FPointerEvent& InTouchEvent );

	/**
	 * Called when motion is detected (controller or device) when polling game device state
	 * 
	 * @param MotionEvent		The motion event generated
	 */
	SLATE_API void ProcessMotionDetectedEvent( const FMotionEvent& InMotionEvent );

	/**
	 * Called by the native application in response to an activation or deactivation. 
	 *
	 * @param ActivateEvent Information about the window activation/deactivation
	 * @return  Was this event handled by the Slate application?
	 */
	SLATE_API bool ProcessWindowActivatedEvent( const FWindowActivateEvent& ActivateEvent );
	
	/**
	 * Called when the application is activated (i.e. one of its windows becomes active) or deactivated.
	 *
	 * @param InAppActivated Whether the application was activated.
	 */
	SLATE_API void ProcessApplicationActivationEvent( bool InAppActivated );

	/**
	 * Returns true if the we're currently processing mouse, keyboard, touch or gamepad input.
	 */
	bool IsProcessingInput() const { return ProcessingInput > 0; }

public:

	TSharedRef<FNavigationConfig> GetNavigationConfig() const { return NavigationConfig; }

	/**
	 * Sets the navigation config.  If you need to control navigation config dynamically, you
	 * should subclass FNavigationConfig to be dynamically adjustable to your needs.
	 */
	SLATE_API void SetNavigationConfig(TSharedRef<FNavigationConfig> InNavigationConfig);

	/**
	 * Sets the navigation config factory.  If you need to control navigation config dynamically, you
	 * should subclass FNavigationConfig to be dynamically adjustable to your needs.
	*/
	UE_DEPRECATED(4.20, "Returning to a simpler method of registering navigation configs.\nSetNavigationConfig, is what you should use now.  Note: You'll need to store per user state information yourself if you have any, like we do for repeats with the analog stick in FNavigationConfig::UserNavigationState,\nrather than Slate creating a new Navigation Config per user.")
	void SetNavigationConfigFactory(TFunction<TSharedRef<FNavigationConfig>()> InNavigationConfigFactory) { }

public:

	/** Closes all active windows immediately */
	SLATE_API void CloseAllWindowsImmediately();

	/**
	 * Destroy the native and slate windows in the array provided.
	 *
	 * @param WindowsToDestroy   Destroy these windows
	 */
	SLATE_API void DestroyWindowsImmediately();

	/**
	 * Apply any requests from the Reply to the application. E.g. Capture mouse
	 *
	 * @param CurrentEventPath   The WidgetPath along which the reply-generated event was routed
	 * @param TheReply           The reply generated by an event that was being processed.
	 * @param UserIndex			 User index that generated the event we are replying to (defaults to 0, at least for now)
	 * @param PointerIndex		 Pointer index that generated the event we are replying to
	 */
	SLATE_API void ProcessExternalReply(const FWidgetPath& CurrentEventPath, const FReply TheReply, const int32 UserIndex = 0, const int32 PointerIndex = 10 /* todo: use the enum */);

	/**
	 * Apply any requests from the Reply to the application. E.g. Capture mouse
	 *
	 * @param CurrentEventPath   The WidgetPath along which the reply-generated event was routed
	 * @param TheReply           The reply generated by an event that was being processed.
	 * @param WidgetsUnderMouse  Optional widgets currently under the mouse; initiating drag and drop needs access to widgets under the mouse.
	 * @param InMouseEvent       Optional mouse event that caused this action.
	 * @param UserIndex			 User index that generated the event we are replying to (defaults to 0, at least for now)
	 */
	SLATE_API void ProcessReply(const FWidgetPath& CurrentEventPath, const FReply& TheReply, const FWidgetPath* WidgetsUnderMouse, const FPointerEvent* InMouseEvent, const uint32 UserIndex = 0);
	
	/** Bubble a request for which cursor to display for widgets under the mouse or the widget that captured the mouse. */
	SLATE_API void QueryCursor();

	/**
	 * Apply any requests from the CursorReply
	 *
	 * @param CursorReply        The reply generated by an event that was being processed.
	 */
	SLATE_API void ProcessCursorReply(const FCursorReply& CursorReply);
	
	/**
	 * Spawns a tool tip window.  If an existing tool tip window is open, it will be dismissed first.
	 *
	 * @param	InToolTip           Widget to display.
	 * @param	InSpawnLocation     Screen space location to show the tool tip (window top left)
	 */
	SLATE_API void SpawnToolTip( const TSharedRef<IToolTip>& InToolTip, const UE::Slate::FDeprecateVector2DParameter& InSpawnLocation );

	/** Closes the open tool-tip, if a tool-tip is open */
	SLATE_API void CloseToolTip();

	SLATE_API void UpdateToolTip( bool bAllowSpawningOfNewToolTips );

	/** @return an array of top-level windows that can be interacted with. e.g. when a modal window is up, only return the modal window */
	SLATE_API TArray< TSharedRef<SWindow> > GetInteractiveTopLevelWindows();

	/** Gets all visible slate windows ordered from back to front based on child hierarchies */
	SLATE_API void GetAllVisibleWindowsOrdered(TArray< TSharedRef<SWindow> >& OutWindows);
	
	/** @return true if mouse events are being turned into touch events, and touch UI should be forced on */
	SLATE_API bool IsFakingTouchEvents() const;

	/** Sets whether the application is treating mouse events as imitating touch events.  Optional CursorLocation can be supplied to override the platform's belief of where the cursor is */
	SLATE_API void SetGameIsFakingTouchEvents(const bool bIsFaking, FVector2D* CursorLocation = nullptr);

	/** Sets whether the application is allowed to fake touches or not */
	SLATE_API void SetGameAllowsFakingTouchEvents(const bool bAllowFaking);

	/** Sets the handler for otherwise unhandled key down events. This is used by the editor to provide a global action list, if the key was not consumed by any widget. */
	SLATE_API void SetUnhandledKeyDownEventHandler( const FOnKeyEvent& NewHandler );

	/** Sets the handler for otherwise unhandled key down events. This is used by the editor to provide a global action list, if the key was not consumed by any widget. */
	SLATE_API void SetUnhandledKeyUpEventHandler(const FOnKeyEvent& NewHandler);

	/** @return the last time a user interacted with a keyboard, mouse, touch device, or controller */
	double GetLastUserInteractionTime() const { return LastUserInteractionTime; }

	DECLARE_EVENT_OneParam(FSlateApplication, FSlateLastUserInteractionTimeUpdateEvent, double);
	/** @return Gets the event for LasterUserInteractionTime update */
	FSlateLastUserInteractionTimeUpdateEvent& GetLastUserInteractionTimeUpdateEvent() { return LastUserInteractionTimeUpdateEvent; }

	/** @return the deadzone size for dragging in screen pixels (aka virtual desktop pixels) */
	SLATE_API float GetDragTriggerDistance() const;

	/** @return the deadzone size squared for dragging in screen pixels (aka virtual desktop pixels) */
	SLATE_API float GetDragTriggerDistanceSquared() const;
	
	/** @return Whether we disable the last drag on a new drag enter event to prevent double drag conflicts. */
	SLATE_API bool IsLastDragOnDragEnterDisabled() const;

	/** @return true if the difference between the ScreenSpaceOrigin and the ScreenSpacePosition is larger than the trigger distance for dragging in Slate. */
	SLATE_API bool HasTraveledFarEnoughToTriggerDrag(const FPointerEvent& PointerEvent, const UE::Slate::FDeprecateVector2DParameter ScreenSpaceOrigin) const;
	SLATE_API bool HasTraveledFarEnoughToTriggerDrag(const FPointerEvent& PointerEvent, const UE::Slate::FDeprecateVector2DParameter ScreenSpaceOrigin, EOrientation Orientation) const;

	/** Set the size of the deadzone for dragging in screen pixels */
	SLATE_API void SetDragTriggerDistance( float ScreenPixels );
	
	/** Toggle disabling last drag on a new drag enter event to prevent double drag conflicts. */
	SLATE_API void ToggleDisableLastDragOnDragEnter( bool bInDisableLastDragOnDragEnter );


	/**
	 * Adds input pre-processor if unique.
	 * @param InputProcessor	The input pre-processor to add. Defaults to using the EInputPreProcessorType::Game bucket type and adding at the end of the bucket.
	 * @return True if added to list of input pre-processors, false if not
	 */
	SLATE_API bool RegisterInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor);

	/** 
	 * Adds input pre-processor if unique. 
	 * @param InputProcessor	The input pre-processor to add.
	 * @param Index				Where to insert the InputProcessor within the type bucket. Defaults to using the EInputPreProcessorType::Game type.
	 * @return True if added to list of input pre-processors, false if not
	 */
	SLATE_API bool RegisterInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor, const int32 Index);

	/**
	 * Adds input pre-processor if unique.
	 * @param InputProcessor	The input pre-processor to add.
	 * @param Type				Which bucket to insert the processor in. Defaults to adding at the end of the bucket.
	 * @return True if added to list of input pre-processors, false if not
	 */
	SLATE_API bool RegisterInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor, const EInputPreProcessorType Type);

	/**
	 * Adds input pre-processor if unique.
	 * @param InputProcessor	The input pre-processor to add.
	 * @param Info				Information specifying which priority type and index should be used to register the pre-processor.
	 * @return True if added to list of input pre-processors, false if not
	 */
	SLATE_API bool RegisterInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor, const FInputPreprocessorRegistrationKey& Info);

	/**
	 * Removes an input pre-processor.
	 * @param InputProcessor	The input pre-processor to Remove.
	 */
	SLATE_API void UnregisterInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor);

	/**
	 * Get the index of a registered pre-processor.
	 * @param InputProcessor	The input pre-processor to find.
	 * @return The index of the pre-processor, or INDEX_NONE if not registered.
	 */
	UE_DEPRECATED(5.5, "This method is deprecated and will use EInputPreProcessorType::Game as a default. For a more accurate search result, please use the new version taking a EInputPreProcessorType")
	SLATE_API int32 FindInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor) const;

	/**
	 * Get the index of a registered pre-processor.
	 * @param InputProcessor	The input pre-processor to find.
	 * @param Type				The priority bracket in which the pre-processor should be registered
	 * @return The index of the pre-processor, or INDEX_NONE if not registered.
	 */
	SLATE_API int32 FindInputPreProcessor(TSharedPtr<class IInputProcessor> InputProcessor, const EInputPreProcessorType& Type) const;

	/** Sets the hit detection radius of the cursor */
	SLATE_API void SetCursorRadius(float NewRadius);

	/** Getter for the cursor radius */
	SLATE_API float GetCursorRadius() const;

	SLATE_API void SetAllowTooltips(bool bCanShow);
	SLATE_API bool GetAllowTooltips() const;
	
	/** Setter and getter for the touch simulation */
	SLATE_API void SetIsFakeTouch(bool bInIsFakingTouch);
	SLATE_API bool GetIsFakeTouch() const;

	bool IsRenderingOffScreen() const { return bRenderOffScreen; }

public:

	//~ Begin FSlateApplicationBase Interface

	virtual bool IsActive() const override
	{
		return bAppIsActive;
	}

	SLATE_API virtual TSharedRef<SWindow> AddWindow( TSharedRef<SWindow> InSlateWindow, const bool bShowImmediately = true ) override;

	virtual void ArrangeWindowToFrontVirtual( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& WindowToBringToFront ) override
	{
		FSlateWindowHelper::ArrangeWindowToFront(Windows, WindowToBringToFront);
	}

	virtual bool FindPathToWidget( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter = EVisibility::Visible ) override
	{
		if ( !FSlateWindowHelper::FindPathToWidget(GetInteractiveTopLevelWindows(), InWidget, OutWidgetPath, VisibilityFilter) )
		{
			return FSlateWindowHelper::FindPathToWidget(SlateVirtualWindows, InWidget, OutWidgetPath, VisibilityFilter);
		}

		return true;
	}

	virtual const double GetCurrentTime() const override
	{
		return CurrentTime;
	}

	SLATE_API virtual TSharedPtr<SWindow> GetActiveTopLevelWindow() const override;

	SLATE_API virtual TSharedPtr<SWindow> GetActiveTopLevelRegularWindow() const override;

	SLATE_API virtual const FSlateBrush* GetAppIcon() const override;
	SLATE_API virtual const FSlateBrush* GetAppIconSmall() const override;

	virtual float GetApplicationScale() const override { return Scale; }
	virtual bool GetSoftwareCursorAvailable() const override { return bSoftwareCursorAvailable; }
	SLATE_API virtual EVisibility GetSoftwareCursorVis() const override;

	SLATE_API virtual UE::Slate::FDeprecateVector2DResult GetCursorPos() const override;
	SLATE_API virtual UE::Slate::FDeprecateVector2DResult GetLastCursorPos() const override;
	SLATE_API virtual UE::Slate::FDeprecateVector2DResult GetCursorSize() const override;

	SLATE_API virtual TSharedPtr<SWidget> GetKeyboardFocusedWidget() const override;

	virtual EWindowTransparency GetWindowTransparencySupport() const override
	{
		return PlatformApplication->GetWindowTransparencySupport();
	}

protected:

	SLATE_API virtual TSharedPtr< SWidget > GetMouseCaptorImpl() const override;

public:

	//~ Begin FSlateApplicationBase Interface

	SLATE_API virtual bool HasAnyMouseCaptor() const override;
	SLATE_API virtual bool HasUserMouseCapture(int32 UserIndex) const override;
	SLATE_API virtual FSlateRect GetPreferredWorkArea() const override;
	SLATE_API virtual bool HasFocusedDescendants( const TSharedRef<const SWidget>& Widget ) const override;
	SLATE_API virtual bool HasUserFocusedDescendants(const TSharedRef< const SWidget >& Widget, int32 UserIndex) const override;
	SLATE_API virtual bool IsExternalUIOpened() override;
	SLATE_API virtual FWidgetPath LocateWindowUnderMouse( UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate, const TArray<TSharedRef<SWindow>>& Windows, bool bIgnoreEnabledStatus = false, int32 UserIndex = INDEX_NONE) override;
	SLATE_API virtual bool IsWindowHousingInteractiveTooltip(const TSharedRef<const SWindow>& WindowToTest) const override;
	SLATE_API virtual TSharedRef<SImage> MakeImage( const TAttribute<const FSlateBrush*>& Image, const TAttribute<FSlateColor>& Color, const TAttribute<EVisibility>& Visibility ) const override;
	SLATE_API virtual TSharedRef<SWidget> MakeWindowTitleBar(const FWindowTitleBarArgs& InArgs, TSharedPtr<IWindowTitleBar>& OutTitleBar) const override;
	SLATE_API virtual TSharedRef<IToolTip> MakeToolTip( const TAttribute<FText>& ToolTipText ) override;
	SLATE_API virtual TSharedRef<IToolTip> MakeToolTip( const FText& ToolTipText ) override;
	SLATE_API virtual void RequestDestroyWindow( TSharedRef<SWindow> WindowToDestroy ) override;
	SLATE_API virtual bool SetKeyboardFocus( const FWidgetPath& InFocusPath, const EFocusCause InCause ) override;
	SLATE_API virtual bool SetUserFocus(const uint32 InUserIndex, const FWidgetPath& InFocusPath, const EFocusCause InCause) override;
	SLATE_API virtual bool SetUserFocusAllowingDescendantFocus(const uint32 InUserIndex, const FWidgetPath& InFocusPath, const EFocusCause InCause) override;
	SLATE_API virtual void SetAllUserFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause) override;
	SLATE_API virtual void SetAllUserFocusAllowingDescendantFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause) override;
	SLATE_API virtual TSharedPtr<SWidget> GetUserFocusedWidget(uint32 UserIndex) const override;
	SLATE_API virtual TSharedPtr<SWidget> GetCurrentDebugContextWidget() const override;
	virtual const TArray<TSharedRef<SWindow>> GetTopLevelWindows() const override { return SlateWindows; }

	DECLARE_EVENT_OneParam(FSlateApplication, FApplicationActivationStateChangedEvent, const bool /*IsActive*/)
	virtual FApplicationActivationStateChangedEvent& OnApplicationActivationStateChanged() { return ApplicationActivationStateChangedEvent; }

public:

	//~ Begin FGenericApplicationMessageHandler Interface

	SLATE_API virtual bool ShouldProcessUserInputMessages( const TSharedPtr< FGenericWindow >& PlatformWindow ) const override;
	SLATE_API virtual bool OnKeyChar( const TCHAR Character, const bool IsRepeat ) override;
	SLATE_API virtual bool OnKeyDown( const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat ) override;
	SLATE_API virtual bool OnKeyUp( const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat ) override;
	SLATE_API virtual void OnInputLanguageChanged() override;
	SLATE_API virtual bool OnMouseDown( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button ) override;
	SLATE_API virtual bool OnMouseDown( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button, const FVector2D CursorPos ) override;
	SLATE_API virtual bool OnMouseUp( const EMouseButtons::Type Button ) override;
	SLATE_API virtual bool OnMouseUp( const EMouseButtons::Type Button, const FVector2D CursorPos ) override;
	SLATE_API virtual bool OnMouseDoubleClick( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button ) override;
	SLATE_API virtual bool OnMouseDoubleClick( const TSharedPtr< FGenericWindow >& PlatformWindow, const EMouseButtons::Type Button, const FVector2D CursorPos ) override;
	SLATE_API virtual bool OnMouseWheel( const float Delta ) override;
	SLATE_API virtual bool OnMouseWheel( const float Delta, const FVector2D CursorPos ) override;
	SLATE_API virtual bool OnMouseMove() override;
	SLATE_API virtual bool OnRawMouseMove( const int32 X, const int32 Y ) override;
	SLATE_API virtual bool OnCursorSet() override;
	SLATE_API virtual bool OnTouchGesture( EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice ) override;
	SLATE_API virtual bool OnNativeGestureBegin( EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId ) override;
	SLATE_API virtual bool OnNativeGestureUpdate( EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId ) override;
	SLATE_API virtual bool OnNativeGestureEnd( EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId ) override;
	SLATE_API virtual bool OnTouchStarted( const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId ) override;
	SLATE_API virtual bool OnTouchMoved( const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID ) override;
	SLATE_API virtual bool OnTouchEnded( const FVector2D& Location, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID ) override;
	SLATE_API virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID) override;
	SLATE_API virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID) override;
	SLATE_API virtual void ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable) override;
	SLATE_API virtual bool OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId) override;
	SLATE_API virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue) override;
	SLATE_API virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	SLATE_API virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	SLATE_API virtual bool OnSizeChanged( const TSharedRef< FGenericWindow >& PlatformWindow, const int32 Width, const int32 Height, bool bWasMinimized = false ) override;
	SLATE_API virtual void OnOSPaint( const TSharedRef< FGenericWindow >& PlatformWindow ) override;
	SLATE_API virtual FWindowSizeLimits GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const override;
	SLATE_API virtual void OnResizingWindow( const TSharedRef< FGenericWindow >& PlatformWindow ) override;
	SLATE_API virtual bool BeginReshapingWindow( const TSharedRef< FGenericWindow >& PlatformWindow ) override;
	SLATE_API virtual void FinishedReshapingWindow( const TSharedRef< FGenericWindow >& PlatformWindow ) override;
	SLATE_API virtual void SignalSystemDPIChanged(const TSharedRef<FGenericWindow>& Window) override;
	SLATE_API virtual void HandleDPIScaleChanged(const TSharedRef<FGenericWindow>& Window) override;
	SLATE_API virtual void OnRegularWindowMovedOrResized(const TSharedRef<FGenericWindow>& Window) override;
	SLATE_API virtual void OnMovedWindow( const TSharedRef< FGenericWindow >& PlatformWindow, const int32 X, const int32 Y ) override;
	SLATE_API virtual bool OnWindowActivationChanged( const TSharedRef< FGenericWindow >& PlatformWindow, const EWindowActivation ActivationType ) override;
	SLATE_API virtual bool OnApplicationActivationChanged( const bool IsActive ) override;
	SLATE_API virtual bool OnConvertibleLaptopModeChanged() override;
	SLATE_API virtual EWindowZone::Type GetWindowZoneForPoint( const TSharedRef< FGenericWindow >& PlatformWindow, const int32 X, const int32 Y ) override;
	SLATE_API virtual void OnWindowClose( const TSharedRef< FGenericWindow >& PlatformWindow ) override;
	SLATE_API virtual EDropEffect::Type OnDragEnterText( const TSharedRef< FGenericWindow >& Window, const FString& Text ) override;
	SLATE_API virtual EDropEffect::Type OnDragEnterFiles( const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files ) override;
	SLATE_API virtual EDropEffect::Type OnDragEnterExternal( const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files ) override;

	SLATE_API EDropEffect::Type OnDragEnter( const TSharedRef< SWindow >& Window, const TSharedRef<FExternalDragOperation>& DragDropOperation );

	SLATE_API virtual EDropEffect::Type OnDragOver( const TSharedPtr< FGenericWindow >& Window ) override;
	SLATE_API virtual void OnDragLeave( const TSharedPtr< FGenericWindow >& Window ) override;
	SLATE_API virtual EDropEffect::Type OnDragDrop( const TSharedPtr< FGenericWindow >& Window ) override;
	SLATE_API virtual bool OnWindowAction( const TSharedRef< FGenericWindow >& PlatformWindow, const EWindowAction::Type InActionType ) override;

public:

	/**
	 * Directly routes a pointer down event to the widgets in the specified widget path
	 *
	 * @param WidgetsUnderPointer	The path of widgets the event is routed to.
	 * @param PointerEvent		The event data that is is routed to the widget path
	 * 
	 * @return The reply returned by the widget that handled the event
	 */
	SLATE_API FReply RoutePointerDownEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent);

	/**
	 * Directly routes a pointer up event to the widgets in the specified widget path
	 *
	 * @param WidgetsUnderPointer	The path of widgets the event is routed to.
	 * @param PointerEvent		The event data that is is routed to the widget path
	 *
	 * @return The reply from the event
	 */
	SLATE_API FReply RoutePointerUpEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent);

	/**
	 * Directly routes a pointer move event to the widgets in the specified widget path
	 *
	 * @param WidgetsUnderPointer	The path of widgets the event is routed to.
	 * @param PointerEvent		The event data that is is routed to the widget path
	 * @param bIsSynthetic		Whether or not the move event is synthetic.  Synthetic pointer moves used simulate an event without the pointer actually moving 
	 */
	SLATE_API bool RoutePointerMoveEvent( const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent, bool bIsSynthetic );

	/**
	 * Directly routes a pointer double click event to the widgets in the specified widget path
	 *
	 * @param WidgetsUnderPointer	The path of widgets the event is routed to.
	 * @param PointerEvent		The event data that is is routed to the widget path
	 */
	SLATE_API FReply RoutePointerDoubleClickEvent( const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& PointerEvent );

	/**
	 * Directly routes a pointer mouse wheel or gesture event to the widgets in the specified widget path.
	 * 
	 * @param WidgetsUnderPointer	The path of widgets the event is routed to.
	 * @param InWheelEvent			The event data that is is routed to the widget path
	 * @param InGestureEvent		The event data that is is routed to the widget path
	 */
	SLATE_API FReply RouteMouseWheelOrGestureEvent(const FWidgetPath& WidgetsUnderPointer, const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent = nullptr);

	/**
	 * @return int user index that the mouse is mapped to.
	 */
	SLATE_API int32 GetUserIndexForMouse() const;

	/**
	 * @return int user index that the keyboard is mapped to.
	 */
	SLATE_API int32 GetUserIndexForKeyboard() const;

	/**
	 * @return InputDeviceId that the mouse is mapped to
	 */
	SLATE_API FInputDeviceId GetInputDeviceIdForMouse() const;

	/**
	 * @return InputDeviceId that the keyboard is mapped to
	 */
	SLATE_API FInputDeviceId GetInputDeviceIdForKeyboard() const;

	/** @return int user index that this controller is mapped to. */
	SLATE_API int32 GetUserIndexForController(int32 ControllerId) const;

	/** @return Gets the slate user index that this input device is mapped to */	
	SLATE_API TOptional<int32> GetUserIndexForInputDevice(FInputDeviceId InputDeviceId) const;
	/** @return Gets the slate user index that this platform user id mapped to*/	
	SLATE_API TOptional<int32> GetUserIndexForPlatformUser(FPlatformUserId PlatformUser) const;
	
	SLATE_API TOptional<int32> GetUserIndexForController(int32 ControllerId, FKey InKey) const;

	/** Establishes the input mapping object used to map input sources to SlateUser indices */
	SLATE_API void SetInputManager(TSharedRef<ISlateInputManager> InputManager);

	/**
	 * Register for a notification when the window action occurs.
	 *
	 * @param Notification          The notification to invoke.
	 *
	 * @return Handle to the registered delegate.
	 */
	SLATE_API FDelegateHandle RegisterOnWindowActionNotification(const FOnWindowAction& Notification);

	/** Event type for when Slate is ticking during a modal dialog loop */
	DECLARE_EVENT_OneParam(FSlateApplication, FOnModalLoopTickEvent, float);

	/**
	 * Get the FOnModalLoopTickEvent for the Slate Application. Allows clients to register for callbacks during modal dialog loops.
	 *
	 * @return The application's FOnModalLoopTickEvent.
	 */
	FOnModalLoopTickEvent& GetOnModalLoopTickEvent() { return ModalLoopTickEvent; }

	/**
	* Unregister the notification because it is no longer desired.

	* @param Handle Handle to the delegate to unregister.
	*/
	SLATE_API void UnregisterOnWindowActionNotification(FDelegateHandle Handle);

	/**
	 * Attempts to navigate directly to the given widget
	 *
	 * @param InUserIndex The user that is doing the navigation
	 * @param NavigationDestination The navigation destination widget
	 * @param NavigationSource The source type of the navigation
	 * @param NavigationType The type of the navigation transition.
	 */
	SLATE_API void NavigateToWidget(const uint32 UserIndex, const TSharedPtr<SWidget>& NavigationDestination, ENavigationSource NavigationSource = ENavigationSource::FocusedWidget, EUINavigation NavigationType = EUINavigation::Invalid);

	/**
	 * Attempts to navigate directly to the widget currently under the given user's cursor
	 *
	 * @param InUserIndex The user that is doing the navigation
	 * @param InNavigationType The navigation type / direction
	 * @param InWindow The window to do the navigation within
	 */
	SLATE_API void NavigateFromWidgetUnderCursor(const uint32 InUserIndex, EUINavigation InNavigationType, TSharedRef<SWindow> InWindow);

	/**
	 * Attempts to navigate to a neighboring widget by searching the given directions
	 *
	 * @param InUserIndex The user that is doing the navigation
	 * @param InNavigationSource The source widget to navigate from. If null, will use current user focus
	 * @param InNavigationTypes The navigation requests to attempt, in order

	 * @return If succesful, returns the direction navigated. Otherwise return EUINavigation::Invalid
	 */
	SLATE_API EUINavigation NavigateFromWidget(const uint32 InUserIndex, TSharedPtr<SWidget> InNavigationSource, const TArray<EUINavigation>& InNavigationTypes);

	/**
	* Given an optional widget, try and get the most suitable parent window to use with dialogs (such as file and directory pickers).
	* This will first try and get the window that owns the widget (if provided), before falling back to using the MainFrame window.
	*/
	SLATE_API TSharedPtr<SWindow> FindBestParentWindowForDialogs(const TSharedPtr<SWidget>& InWidget, const ESlateParentWindowSearchMethod InParentWindowSearchMethod = ESlateParentWindowSearchMethod::ActiveWindow);

	/**
	* Given an optional widget, try and get the most suitable parent window handle to use with dialogs (such as file and directory pickers).
	* This will first try and get the window that owns the widget (if provided), before falling back to using the MainFrame window.
	*/
	SLATE_API const void* FindBestParentWindowHandleForDialogs(const TSharedPtr<SWidget>& InWidget, const ESlateParentWindowSearchMethod InParentWindowSearchMethod = ESlateParentWindowSearchMethod::ActiveWindow);

	/**
	* @return Returns the current Id of the application drawing phase. The value returned can be used for == and != operations only.
	*/
	SLATE_API uint32 GetCurrentDrawId() const;

public:
#if WITH_EDITORONLY_DATA
	FDragDropCheckingOverride OnDragDropCheckOverride;
#endif

	SLATE_API const TSet<FKey>& GetPressedMouseButtons() const;

private:

	SLATE_API TSharedRef< FGenericWindow > MakeWindow( TSharedRef<SWindow> InSlateWindow, const bool bShowImmediately );

	/**
	 * Destroys an SWindow, removing it and all its children from the Slate window list.  Notifies the native window to destroy itself and releases rendering resources
	 *
	 * @param DestroyedWindow The window to destroy
	 */
	SLATE_API void PrivateDestroyWindow( const TSharedRef<SWindow>& DestroyedWindow );

	/**
	 * Attempts to navigate to the next widget in the direction specified
	 *
	 * @return if a new widget was navigated too
	 */
	SLATE_API bool AttemptNavigation(const FWidgetPath& NavigationSource, const FNavigationEvent& NavigationEvent, const FNavigationReply& NavigationReply, const FArrangedWidget& BoundaryWidget);

	/**
	 * Executes a navigate to the specified widget if possible
	 *
	 * @return if the widget was navigated too
	 */
	SLATE_API bool ExecuteNavigation(const FWidgetPath& NavigationSource, TSharedPtr<SWidget> DestinationWidget, const uint32 UserIndex, bool bAlwaysHandleNavigationAttempt);


	/* Event receiver for when a menu in the menu stack is destroyed */
	SLATE_API void OnMenuDestroyed(const TSharedRef<IMenu>& Menu);

public:	
	struct FNavigationResult
	{
		TSharedPtr<SWidget> DestinationWidget;
		bool bAlwaysHandleNavigationAttempt = false;
#if WITH_SLATE_DEBUGGING
		ESlateDebuggingNavigationMethod NavigationMethod = ESlateDebuggingNavigationMethod::Unknown;
#endif
	};

	SLATE_API FNavigationResult CalculateDestinationWidget(const FWidgetPath& NavigationSource, const FNavigationReply& NavigationReply, EUINavigation NavigationType, uint32 UserIndex, const FArrangedWidget& BoundaryWidget) const;

private:
	SLATE_API FSlateApplication();
	SLATE_API void SetLastUserInteractionTime(const double InCurrentTime);	
	SLATE_API bool SetUserFocus(FSlateUser& User, const FWidgetPath& InFocusPath, const EFocusCause InCause);

private:
	/**
	 * Will be invoked when the size of the geometry of the virtual
	 * desktop changes (e.g. resolution change or monitors re-arranged)
	 */
	SLATE_API void OnVirtualDesktopSizeChanged(const FDisplayMetrics& NewDisplayMetric);
	
	SLATE_API bool HandleTouchGesture(FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D &Delta, const float Value, bool bIsDirectionInvertedFromDevice);

	/** Application singleton */
	static SLATE_API TSharedPtr< FSlateApplication > CurrentApplication;

	TSet<FKey> PressedMouseButtons;

	/** true when the slate app is active; i.e. the current foreground window is from our Slate app*/
	bool bAppIsActive;

	/** true if any slate window is currently active (not just top level windows) */
	bool bSlateWindowActive;

	/** true if rendering windows even when they are set to invisible */
	bool bRenderOffScreen;

	/** Application-wide scale for supporting monitors of varying pixel density */
	float Scale;

	/** The dead zone distance in virtual desktop pixels (a.k.a screen pixels) that the user has to move their finder before it is considered a drag.*/
	float DragTriggerDistance;

	/** All the top-level windows owned by this application; they are tracked here in a platform-agnostic way. */
	TArray< TSharedRef<SWindow> > SlateWindows;

	/** All the virtual windows, which can be anywhere - likely inside the virtual world. */
	TArray< TSharedRef<SWindow> > SlateVirtualWindows;

	/** The currently active slate window that is a top-level window (full fledged window; not a menu or tooltip)*/
	TWeakPtr<SWindow> ActiveTopLevelWindow;
	
	/** List of active modal windows.  The last item in the list is the top-most modal window */
	TArray< TSharedPtr<SWindow> > ActiveModalWindows;

	/** These windows will be destroyed next tick. */
	TArray< TSharedRef<SWindow> > WindowDestroyQueue;

	/** The stack of menus that are open */
	FMenuStack MenuStack;

	/** The hit-test radius of the cursor. Default value is 0. */
	float CursorRadius;

	/** The current DrawId. Value changes whenever we do a new pass*/
	uint32 DrawId;

	/**
	 * All users currently registered with Slate.
	 * Normally there's just one, but any case where multiple users can provide input to the same application will register multiple users.
	 * 
	 * Note: The array may contain invalid entries. Users have associated indices that they expect to maintain, independent of the existence of other users.
	 */
	TArray<TSharedPtr<FSlateUser>> Users;

	/** Weak pointers to the allocated virtual users. */
	TArray<TWeakPtr<FSlateVirtualUserHandle>> VirtualUsers;

	/** Per-user navigation context payloads. Written during OnNavigation, read during OnFocusReceived.
	 * Cleared automatically at the end of each navigation event by FScopedNavigationTransition.
	 */
	TMap<uint32, TSharedPtr<ISlateMetaData>> PendingNavigationContexts;

	/** Last widget that was set for 'all users' focus and the cause. */
	TWeakPtr<SWidget> LastAllUsersFocusWidget;
	EFocusCause LastAllUsersFocusCause;

	/** The painting SWindow. */
	TWeakPtr<SWidget> CurrentDebugContextWidget;
	TWeakPtr<SWindow> CurrentDebuggingWindow;

	/**
	 * Application throttling
	 */

	/** Holds a current request to ensure Slate is responsive in low FPS situations, based in mouse button pressed state */
	FThrottleRequest MouseButtonDownResponsivnessThrottle;

	/** Separate throttle handle that engages automatically based on mouse movement and other user behavior */
	FThrottleRequest UserInteractionResponsivnessThrottle;

	/** The last real time that the user pressed a key or mouse button */
	double LastUserInteractionTime;

	/** Subset of LastUserInteractionTime that is used only when considering when to throttle */
	double LastUserInteractionTimeForThrottling;

	/** Delegate that gets called for LastUserInteractionTime Update */
	FSlateLastUserInteractionTimeUpdateEvent LastUserInteractionTimeUpdateEvent;

	/** Used when considering whether to put Slate to sleep */
	double LastMouseMoveTime;

	/** Support for auto-dismissing pop-ups */
	FPopupSupport PopupSupport;
	
	/** Pointer to the currently registered game viewport widget if any */
	TWeakPtr<SViewport> GameViewportWidget;

#if WITH_EDITOR
	/** List of all registered game viewports since the last time UnregisterGameViewport was called. */
	TSet<TWeakPtr<SViewport>> AllGameViewports;

	/** List of reason why we can't enter in debugging mode. */
	TArray<TPair<FText, int32>> PreventDebuggingModeStack;
#endif

	/** The message when the EnterDebugginMode failed. */
	TWeakPtr<SNotificationItem> DebuggingModeNotificationMessage;

	TSharedPtr<ISlateSoundDevice> SlateSoundDevice;

	/** The current cached absolute real time, right before we tick widgets */
	double CurrentTime;

	/** Last absolute real time that we ticked */
	double LastTickTime;

	/** Running average time in seconds between calls to Tick (used for monitoring responsiveness) */
	float AverageDeltaTime;

	/** Average delta time for application responsiveness tracking.  This is like AverageDeltaTime, but it excludes frame
	    deltas spent while the the application is in a throttled state */
	float AverageDeltaTimeForResponsiveness;

	
	/**
	 * Provides a platform-agnostic method for requesting that the application exit.
	 * Implementations should assign a handler that terminates the process when this delegate is invoked.
	 */
	FSimpleDelegate OnExitRequested;
	
	/** A Widget that introspects the current UI hierarchy */
	TWeakPtr<IWidgetReflector> WidgetReflectorPtr;

	/** Delegate for accessing source code, to pass to any widget inspectors. */
	FAccessSourceCode SourceCodeAccessDelegate;

	/** Delegate for querying if source code access is available */
	FQueryAccessSourceCode QuerySourceCodeAccessDelegate;

	/** Delegate for accessing assets, to pass to any widget inspectors. */
	FAccessAsset AssetAccessDelegate;

	/** Delegate for accessing debug objects, to pass to any widget inspectors. */
	FAccessDebugObject DebugObjectAccessDelegate;

	/** Allows us to track the number of non-slate modal windows active. */
	int32 NumExternalModalWindowsActive;

	/** List of delegates that need to be called when the window action occurs. */
	TArray<FOnWindowAction> OnWindowActionNotifications;

	/** The top of the Style tree. */
	const class FStyleNode* RootStyleNode;

	/** Whether or not we are requesting that we leave debugging mode after the tick is complete */
	bool bRequestLeaveDebugMode;
	/** Whether or not we need to leave debug mode for single stepping */
	bool bLeaveDebugForSingleStep;
	TAttribute<bool> NormalExecutionGetter;

	/**
	 * Modal Windows
	 */

	/** Delegates for when modal windows open or close */
	FModalWindowStackStarted ModalWindowStackStartedDelegate;
	FModalWindowStackEnded ModalWindowStackEndedDelegate;

	/** Keeps track of whether or not the UI for services such as Steam is open. */
	bool bIsExternalUIOpened;

	/** Handle to a throttle request made to ensure the window is responsive in low FPS situations */
	FThrottleRequest ThrottleHandle;

	/** When an drag and drop is happening, we keep track of whether slate knew what to do with the payload on last mouse move */
	bool DragIsHandled;

	/**
	 * Virtual keyboard text field
	 */
	TUniquePtr<IPlatformTextField> SlateTextField;

	/** For desktop platforms that want to test touch style input, pass -faketouches or -simmobile on the commandline to set this */
	bool bIsFakingTouch;

	/** For games that want to allow mouse to imitate touch */
	bool bIsGameFakingTouch;

	/**For desktop platforms that the touch move event be called when this variable is true */
	bool bIsFakingTouched;

	/** Restricts or allows the faking of touches. */
	bool bAllowFakingTouch;

	/** Force Mouse Pointer Capture to always occur even when the application is not active or widget is not a virtual window */
	bool bHandleDeviceInputWhenApplicationNotActive;

	/** Delegate for when a key down event occurred but was not handled in any other way by ProcessKeyDownMessage */
	FOnKeyEvent UnhandledKeyDownEventHandler;

	/** Delegate for when a key down event occurred but was not handled in any other way by ProcessKeyDownMessage */
	FOnKeyEvent UnhandledKeyUpEventHandler;

	/** controls whether unhandled touch events fall back to sending mouse events */
	bool bTouchFallbackToMouse;

	/** .ini controlled option to allow or disallow software cursor rendering */
	bool bSoftwareCursorAvailable;

	/**
	 * Slate look and feel
	 */

	/** Globally enables or disables transition effects for pop-up menus (menu stacks) */
	bool bMenuAnimationsEnabled;

	/** The icon to use on application windows */
	const FSlateBrush *AppIcon;

	FApplicationActivationStateChangedEvent ApplicationActivationStateChangedEvent;
	//
	// Hittest 2.0
	//

	// The rectangle that bounds all the physical monitors given their arrangement.
	// Info comes from the native platform.
	// e.g. On windows the origin (coordinates X=0, Y=0) is the upper left of the primary monitor,
	// but there could be another monitor on any of the sides.
	FSlateRect VirtualDesktopRect;

	TSharedRef<FNavigationConfig> NavigationConfig;

#if WITH_EDITOR
	/** When PIE runs, the game's navigation config will overwrite the editor's navigation config.
	    This separate config allows editor navigation to work even when PIE is running. */
	TSharedRef<FNavigationConfig> EditorNavigationConfig;
#endif

	/** The simulated gestures Slate Application will be in charge of. */
	TBitArray<FDefaultBitArrayAllocator> SimulateGestures;

	/** Delegate for pre slate tick */
	FSlateTickEvent PreTickEvent;

	/** Delegate for post slate Tick */
	FSlateTickEvent PostTickEvent;

	/** Delegate for pre shutdown */
	FSimpleMulticastDelegate PreShutdownEvent;

	/** Delegate for when a new user has been registered. */
	FUserRegisteredEvent UserRegisteredEvent;

	/** Delegate for when a window is in the process of being destroyed */
	FOnWindowBeingDestroyed WindowBeingDestroyedEvent;

	/** Delegate for a when a menu has been dismissed */
	FOnMenuDestroyed MenuBeingDestroyedEvent;

	/** Delegate for slate Tick during modal dialogs */
	FOnModalLoopTickEvent ModalLoopTickEvent;

	/** Delegate for when focus might be about to change */
	FOnFocusChanging FocusChangingDelegate;
	
	/** Delegate for when we are about to take a screenshot of a Slate widget */
	FOnPreScreenshot PreScreenshotDelegate;

	/** Critical section to avoid multiple threads calling Slate Tick when we're synchronizing between the Slate Loading Thread and the Game Thread. */
	FCriticalSection SlateTickCriticalSection;

	/** Are we currently processing input in slate?  If so this value will be greater than 0. */
	int32 ProcessingInput;

	/** Did we synthesize cursor input this frame? */
	bool bSynthesizedCursorMove = false;
	
	/** Are we ticking the SlateApplication. */
	bool bIsTicking = false;
	
	/** Are we disabling the last drag on a new drag enter event to prevent double drag conflicts?  */
	bool bDisableLastDragOnDragEnter = false;

	/** Platform mouse movement event count. */
	uint64 PlatformMouseMovementEvents = 0;

	/** The location of the cursor the last time OnMouseMove was called */
	FVector2f LastPlatformCursorPosition = FVector2f::ZeroVector;

	/** Constant delta time used on every Slate widget Tick if CVarSlateUseFixedDeltaTime is enabled. */
	static SLATE_API double FixedDeltaTime;
	/**
	 * A helper class to wrap the list of input pre-processors. 
	 */
	class InputPreProcessorsHelper
	{
	public:
		// Wrapper functions that call the corresponding function of IInputProcessor for each InputProcessor in the list.
		void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor);
		bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent);
		bool HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent);
		bool HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent);
		bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent);
		bool HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent);
		bool HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent);
		bool HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent);
		bool HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& WheelEvent, const FPointerEvent* GestureEvent);
		bool HandleMotionDetectedEvent(FSlateApplication& SlateApp, const FMotionEvent& MotionEvent);

		/**
		 * Adds or inserts an unique input pre-processor. 
		 * @param Registration	Information needed for the processor registration, including the processor itself, its type, priority and (in legacy mode) index.
		 */
		bool Add(const FInputPreprocessorRegistration& Registration);

		/**
		 * Remove an input pre-processor. 
		 * @param InputProcessor	The InputProcessor to remove.
		 */
		void Remove(TSharedPtr<IInputProcessor> InputProcessor);

		/**
		 * Remove all registered input pre-processors.
		 */
		void RemoveAll();

		/**
		 * Get the index of an input pre-processor.
		 * @param InputProcessor	The InputProcessor to find.
		 * @param Type				The priority bucket in which the InputProcessor was registered
		 * @return The index of the pre-processor, or INDEX_NONE if not registered.
		 */
		int32 Find(TSharedPtr<IInputProcessor> InputProcessor, const EInputPreProcessorType& Type) const;

	private:
		bool PreProcessInput(ESlateDebuggingInputEvent InputEvent, TFunctionRef<bool(IInputProcessor&)> InputProcessFunc);

		void AddInternal(const FInputPreprocessorRegistration& Registration);

		using FProcessorTypeStorage = TArray<TSharedPtr<IInputProcessor>>;
		using FInputProcessorStorage = TSparseArray<FProcessorTypeStorage, TInlineSparseArrayAllocator<(uint32)EInputPreProcessorType::Count>>;

		/** The container of input pre-processors, indexed by type and ordered by priority. */
		FInputProcessorStorage InputPreProcessors;

		/** Flat array built from the ordered contents of the InputPreProcessors map to speed iteration */
		TArray<TSharedPtr<IInputProcessor>> InputPreProcessorsIteratorList;

		/** Guard value for if we are currently iterating our preprocessors. */
		bool bIsIteratingPreProcessors = false;

		/** A list of pre-processors to remove if we are iterating them while removal is requested. */
		TArray<TSharedPtr<IInputProcessor>> ProcessorsPendingRemoval;

		/** A list of pre-processors to add if we are iterating them while addition is requested. */
		TArray<FInputPreprocessorRegistration> ProcessorsPendingAddition;
	};

	/** A list of input pre-processors, gets an opportunity to parse input before anything else. */
	InputPreProcessorsHelper InputPreProcessors;

	/** Allows applications finer control over how we map controllers to users. */
	TSharedRef<ISlateInputManager> InputManager;

	/**
	 * Called when a regular window is moved or resized
	 */
	FOnRegularWindowMovedOrResized OnRegularWindowMovedOrResizedEvent;

#if WITH_EDITOR
	/**
	 * Delegate that is invoked before the input key get process by slate widgets bubble system.
	 * User Function cannot mark the input as handled.
	 */
	FOnApplicationPreInputKeyDownListener OnApplicationPreInputKeyDownListenerEvent;

	/**
	 * Delegate that is invoked before the mouse input button get process by slate widgets bubble system.
	 * User Function cannot mark the input as handled.
	 */
	FOnApplicationMousePreInputButtonDownListener OnApplicationMousePreInputButtonDownListenerEvent;

	/**
	* Called before the dpi scale of a particular window is about to changed
	*/
	FOnWindowDPIScaleChanged OnSignalSystemDPIChangedEvent;

	/**
	 * Called when an editor window dpi scale is changed
	 */
	FOnWindowDPIScaleChanged OnWindowDPIScaleChangedEvent;
#endif // WITH_EDITOR
};
