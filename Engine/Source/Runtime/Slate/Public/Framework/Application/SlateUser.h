// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateScope.h"
#include "Layout/WidgetPath.h"
#include "GestureDetector.h"

class SWidget;
class SWindow;
class ICursor;
class FDragDropOperation;
class FMenuStack;
class FNavigationConfig;

/** Handle to a virtual user of slate. */
class FSlateVirtualUserHandle
{
public:
	SLATE_API virtual ~FSlateVirtualUserHandle();

	inline int32 GetUserIndex() const { return UserIndex; }
	inline int32 GetVirtualUserIndex() const { return VirtualUserIndex; }

SLATE_SCOPE:
	FSlateVirtualUserHandle(int32 InUserIndex, int32 InVirtualUserIndex);

private:
	/** The index the user was assigned. */
	int32 UserIndex;

	/** The virtual index the user was assigned. */
	int32 VirtualUserIndex;
};

/**
 * Slate's representation of an individual input-providing user.  
 * As new input sources are connected, new SlateUsers are created.
 */
class FSlateUser : public TSharedFromThis<FSlateUser>
{
public:
	SLATE_API virtual ~FSlateUser();

	inline int32 GetUserIndex() const { return UserIndex; }
	inline FPlatformUserId GetPlatformUserId() const { return PlatformUser; }
	inline bool IsVirtualUser() const { return !Cursor.IsValid(); }

	SLATE_API TSharedPtr<SWidget> GetFocusedWidget() const;
	SLATE_API bool ShouldShowFocus(TSharedPtr<const SWidget> Widget) const;
	SLATE_API bool SetFocus(const TSharedRef<SWidget>& WidgetToFocus, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);
	SLATE_API void ClearFocus(EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly);

	/** Returns the cause for which the provided widget was focused, or nothing if the given widget is not the current focus target. */
	SLATE_API TOptional<EFocusCause> HasFocus(TSharedPtr<const SWidget> Widget) const;

	/** Returns true if the given widget is in the focus path, but is not the focused widget itself. */
	SLATE_API bool HasFocusedDescendants(TSharedRef<const SWidget> Widget) const;

	/** Returns true if the given widget is anywhere in the focus path, including the focused widget itself. */
	SLATE_API bool IsWidgetInFocusPath(TSharedPtr<const SWidget> Widget) const;
	
	SLATE_API bool HasAnyCapture() const;
	SLATE_API bool HasCursorCapture() const;
	SLATE_API bool HasCapture(uint32 PointerIndex) const;
	
	SLATE_API bool DoesWidgetHaveAnyCapture(TSharedPtr<const SWidget> Widget) const;
	SLATE_API bool DoesWidgetHaveCursorCapture(TSharedPtr<const SWidget> Widget) const;
	SLATE_API bool DoesWidgetHaveCapture(TSharedPtr<const SWidget> Widget, uint32 PointerIndex) const;
	
	SLATE_API bool SetCursorCaptor(TSharedRef<const SWidget> Widget, const FWidgetPath& EventPath);
	SLATE_API bool SetPointerCaptor(uint32 PointerIndex, TSharedRef<const SWidget> Widget, const FWidgetPath& EventPath);
	
	SLATE_API void ReleaseAllCapture();
	SLATE_API void ReleaseCursorCapture();
	SLATE_API void ReleaseCapture(uint32 PointerIndex);

	SLATE_API TArray<FWidgetPath> GetCaptorPaths();
	SLATE_API FWidgetPath GetCursorCaptorPath(FWeakWidgetPath::EInterruptedPathHandling::Type InterruptedPathHandling = FWeakWidgetPath::EInterruptedPathHandling::Truncate, const FPointerEvent* PointerEvent = nullptr);
	SLATE_API FWidgetPath GetCaptorPath(uint32 PointerIndex, FWeakWidgetPath::EInterruptedPathHandling::Type InterruptedPathHandling = FWeakWidgetPath::EInterruptedPathHandling::Truncate, const FPointerEvent* PointerEvent = nullptr);

	SLATE_API FWeakWidgetPath GetWeakCursorCapturePath() const;
	SLATE_API FWeakWidgetPath GetWeakCapturePath(uint32 PointerIndex) const;

	SLATE_API TArray<TSharedRef<SWidget>> GetCaptorWidgets() const;
	SLATE_API TSharedPtr<SWidget> GetCursorCaptor() const;
	SLATE_API TSharedPtr<SWidget> GetPointerCaptor(uint32 PointerIndex) const;

	SLATE_API void SetCursorVisibility(bool bDrawCursor);
	SLATE_API bool IsCursorVisible() const;

	SLATE_API void SetCursorPosition(int32 PosX, int32 PosY);
	SLATE_API void SetCursorPosition(const UE::Slate::FDeprecateVector2DParameter& NewCursorPos);
	SLATE_API void SetPointerPosition(uint32 PointerIndex, int32 PosX, int32 PosY);
	SLATE_API void SetPointerPosition(uint32 PointerIndex, const UE::Slate::FDeprecateVector2DParameter& NewPointerPos);
	
	SLATE_API UE::Slate::FDeprecateVector2DResult GetCursorPosition() const;
	SLATE_API UE::Slate::FDeprecateVector2DResult GetPreviousCursorPosition() const;
	
	SLATE_API UE::Slate::FDeprecateVector2DResult GetPointerPosition(uint32 PointerIndex) const;
	SLATE_API UE::Slate::FDeprecateVector2DResult GetPreviousPointerPosition(uint32 PointerIndex) const;

	SLATE_API bool IsWidgetUnderCursor(TSharedPtr<const SWidget> Widget) const;
	SLATE_API bool IsWidgetUnderPointer(TSharedPtr<const SWidget> Widget, uint32 PointerIndex) const;
	SLATE_API bool IsWidgetUnderAnyPointer(TSharedPtr<const SWidget> Widget) const;

	SLATE_API bool IsWidgetDirectlyUnderCursor(TSharedPtr<const SWidget> Widget) const;
	SLATE_API bool IsWidgetDirectlyUnderPointer(TSharedPtr<const SWidget> Widget, uint32 PointerIndex) const;
	SLATE_API bool IsWidgetDirectlyUnderAnyPointer(TSharedPtr<const SWidget> Widget) const;

	SLATE_API FWeakWidgetPath GetLastWidgetsUnderCursor() const;
	SLATE_API FWeakWidgetPath GetLastWidgetsUnderPointer(uint32 PointerIndex) const;
	const TMap<uint32, FWeakWidgetPath>& GetWidgetsUnderPointerLastEventByIndex() const { return WidgetsUnderPointerLastEventByIndex; }

	TSharedPtr<FDragDropOperation> GetDragDropContent() const { return DragDropContent; }
	SLATE_API bool IsDragDropping() const;
	SLATE_API bool IsDragDroppingAffected(const FPointerEvent& InPointerEvent) const;
	SLATE_API void CancelDragDrop();
	
	SLATE_API void ShowTooltip(const TSharedRef<IToolTip>& InTooltip, const UE::Slate::FDeprecateVector2DParameter& InSpawnLocation, TSharedPtr<SWindow> OwnerWindow = nullptr);
	SLATE_API void CloseTooltip();
	SLATE_API FVector2f GetTooltipPosition() const;

	const FGestureDetector& GetGestureDetector() const { return GestureDetector; }

	SLATE_API void SetUserNavigationConfig(TSharedPtr<FNavigationConfig> InNavigationConfig);
	TSharedPtr<FNavigationConfig> GetUserNavigationConfig() const { return UserNavigationConfig; }

	SLATE_API bool HasValidFocusPath() const;
	SLATE_API const FWeakWidgetPath& GetWeakFocusPath() const;

SLATE_SCOPE:
	static TSharedRef<FSlateUser> Create(int32 InUserIndex, TSharedPtr<ICursor> InCursor);
	static SLATE_API TSharedRef<FSlateUser> Create(FPlatformUserId InPlatformUserId, TSharedPtr<ICursor> InCursor);

	inline TSharedRef<FWidgetPath> GetFocusPath() const
	{
		if (!StrongFocusPath.IsValid())
		{
			StrongFocusPath = WeakFocusPath.ToWidgetPathRef();
		}
		return StrongFocusPath.ToSharedRef();
	}

	SLATE_API void DrawWindowlessDragDropContent(const TSharedRef<SWindow>& WindowToDraw, FSlateWindowElementList& WindowElementList, int32& MaxLayerId);
	SLATE_API void DrawCursor(const TSharedRef<SWindow>& WindowToDraw, FSlateWindowElementList& WindowElementList, int32& MaxLayerId);

	SLATE_API void QueueSyntheticCursorMove();
	SLATE_API bool SynthesizeCursorMoveIfNeeded();

	TSharedPtr<ICursor> GetCursor() const { return Cursor; }
	SLATE_API void LockCursor(const TSharedRef<SWidget>& Widget);
	SLATE_API void UnlockCursor();
	SLATE_API void UpdateCursor();
	SLATE_API void ProcessCursorReply(const FCursorReply& CursorReply);
	void RequestCursorQuery() { bQueryCursorRequested = true; }
	SLATE_API void QueryCursor();
	void OverrideCursor(const TSharedPtr<ICursor> InCursor) { Cursor = InCursor; }
	
	SLATE_API void SetFocusPath(const FWidgetPath& NewFocusPath, EFocusCause InFocusCause, bool bInShowFocus);
	
	SLATE_API void FinishFrame();
	SLATE_API void NotifyWindowDestroyed(TSharedRef<SWindow> DestroyedWindow);

	SLATE_API bool IsTouchPointerActive(int32 TouchPointerIndex) const;

	SLATE_API void NotifyTouchStarted(const FPointerEvent& TouchEvent);
	SLATE_API void NotifyPointerMoveBegin(const FPointerEvent& PointerEvent);
	SLATE_API void NotifyPointerMoveComplete(const FPointerEvent& PointerEvent, const FWidgetPath& WidgetsUnderPointer);
	SLATE_API void NotifyPointerReleased(const FPointerEvent& PointerEvent, const FWidgetPath& WidgetsUnderCursor, TSharedPtr<FDragDropOperation> DroppedContent, bool bWasHandled);
	SLATE_API void UpdatePointerPosition(const FPointerEvent& PointerEvent);

	SLATE_API void StartDragDetection(const FWidgetPath& PathToWidget, int32 PointerIndex, FKey DragButton, UE::Slate::FDeprecateVector2DParameter StartLocation);
	SLATE_API FWidgetPath DetectDrag(const FPointerEvent& PointerEvent, float DragTriggerDistance);
	SLATE_API bool IsDetectingDrag(uint32 PointerIndex) const;
	SLATE_API void ResetDragDetection();

	SLATE_API void SetDragDropContent(TSharedRef<FDragDropOperation> InDragDropContent);
	SLATE_API void ResetDragDropContent();

	int32 GetFocusVersion() const { return FocusVersion; }
	void IncrementFocusVersion() { FocusVersion++; }

	SLATE_API void UpdateTooltip(const FMenuStack& MenuStack, bool bCanSpawnNewTooltip);
	SLATE_API void ResetTooltipWindow();
	SLATE_API bool IsWindowHousingInteractiveTooltip(const TSharedRef<const SWindow>& WindowToTest) const;

	FGestureDetector& GetGestureDetector() { return GestureDetector; }

#if PLATFORM_MAC
	// Unclear why from existing code, but Mac seems to need to cache & restore all mouse captor paths when activating the top level window
	const TMap<uint32, FWeakWidgetPath> GetCaptorPathsByIndex() const { return PointerCaptorPathsByIndex; }
	void RestoreCaptorPathsByIndex(const TMap<uint32, FWeakWidgetPath>& InPointerCaptorPathsByIndex) { PointerCaptorPathsByIndex = InPointerCaptorPathsByIndex; }
#endif

private:
	SLATE_API FSlateUser(int32 InUserIndex, TSharedPtr<ICursor> InCursor);
	
	SLATE_API FSlateUser(FPlatformUserId InPlatformUser, TSharedPtr<ICursor> InCursor);
	SLATE_API void UpdatePointerPosition(uint32 PointerIndex, const FVector2f& Position);
	SLATE_API void LockCursorInternal(const FWidgetPath& WidgetPath);
	SLATE_API TSharedRef<SWindow> GetOrCreateTooltipWindow(TSharedPtr<SWindow> OwnerWindow = nullptr);

#if WITH_SLATE_DEBUGGING
	SLATE_API void ClearForceFieldDebug();
	SLATE_API void HandlePaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);

	TArray<FSlateRect> DebugForceFieldRects;
#endif

	/** The index the user was assigned. */
	int32 UserIndex = INDEX_NONE;

	/** The owning platform user of this slate user. */
	FPlatformUserId PlatformUser = PLATFORMUSERID_NONE;

	/** The cursor this user is in control of. Guaranteed to be valid for all real users, absence implies this is a virtual user. */
	TSharedPtr<ICursor> Cursor;

	/** Store the last time the cursor position was changed */
	double LastCursorSignificantMoveTime = 0.0;

	/** SlateUsers can optionally be individually assigned a navigation config to use. This overrides the global nav config that lives on FSlateApplication when valid. */
	TSharedPtr<FNavigationConfig> UserNavigationConfig;

	/** Whether this user is currently drawing the cursor onscreen each frame */
	bool bCanDrawCursor = true;

	/** Whether we need to call QueryCursor because bCanDrawCursor was enabled */
	bool bNeedQueryForVisibleCursor = true;

	/** The OS or actions taken by the user may require we refresh the current state of the cursor. */
	bool bQueryCursorRequested = false;

	/**
	 * Whenever something happens that can affect Slate layout, we need to process a mouse move even if the mouse didn't move at all.
	 * @see SynthesizeCursorMoveIfNeeded
	 */
	int32 NumPendingSyntheticCursorMoves = 0;

	/** The cursor widget and window to render that cursor for the current software cursor.*/
	TWeakPtr<SWindow> CursorWindowPtr;
	TWeakPtr<SWidget> CursorWidgetPtr;

	/** A weak path to the widget currently focused by a user, if any. */
	FWeakWidgetPath WeakFocusPath;

	/** A strong widget path to the focused widget, if any. This is cleared after the end of pumping messages. */
	mutable TSharedPtr<FWidgetPath> StrongFocusPath;

	/** Reason a widget was focused by a user, if any. */
	EFocusCause FocusCause = EFocusCause::Cleared;

	/** If we should show this focus */
	bool bShowFocus = false;

	/**
	 * The FocusVersion is used to know if the focus state is modified for a user while processing focus
	 * events, that way upon returning from focus calls, we know if we should abandon the remainder of the event.
	 */
	int32 FocusVersion = 0;

	/** Current position of all pointers controlled by this user */
	TMap<uint32, FVector2f> PointerPositionsByIndex;
	TMap<uint32, FVector2f> PreviousPointerPositionsByIndex;

	/** Weak paths to widgets that are currently capturing a particular pointer */
	TMap<uint32, FWeakWidgetPath> PointerCaptorPathsByIndex;

	struct FDragDetectionState
	{
		FDragDetectionState(const FWidgetPath& PathToWidget, int32 PointerIndex, FKey DragButton, const FVector2f& StartLocation)
			: DetectDragForWidget(PathToWidget)
			, DragStartLocation(StartLocation)
			, TriggerButton(DragButton)
			, PointerIndex(PointerIndex)
		{
		}

		/** If not null, a widget has requested that we detect a drag being triggered in this widget and send an OnDragDetected() event*/
		FWeakWidgetPath DetectDragForWidget;

		FVector2f DragStartLocation = FVector2f::ZeroVector;
		FKey TriggerButton = EKeys::Invalid;
		int32 PointerIndex = INDEX_NONE;
	};

	/** Current drag status for pointers currently executing a drag/drop operation */
	TMap<uint32, FDragDetectionState> DragStatesByPointerIndex;

	/** When not null, the content of the current drag drop operation. */
	TSharedPtr<FDragDropOperation> DragDropContent;

	/** The window the drag drop content is over. */
	TWeakPtr<SWindow> DragDropWindowPtr;

	/** Weak paths to the last widget each pointer was under last time an event was processed */
	TMap<uint32, FWeakWidgetPath> WidgetsUnderPointerLastEventByIndex;

	/** Path to widget that currently holds the cursor lock; invalid path if no cursor lock. */
	FWeakWidgetPath LockingWidgetPath;

	/** Desktop Space Rect that bounds the cursor. */
	FSlateRect LastComputedLockBounds;

	/** Per-owner tooltip windows. On Wayland, tooltips are xdg_popups that must be parented
	 *  to the correct toplevel. Each toplevel gets its own tooltip window. */
	TMap<TWeakPtr<SWindow>, TWeakPtr<SWindow>> TooltipWindowsByOwner;
	/** The currently active tooltip window (from TooltipWindowsByOwner) */
	TWeakPtr<SWindow> TooltipWindowPtr;

	enum class ETooltipOffsetDirection : uint8
	{
		Undetermined,
		Down,
		Right
	};
	struct FActiveTooltipInfo
	{
		void Reset();

		/** The actual tooltip object running the show here (not necessarily the same object as the tooltip SWidget, but can be) */
		TWeakPtr<IToolTip> Tooltip;

		/** The widget that is taking care of visualizing this tooltip (can be null) */
		TWeakPtr<SWidget> TooltipVisualizer;

		/** The widget that sourced the currently active tooltip */
		TWeakPtr<SWidget> SourceWidget;

		/** Desired position of the tooltip in screen space, updated whenever the mouse moves */
		FVector2f DesiredLocation = FVector2f::ZeroVector;

		/** Desired size of the tooltip in screen space */
		FVector2f DesiredSize = FVector2f::ZeroVector;

		/** The time at which the tooltip was summoned */
		double SummonTime = 0.0;

		/** 
		 * Direction in which the tooltip widget is being offset from the source widget  
		 * Cached to prevent the direction of offset from changing as the user moves the mouse cursor (otherwise the tooltip teleports around)
		 */
		ETooltipOffsetDirection OffsetDirection = ETooltipOffsetDirection::Undetermined;

		/** Caches whether the tooltip's IsInteractive() method returned true last frame. */
		bool WasInteractive = false;

		/** Caches whether the tooltip has been positioned yet. */
		bool HasBeenPositioned = false;

		/** The toplevel window that sourced the tooltip. Used to detect when the tooltip
		 *  needs to be recreated for a different parent (required on Wayland). */
		TWeakPtr<SWindow> SourceWindow;
	};
	FActiveTooltipInfo ActiveTooltipInfo;

	FGestureDetector GestureDetector;
};
