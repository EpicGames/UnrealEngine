// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/InputBindingManager.h"

//
// COMMAND DECLARATION
//

#define LOC_DEFINE_REGION

/** Internal function used by the UI_COMMAND macros to build the command. Do not call this directly as only the macros are gathered for localization; instead use FUICommandInfo::MakeCommandInfo for dynamic content */
SLATE_API void MakeUICommand_InternalUseOnly(FBindingContext* This, TSharedPtr< FUICommandInfo >& OutCommand, const TCHAR* InSubNamespace, const TCHAR* InCommandName, const TCHAR* InCommandNameUnderscoreTooltip, const ANSICHAR* DotCommandName, const TCHAR* FriendlyName, const TCHAR* InDescription, const EUserInterfaceActionType CommandType, const FInputChord& InDefaultChord, const FInputChord& InAlternateDefaultChord = FInputChord());

// This macro requires LOCTEXT_NAMESPACE to be defined. If you don't want the command to be placed under a sub namespace, provide "" as the namespace.
#define UI_COMMAND_EXT( BindingContext, OutUICommandInfo, CommandIdName, FriendlyName, InDescription, CommandType, InDefaultChord, ... ) \
	MakeUICommand_InternalUseOnly( BindingContext, OutUICommandInfo, TEXT(LOCTEXT_NAMESPACE), TEXT(CommandIdName), TEXT(CommandIdName) TEXT("_ToolTip"), "." CommandIdName, TEXT(FriendlyName), TEXT(InDescription), CommandType, InDefaultChord, ## __VA_ARGS__ );

#define UI_COMMAND( CommandId, FriendlyName, InDescription, CommandType, InDefaultChord, ... ) \
	MakeUICommand_InternalUseOnly( this, CommandId, TEXT(LOCTEXT_NAMESPACE), TEXT(#CommandId), TEXT(#CommandId) TEXT("_ToolTip"), "." #CommandId, TEXT(FriendlyName), TEXT(InDescription), CommandType, InDefaultChord, ## __VA_ARGS__ );

#undef LOC_DEFINE_REGION

#define UE_DECLARE_TCOMMANDS(Type, Api) template<> Api TWeakPtr<Type>& TCommands<Type>::GetInstance();
#define UE_DEFINE_TCOMMANDS(Type) template<> TWeakPtr<Type>& TCommands<Type>::GetInstance() { static TWeakPtr<Type> Inst; return Inst; }

/** A base class for a set of commands. Inherit from it to make a set of commands. See MainFrameActions for an example. */
template<typename CommandContextType>
class TCommands : public FBindingContext
{
public:
	

	/** Use this method to register commands. Usually done in StartupModule(). */
	inline static void Register()
	{
		TWeakPtr<CommandContextType>& Instance = GetInstance();
		if ( !Instance.IsValid() )
		{
			// We store the singleton instances in the FInputBindingManager in order to prevent
			// different modules from instantiating their own version of TCommands<CommandContextType>.
			TSharedRef<CommandContextType> NewInstance = MakeShareable( new CommandContextType() );

			TSharedPtr<FBindingContext> ExistingBindingContext = FInputBindingManager::Get().GetContextByName( NewInstance->GetContextName() );
			if (ExistingBindingContext.IsValid())
			{
				// Someone already made this set of commands, and registered it.
				Instance = StaticCastSharedPtr<CommandContextType>(ExistingBindingContext);
			}
			else
			{
				// Make a new set of commands and register it.
				Instance = NewInstance;

				// Registering the first command will add the NewInstance into the Binding Manager, who holds on to it.
				NewInstance->RegisterCommands();

				// Notify that new commands have been registered
				CommandsChanged.Broadcast(*NewInstance);
			}
		}
	}

	inline static bool IsRegistered()
	{
		return GetInstance().IsValid();
	}

	/** Get the singleton instance of this set of commands. */
	inline static CommandContextType& Get()
	{
		// If you get a crash here, then either your commands object wasn't registered, or it was not kept
		//  alive. The latter could happen if your RegisterCommands() function does not actually register 
		//  any FUICommandInfo objects that use your TCommands object, so nothing keeps it alive.
		return *( GetInstance().Pin() );
	}

	/** Use this method to clean up any resources used by the command set. Usually done in ShutdownModule() */
	inline static void Unregister()
	{
		TWeakPtr<CommandContextType>& Instance = GetInstance();
		// The instance may not be valid if it was never used.
		if( Instance.IsValid() )
		{
			auto InstancePtr = Instance.Pin();
			FInputBindingManager::Get().RemoveContextByName(InstancePtr->GetContextName());
			
			// Notify that new commands have been unregistered
			CommandsChanged.Broadcast(*InstancePtr);

			check(InstancePtr.IsUnique());
			InstancePtr.Reset();
		}
	}

	/** Get the BindingContext for this set of commands. */
	inline static const FBindingContext& GetContext()
	{
		TWeakPtr<CommandContextType>& Instance = GetInstance();
		check( Instance.IsValid() );
		return *(Instance.Pin());
	}
	
protected:
	
	/** Construct a set of commands; call this from your custom commands class. */
	TCommands( const FName InContextName, const FText& InContextDesc, const FName InContextParent, const FName InStyleSetName )
		: FBindingContext( InContextName, InContextDesc, InContextParent, InStyleSetName )
	{
	}
	virtual ~TCommands() = default;

	/** A static instance of the command set. */
#if PLATFORM_UNIX || PLATFORM_APPLE
	static SLATE_API TWeakPtr<CommandContextType>& GetInstance()
	{
		static TWeakPtr<CommandContextType> Inst;
		return Inst;
	}
#else
	static TWeakPtr<CommandContextType>& GetInstance()
	{
		static TWeakPtr<CommandContextType> Inst;
		return Inst;
	}
#endif

	/** Pure virtual to override; describe and instantiate the commands in here by using the UI COMMAND macro. */
	virtual void RegisterCommands() = 0;

};
