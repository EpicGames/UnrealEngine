// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/UIAction.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"
#include "Framework/SlateDelegates.h"

/**
 * Context for generating content from a FUICommandInfo
 */
struct FUIContentContext : public IUIActionContextBase
{
	virtual ~FUIContentContext() = default;

	/** Holds a delegate that is executed to generate menu actions from the stack. */
	FOnGetContent OnGetContent;

	//~ Begin IUIActionContextBase Interface
	virtual FName GetContextName() const override
	{
		return ContextName;
	}
	//~ End IUIActionContextBase Interface

	inline static const FName ContextName = FName("ContentContext");
};
