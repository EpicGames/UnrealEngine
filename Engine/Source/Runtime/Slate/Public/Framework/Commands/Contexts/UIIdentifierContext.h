// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/UIAction.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "Textures/SlateIcon.h"

/** Delegate for dynamic generation of icons */
DECLARE_DELEGATE_RetVal(const FSlateIcon, FOnGetContextIcon)

/** Delegate for dynamic generation of text */
DECLARE_DELEGATE_RetVal(const FText, FOnGetContextText)

/**
 * Context for generating identifiers such as icons, labels, and descriptions from a FUICommandInfo
 */
struct FUIIdentifierContext : public IUIActionContextBase
{
	virtual ~FUIIdentifierContext() = default;

	/** Holds a delegate that is executed to generate context icon. */
	FOnGetContextIcon OnGetContextIcon;

	/** Holds a delegate that is executed to generate context label. */
	FOnGetContextText OnGetContextabel;

	/** Holds a delegate that is executed to generate context description. */
	FOnGetContextText OnGetContextDescription;

	//~ Begin IUIActionContextBase Interface
	virtual FName GetContextName() const override
	{
		return ContextName;
	}
	//~ End IUIActionContextBase Interface

	inline static const FName ContextName = FName("IdentifierContext");
};
