// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/CoreStyle.h"
#include "Framework/Commands/Commands.h"

UE_DECLARE_TCOMMANDS(class FGenericCommands, SLATE_API)

class FGenericCommands : public TCommands<FGenericCommands>
{
public:
	
	FGenericCommands()
		: TCommands<FGenericCommands>( TEXT("GenericCommands"), NSLOCTEXT("GenericCommands", "Generic Commands", "Common Commands"), NAME_None, FCoreStyle::Get().GetStyleSetName() )
	{
	}

	virtual ~FGenericCommands() = default;

	SLATE_API virtual void RegisterCommands() override;	

	TSharedPtr<FUICommandInfo> Cut;
	TSharedPtr<FUICommandInfo> Copy;
	TSharedPtr<FUICommandInfo> Paste;
	TSharedPtr<FUICommandInfo> Duplicate;
	TSharedPtr<FUICommandInfo> Undo;
	TSharedPtr<FUICommandInfo> Redo;
	TSharedPtr<FUICommandInfo> Delete;
	TSharedPtr<FUICommandInfo> Rename;
	TSharedPtr<FUICommandInfo> SelectAll;	
};
