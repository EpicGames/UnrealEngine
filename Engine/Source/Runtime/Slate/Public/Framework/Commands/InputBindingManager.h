// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UICommandList.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

class FUserDefinedChords;

typedef TMap<FName, TSharedPtr<FUICommandInfo>> FCommandInfoMap;
typedef TMap<FInputChord, FName> FChordMap;


/** Delegate for alerting subscribers the input manager records a user-defined chord */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUserDefinedChordChanged, const FUICommandInfo& );

/** Delegate for exposing new command lists to subscribers */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRegisterCommandList, const FName, TSharedRef<FUICommandList>);

/**
 * Manager responsible for creating and processing input bindings.                  
 */
class FInputBindingManager
{
public:

	/**
	 * @return The instance of this manager                   
	 */
	static SLATE_API FInputBindingManager& Get();

	/**
	 * Virtual destructor
	 */
	virtual ~FInputBindingManager() = default;

	/**
	 * Returns a list of all known input contexts
	 *
	 * @param OutInputContexts	The generated list of contexts
	 * @return A list of all known input contexts                   
	 */
	SLATE_API void GetKnownInputContexts( TArray< TSharedPtr<FBindingContext> >& OutInputContexts ) const;

	/**
	 * Look up a binding context by name.
	 */
	SLATE_API TSharedPtr<FBindingContext> GetContextByName( const FName& InContextName );

	/**
	 * Remove the context with this name
	 */
	SLATE_API void RemoveContextByName( const FName& InContextName );

	/**
	 * Creates an input command from the specified user interface action
	 * 
	 * @param InBindingContext		The context where the command is valid
	 * @param InUICommandInfo		The user interface action to create the input command from
	 *								Will automatically bind the user interface action to the command specified 
	 *								by the action so when the command's chord is pressed, the action is executed
	 */
	SLATE_API void CreateInputCommand( const TSharedRef<FBindingContext>& InBindingContext, TSharedRef<FUICommandInfo> InUICommandInfo );

	/**
	 * Removes an input command, allowing a new one to take its place
	 *
	 * @param InBindingContext		The context where the command is valid
	 * @param InUICommandInfo		The user interface action to remove
	 */
	SLATE_API void RemoveInputCommand( const TSharedRef<FBindingContext>& InBindingContext, TSharedRef<FUICommandInfo> InUICommandInfo );

	/**
	 * Returns a command info that is has the same active chord as the provided chord and is in the same binding context or parent context
	 *
	 * @param	InBindingContext		The context in which the command is valid
	 * @param	InChord					The chord to match against commands
	 * @param	bCheckDefault			Whether or not to check the default chord.  Will check the active chord if false
	 * @return	A pointer to the command info which has the InChord as its active chord or null if one cannot be found
	 */
	SLATE_API const TSharedPtr<FUICommandInfo> GetCommandInfoFromInputChord( const FName InBindingContext, const FInputChord& InChord, bool bCheckDefault ) const;

	/** 
	 * Finds the command in the provided context which uses the provided input chord
	 * 
	 * @param InBindingContext	The binding context name
	 * @param InChord			The chord to check against when looking for commands
	 * @param bCheckDefault		Whether or not to check the default chord of commands instead of active chords
	 * @param OutActiveChordIndex	The index into the commands active chord array where the passed in chord was matched.  INDEX_NONE if bCheckDefault is true or nothing was found.
	 */
	SLATE_API const TSharedPtr<FUICommandInfo> FindCommandInContext( const FName InBindingContext, const FInputChord& InChord, bool bCheckDefault ) const;

	/** 
	 * Finds the command in the provided context which has the provided name 
	 * 
	 * @param InBindingContext	The binding context name
	 * @param CommandName		The name of the command to find
	 */
	SLATE_API const TSharedPtr<FUICommandInfo> FindCommandInContext( const FName InBindingContext, const FName CommandName ) const;

	/**
	 * Called when the active chord is changed on a command
	 *
	 * @param CommandInfo	The command that had the active chord changed. 
	 */
	SLATE_API virtual void NotifyActiveChordChanged( const FUICommandInfo& CommandInfo, const EMultipleKeyBindingIndex InChordIndex);
	
	/**
	 * Saves the user defined chords to a json file
	 */
	SLATE_API void SaveInputBindings();

	/**
	 * Removes any user defined chords
	 */
	SLATE_API void RemoveUserDefinedChords();

	/**
	 * Returns all known command infos for a given binding context
	 *
	 * @param InBindingContext	The binding context to get command infos from
	 * @param OutCommandInfos	The list of command infos for the binding context
	 */
	SLATE_API void GetCommandInfosFromContext( const FName InBindingContext, TArray< TSharedPtr<FUICommandInfo> >& OutCommandInfos ) const;

	/** Registers a delegate to be called when a user-defined chord is edited */
	FDelegateHandle RegisterUserDefinedChordChanged(const FOnUserDefinedChordChanged::FDelegate& Delegate)
	{
		return OnUserDefinedChordChanged.Add(Delegate);
	}

	/** Unregisters a delegate to be called when a user-defined chord is edited */
	void UnregisterUserDefinedChordChanged(FDelegateHandle DelegateHandle)
	{
		OnUserDefinedChordChanged.Remove(DelegateHandle);
	}

	/**
	* Gets the user defined chord (if any) from the provided command name
	*
	* @param InBindingContext	The context in which the command is active
	* @param InCommandName		The name of the command to get the chord from
	* @param ChordIndex		The index of the key binding (in the multiple key bindings array)
	*/
	SLATE_API bool GetUserDefinedChord(const FName InBindingContext, const FName InCommandName, const EMultipleKeyBindingIndex InChordIndex, FInputChord& OutUserDefinedChord);

	/**
	* Gets the project defined chord (if any) from the provided command name
	*
	* @param InBindingContext	The context in which the command is active
	* @param InCommandName		The name of the command to get the chord from
	* @param ChordIndex		The index of the key binding (in the multiple key bindings array)
	*/
	SLATE_API bool GetProjectDefinedChord(const FName InBindingContext, const FName InCommandName, const EMultipleKeyBindingIndex InChordIndex, FInputChord& OutUserDefinedChord);

	// Type of filter
	enum class ECommandFilterType
	{
		None,

		DenyList,
		AllowList
	};

	/**
	* Filters a command by name
	*
	* @param InOwnerName		The name of the owner adding filter so it can be unregistered later
	* @param InBindingContext	The context in which the command is active
	* @param InCommandName		The name of the command
	* @param FilterType			Type of filter to apply
	*/
	SLATE_API void AddCommandFilter(const FName InOwnerName, const FName InBindingContext, const FName InCommandName, const ECommandFilterType FilterType);

	/** Removes all filters associated with an owner */
	SLATE_API void UnregisterCommandFilterOwner(const FName InOwnerName);

	/**
	* Command passes allow/deny list filters
	*
	* @param InBindingContext	The context in which the command is active
	* @param InCommandName		The name of the command
	*/
	SLATE_API bool CommandPassesFilter(const FName InBindingContext, const FName InCommandName) const;

	/**
	 * Exposes the given command list and its context to all OnRegisterCommandList subscribers
	 *
	 * @param InBindingContext	The context in which the command list is active
	 * @param CommandList		The command list to expose to subscribers
	 */
	SLATE_API bool RegisterCommandList(const ::FName InBindingContext, TSharedRef<FUICommandList> CommandList) const;

	/**
	 * Exposes a new given command list from the given context to all OnRegisterCommandList subscribers
	 * 
	 * @param InBindingContext	The context in which the command list is active
	 * @return	The new command list exposed to subscribers
	 */
	SLATE_API TSharedPtr<FUICommandList> RegisterNewCommandList(const FName InBindingContext) const;

	/**
	 * Exposes the given command list and its context to all OnUnregisterCommandList subscribers
	 * 
	 * @param InBindingContext	The context in which the command list is active
	 * @param CommandList		The command list to expose to subscribers
	 */
	SLATE_API bool UnregisterCommandList(const ::FName InBindingContext, TSharedRef<FUICommandList> CommandList) const;

	/** A delegate for systems to subscribe to any other system exposing their command lists */
	FOnRegisterCommandList OnRegisterCommandList;

	/** A delegate for systems to subscribe to any other system unregistering their command lists */
	FOnRegisterCommandList OnUnregisterCommandList;

private:

	/**
	 * Hidden default constructor.
	 */
	SLATE_API FInputBindingManager();

	/**
	 *	Checks a binding context for duplicate chords 
	 */
	SLATE_API void CheckForDuplicateDefaultChords( const FBindingContext& InBindingContext, TSharedPtr<FUICommandInfo> InCommandInfo ) const;

	/**
	 * Recursively finds all child contexts of the provided binding context.  
	 *
	 * @param InBindingContext	The binding context to search
	 * @param AllChildren		All the children of InBindingContext. InBindingContext is the first element in AllChildren
	 */
	SLATE_API void GetAllChildContexts( const FName InBindingContext, TArray<FName>& AllChildren ) const;

	/**
	 * Displays list of commands.
	 */
	SLATE_API void PrintAllInputCommands(bool bBoundOnly);

private:

	struct FContextEntry
	{
		FContextEntry()
		{
			ChordToCommandInfoMaps.Init(FChordMap(), static_cast<uint8>(EMultipleKeyBindingIndex::NumChords));
		}
		/** A list of commands associated with the context */
		FCommandInfoMap CommandInfoMap;

		/** Chord to command info maps, one for each set of key bindings */
		TArray<FChordMap> ChordToCommandInfoMaps;

		/** The binding context for this entry*/
		TSharedPtr< FBindingContext > BindingContext;
	};

	struct FCommandFilterOwners
	{
		TArray<FName> OwnerNames;
	};

	struct FCommandFilterForContext
	{
		TMap<FName, FCommandFilterOwners> CommandDenyList;
		TMap<FName, FCommandFilterOwners> CommandAllowList;
	};

	/** Mapping of command filters by context */
	TMap<FName, FCommandFilterForContext> CommandFiltersByContext;

	/** A mapping of context name to the associated entry map */
	TMap< FName, FContextEntry > ContextMap;
	
	/** A mapping of contexts to their child contexts */
	TMultiMap< FName, FName > ParentToChildMap;

	/** User defined chord overrides for commands */
	TSharedPtr< class FUserDefinedChords > UserDefinedChords;

	/** Project defined chord overrides for commands */
	TSharedPtr< class FUserDefinedChords > ProjectDefinedChords;

	/** Delegate called when a user-defined chord is edited */
	FOnUserDefinedChordChanged OnUserDefinedChordChanged;
};
