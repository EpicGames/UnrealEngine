// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "UObject/ObjectMacros.h"

#include "InputChord.generated.h"

struct FInputChord;

// Was supposed to be deprecated in 4.8, but was never properly marked up
UE_DEPRECATED(4.21, "Use FInputChord instead of FInputGesture") 
typedef struct FInputChord FInputGesture;

/** An Input Chord is a key and the modifier keys that are to be held with it. */
USTRUCT(BlueprintType)
struct FInputChord
{
	GENERATED_USTRUCT_BODY()

	/** The Key is the core of the chord. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Key)
	FKey Key;

	/** Whether the shift key is part of the chord.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifier)
	uint32 bShift:1;

	/** Whether the control key is part of the chord.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifier)
	uint32 bCtrl:1;

	/** Whether the alt key is part of the chord.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifier)
	uint32 bAlt:1;

	/** Whether the command key is part of the chord.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifier)
	uint32 bCmd:1;

	/** 
	  * The ways two chords can be related to each other. A chord is considered masking 
	  * when it has all the same modifier keys as another chord plus more.
	  */
	enum class ERelationshipType
	{
		None,
		Same,
		Masked,
		Masks
	};

	/** Returns the relationship between this chord and another. */
	SLATE_API ERelationshipType GetRelationship(const FInputChord& OtherChord) const;

	SLATE_API FInputChord();

	FInputChord(const FKey InKey)
		: Key(InKey)
		, bShift(false)
		, bCtrl(false)
		, bAlt(false)
		, bCmd(false)
	{
	}

	FInputChord(const FKey InKey, const bool bInShift, const bool bInCtrl, const bool bInAlt, const bool bInCmd)
		: Key(InKey)
		, bShift(bInShift)
		, bCtrl(bInCtrl)
		, bAlt(bInAlt)
		, bCmd(bInCmd)
	{
	}

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InKey
	 * @param InModifierKeys
	 */
	FInputChord( const EModifierKey::Type InModifierKeys, const FKey InKey )
		: Key(InKey)
		, bShift((InModifierKeys & EModifierKey::Shift) !=0)
		, bCtrl((InModifierKeys & EModifierKey::Control) != 0)
		, bAlt((InModifierKeys & EModifierKey::Alt) != 0)
		, bCmd((InModifierKeys & EModifierKey::Command) != 0)
	{
	}

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InKey
	 * @param InModifierKeys
	 */
	FInputChord( const FKey InKey, const EModifierKey::Type InModifierKeys )
		: Key(InKey)
		, bShift((InModifierKeys & EModifierKey::Shift) !=0)
		, bCtrl((InModifierKeys & EModifierKey::Control) != 0)
		, bAlt((InModifierKeys & EModifierKey::Alt) != 0)
		, bCmd((InModifierKeys & EModifierKey::Command) != 0)
	{
	}

	/**
	 * Copy constructor.
	 *
	 * @param Other
	 */
	FInputChord(const FInputChord& Other) = default;

	/**
	 * Move constructor.
	 *
	 * @param Other
	 */
	FInputChord(FInputChord&& Other) = default;

	/**
	 * Copy assignment operator.
	 *
	 * @param Other
	 */
	FInputChord& operator=(const FInputChord& Other) = default;

	/**
	 * Move assignment operator.
	 *
	 * @param Other
	 */
	FInputChord& operator=(FInputChord&& Other) = default;

	/**
	 * Compares this input chord with another for equality.
	 *
	 * @param Other The other chord to compare with.
	 * @return true if the chords are equal, false otherwise.
	 */
	bool operator==(const FInputChord& Other) const
	{
		return (   Key == Other.Key
				&& bShift == Other.bShift
				&& bCtrl == Other.bCtrl
				&& bAlt == Other.bAlt
				&& bCmd == Other.bCmd);
	}

	/**
	 * Compares this input chord with another for inequality.
	 *
	 * @param Other The other chord to compare with.
	 * @return true if the chords are equal, false otherwise.
	 */
	bool operator!=( const FInputChord& Other ) const
	{
		return !(*this == Other);
	}

#if PLATFORM_MAC
	bool NeedsControl() const { return bCmd; }
	bool NeedsCommand() const { return bCtrl; }
#else
	bool NeedsControl() const { return bCtrl; }
	bool NeedsCommand() const { return bCmd; }
#endif
	bool NeedsAlt() const { return bAlt; }
	bool NeedsShift() const { return bShift; }


	/**
	 * Gets the corresponding EModifierKey::Type
	 */
	SLATE_API EModifierKey::Type GetModifierKey() const;

	/** 
	 * Gets a localized string that represents the chord.
	 *
	 * @return A localized string.
	 */
	SLATE_API FText GetInputText(const bool bLongDisplayName = true) const;
	
	/**
	 * Gets the key represented as a localized string.
	 *
	 * @return A localized string.
	 */
	SLATE_API FText GetKeyText(const bool bLongDisplayName = true) const;

	/**
	 * Gets the localized string for the modifier portion of the coord. 
	 * 
	 * @param ModifierAppender. Text to append between each modifier and the key. If this is not specified a '+' character will be used
	 */
	SLATE_API FText GetModifierText(TOptional<FText> ModifierAppender = TOptional<FText>()) const;

	/**
	 * Checks whether this chord requires an modifier keys to be pressed.
	 *
	 * @return true if modifier keys must be pressed, false otherwise.
	 */
	bool HasAnyModifierKeys( ) const
	{
		return (bAlt || bCtrl || bCmd || bShift);
	}

	/**
	 * Determines if this chord is valid.  A chord is valid if it has a non modifier key that must be pressed
	 * and zero or more modifier keys that must be pressed
	 *
	 * @return true if the chord is valid
	 */
	bool IsValidChord( ) const
	{
		return (Key.IsValid() && !Key.IsModifierKey());
	}

	/**
	 * Sets this chord to a new key and modifier state based on the provided template
	 * Should not be called directly.  Only used by the key binding editor to set user defined keys
	 */
	void Set( const FInputChord& InTemplate )
	{
		*this = InTemplate;
	}

public:

	/**
	 * Gets a type hash value for the specified chord.
	 *
	 * @param Chord The input chord to get the hash value for.
	 * @return The hash value.
	 */
	friend uint32 GetTypeHash( const FInputChord& Chord )
	{
		return GetTypeHash(Chord.Key) ^ (Chord.bShift | Chord.bCtrl << 1 | Chord.bAlt << 2 | Chord.bCmd << 3);
	}
};

