// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"

/** Defines FExecuteAction delegate interface */
DECLARE_DELEGATE(FExecuteAction);

/** Defines FCanExecuteAction delegate interface.  Returns true when an action is able to execute. */
DECLARE_DELEGATE_RetVal(bool, FCanExecuteAction);

/** Defines FIsActionChecked delegate interface.  Returns true if the action is currently toggled on. */
DECLARE_DELEGATE_RetVal(bool, FIsActionChecked);

/** Defines FGetActionCheckState delegate interface.  Returns the ECheckBoxState for the action. */
DECLARE_DELEGATE_RetVal(ECheckBoxState, FGetActionCheckState);

/** Defines FIsActionButtonVisible delegate interface.  Returns true when UI buttons associated with the action should be visible. */
DECLARE_DELEGATE_RetVal(bool, FIsActionButtonVisible);


/** Enum controlling whether a given UI action can be repeated if the chord used to call it is held down */
enum class EUIActionRepeatMode
{
	RepeatDisabled,
	RepeatEnabled,
};


/**
 * Implements an UI action.
 */
struct FUIAction
{
	/** Holds a delegate that is executed when this action is activated. */
	FExecuteAction ExecuteAction;

	/** Holds a delegate that is executed when determining whether this action can execute. */
	FCanExecuteAction CanExecuteAction;

	/** Holds a delegate that is executed when determining the check state of this action. */
	FGetActionCheckState GetActionCheckState;

	/** Holds a delegate that is executed when determining whether this action is visible. */
	FIsActionButtonVisible IsActionVisibleDelegate;

	/** Can this action can be repeated if the chord used to call it is held down? */
	EUIActionRepeatMode RepeatMode;

public:

	/** Default constructor. */
	SLATE_API FUIAction();

	/**
	 * Constructor that takes delegates to initialize the action with
	 *
	 * @param	ExecuteAction		The delegate to call when the action should be executed
	 * @param	RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API FUIAction(FExecuteAction InitExecuteAction, EUIActionRepeatMode InitRepeatMode = EUIActionRepeatMode::RepeatDisabled);

	/**
	 * Constructor that takes delegates to initialize the action with
	 *
	 * @param	ExecuteAction		The delegate to call when the action should be executed
	 * @param	CanExecuteAction	The delegate to call to see if the action can be executed
	 * @param	RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, EUIActionRepeatMode InitRepeatMode = EUIActionRepeatMode::RepeatDisabled);

	/**
	 * Constructor that takes delegates to initialize the action with
	 *
	 * @param	ExecuteAction		The delegate to call when the action should be executed
	 * @param	CanExecuteAction	The delegate to call to see if the action can be executed
	 * @param	IsCheckedDelegate	The delegate to call to see if the action should appear checked when visualized
	 * @param	RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FIsActionChecked InitIsCheckedDelegate, EUIActionRepeatMode InitRepeatMode = EUIActionRepeatMode::RepeatDisabled);

	/**
	 * Constructor that takes delegates to initialize the action with
	 *
	 * @param	ExecuteAction		The delegate to call when the action should be executed
	 * @param	CanExecuteAction	The delegate to call to see if the action can be executed
	 * @param	GetActionCheckState	The delegate to call to see what the check state of the action should be
	 * @param	RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FGetActionCheckState InitGetActionCheckStateDelegate, EUIActionRepeatMode InitRepeatMode = EUIActionRepeatMode::RepeatDisabled);

	/**
	 * Constructor that takes delegates to initialize the action with
	 *
	 * @param	ExecuteAction		The delegate to call when the action should be executed
	 * @param	CanExecuteAction	The delegate to call to see if the action can be executed
	 * @param	IsCheckedDelegate	The delegate to call to see if the action should appear checked when visualized
	 * @param	IsActionVisible		The delegate to call to see if the action should be visible
	 * @param	RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FIsActionChecked InitIsCheckedDelegate, FIsActionButtonVisible InitIsActionVisibleDelegate, EUIActionRepeatMode InitRepeatMode = EUIActionRepeatMode::RepeatDisabled);

	/**
	 * Constructor that takes delegates to initialize the action with
	 *
	 * @param	ExecuteAction		The delegate to call when the action should be executed
	 * @param	CanExecuteAction	The delegate to call to see if the action can be executed
	 * @param	GetActionCheckState	The delegate to call to see what the check state of the action should be
	 * @param	IsActionVisible		The delegate to call to see if the action should be visible
	 * @param	RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API FUIAction(FExecuteAction InitExecuteAction, FCanExecuteAction InitCanExecuteAction, FGetActionCheckState InitGetActionCheckStateDelegate, FIsActionButtonVisible InitIsActionVisibleDelegate, EUIActionRepeatMode InitRepeatMode = EUIActionRepeatMode::RepeatDisabled);

public:

	/**
	 * Checks to see if its currently safe to execute this action.
	 *
	 * @return	True if this action can be executed and we should reflect that state visually in the UI
	 */
	bool CanExecute( ) const
	{
		// Fire the 'can execute' delegate if we have one, otherwise always return true
		return CanExecuteAction.IsBound() ? CanExecuteAction.Execute() : true;
	}

	/**
	 * Executes this action
	 */
	bool Execute( ) const
	{
		// It's up to the programmer to ensure that the action is still valid by the time the user clicks on the
		// button.  Otherwise the user won't know why the action didn't take place!
		if( CanExecute() )
		{
			return ExecuteAction.ExecuteIfBound();
		}

		return false;
	}

	/**
	 * Queries the checked state for this action.  This is only valid for actions that are toggleable!
	 *
	 * @return	Checked state for this action
	 */
	ECheckBoxState GetCheckState( ) const
	{
		if (GetActionCheckState.IsBound())
		{
			return GetActionCheckState.Execute();
		}

		return ECheckBoxState::Unchecked;
	}

	/**
	 * Checks whether this action's execution delegate is bound.
	 *
	 * @return true if the delegate is bound, false otherwise.
	 */
	bool IsBound( ) const
	{
		return ExecuteAction.IsBound();
	}

	/**
	 * Queries the visibility for this action.
	 *
	 * @return	Visibility state for this action
	 */
	EVisibility IsVisible( ) const
	{
		if (IsActionVisibleDelegate.IsBound())
		{
			const bool bIsVisible = IsActionVisibleDelegate.Execute();
			return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Visible;
	}

	/**
	 * Checks whether this action can be repeated if the chord used to call it is held down.
	 *
	 * @return true if the delegate can be repeated, false otherwise.
	 */
	bool CanRepeat( ) const
	{
		return RepeatMode == EUIActionRepeatMode::RepeatEnabled;
	}

	/**
	 * Passthrough function to convert the result from an FIsActionChecked delegate into something that works with a FGetActionCheckState delegate
	 */
	static ECheckBoxState IsActionCheckedPassthrough(FIsActionChecked InDelegate)
	{
		if (InDelegate.IsBound())
		{
			const bool bIsChecked = InDelegate.Execute();
			return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}
};

/**
 * Base type for UI Action context object. 
 * 
 * Note: not using UObjects to avoid overhead.
 */
struct IUIActionContextBase
{

public:
	virtual FName GetContextName() const = 0;
};

/**
 * Container of multiple 'IUIActionContextBase' context objects for FUIActions.
 * Use if you need or want to associate a FUICommandInfo with additional data.
 * 
 * Ex: An FOnGetContent delegate that can be called to generate a widget.
 */
struct FUIActionContext
{
public:

	SLATE_API TSharedPtr<IUIActionContextBase> FindContext(const FName InName) const;

	SLATE_API void AddContext(const TSharedPtr<IUIActionContextBase>& InContext);

private:
	TArray<TSharedPtr<IUIActionContextBase>> Contexts;
};
