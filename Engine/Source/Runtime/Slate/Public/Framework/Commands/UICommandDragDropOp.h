// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxDefs.h"

class SWidget;

/**
 * A drag drop operation for UI Commands
 */
class FUICommandDragDropOp
	: public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FUICommandDragDropOp, FDragDropOperation)

	static SLATE_API TSharedRef<FUICommandDragDropOp> New( FName InItemName, EMultiBlockType InBlockType, bool bInIsDraggingSection, FName InOriginMultiBox, TSharedPtr<SWidget> CustomDectorator, FVector2D DecoratorOffset );

	FUICommandDragDropOp( FName InItemName, EMultiBlockType InBlockType, bool bInIsDraggingSection, FName InOriginMultiBox, TSharedPtr<SWidget> InCustomDecorator, FVector2D DecoratorOffset )
		: ItemName( InItemName )
		, BlockType( InBlockType )
		, bIsDraggingSection(bInIsDraggingSection)
		, OriginMultiBox( InOriginMultiBox )
		, CustomDecorator( InCustomDecorator )
		, Offset( DecoratorOffset )
	{ }

	/**
	 * Sets a delegate that will be called when the command is dropped
	 */
	void SetOnDropNotification( FSimpleDelegate InOnDropNotification ) { OnDropNotification = InOnDropNotification; }

	/** FDragDropOperation interface */
	SLATE_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	SLATE_API virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;

public:

	/** UI entry being dragged */
	FName ItemName;

	/** UI entry type being dragged */
	EMultiBlockType BlockType;

	/** UI entry being dragged is a section header or section separator */
	bool bIsDraggingSection;

	/** Multibox the UI command was dragged from if any*/
	FName OriginMultiBox;

	/** Custom decorator to display */
	TSharedPtr<SWidget> CustomDecorator;

	/** Offset from the cursor where the decorator should be displayed */
	FVector2D Offset;

	/** Delegate called when the command is dropped */
	FSimpleDelegate OnDropNotification;
};
