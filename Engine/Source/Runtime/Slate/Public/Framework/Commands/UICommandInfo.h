// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Textures/SlateIcon.h"
#include "Trace/SlateMemoryTags.h"
#include "Framework/Commands/InputChord.h"
#include "UICommandInfo.generated.h"

class FBindingContext;
class FUICommandInfo;

/** Types of user interfaces that can be associated with a user interface action */
UENUM(BlueprintType)
enum class EUserInterfaceActionType : uint8
{
	/** An action which should not be associated with a user interface action */
	None,

	/** Momentary buttons or menu items.  These support enable state, and execute a delegate when clicked. */
	Button,

	/** Toggleable buttons or menu items that store on/off state.  These support enable state, and execute a delegate when toggled. */
	ToggleButton,
		
	/** Radio buttons are similar to toggle buttons in that they are for menu items that store on/off state.  However they should be used to indicate that menu items in a group can only be in one state */
	RadioButton,

	/** Similar to Button but will display a readonly checkbox next to the item. */
	Check,

	/** Similar to Button but has the checkbox area collapsed */
	CollapsedButton
};

UENUM()
enum class EMultipleKeyBindingIndex : uint8
{
	Primary = 0,
	Secondary,
	NumChords
};

class FUICommandInfo;


/**
 *
 */
class FUICommandInfoDecl
{
	friend class FBindingContext;

public:

	SLATE_API FUICommandInfoDecl& DefaultChord( const FInputChord& InDefaultChord, const EMultipleKeyBindingIndex InChordIndex = EMultipleKeyBindingIndex::Primary);
	SLATE_API FUICommandInfoDecl& UserInterfaceType( EUserInterfaceActionType InType );
	SLATE_API FUICommandInfoDecl& Icon( const FSlateIcon& InIcon );
	SLATE_API FUICommandInfoDecl& Description( const FText& InDesc );

	SLATE_API operator TSharedPtr<FUICommandInfo>() const;
	SLATE_API operator TSharedRef<FUICommandInfo>() const;

public:

	SLATE_API FUICommandInfoDecl( const TSharedRef<class FBindingContext>& InContext, const FName InCommandName, const FText& InLabel, const FText& InDesc, const FName InBundle = NAME_None);

private:

	TSharedPtr<class FUICommandInfo> Info;
	const TSharedRef<FBindingContext>& Context;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingContextChanged, const FBindingContext&);

/**
 * Represents a context in which input bindings are valid
 */
class FBindingContext
	: public TSharedFromThis<FBindingContext>
{
public:
	/**
	 * Constructor
	 *
	 * @param InContextName		The name of the context
	 * @param InContextDesc		The localized description of the context
	 * @param InContextParent	Optional parent context.  Bindings are not allowed to be the same between parent and child contexts
	 * @param InStyleSetName	The style set to find the icons in, eg) FCoreStyle::Get().GetStyleSetName()
	 */
	FBindingContext( const FName InContextName, const FText& InContextDesc, const FName InContextParent, const FName InStyleSetName )
		: ContextName( InContextName )
		, ContextParent( InContextParent )
		, ContextDesc( InContextDesc )
		, StyleSetName( InStyleSetName )
	{
		check(!InStyleSetName.IsNone());
	}

	FBindingContext(const FBindingContext&) = default;
	FBindingContext(FBindingContext&&) = default;
	FBindingContext& operator=(const FBindingContext&) = default;
	FBindingContext& operator=(FBindingContext&&) = default;

	/**
	 * Creates a new command declaration used to populate commands with data
	 */
	SLATE_API FUICommandInfoDecl NewCommand( const FName InCommandName, const FText& InCommandLabel, const FText& InCommandDesc );

	/**
	 * @return The name of the context
	 */
	FName GetContextName() const { return ContextName; }

	/**
	 * @return The name of the parent context (or NAME_None if there isnt one)
	 */
	FName GetContextParent() const { return ContextParent; }

	/**
	 * @return The name of the style set to find the icons in
	 */
	FName GetStyleSetName() const { return StyleSetName; }

	/**
	 * @return The localized description of this context
	 */
	const FText& GetContextDesc() const { return ContextDesc; }

	/**
	 * Adds a new command bundle to this context that can be referenced by name
	 * from commands within the context.
	 *
	 * @param Name A unique identifier for the bundle in this context
	 * @param Desc A localized description of the bundle
	 */
	SLATE_API void AddBundle(const FName Name, const FText& Desc);

	/**
	 * Gets a localized label of a command bundle
	 *
	 * @param Name The name of the bundle to get a label for
	 * @return The localized label of the bundle
	 */
	SLATE_API const FText& GetBundleLabel(const FName Name);

	friend uint32 GetTypeHash( const FBindingContext& Context )
	{
		return GetTypeHash( Context.ContextName );
	}

	bool operator==( const FBindingContext& Other ) const
	{
		return ContextName == Other.ContextName;
	}

	/** A delegate that is called when commands are registered or unregistered with a binding context */
	static SLATE_API FOnBindingContextChanged CommandsChanged;

private:

	/** The name of the context */
	FName ContextName;

	/** The name of the parent context */
	FName ContextParent;

	/** The description of the context */
	FText ContextDesc;

	/** The style set to find the icons in */
	FName StyleSetName;

	/** A list of command bundles and their friendly names that can be referenced in this context */
	TMap<FName, FText> Bundles;
};


class FUICommandInfo
{
	friend class FInputBindingManager;
	friend class FUICommandInfoDecl;

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InBindingContext The name of the binding context to use.
	 */
	SLATE_API FUICommandInfo(const FName InBindingContext);

	/**
	 * Returns the friendly, localized string name of the first valid chord in the key bindings list that is required to perform the command
	 *
	 * @return	Localized friendly text for the chord
	 */
	SLATE_API const FText GetInputText() const;

	/**
	 * @return	Returns the active chord at the specified index for this command
	 */
	const TSharedRef<const FInputChord> GetActiveChord(const EMultipleKeyBindingIndex InChordIndex) const { return ActiveChords[static_cast<uint8>(InChordIndex)]; }
	
	/**
	* @return	Checks if there is an active chord for this command matching the input chord
	*/
	const bool HasActiveChord(const FInputChord InChord) const { return *ActiveChords[static_cast<uint8>(EMultipleKeyBindingIndex::Primary)] == InChord ||
																	*ActiveChords[static_cast<uint8>(EMultipleKeyBindingIndex::Secondary)] == InChord; }

	/**
	* @return	Checks if there is an active chord for this command matching the input chord
	*/
	const TSharedRef<const FInputChord> GetFirstValidChord() const {
		return ActiveChords[static_cast<uint8>(EMultipleKeyBindingIndex::Primary)]->IsValidChord()
				? ActiveChords[static_cast<uint8>(EMultipleKeyBindingIndex::Primary)]
				: ActiveChords[static_cast<uint8>(EMultipleKeyBindingIndex::Secondary)];
	}

	/**
	* @return	Checks if there is an active chord for this command matching the input chord
	*/
	const bool HasDefaultChord(const FInputChord InChord) const {
		return (DefaultChords[static_cast<uint8>(EMultipleKeyBindingIndex::Primary)] == InChord) ||
			(DefaultChords[static_cast<uint8>(EMultipleKeyBindingIndex::Secondary)] == InChord);
	}

	const FInputChord& GetDefaultChord(const EMultipleKeyBindingIndex InChordIndex) const { return DefaultChords[static_cast<uint8>(InChordIndex)]; }

	/** Utility function to make an FUICommandInfo */
	static SLATE_API void MakeCommandInfo( const TSharedRef<class FBindingContext>& InContext, TSharedPtr< FUICommandInfo >& OutCommand, const FName InCommandName, const FText& InCommandLabel, const FText& InCommandDesc, const FSlateIcon& InIcon, const EUserInterfaceActionType InUserInterfaceType, const FInputChord& InDefaultChord, const FInputChord& InAlternateDefaultChord = FInputChord(), const FName InBundle = NAME_None);

	/** Utility function to unregister an FUICommandInfo */
	static SLATE_API void UnregisterCommandInfo(const TSharedRef<class FBindingContext>& InContext, const TSharedRef<FUICommandInfo>& InCommand);

	/** @return The display label for this command */
	const FText& GetLabel() const { return Label; }

	/** @return The description of this command */
	const FText& GetDescription() const { return Description; }

	/** @return The icon to used when this command is displayed in UI that shows icons */
	const FSlateIcon& GetIcon() const { return Icon; }

	/** @return The type of command this is.  Used to determine what UI to create for it */
	EUserInterfaceActionType GetUserInterfaceType() const { return UserInterfaceType; }
	
	/** @return The name of the command */
	FName GetCommandName() const { return CommandName; }

	/** @return The name of the context where the command is valid */
	FName GetBindingContext() const { return BindingContext; }

	/** @return The name of the bundle this command is assigned to */
	FName GetBundle() const { return Bundle; }

	/** @return True if should we use long names for when getting text for input chords */
	bool GetUseLongDisplayName() const { return bUseLongDisplayName; }

	/** Sets if we should use a long display name or not */
	void SetUseLongDisplayName(const bool bInUseLongDisplayName) { bUseLongDisplayName = bInUseLongDisplayName; }

	/** Sets the new active chord for this command */
	SLATE_API void SetActiveChord( const FInputChord& NewChord, const EMultipleKeyBindingIndex InChordIndex );

	/** Removes the active chord from this command */
	SLATE_API void RemoveActiveChord(const EMultipleKeyBindingIndex InChordIndex);

	/** 
	 * Makes a tooltip for this command.
	 * @param	InText	Optional dynamic text to be displayed in the tooltip.
	 * @return	The tooltip widget
	 */
	SLATE_API TSharedRef<class SToolTip> MakeTooltip( const TAttribute<FText>& InText = TAttribute<FText>() , const TAttribute< EVisibility >& InToolTipVisibility = TAttribute<EVisibility>()) const;

private:

	/** Input commands that executes this action */
	TArray<TSharedRef<FInputChord>> ActiveChords;

	/** Default display name of the command */
	FText Label;

	/** Localized help text for this command */
	FText Description;

	/** The default input chords for this command (can be invalid) */
	TArray<FInputChord> DefaultChords;

	/** Brush name for icon to use in tool bars and menu items to represent this command */
	FSlateIcon Icon;

	/** Brush name for icon to use in tool bars and menu items to represent this command in its toggled on (checked) state*/
	FName UIStyle;

	/** Name of the command */
	FName CommandName;

	/** The context in which this command is active */
	FName BindingContext;

	/** The bundle to group this command into. The bundle must have been added to the BindingContext first. */
	FName Bundle;

	/** The type of user interface to associated with this action */
	EUserInterfaceActionType UserInterfaceType;

	/** True if should we use long names for when getting text for input chords */
	bool bUseLongDisplayName;
};
