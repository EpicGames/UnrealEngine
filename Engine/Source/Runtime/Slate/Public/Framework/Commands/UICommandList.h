// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"

struct FKeyEvent;
struct FPointerEvent;

class FUICommandList
	: public TSharedFromThis<FUICommandList>
{
public:

	/** Virtual destructor */
	virtual ~FUICommandList() = default;

	/** Determines if this UICommandList is capable of producing an action for the supplied command */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FCanProduceActionForCommand, const TSharedRef<const FUICommandInfo>& /*Command*/);

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo	The command info to map
	 * @param ExecuteAction		The delegate to call when the command should be executed
	 * @param RepeatMode		Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, EUIActionRepeatMode RepeatMode = EUIActionRepeatMode::RepeatDisabled );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo	The command info to map
	 * @param ExecuteAction		The delegate to call when the command should be executed
	 * @param CanExecuteAction	The delegate to call to see if the command can be executed
	 * @param RepeatMode		Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, EUIActionRepeatMode RepeatMode = EUIActionRepeatMode::RepeatDisabled );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo	The command info to map
	 * @param ExecuteAction		The delegate to call when the command should be executed
	 * @param CanExecuteAction	The delegate to call to see if the command can be executed
	 * @param IsCheckedDelegate	The delegate to call to see if the command should appear checked when visualized in a multibox
	 * @param RepeatMode		Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FIsActionChecked IsCheckedDelegate, EUIActionRepeatMode RepeatMode = EUIActionRepeatMode::RepeatDisabled );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo		The command info to map
	 * @param ExecuteAction			The delegate to call when the command should be executed
	 * @param CanExecuteAction		The delegate to call to see if the command can be executed
	 * @param GetActionCheckState	The delegate to call to see what the check state of the action should be when visualized in a multibox
	 * @param RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FGetActionCheckState GetActionCheckState, EUIActionRepeatMode RepeatMode = EUIActionRepeatMode::RepeatDisabled );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo	The command info to map
	 * @param ExecuteAction		The delegate to call when the command should be executed
	 * @param CanExecuteAction	The delegate to call to see if the command can be executed
	 * @param IsCheckedDelegate	The delegate to call to see if the command should appear checked when visualized in a multibox
	 * @param IsVisibleDelegate	The delegate to call to see if the command should appear or be hidden when visualized in a multibox
	 * @param RepeatMode		Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FIsActionChecked IsCheckedDelegate, FIsActionButtonVisible IsVisibleDelegate, EUIActionRepeatMode RepeatMode = EUIActionRepeatMode::RepeatDisabled );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo		The command info to map
	 * @param ExecuteAction			The delegate to call when the command should be executed
	 * @param CanExecuteAction		The delegate to call to see if the command can be executed
	 * @param GetActionCheckState	The delegate to call to see what the check state of the action should be when visualized in a multibox
	 * @param IsVisibleDelegate		The delegate to call to see if the command should appear or be hidden when visualized in a multibox
	 * @param RepeatMode			Can this action can be repeated if the chord used to call it is held down?
	 */
	SLATE_API void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExecuteAction, FGetActionCheckState GetActionCheckState, FIsActionButtonVisible IsVisibleDelegate, EUIActionRepeatMode RepeatMode = EUIActionRepeatMode::RepeatDisabled );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 *
	 * @param InUICommandInfo	The command info to map
	 * @param InUIAction		Action to map to this command
	 */
	SLATE_API virtual void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, const FUIAction& InUIAction );

	/**
	 * Maps a command info to a series of delegates that are executed by a multibox or mouse/keyboard input
	 * Additionally, add a container of context objects to also associate with this command
	 *
	 * @param InUICommandInfo	The command info to map
	 * @param InUIAction		Action to map to this command
	 * @param InUIActionContext	Action context to map to this command
	 */
	SLATE_API virtual void MapAction(const TSharedPtr< const FUICommandInfo > InUICommandInfo, const FUIAction& InUIAction, const FUIActionContext& InUIActionContext);

	/**
	 * Append commands in InCommandsToAppend to this command list.
	 */
	SLATE_API void Append( const TSharedRef<FUICommandList>& InCommandsToAppend );

	/**
	 * Unmaps a command info that was previously mapped
	 *
	 * @param InUICommandInfo		The command info to unmap
	 */
	SLATE_API void UnmapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo );

	/**
	 * Checks if an action is already mapped
	 * 
	 * @return True if an action is already mapped or false otherwise
	 */
	SLATE_API bool IsActionMapped( const TSharedPtr< const FUICommandInfo > InUICommandInfo ) const;

	/**
	 * Executes the action associated with the provided command info
	 * Note: It is assumed at this point that CanExecuteAction was already checked
	 *
	 * @param InUICommandInfo	The command info execute
	 */
	SLATE_API virtual bool ExecuteAction( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const;

	/**
	 * Calls the CanExecuteAction associated with the provided command info to see if ExecuteAction can be called
	 *
	 * @param InUICommandInfo	The command info execute
	 */
	SLATE_API bool CanExecuteAction( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const;

	/**
	 * Attempts to execute the action associated with the provided command info
	 * Note: This will check if the action can be executed before finally executing the action
	 *
	 * @param InUICommandInfo	The command info execute
	 */
	SLATE_API bool TryExecuteAction( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const;

	/**
	 * Calls the IsVisible delegate associated with the provided command info to see if the command should be visible in a toolbar
	 *
	 * @param InUICommandInfo	The command info execute
	 */
	SLATE_API EVisibility GetVisibility( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const;

	/**
	 * Calls the GetCheckState delegate to see if the visualization of this command in a multibox should appear checked
	 *
	 * @param InUICommandInfo	The command info execute
	 */
	SLATE_API ECheckBoxState GetCheckState( const TSharedRef< const FUICommandInfo > InUICommandInfo ) const;

	/**
	 * Processes any UI commands which are activated by the specified key event
	 *
	 * @param InKeyEvent	The key event to check
	 * @return true if an action was processed
	 */
	SLATE_API bool ProcessCommandBindings( const FKeyEvent& InKeyEvent ) const;

	/**
	 * Processes any UI commands which are activated by the specified mouse event
	 *
	 * @param InKeyEvent	The mouse event to check
	 * @return true if an action was processed
	 */
	SLATE_API bool ProcessCommandBindings( const FPointerEvent& InMouseEvent ) const;

	/**
	 * Processes any UI commands which are activated by the specified key, modifier keys state and input event
	 *
	 * @param Key				The current key that is pressed
	 * @param ModifierKeysState	Pressed state of keys that are commonly used as modifiers
	 * @param bRepeat			True if input is repeating (held)
	 * @return true if an action was processed
	 */
	SLATE_API bool ProcessCommandBindings( const FKey Key, const FModifierKeysState& ModifierKeysState, const bool bRepeat ) const;

	/** Sets the delegate that determines if this UICommandList is capable of producing an action for the supplied command */
	void SetCanProduceActionForCommand( const FCanProduceActionForCommand& NewCanProduceActionForCommand ) { CanProduceActionForCommand = NewCanProduceActionForCommand; }

	/** 
	  * Attempts to find an action for the specified command in the current UICommandList. This is a wrapper for GetActionForCommandRecursively.
	  *
	  * @param Command				The UI command for which you are discovering an action
	  */
	SLATE_API const FUIAction* GetActionForCommand(TSharedPtr<const FUICommandInfo> Command) const;

	/** 
	  * Attempts to find an context for the specified command in the current UICommandList. This is a wrapper for GetContextForCommandRecursively.
	  *
	  * @param Command				The UI command for which you are discovering an action
	  */
	SLATE_API const FUIActionContext* GetContextForCommand(TSharedPtr<const FUICommandInfo> Command) const;

protected:

	/**
	 * Helper function to execute delegate or exec command associated with a command (if valid)
	 *
	 * @param Key		The current key that is pressed
	 * @param bCtrl		True if control is pressed
	 * @param bAlt		True if alt is pressed
	 * @param bShift	True if shift is pressed
	 * @param bRepeat	True if command is repeating (held)
	 * @return True if a command was executed, False otherwise
	 */
	SLATE_API bool ConditionalProcessCommandBindings( const FKey Key, bool bCtrl, bool bAlt, bool bShift, bool bCmd, bool bRepeat ) const;

	/** 
	  * Attempts to find an action for the specified command in the current UICommandList. If it is not found, the action for the
	  * specified command is discovered in the children recursively then the parents recursively.
	  *
	  * @param Command				The UI command for which you are discovering an action
	  * @param bIncludeChildren		If true, children of this command list will be searched in the event that the action is not found
	  * @param bIncludeParents		If true, parents of this command list will be searched in the event that the action is not found
	  * @param OutVisitedLists		The set of visited lists during recursion. This is used to prevent cycles.
	  */
	SLATE_API const FUIAction* GetActionForCommandRecursively(const TSharedRef<const FUICommandInfo>& Command, bool bIncludeChildren, bool bIncludeParents, TSet<TSharedRef<const FUICommandList>>& InOutVisitedLists) const;

	/**
	  * Attempts to find an context for the specified command in the current UICommandList. If it is not found, the context for the
	  * specified command is discovered in the children recursively then the parents recursively.
	  *
	  * @param Command				The UI command for which you are discovering an action
	  * @param bIncludeChildren		If true, children of this command list will be searched in the event that the action is not found
	  * @param bIncludeParents		If true, parents of this command list will be searched in the event that the action is not found
	  * @param OutVisitedLists		The set of visited lists during recursion. This is used to prevent cycles.
	  */
	SLATE_API const FUIActionContext* GetContextForCommandRecursively(const TSharedRef<const FUICommandInfo>& Command, bool bIncludeChildren, bool bIncludeParents, TSet<TSharedRef<const FUICommandList>>& InOutVisitedLists) const;

	/** Returns all contexts associated with this list. This is a wrapper for GatherContextsForListRecursively */
	SLATE_API void GatherContextsForList(TSet<FName>& OutAllContexts) const;

	/** Returns all contexts associated with this list. */
	SLATE_API void GatherContextsForListRecursively(TSet<FName>& OutAllContexts, TSet<TSharedRef<const FUICommandList>>& InOutVisitedLists) const;

private:

	using FUIBindingMap = TMap<const TSharedPtr<const FUICommandInfo>, FUIAction>;
	using FUIContextMap = TMap<const TSharedPtr<const FUICommandInfo>, FUIActionContext>;
	
	/** Known contexts in this list.  Each context must be known so we can quickly look up commands from bindings */
	TSet<FName> ContextsInList;

	/** Mapping of command to action */
	FUIBindingMap UICommandBindingMap;

	/** Mapping of command to action context, used for special cases and thus much more sparse */
	FUIContextMap UICommandContextMap;

	/** The list of parent and children UICommandLists */
	TArray<TWeakPtr<FUICommandList>> ParentUICommandLists;
	TArray<TWeakPtr<FUICommandList>> ChildUICommandLists;

	/** Determines if this UICommandList is capable of producing an action for the supplied command */
	FCanProduceActionForCommand CanProduceActionForCommand;

	/** Helper struct for list recursion */
	friend struct RecursiveCommandDataGetter;
};
