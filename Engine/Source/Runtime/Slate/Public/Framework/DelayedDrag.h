// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Framework/Application/SlateApplication.h"

/** Helper class to assist with delayed drag operations */
class FDelayedDrag
{
public:

	/** Construct this drag helper with an initial posision, and a key (probably mouse button) required for dragging */
	FDelayedDrag(FVector2D InInitialPosition, FKey InEffectiveKey)
		: InitialPosition(InInitialPosition), bHasInitiatedDrag(false), DistanceDragged(0), EffectiveKey(InEffectiveKey)
	{
		TriggerDistance = FSlateApplication::Get().GetDragTriggerDistance();
	}

	/** Get the initial start position (before any drag has started) */
	const FVector2D& GetInitialPosition() const { return InitialPosition; }

	/** Check whether we have initiated a drag or not */
	bool IsDragging() const { return bHasInitiatedDrag; }

	/** Force the state of this helper to be 'dragging' */
	void ForceDragStart() { bHasInitiatedDrag = true; }

	/** Attempt to start a drag from the given mouse event. */
	bool AttemptDragStart(const FPointerEvent& MouseEvent)
	{
		if (!bHasInitiatedDrag && MouseEvent.IsMouseButtonDown(EffectiveKey))
		{
			DistanceDragged += MouseEvent.GetCursorDelta().Size();
			if (!bHasInitiatedDrag && DistanceDragged > TriggerDistance)
			{
				ForceDragStart();
				return true;
			}
		}
		return false;
	}

	/** Assign a new scale factor to apply to the drag trigger distance */
	void SetTriggerScaleFactor(float InTriggerScaleFactor)
	{
		TriggerDistance = FMath::Max(FSlateApplication::Get().GetDragTriggerDistance() * InTriggerScaleFactor, 1.f);
	}

protected:

	/** The initial position of the drag start */
	FVector2D InitialPosition;
	/** True where the distance dragged is sufficient to have started a drag */
	bool bHasInitiatedDrag;
	/** The amount we have dragged */
	float DistanceDragged;
	/** The minimum distance that must be moved before the drag initiates. */
	float TriggerDistance;
	/** The key that must be pressed to initiate the drag */
	FKey EffectiveKey;
};
