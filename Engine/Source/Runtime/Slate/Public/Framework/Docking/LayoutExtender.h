// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "Framework/Docking/TabManager.h"

typedef TFunction<void(TSharedRef<FTabManager::FArea>)> FAreaExtension;

/** Extension position enumeration */
enum class ELayoutExtensionPosition
{
	/** Extend the layout before the specified element */
	Before,
	/** Extend the layout after the specified element */
	After,
	/** Extend the layout above the specified element in the parent splitter */
	Above,
	/** Extend the layout below the specified element in the parent splitter */
	Below,
};

/** Class used for extending default layouts */
class FLayoutExtender : public TSharedFromThis<FLayoutExtender>
{
public:

	/**
	 * Extend the layout by defining a tab before or after the specified predicate tab ID
	 *
	 * @param PredicateTabId		The existing tab to place the extended tab before/after
	 * @param Position				Whether to place the new tab before or after this tab
	 * @param TabToAdd				The new tab definition
	 */
	SLATE_API void ExtendLayout(FTabId PredicateTabId, ELayoutExtensionPosition Position, FTabManager::FTab TabToAdd);


	/**
	 * Extend the area identified by the specified extension ID
	 *
	 * @param ExtensionId			The ID of the area to extend (FTabManager::FArea::ExtensionId)
	 * @param AreaExtensions		A callback to call with the default layout for the desired area
	 */
	SLATE_API void ExtendArea(FName ExtensionId, const FAreaExtension& AreaExtension);

	/**
	 * Extend the tab stack identified by the specified extension ID
	 *
	 * @param ExtensionId			The ID of the area to extend (FTabManager::FStack::ExtensionId)
	 * @param AreaExtensions		A callback to call with the default layout for the desired area
	 * @param Position				Whether to place the new tab at the start or end of this stack. Above and below are not used for this case.
	 * @param TabToAdd				The new tab definition
	 * @param bShouldBeForegrounded	Whether the tab should be foregrounded, first foreground tab found when applying extension will be used. So competing foregrounded tabs are not handled.
	 */
	SLATE_API void ExtendStack(FName ExtensionId, ELayoutExtensionPosition Position, FTabManager::FTab TabToAdd, bool bShouldBeForegrounded=false);


	/**
	 * Populate the specified container with the tabs that relate to the specified tab ID
	 *
	 * @param TabId					The existing tab that may be extended
	 * @param Position				The position to acquire extensions for (before/after)
	 * @param OutValues				The container to populate with extended tabs
	 */
	template<typename AllocatorType>
	void FindTabExtensions(FTabId TabId, ELayoutExtensionPosition Position, TArray<FTabManager::FTab, AllocatorType>& OutValues) const
	{
		OutValues.Reset();

		TArray<FExtendedTab, TInlineAllocator<2>> Extensions;
		TabExtensions.MultiFind(TabId, Extensions, true);

		for (FExtendedTab& Extension : Extensions)
		{
			if (Extension.Position == Position)
			{
				OutValues.Add(Extension.TabToAdd);
			}
		}
	}

	/**
	 * Populate the specified container with the tabs that relate to the specified stack
	 *
	 * @param TabId					The existing tab that may be extended
	 * @param Position				The position to acquire extensions for (before/after)
	 * @param OutValues				The container to populate with extended tabs
	 */
	template<typename AllocatorType>
	void FindStackExtensions(FName ExtensionID, ELayoutExtensionPosition Position, TArray<FTabManager::FTab, AllocatorType>& OutValues) const
	{
		OutValues.Reset();

		TArray<FExtendedTab, TInlineAllocator<2>> Extensions;
		StackExtensions.MultiFind(ExtensionID, Extensions, true);

		for (FExtendedTab& Extension : Extensions)
		{
			if (Extension.Position == Position)
			{
				OutValues.Add(Extension.TabToAdd);
			}
		}
	}
	
	FTabId FindFirstForegroundTab(FName ExtensionID) const
	{
		TArray<FExtendedTab, TInlineAllocator<2>> Extensions;
		StackExtensions.MultiFind(ExtensionID, Extensions, true);

		for (FExtendedTab& Extension : Extensions)
		{
			if (Extension.bShouldBeForegrounded)
			{
				return Extension.TabToAdd.TabId;
			}
		}
		
		return FTabId();
	}

	/**
	 * Recursively extend the specified area
	 */
	void ExtendAreaRecursive(const TSharedRef<FTabManager::FArea>& Area) const;

private:
	
	/** Extended tab information */
	struct FExtendedTab
	{
		FExtendedTab(ELayoutExtensionPosition InPosition, const FTabManager::FTab& InTabToAdd, bool bInShouldBeForegrounded=false)
			: Position(InPosition), TabToAdd(InTabToAdd), bShouldBeForegrounded(bInShouldBeForegrounded)
		{}

		/** The tab position */
		ELayoutExtensionPosition Position;
		/** The tab definition */
		FTabManager::FTab TabToAdd;
		/** Whether the tab should be foregrounded, competing foregrounded tabs are not handled */ 
		bool bShouldBeForegrounded;
	};

	/** Extended area information */
	struct FExtendedArea
	{
		FExtendedArea(const FAreaExtension& InExtensionCallback)
			: ExtensionCallback(InExtensionCallback)
		{}

		/** The area extension callback */
		FAreaExtension ExtensionCallback;
	};

	/** Map of extensions for tabs */
	TMultiMap<FTabId, FExtendedTab> TabExtensions;

	/** Map of extensions for stacks */
	TMultiMap<FName, FExtendedTab> StackExtensions;

	/** Map of extensions for areas */
	TMultiMap<FName, FExtendedArea> AreaExtensions;
};
