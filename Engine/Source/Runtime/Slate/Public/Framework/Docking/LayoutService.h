// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/TabManager.h"

struct FLayoutSaveRestore
{
	/** Gets the ini section label for the additional layout configs */
	static SLATE_API const FString& GetAdditionalLayoutConfigIni();

	/**
	 * Write the layout out into a named config file.
	 *
	 * @param InConfigFileName file to be saved to.
	 * @param InLayoutToSave the layout to save.
	 */
	static SLATE_API void SaveToConfig(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InLayoutToSave );

	/**
	 * Given a named DefaultLayout, return any saved version of it from the given ini file, otherwise return the default, also default to open tabs based on bool.
	 *
	 * @param InConfigFileName File to be used to load an existing layout.
	 * @param InDefaultLayout The layout to be used if the file does not exist.
	 * @param InPrimaryAreaOutputCanBeNullptr Analog to the EOutputCanBeNullptr argument of FTabManager::RestoreFrom(), but only applied to the PrimaryArea. It
	 * specifies if the primary area can still be nullptr even if no valid tabs (or opened tabs) are found. By default, set to EOutputCanBeNullptr::Never.
	 * @param InOutRemovedOlderLayoutVersions If this TArray is not added, default behavior. If it is added as an argument, older versions of this layout field
	 * will be also cleaned from the layout ini file and their names returned in OutRemovedOlderLayoutVersions. To be precise, it will remove fields with a
	 * name that contains the same characters, other than the final number(s) and dot(s). E.g., for "UnrealEd_Layout_v1.4", any layout field starting by
	 * "UnrealEd_Layout_v" with some numbers and/or dots after the final "v".
	 *
	 * @return Loaded layout or the default.
	 */
	static SLATE_API TSharedRef<FTabManager::FLayout> LoadFromConfig(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InDefaultLayout,
		const EOutputCanBeNullptr InPrimaryAreaOutputCanBeNullptr = EOutputCanBeNullptr::Never);
	static SLATE_API TSharedRef<FTabManager::FLayout> LoadFromConfig(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InDefaultLayout,
		const EOutputCanBeNullptr InPrimaryAreaOutputCanBeNullptr, TArray<FString>& OutRemovedOlderLayoutVersions);

	/**
	 * Write the desired value into the desired section of a named config file.
	 * This function should only be used to save localization names (e.g., LayoutName, LayoutDescription).
	 * For saving the FTabManager::FLayout, check SaveToConfig.
	 *
	 * @param InConfigFileName file to be saved to.
	 * @param InSectionName the section name where to save the value.
	 * @param InSectionValue the value to save.
	 */
	static SLATE_API void SaveSectionToConfig(const FString& InConfigFileName, const FString& InSectionName, const FString& InSectionValue);

	UE_DEPRECATED(5.7, "SaveSectionToConfig using FText value is deprecated. Please use the SaveSectionToConfig function using FString.")
	static SLATE_API void SaveSectionToConfig(const FString& InConfigFileName, const FString& InSectionName, const FText& InSectionValue);

	/**
	 * Read the desired FText value from the desired section of a named config file.
	 * This function should only be used to load localization names (e.g., LayoutName, LayoutDescription).
	 * For loading the FTabManager::FLayout, check SaveToConfig.
	 *
	 * @param InConfigFileName file to be used to load an existing layout.
	 * @param InSectionName the name of the section to be read.
	 * @param bIsOptional if false, an error may be output if the section doesn't exist.
	 *
	 * @return Loaded FText associated for that section.
	 */
	static SLATE_API FText LoadSectionFromConfig(const FString& InConfigFileName, const FString& InSectionName, const bool bIsOptional = false);

	/**
	 * Migrates the layout configuration from one config file to another.
	 *
	 * @param OldConfigFileName The name of the old configuration file.
	 * @param NewConfigFileName The name of the new configuration file.
	 */
	static SLATE_API void MigrateConfig(const FString& OldConfigFileName, const FString& NewConfigFileName);

	/** 
	 * Duplicate the layout config from one file to another.
	 * @param SourceConfigFileName The name of the source configuration file.
	 * @param TargetConfigFileName The name of the target configuration file.
	 */
	static SLATE_API bool DuplicateConfig(const FString& SourceConfigFileName, const FString& TargetConfigFileName);

	/**
	 * It checks whether a file is a valid layout config file.
	 * @param InConfigFileName file to be read.
	 * @param bAllowFallback Whether a config should be considered valid if EditorLayoutsSectionName is missing but DefaultEditorLayoutsSectionName is found
	 * @return Whether the file is a valid layout config file.
	 */
	static SLATE_API bool IsValidConfig(const FString& InConfigFileName, bool bAllowFallback = true);

private:

	/**
	 * Auxiliary function for both public versions of LoadFromConfig
	 *
	 * @param bInRemoveOlderLayoutVersions If true, it will be equivalent to use the public LoadFromConfig() with
	 * "TArray<FString>& OutRemovedOlderLayoutVersions". If false, it won't search nor remove old versions (i.e.,
	 * public LoadFromConfig() without the final TArray argument).
	 */
	static TSharedRef<FTabManager::FLayout> LoadFromConfigPrivate(const FString& InConfigFileName, const TSharedRef<FTabManager::FLayout>& InDefaultLayout,
		const EOutputCanBeNullptr InPrimaryAreaOutputCanBeNullptr, const bool bInRemoveOlderLayoutVersions, TArray<FString>& OutRemovedOlderLayoutVersions);
};
