// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Layout/SSplitter.h"

class SDockingArea;
class SDockingSplitter;

/**
 * A node in the Docking/Tabbing hierarchy.
 * Any SDockingNode can be either a stack of tabs or a splitter.
 */
class SDockingNode : public SCompoundWidget
{
public:
	enum Type
	{
		/**
		 * A tab stack is a collection of tabs with associated content.
		 * Only one of the tabs is active at a time; its content will be shown.
		 */
		DockTabStack,
		/** Displays multiple child SDockNodes horizontally or vertically. */
		DockSplitter,
		/** Top-level dock splitters */
		DockArea,
		/** Takes up some room during layout re-arranging. */
		PlaceholderNode
	};

	/** Direction relative to some DockNode. For example, we may wish to say "Dock a tab RightOf this node." */
	enum RelativeDirection
	{
		LeftOf,
		Above,
		RightOf,
		Below,
		Center
	};

	/** Tracking for the docking cross, content, and background of the TabStack */
	struct FOverlayManagement
	{
		FOverlayManagement()
			: bShowingCross(false)
		{
		}

		/** The overlay widget that shows TabStack's content*/
		TSharedPtr<SOverlay> ContentAreaOverlay;

		/** true when we're showing the DockCross */
		bool bShowingCross;
	};

	/** @return Is this dock node a tab stack, splitter or something else? */
	virtual Type GetNodeType() const = 0;

	/**
	 * All DockNodes are aware of their parent DockNode (unless they are a root node; a.k.a DockArea)
	 * The parent is set whenever a node is inserted into a docking hierarchy.
	 *
	 * @param InParent      Set this node's parent.
	 */
	virtual void SetParentNode( TSharedRef<SDockingSplitter> InParent )
	{
		ParentNodePtr = InParent;
	}

	/** A tab can be removed from a stack when a uset drags it away or when the user closes it */
	enum ELayoutModification
	{
		/** Tab was dragged out of the node */
		TabRemoval_DraggedOut,
		/** Tab is being closed */
		TabRemoval_Closed,
		/** Tab is being put in a sidebar */
		TabRemoval_Sidebar,
		TabRemoval_None
	};
	

	/** Gets the dock area that this resides in */
	SLATE_API virtual TSharedPtr<SDockingArea> GetDockArea();
	SLATE_API virtual TSharedPtr<const SDockingArea> GetDockArea() const;
	
	/** Recursively searches through all children looking for child tabs */
	virtual TArray< TSharedRef<SDockTab> > GetAllChildTabs() const {return TArray< TSharedRef<SDockTab> >();}

	/** Gets the number of tabs in all children recursively */
	virtual int32 GetNumTabs() const { return 0; }

	/**
	 * Attempt to dock the TabWidget from the DragDropEvent next to this node.
	 * 
	 * @param Direction       Where relative to this direction to dock the node: e.g. to the right
	 * @param DragDropEvent   DragAndDrop event that potentially carries
	 *
	 * @return An FReply::Handled() if we successfully docked the content; FReply::Unhandled() otherwise.
	 */
	SLATE_API virtual FReply OnUserAttemptingDock( SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent );

	/** Should this node auto-size or be a percentage of its parent size; this setting is usually determined by users */
	virtual SSplitter::ESizeRule GetSizeRule() const { return SSplitter::FractionOfParent; }

	/** Minimum pixel size for this node when it is a slot in a parent splitter. Zero means no minimum. */
	virtual float GetNodeMinSize() const { return 0.f; }

	/** @return the numerator for the fraction of available space that this dock node should occupy. */
	SLATE_API float GetSizeCoefficient() const;

	/** Set the coefficient size */
	SLATE_API void SetSizeCoefficient( float InSizeCoefficient );

	/** @returns true when this node is showing a live DockableTab; false when this nodes content is only kept around for history's sake */
	//virtual bool IsShowingLiveTabs() const;

	/**
	 * Recursively build up an tree of FLayoutNodes that represent the persistent layout state of this DockingNode and all its descendants.
	 * The result can be easily serialized to disk or used as history for restoring tabs to their previous locations.
	 *
	 * @return a pointer to persistent layout node; invalid pointer if this dock node had no tab layouts worth saving.
	 */
	virtual TSharedPtr<FTabManager::FLayoutNode> GatherPersistentLayout() const = 0;

	enum ECleanupRetVal
	{
		VisibleTabsUnderNode,
		HistoryTabsUnderNode,
		NoTabsUnderNode
	};
	virtual SDockingNode::ECleanupRetVal CleanUpNodes(){ return NoTabsUnderNode; }

protected:

	SLATE_API SDockingNode();

	/** A live tab was added to this node or one of its descendants */
	SLATE_API virtual void OnLiveTabAdded();

	virtual void OnResized() {}

	/**
	 * Weak reference to the parent node. It is NULL until the node
	 * is inserted into the hierarchy. Also NULL for root nodes (aka SDockingArea)
	 */
	TWeakPtr<SDockingSplitter> ParentNodePtr;

	float SizeCoefficient = 1.0f;
};

