// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Animation/CurveSequence.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/SDockingNode.h"

#define DEBUG_TAB_MANAGEMENT 0

class FUICommandList;
class FWeakWidgetPath;
class FWidgetPath;
class SDockingTabWell;
class SBorder;

struct FDockingStackOptionalContent
{
	FDockingStackOptionalContent()
		: ContentLeft(SNullWidget::NullWidget)
		, ContentRight(SNullWidget::NullWidget)
		, TitleBarContentRight(SNullWidget::NullWidget)
	{}

	// Content that appears on the left side of tabs in this docking stack
	TSharedRef<SWidget> ContentLeft;

	// Content that appears on the right side of tabs in this docking stack
	TSharedRef<SWidget> ContentRight;

	// Content that appears on the right side of the title bar in the window this stack is in
	TSharedRef<SWidget> TitleBarContentRight;
};


/**
 * A node in the Docking/Tabbing hierarchy.
 * A DockTabStack shows a row of tabs and the content of one selected tab.
 * It also supports re-arranging tabs and dragging them out for the stack.
 */
class SDockingTabStack : public SDockingNode
{
public:
	SLATE_BEGIN_ARGS(SDockingTabStack)
			: _IsDocumentArea(false) {}
		SLATE_ARGUMENT( bool, IsDocumentArea )
	SLATE_END_ARGS()

	virtual Type GetNodeType() const override
	{
		return SDockingNode::DockTabStack;
	}

	virtual float GetNodeMinSize() const override { return NodeMinSize; }
	void SetNodeMinSize(float InMinSize) { NodeMinSize = InMinSize; }

	SLATE_API void OnLastTabRemoved();

	SLATE_API void OnTabClosed(const TSharedRef<SDockTab>& ClosedTab, SDockingNode::ELayoutModification RemovalMethod);

	SLATE_API void OnTabRemoved( const FTabId& TabId );

	SLATE_API void Construct( const FArguments& InArgs, const TSharedRef<FTabManager::FStack>& PersistentNode );

	// TabStack methods

	SLATE_API void OpenTab(const TSharedRef<SDockTab>& InTab, int32 InsertAtLocation = INDEX_NONE, bool bKeepInactive = false);

	SLATE_API void AddTabWidget(const TSharedRef<SDockTab>& InTab, int32 AtLocation = INDEX_NONE, bool bKeepInactive = false);

	SLATE_API void AddSidebarTab(const TSharedRef<SDockTab>& InTab);

	SLATE_API float GetTabSidebarSizeCoefficient(const TSharedRef<SDockTab>& InTab);
	SLATE_API void SetTabSidebarSizeCoefficient(const TSharedRef<SDockTab>& InTab, float InSizeCoefficient);

	SLATE_API bool IsTabPinnedInSidebar(const TSharedRef<SDockTab>& InTab);
	SLATE_API void SetTabPinnedInSidebar(const TSharedRef<SDockTab>& InTab, bool bPinnedInSidebar);

	SLATE_API bool IsTabLocked(const TSharedRef<const SDockTab>& InTab) const;
	SLATE_API void SetTabLocked(const TSharedRef<const SDockTab>& InTab, TOptional<bool> bInLocked);

	/** @return true if the specified tab can be moved to a sidebar */
	SLATE_API bool CanMoveTabToSideBar(TSharedRef<SDockTab> Tab) const;

	/** Moves a specific tab from this stack into a sidebar */
	SLATE_API void MoveTabToSidebar(TSharedRef<SDockTab> Tab);

	/** Restores a specific tab from this stack to its original position */
	SLATE_API void RestoreTabFromSidebar(TSharedRef<SDockTab> Tab);

	/** @return All child tabs in this node */
	SLATE_API const TSlotlessChildren<SDockTab>& GetTabs() const;

	SLATE_API bool HasTab( const struct FTabMatcher& TabMatcher ) const;

	/** @return the last known geometry of this TabStack */
	SLATE_API FGeometry GetTabStackGeometry() const;

	SLATE_API void RemoveClosedTabsWithName( FName InName );

	SLATE_API bool IsShowingLiveTabs() const;

	SLATE_API void BringToFront( const TSharedRef<SDockTab>& TabToBringToFront );

	/** Set the content that the DockNode is presenting. */
	SLATE_API void SetNodeContent( const TSharedRef<SWidget>& InContent, const FDockingStackOptionalContent& OptionalContent);

	SLATE_API virtual FReply OnUserAttemptingDock( SDockingNode::RelativeDirection Direction, const FDragDropEvent& DragDropEvent ) override;
	
	/** Recursively searches through all children looking for child tabs */
	SLATE_API virtual TArray< TSharedRef<SDockTab> > GetAllChildTabs() const override;

	/** Gets the number of tabs in all children recursively */
	SLATE_API virtual int32 GetNumTabs() const override;

	SLATE_API virtual SSplitter::ESizeRule GetSizeRule() const override;

	/**
	 * Returns a bool indicating whether or not a tab is in a state where it can be hidden. A tab typically shouldn't
	 * be hidden in certain conditions like if there is more than one tab in the well, or if it's in a floating window.
	 *
	 * @return returns true if the tab can be hidden, else it returns false */
	SLATE_API bool CanHideTabWell() const;

	
	/**
	 * Sets the tab well to hidden/not hidden. 
	 *
	 * @param bShouldHideTabWell the bool that defines whether or not the tab well should be hidden
	 */
	SLATE_API void SetTabWellHidden( bool bShouldHideTabWell );
	SLATE_API bool IsTabWellHidden() const;

	SLATE_API virtual TSharedPtr<FTabManager::FLayoutNode> GatherPersistentLayout() const override;

	SLATE_API TSharedPtr<SDockTab> GetSelectedTabFromWell();

	/** Elements for which we might want to reserve space. */
	enum class EChromeElement
	{
		Icon,
		Controls
	};

	/**
	 * Remove all the space that might have been reserved for
	 * various window chrome elements (app icons, minimize, close, etc.)
	 */
	SLATE_API void ClearReservedSpace();
	/**
	 * Add some extra padding so that the corresponding window chrome element
	 * does not overlap our tabs.
	 */
	SLATE_API void ReserveSpaceForWindowChrome(EChromeElement Element, bool bIncludePaddingForMenuBar, bool bOnlyMinorTabs);

	//~ Begin SWidget
	SLATE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual void OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	//~ End SWidget

	/** Create and show context menu for a specific tab */
	void ShowContextMenu(const TSharedRef<SDockTab>& InContextTab, const FWidgetPath& InWidgetPath);

protected:

	virtual EWindowZone::Type GetWindowZoneOverride() const override
	{
		// Pretend we are a title bar so the user can grab the area to move the window around
		return EWindowZone::TitleBar;
	}

	SLATE_API void CloseForegroundTab();
	
	void CloseTab(TSharedPtr<SDockTab> InTab);

	enum class ETabsToClose : uint8
	{
		CloseDocumentTabs,
		CloseDocumentAndMajorTabs,
		CloseAllTabs
	};

	enum class ECloseTabsDirection : uint8
	{
		Left,
		Right
	};

	SLATE_API void CloseTabsInDirectionFromForegroundTab(ETabsToClose TabsToClose, ECloseTabsDirection Direction);
	
	void CloseTabsInDirectionFromTab(TSharedPtr<SDockTab> InTab, ETabsToClose InTabsToClose, ECloseTabsDirection InDirection);

	/**
	 * Close all the background tabs.
	 *
	 * @param  TabsToClose   Close just document tabs or both document and tool tabs.
	 */
	SLATE_API void CloseAllButForegroundTab(ETabsToClose TabsToClose);
	
	void CloseAllButTab(TSharedPtr<SDockTab> InTab, ETabsToClose InTabsToClose);

	SLATE_API FReply TabWellRightClicked( const FGeometry& TabWellGeometry, const FPointerEvent& MouseEvent );

	SLATE_API virtual SDockingNode::ECleanupRetVal CleanUpNodes() override;

	SLATE_API int32 OpenPersistentTab( const FTabId& TabId, int32 OpenLocationAmongActiveTabs = INDEX_NONE );

	SLATE_API int32 ClosePersistentTab( const FTabId& TabId );

	SLATE_API void RemovePersistentTab( const FTabId& TabId );

	/** Overridden from SDockingNode */
	SLATE_API virtual void SetParentNode( TSharedRef<class SDockingSplitter> InParent ) override;

	/** @return false if the content is disabled (non-interactable) */
	bool IsContentEnabled() const;

private:

	/** Minimum pixel size enforced on the SSplitter slot containing this stack. Set from FTabManager::FStack::SetMinSize(). */
	float NodeMinSize = 0.f;

	/** Data that persists across sessions and when the widget associated with this node is removed. */
	TArray<FTabManager::FTab> Tabs;


	/**
	 * Creates a SDockingTabStack by adding a new split to this stack's parent splitter and attaching the new SDockingTabStack.
	 * 
	 * @param	Direction	The relative direction to split the parent splitter
	 *
	 * @return	The newly created empty SDockingTabStack, ready for a tab to be added to it
	 */
	SLATE_API TSharedRef< SDockingTabStack > CreateNewTabStackBySplitting( const SDockingNode::RelativeDirection Direction );

	/** What should the content area look like for the current tab? */
	SLATE_API const FSlateBrush* GetContentAreaBrush() const;

	/** How much padding to show around the content currently being presented */
	SLATE_API FMargin GetContentPadding() const;

	/** Depending on the tabs we put into the tab well, we want a different background brush. */
	SLATE_API const FSlateBrush* GetTabWellBrush() const;

	/** Show the tab well? */
	SLATE_API EVisibility GetTabWellVisibility() const;

	/** Show the stuff needed to unhide the tab well? */
	SLATE_API EVisibility GetUnhideButtonVisibility() const;

	/** Show/Hide the tab well; do it smoothly with an animation */
	SLATE_API void ToggleTabWellVisibility();

	/** Moves the foreground tab to a sidebar */
	SLATE_API void MoveForegroundTabToSidebar();
	
	/** Moves the foreground tab to a sidebar */
	void MoveTabToSidebar(TSharedPtr<SDockTab> InTab);

	/** Unhides the tab well, revealing all tab headers */
	SLATE_API FReply UnhideTabWell();

	/** Only allow closing the tab well when the tab allows it. */
	SLATE_API bool CanCloseForegroundTab() const;
	
	bool CanCloseTab(TSharedPtr<SDockTab> InTab) const;

	static int32 GetIncrementFromCloseTabsDirection(ECloseTabsDirection);

	/** Only allow closing tabs to the right when there is more than one tab open, the tab is of type Document or Major, and the tab is not furthest to the right. */
	SLATE_API bool CanCloseTabsInDirectionOfForegroundTab(ECloseTabsDirection Direction) const;
	
	bool CanCloseTabsInDirectionOfTab(TSharedPtr<SDockTab> InTab, ECloseTabsDirection InDirection) const;

	/** Only allow closing all other tabs when there are more then one tab open and the tab is of type Document or Major. */
	SLATE_API bool CanCloseAllButForegroundTab() const;
	
	bool CanCloseAllButTab(TSharedPtr<SDockTab> InTab) const;

	/** @return true if the foreground tab can be moved to a sidebar */
	SLATE_API bool CanMoveForegroundTabToSidebar() const;
	
	/** @return true if the tab can be moved to a sidebar */
	bool CanMoveTabToSidebar(TSharedPtr<SDockTab> InTab) const;

	/** @return true if the specified tab can be moved to a sidebar */
	SLATE_API bool IsTabAllowedInSidebar(TSharedPtr<SDockTab> Tab) const;

	/** The tab well widget shows all tabs, keeps track of the selected tab, allows tab rearranging, etc. */
	TSharedPtr<class SDockingTabWell> TabWell;

	/** The borders that hold any potential inline content areas. */
	SHorizontalBox::FSlot* InlineContentAreaLeft;
	SHorizontalBox::FSlot* InlineContentAreaRight;

	SVerticalBox::FSlot* TitleBarSlot;
	TSharedPtr<SWidget> TitleBarContent;

	TSharedPtr<SBorder> ContentSlot;

	FOverlayManagement OverlayManagement;

	SLATE_API TSharedRef<SWidget> MakeContextMenu(const TSharedPtr<SDockTab>& InTab);

	/** Show the docking cross */
	SLATE_API void ShowCross();

	/** Hide the docking cross */
	SLATE_API void HideCross();

	/** Document Areas do not disappear when out of tabs, and instead say 'Document Area' */
	bool bIsDocumentArea;

	/** Animation that shows/hides the tab well; also used as a state machine to determine whether tab well is shown/hidden */
	FCurveSequence ShowHideTabWell;

	/** Grabs the scaling factor for the tab well size from the tab well animation. */
	SLATE_API FVector2D GetTabWellScale() const;

	/** Get the scale for the button that unhides the tab well */
	SLATE_API FVector2D GetUnhideTabWellButtonScale() const;
	/** Get the opacity for the button that unhides the tab well */
	SLATE_API FSlateColor GetUnhideTabWellButtonOpacity() const;

	/** Gets the background behind the tab stack */
	SLATE_API const FSlateBrush* GetTabStackBorderImage() const;

	/** Visibility of TitleBar spacer based on maximize/restore status of the window.
	 ** This gives us a little more space to grab the title bar when the window is not maximized
	*/
	SLATE_API EVisibility GetMaximizeSpacerVisibility() const;

	/** Bind tab commands into the ActionList */
	SLATE_API void BindTabCommands();

	/** Attempts to close the foreground tab when the CloseMajorTab command is executed */
	SLATE_API void ExecuteCloseMajorTabCommand();

	/** Attempts to find the foreground tab that can be closed by the CloseMajorTab command */
	SLATE_API bool CanExecuteCloseMajorTabCommand();

	/** Attempts to close the active tab when the CloseMinorTab command is executed */
	SLATE_API void ExecuteCloseMinorTabCommand();

	/** Attempts to find the active tab that can be closed by the CloseMinorTab command */
	SLATE_API bool CanExecuteCloseMinorTabCommand();

	/** Attempts to close the focused tab when the CloseFocusedTab command is executed */
	SLATE_API void ExecuteCloseFocusedTabCommand();

	/** Attempts to find the active tab that can be closed by the CloseFocusedTab command */
	SLATE_API bool CanExecuteCloseFocusedTabCommand();

	/** Called when this stack is resized. */
	SLATE_API virtual void OnResized() override;

#if DEBUG_TAB_MANAGEMENT
	SLATE_API FString ShowPersistentTabs() const;
#endif

	/** Tab command list */
	TSharedPtr<FUICommandList> ActionList;

	/** Whether or not this tab stack is part of the title bar area */
	bool bShowingTitleBarArea;
};


struct FTabMatcher
{
	FTabMatcher( const FTabId& InTabId, ETabState::Type InTabState = static_cast<ETabState::Type>(ETabState::ClosedTab | ETabState::OpenedTab | ETabState::SidebarTab), const bool InTreatIndexNoneAsWildcard = true )
		: TabIdToMatch( InTabId )
		, RequiredTabState( InTabState )
		, TreatIndexNoneAsWildcard( InTreatIndexNoneAsWildcard )
	{
	}

	bool operator()(const FTabManager::FTab& Candidate) const
	{
		return
			((Candidate.TabState & RequiredTabState) != 0) &&
			(Candidate.TabId.TabType == TabIdToMatch.TabType) &&
			// INDEX_NONE is treated as a wildcard
			((TreatIndexNoneAsWildcard && TabIdToMatch.InstanceId == INDEX_NONE) || TabIdToMatch.InstanceId == Candidate.TabId.InstanceId);;
	}

	FTabId TabIdToMatch;
	ETabState::Type RequiredTabState;
	bool TreatIndexNoneAsWildcard;
};
