// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Application/ThrottleManager.h"
#include "Animation/CurveSequence.h"

struct FSplitterStyle;

class STabDrawer;
class SDockTab;

/** The direction that a tab drawer opens relative to the location of the sidebar it is in */
enum class ETabDrawerOpenDirection : uint8
{
	/** Open from the left to the right */
	Left,

	/** Open from the right to the left */
	Right,

	/** Open from the top to the bottom */
	Bottom
};

DECLARE_DELEGATE_TwoParams(FOnDrawerTargetSizeChanged, TSharedRef<STabDrawer>, float);
DECLARE_DELEGATE_OneParam(FOnDrawerFocusLost, TSharedRef<STabDrawer>);
DECLARE_DELEGATE_OneParam(FOnDrawerClosed, TSharedRef<STabDrawer>);

/**
 * A tab drawer is a widget that contains the contents of a widget when that widget is in a sidebar
 */
class STabDrawer : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(STabDrawer, SCompoundWidget)

	SLATE_BEGIN_ARGS(STabDrawer)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
		_ShadowOffset = FVector2D(8.0f, 8.0f);
	}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		/** The minimum size a drawer can be when opened. This unit is in window space */
		SLATE_ARGUMENT(float, MinDrawerSize)
		/** The maximum size a drawer can be when opened. This unit is in window space */
		SLATE_ARGUMENT(float, MaxDrawerSize)
		/** The size that the drawer should be, clamped to the above min/max values. This unit is in window space */
		SLATE_ARGUMENT(float, TargetDrawerSize)
		/** Called when the target drawer size is changed by the user */
		SLATE_EVENT(FOnDrawerTargetSizeChanged, OnTargetDrawerSizeChanged)
		/** Called when the drawer loses focus */
		SLATE_EVENT(FOnDrawerFocusLost, OnDrawerFocusLost)
		/** Called when the drawer is completely closed (i.e will be called once the close animation completes */
		SLATE_EVENT(FOnDrawerClosed, OnDrawerClosed)
		/** The side of the drop shadow surrounding the drawer */
		SLATE_ARGUMENT(FVector2D, ShadowOffset)
	SLATE_END_ARGS()

	~STabDrawer();
	void Construct(const FArguments& InArgs, TSharedRef<SDockTab> InTab, TWeakPtr<SWidget> InTabButton, ETabDrawerOpenDirection InOpenDirection);

	/** Sets the current size of the drawer, ignoring any open/close animation */
	void SetCurrentSize(float InSize);

	/**
	 * Opens the drawer
	 *
	 * @param bAnimateOpen Whether to play an animation when opening the drawer, defaults to true
	 */
	void Open(bool bAnimateOpen=true);

	/** Begins an animation which closes the drawer */
	void Close();

	/** @return true if the drawer is open */
	bool IsOpen() const;

	/** @return true if the drawer is currently playing the close animation */
	bool IsClosing() const;

	/** @return the tab whose contents is being shown in the drawer */
	const TSharedRef<SDockTab> GetTab() const;

	/** SWidget interface */
	virtual bool SupportsKeyboardFocus() const override;
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	FGeometry GetRenderTransformedGeometry(const FGeometry& AllottedGeometry) const;
	FGeometry GetResizeHandleGeometry(const FGeometry& AllottedGeometry) const;
	EActiveTimerReturnType UpdateAnimation(double CurrentTime, float DeltaTime);
	void OnGlobalFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget);
	void OnActiveTabChanged(TSharedPtr<SDockTab> NewlyActivated, TSharedPtr<SDockTab> PreviouslyActive);

private:
	FGeometry InitialResizeGeometry;
	FOnDrawerTargetSizeChanged OnTargetDrawerSizeChanged;
	FOnDrawerFocusLost OnDrawerFocusLost;
	FOnDrawerClosed OnDrawerClosed;
	TSharedPtr<FActiveTimerHandle> OpenCloseTimer;
	TSharedPtr<SDockTab> ForTab;
	TWeakPtr<SWidget> TabButton;
	FThrottleRequest ResizeThrottleHandle;
	FThrottleRequest AnimationThrottle;
	FCurveSequence OpenCloseAnimation;
	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* ShadowBrush;
	const FSlateBrush* BorderBrush; // border with all corners rounded
	const FSlateBrush* BorderSquareEdgeBrush; // border with corners squared on one edge depending on open direction
	const FSplitterStyle* SplitterStyle;
	FVector2D ShadowOffset;
	float ExpanderSize;
	float CurrentSize;
	float MinDrawerSize;
	float MaxDrawerSize;
	float TargetDrawerSize;
	float InitialSizeAtResize;
	ETabDrawerOpenDirection OpenDirection;
	bool bIsResizing;
	bool bIsResizeHandleHovered;
};

