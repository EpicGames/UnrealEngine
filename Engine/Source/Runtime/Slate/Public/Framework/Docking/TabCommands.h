// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/CoreStyle.h"
#include "Framework/Commands/Commands.h"

class FTabCommands : public TCommands < FTabCommands >
{
public:

	FTabCommands()
		: TCommands<FTabCommands>(TEXT("TabCommands"), NSLOCTEXT("TabCommands", "DockingTabCommands", "Docking Tab Commands"), NAME_None, FCoreStyle::Get().GetStyleSetName())
	{
	}

	virtual ~FTabCommands() = default;

	SLATE_API virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> CloseMajorTab;
	TSharedPtr<FUICommandInfo> CloseMinorTab;
	TSharedPtr<FUICommandInfo> CloseFocusedTab;
};
