// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Containers/Ticker.h"
#include "Containers/ContainersFwd.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SWindow.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Commands/UIAction.h"
#include "Templates/Function.h"

class FJsonObject;
class FMenuBuilder;
class FNamePermissionList;
class FMultiBox;
class FProxyTabmanager;
class SDockingArea;
class SDockingTabStack;
class FLayoutExtender;
struct FTabMatcher;
struct FSidebarTabLists;

namespace UE::Editor::HomeScreen
{
	inline bool IsHomeScreenEnabled()
	{
		static bool bIsHomeScreenEnabled = false;
		UE_CALL_ONCE([&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("HomeScreen.EnableHomeScreen")))
			{
				(void)ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline."));
				bIsHomeScreenEnabled = CVar->GetBool();
			}
		});
		return bIsHomeScreenEnabled;
	}
}

DECLARE_MULTICAST_DELEGATE_TwoParams(
	FOnActiveTabChanged,
	/** Newly active tab */
	TSharedPtr<SDockTab>,
	/** Previously active tab */
	TSharedPtr<SDockTab> );
	

enum class ESidebarLocation : uint8
{
	/** Tab is in a sidebar on the left side of its parent area */
	Left,
	/** Tab is in a sidebar on the right side of its parent area */
	Right,

	/** Tab is not in a sidebar */
	None,
};


enum class ETabIdFlags : uint8
{
	None            = 0x0,  // No flags
	SaveLayout      = 0x1,  // This tab should be included when saving the Slate layout
};

ENUM_CLASS_FLAGS(ETabIdFlags);

struct FTabId
{
	FTabId()
		: TabType(NAME_None) 
		, InstanceId(INDEX_NONE)
		, Flags(ETabIdFlags::SaveLayout)
	{ }

	FTabId(const FName InTabType, const int32 InInstanceId)
		: TabType(InTabType)
		, InstanceId(InInstanceId)
		, Flags(ETabIdFlags::SaveLayout)
	{ }

	FTabId(const FName InTabType)
		: TabType(InTabType)
		, InstanceId(INDEX_NONE)
		, Flags(ETabIdFlags::SaveLayout)
	{ }

	FTabId(const FName InTabType, const ETabIdFlags InFlags)
		: TabType(InTabType)
		, InstanceId(INDEX_NONE)
		, Flags(InFlags)
	{ }

	/** Document tabs allow multiple instances of the same tab type. The placement rules for these tabs are left up for the specific use-cases. These tabs are not persisted. */
	bool IsTabPersistable() const
	{
		return InstanceId == INDEX_NONE;
	}

	bool operator==(const FTabId& Other) const
	{
		return TabType == Other.TabType && (InstanceId == INDEX_NONE || Other.InstanceId == INDEX_NONE || InstanceId == Other.InstanceId) ;
	}

	bool ShouldSaveLayout() const { return EnumHasAnyFlags(Flags, ETabIdFlags::SaveLayout); }

	FString ToString() const
	{
		// This function is useful to allow saving the layout on disk.
		// Alternative: We could save (InstanceId == INDEX_NONE) ? TabType.ToString() : FString::Printf( TEXT("%s : %d"), *(TabType.ToString()), InstanceId ), which would include the InstanceId.
		// Problem: InstanceId depends on the Editor/Slate runtime session rather than the layout itself. I.e., this would lead to the exact same layout being saved differently, depending
		// on how many tabs were opened/closed before arriving to that final layout, or in the order in which those tabs were created.
		// Conclusion: We do not want to save the InstanceId, which is session-dependent rather than layout-dependent.
		// More detailed explanation: When the Editor is opened, InstanceId is a static value starting in 0 and increasing every time a new FTabID is created (but not decreased when one is
		// closed). Loading/saving layouts do not reset this number either (and there is no point in doing so given that it is a runtime variable). E.g., opening and closing the same tab
		// multiple times will lead to higher InstanceId numbers in the final layout. In addition, creating tab A and then B would lead to A::InstanceId = 0, B::InstanceId = 1. While creating
		// B first and A latter would lead to the opposite values. Any of these examples would wrongly make 2 exact layouts look "different" if we saved the InstanceIds.
		return TabType.ToString();
	}

	FText ToText() const
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("TabType"), FText::FromName( TabType ) );
		Args.Add( TEXT("InstanceIdNumber"), FText::AsNumber( InstanceId ) );

		return (InstanceId == INDEX_NONE)
			? FText::FromName( TabType )
			: FText::Format( NSLOCTEXT("TabManager", "TabIdFormat", "{TabType} : {InstanceIdNumber}"), Args );
	}

	friend uint32 GetTypeHash(const FTabId& In)
	{
		return GetTypeHash(In.TabType) ^ In.InstanceId;
	}

	FName TabType;
	int32 InstanceId;

	private:
	ETabIdFlags Flags;
};


class FSpawnTabArgs
{
public:
	FSpawnTabArgs( const TSharedPtr<SWindow>& InOwnerWindow, const FTabId& InTabBeingSpawenedId )
	: TabIdBeingSpawned(InTabBeingSpawenedId)
	, OwnerWindow(InOwnerWindow)
	{
	}

	const TSharedPtr<SWindow>& GetOwnerWindow() const
	{
		return OwnerWindow;
	}

	const FTabId& GetTabId() const 
	{
		return TabIdBeingSpawned;
	}

private:
	FTabId TabIdBeingSpawned;
	TSharedPtr<SWindow> OwnerWindow;
};


/**
 * Invoked when a tab needs to be spawned.
 */
DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<SDockTab>, FOnSpawnTab, const FSpawnTabArgs& );
DECLARE_DELEGATE_RetVal_OneParam(bool, FCanSpawnTab, const FSpawnTabArgs&);
/**
 * Allows users to provide custom logic when searching for a tab to reuse.
 * The TabId that is being searched for is provided as a courtesy, but does not have to be respected.
 */
DECLARE_DELEGATE_RetVal_OneParam( TSharedPtr<SDockTab>, FOnFindTabToReuse, const FTabId& )

DECLARE_MULTICAST_DELEGATE_TwoParams(
	FOnToggleTemporarilySidebaredTabs,
	const bool /*bInSidebared*/,
	/** Tab exceptions that were requested to not be sidebared */
	const TSet<FTabId>& /*InExceptions*/)

struct FMinorTabConfig
{
public:
	FMinorTabConfig()
	{
	}

	FMinorTabConfig(const FName& InTabID)
		: TabId(InTabID)
	{
	}

	FName TabId;

	FText TabLabel;

	FText TabTooltip;

	FSlateIcon TabIcon;

	FOnSpawnTab OnSpawnTab;

	FCanSpawnTab CanSpawnTab;

	FOnFindTabToReuse OnFindTabToReuse;

	TSharedPtr<FWorkspaceItem> WorkspaceGroup;
};

/** An enum to describe how TabSpawnerEntries will be handled by menus. */
namespace ETabSpawnerMenuType
{
	enum Type
	{
		Enabled,		// Display this spawner in menus
		Disabled,		// Display this spawner in menus, but make it disabled
		Hidden,			// Do not display this spawner in menus, it will be invoked manually
	};
}

/** An enum to describe how TabSpawnerEntries behave when the tab manager is in read only mode */
enum class ETabReadOnlyBehavior : int32
{
	Disabled, // Default behavior - This tab will show up in read only modes but the contents will be disabled
	Hidden,   // This tab will not show up in read only modes
	Custom    // This tab will show up in read only modes and the tab owner will decide how it behaves (FTabManager::OnReadOnlyStateChanged to keep track of read only state)
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnReadOnlyModeChanged, bool /*bReadOnly*/);

struct FTabSpawnerEntry : public FWorkspaceItem
{
	FTabSpawnerEntry(const FName& InTabType, const FOnSpawnTab& InSpawnTabMethod, const FCanSpawnTab& InCanSpawnTab)
		: FWorkspaceItem(InTabType, FText(), FSlateIcon(), false)
		, TabType(InTabType)
		, OnSpawnTab(InSpawnTabMethod)
		, CanSpawnTab(InCanSpawnTab)
		, OnFindTabToReuse()
		, MenuType(ETabSpawnerMenuType::Enabled)
		, bAutoGenerateMenuEntry(true)
		, bCanSidebarTab(true)
		, bHiddenTabName(false)
		, SpawnedTabPtr()
		, ReadOnlyBehavior(ETabReadOnlyBehavior::Disabled)
	{
		// Special treatment for the HomeScreen tab to not have its name displayed and to be locked, this is currently intended to not be a public feature
		if (InTabType == FName("HomeScreen"))
		{
			bHiddenTabName = true;
			bIsLocked = true;
		}

		// Special treatment for the LevelEditor tab to be locked, this is currently intended to not be a public feature
		if (InTabType == FName("LevelEditor"))
		{
			bIsLocked = true;
		}
	}

	FTabSpawnerEntry& SetReadOnlyBehavior( const ETabReadOnlyBehavior& InReadOnlyBehavior)
	{
		ReadOnlyBehavior = InReadOnlyBehavior;
		return *this;
	}
	
	FTabSpawnerEntry& SetIcon( const FSlateIcon& InIcon)
	{
		Icon = InIcon;
		return *this;
	}

	FTabSpawnerEntry& SetDisplayNameAttribute(const TAttribute<FText>& InLegibleName)
	{
		DisplayNameAttribute = InLegibleName;
		return *this;
	}

	FTabSpawnerEntry& SetTooltipTextAttribute(const TAttribute<FText>& InTooltipText)
	{
		TooltipTextAttribute = InTooltipText;
		return *this;
	}

	FTabSpawnerEntry& SetDisplayName( const FText& InLegibleName )
	{
		DisplayNameAttribute = InLegibleName;
		return *this;
	}

	FTabSpawnerEntry& SetTooltipText( const FText& InTooltipText )
	{
		TooltipTextAttribute = InTooltipText;
		return *this;
	}

	FTabSpawnerEntry& SetGroup( const TSharedRef<FWorkspaceItem>& InGroup )
	{
		InGroup->AddItem(SharedThis(this));
		return *this;
	}

	FTabSpawnerEntry& SetReuseTabMethod( const FOnFindTabToReuse& InReuseTabMethod )
	{
		OnFindTabToReuse = InReuseTabMethod;
		return *this;
	}

	FTabSpawnerEntry& SetMenuType( const TAttribute<ETabSpawnerMenuType::Type>& InMenuType )
	{
		MenuType = InMenuType;
		return *this;
	}

	FTabSpawnerEntry& SetAutoGenerateMenuEntry( bool bInAutoGenerateMenuEntry )
	{
		bAutoGenerateMenuEntry = bInAutoGenerateMenuEntry;
		return *this;
	}

	FTabSpawnerEntry& SetCanSidebarTab(bool bInCanSidebarTab)
	{
		bCanSidebarTab = bInCanSidebarTab;
		return *this;
	}

	bool CanSidebarTab() const
	{
		return bCanSidebarTab;
	}

	TOptional<bool> IsTabLocked() const
	{
		return bIsLocked;
	}

	bool IsTabNameHidden() const
	{
		return bHiddenTabName;
	}

	virtual TSharedPtr<FTabSpawnerEntry> AsSpawnerEntry() override
	{
		return SharedThis(this);
	}

	const FName GetTabType() const
	{
		return TabType;
	};

	const bool IsHidden() const
	{
		return MenuType.Get() == ETabSpawnerMenuType::Hidden;
	}

private:
	FName TabType;
	FOnSpawnTab OnSpawnTab;
	FCanSpawnTab CanSpawnTab;
	/** When this method is not provided, we assume that the tab should only allow 0 or 1 instances */
	FOnFindTabToReuse OnFindTabToReuse;
	/** Whether this menu item should be enabled, disabled, or hidden */
	TAttribute<ETabSpawnerMenuType::Type> MenuType;
	/** Whether to automatically generate a menu entry for this tab spawner */
	bool bAutoGenerateMenuEntry;
	/** Whether or not this tab can ever be in a sidebar */
	bool bCanSidebarTab;
	/** Whether the tab name should be hidden */
	bool bHiddenTabName;
	/** Whether the tab default state should be locked or not */
	TOptional<bool> bIsLocked;

	TWeakPtr<SDockTab> SpawnedTabPtr;

	/** How this tab behaves when the tab manager is in a read only mode */
	ETabReadOnlyBehavior ReadOnlyBehavior;

	FORCENOINLINE bool IsSoleTabInstanceSpawned() const
	{
		// Items that allow multiple instances need a custom way to find and reuse tabs.
		return SpawnedTabPtr.IsValid();
	}

	friend class FTabManager;
};


namespace ETabState
{
	enum Type
	{
		OpenedTab = 0x1 << 0,
		ClosedTab = 0x1 << 1,
		SidebarTab = 0x1 << 2,
		StatusbarTab = 0x1 << 3,

		/**
		 * InvalidTab refers to tabs that were not recognized by the Editor (e.g., LiveLink when its plugin its disabled).
		 */
		InvalidTab = 0x1 << 4
	};
}

enum class EOutputCanBeNullptr
{
	/**
	 * RestoreArea_Helper() will always return a SWidget. It will return an "Unrecognized Tab" dummy SWidget if it cannot find the way to create the desired SWidget.
	 * Default behavior and the only one used before UE 4.24.
	 * This is the most strict condition and will never return nullptr.
	 */
	Never,
	/**
	 * RestoreArea_Helper() will return nullptr if its parent FTabManager::FStack does not contain at least a valid tab.
	 * Useful for docked SWidgets that contain closed Tabs.
	 * With IfNoTabValid, if a previously docked Tab is re-opened (i.e., its ETabState value changes from ClosedTab to OpenedTab), it will be displayed in the same place
	 * it was placed before being initially closed. However, with IfNoOpenTabValid, if that Tab were re-opened, it might be instead displayed in a new standalone SWidget.
	 */
	IfNoTabValid,
	/**
	 * RestoreArea_Helper() will return nullptr if its parent FTabManager::FStack does not contain at least a valid tab whose ETabState is set to OpenedTab.
	 * Useful for standalone SWidgets that otherwise will display a blank UI with no Tabs on it.
	 * This is the most relaxed condition and the one that will return nullptr the most.
	 */
	IfNoOpenTabValid
};


class FTabManager : public TSharedFromThis<FTabManager>
{
	friend class FGlobalTabmanager;
	public:

		class FStack;
		class FSplitter;
		class FArea;
		class FLayout;

		DECLARE_DELEGATE_OneParam( FOnPersistLayout, const TSharedRef<FLayout>& );

		class FLayoutNode : public TSharedFromThis<FLayoutNode>
		{
			friend class FTabManager;

		public:

			virtual ~FLayoutNode() = default;
			
			SLATE_API virtual TSharedPtr<FStack> AsStack();

			SLATE_API virtual TSharedPtr<FSplitter> AsSplitter();

			SLATE_API virtual TSharedPtr<FArea> AsArea();

			float GetSizeCoefficient() const { return SizeCoefficient; }

			float GetMinSize() const { return MinSize; }

		protected:
			FLayoutNode()
			: SizeCoefficient(1.0f)
			, MinSize(0.f)
			{
			}

			float SizeCoefficient;
			float MinSize;
		};

		struct FTab
		{
			FTab(const FTabId& InTabId, ETabState::Type InTabState)
				: TabId(InTabId)
				, TabState(InTabState)
				, SidebarLocation(ESidebarLocation::None)
				, SidebarSizeCoefficient(0.0f)
				, bPinnedInSidebar(false)
				, bIsLockedInPlace(TOptional<bool>())
			{
				checkf(InTabState != ETabState::SidebarTab, TEXT("Cannot add sidebar tab without specifying sidebar location."));
			}

			FTab(const FTabId& InTabId, ETabState::Type InTabState, ESidebarLocation InSidebarLocation, float InSidebarSizeCoefficient, bool bInPinnedInSidebar)
				: TabId(InTabId)
				, TabState(InTabState)
				, SidebarLocation(InSidebarLocation)
				, SidebarSizeCoefficient(InSidebarSizeCoefficient)
				, bPinnedInSidebar(bInPinnedInSidebar)
				, bIsLockedInPlace(TOptional<bool>())
			{
				checkf(InTabState != ETabState::SidebarTab || InSidebarLocation != ESidebarLocation::None, TEXT("Cannot add sidebar tab without specifying sidebar location."));
			}

			bool operator==( const FTab& Other ) const
			{
				return this->TabId == Other.TabId && this->TabState == Other.TabState && this->SidebarLocation == Other.SidebarLocation;
			}

			FTabId TabId;
			ETabState::Type TabState;
			ESidebarLocation SidebarLocation;
			float SidebarSizeCoefficient;
			bool bPinnedInSidebar;
			TOptional<bool> bIsLockedInPlace;
		};

		class FStack : public FLayoutNode
		{
				friend class FTabManager;
				friend class FLayout;
				friend class SDockingTabStack;

			public:				
				TSharedRef<FStack> AddTab(const FName TabType, ETabState::Type InTabState)
				{
					check(InTabState != ETabState::SidebarTab);
					Tabs.Add(FTab( FTabId(TabType), InTabState));
					return SharedThis(this);
				}

				TSharedRef<FStack> AddTab(const FTabId TabId, ETabState::Type InTabState)
				{
					check(InTabState != ETabState::SidebarTab);

					Tabs.Add(FTab(TabId, InTabState));

					return SharedThis(this);
				}

				TSharedRef<FStack> AddTab(const FName TabType, ETabState::Type InTabState, ESidebarLocation InSidebarLocation, float SidebarSizeCoefficient, bool bPinnedInSidebar=false)
				{
					check(InTabState != ETabState::SidebarTab || InSidebarLocation != ESidebarLocation::None);
					Tabs.Add(FTab(FTabId(TabType), InTabState, InSidebarLocation, SidebarSizeCoefficient, bPinnedInSidebar));
					return SharedThis(this);
				}

				TSharedRef<FStack> AddTab(const FTabId TabId, ETabState::Type InTabState, ESidebarLocation InSidebarLocation, float SidebarSizeCoefficient, bool bPinnedInSidebar=false)
				{
					check(InTabState != ETabState::SidebarTab || InSidebarLocation != ESidebarLocation::None);

					Tabs.Add(FTab(TabId, InTabState, InSidebarLocation, SidebarSizeCoefficient, bPinnedInSidebar));

					return SharedThis(this);
				}

				TSharedRef<FStack> AddTab(const FTab& Tab)
				{
					Tabs.Add(Tab);
					return SharedThis(this);
				}

				TSharedRef<FStack> SetSizeCoefficient( const float InSizeCoefficient )
				{
					SizeCoefficient = InSizeCoefficient;
					return SharedThis(this);
				}

				TSharedRef<FStack> SetMinSize( const float InMinSize )
				{
					MinSize = InMinSize;
					return SharedThis(this);
				}

				TSharedRef<FStack> SetHideTabWell( const bool InHideTabWell )
				{
					bHideTabWell = InHideTabWell;
					return SharedThis(this);
				}

				TSharedRef<FStack> SetForegroundTab( const FTabId& TabId )
				{
					ForegroundTabId = TabId;
					return SharedThis(this);
				}

				virtual TSharedPtr<FStack> AsStack() override
				{
					return SharedThis(this);
				}

				virtual ~FStack() = default;

				TSharedRef<FStack> SetExtensionId(FName InExtensionId)
				{
					ExtensionId = InExtensionId;
					return SharedThis(this);
				}

				FName GetExtensionId() const
				{
					return ExtensionId;
				}

				TConstArrayView<FTab> GetTabs() const
				{
					return Tabs;
				}

			protected:

				FStack()
				: Tabs()
				, bHideTabWell(false)
				, ForegroundTabId(NAME_None)
				{
				}

				TArray<FTab> Tabs;
				bool bHideTabWell;
				FTabId ForegroundTabId;
				FName ExtensionId;
		};


		class FSplitter : public FLayoutNode
		{
				friend class FTabManager;
				friend class FLayoutExtender;
		
			public:

				TSharedRef<FSplitter> Split( TSharedRef<FLayoutNode> InNode )
				{
					ChildNodes.Add(InNode);
					return SharedThis(this);
				}

				TSharedRef<FSplitter> InsertBefore(TSharedRef<FLayoutNode> NodeToInsertBefore, TSharedRef<FLayoutNode> NodeToInsert)
				{
					int32 InsertAtIndex = ChildNodes.Find(NodeToInsertBefore);
					check(InsertAtIndex != INDEX_NONE);
					ChildNodes.Insert(NodeToInsert, InsertAtIndex);
					return SharedThis(this);
				}

				TSharedRef<FSplitter> InsertAfter(TSharedRef<FLayoutNode> NodeToInsertAfter, TSharedRef<FLayoutNode> NodeToInsert)
				{
					int32 InsertAtIndex = ChildNodes.Find(NodeToInsertAfter);
					check(InsertAtIndex != INDEX_NONE);
					ChildNodes.Insert(NodeToInsert, InsertAtIndex + 1);
					return SharedThis(this);
				}


				TSharedRef<FSplitter> SetSizeCoefficient( const float InSizeCoefficient )
				{
					SizeCoefficient = InSizeCoefficient;
					return SharedThis(this);
				}

				TSharedRef<FSplitter> SetOrientation( const EOrientation InOrientation )
				{
					Orientation = InOrientation;
					return SharedThis(this);
				}

				virtual TSharedPtr<FSplitter> AsSplitter() override
				{
					return SharedThis(this);
				}

				EOrientation GetOrientation() const { return Orientation; }

				virtual ~FSplitter() = default;

				TConstArrayView<TSharedRef<FLayoutNode>> GetChildNodes() const
				{
					return ChildNodes;
				}

			protected:
				FSplitter()
				: Orientation(Orient_Horizontal)
				{
				}

				EOrientation Orientation;
				TArray< TSharedRef<FLayoutNode> > ChildNodes;
		};

		struct FPanelDrawerSize
		{
			float MainContentCoefficient = 0.75f;
			float PanelDrawerCoefficient = 0.25f;
		};

		struct FPanelDrawerTab
		{
			FTabId TabId;
			FPanelDrawerSize Size;
		};

		class FArea : public FSplitter
		{
			friend class FTabManager;

			// Used for a workaround for a config compatibility issue during the development of 5.8, it will be removed in 5.9.
			friend class FMainFrameModule;
		
			protected:
				/** An enum which specifies how content appears within a window. */
				enum class EWindowPlacementInternal : uint8
				{
					/** The content is docked within a primary window. */
					Placement_NoWindow,

					/** The content is docked within a primary window, but determine the placement of the primary window */
					Placement_ParentSpecified,

					/** The content is docked within a floating window that is initially automatically positioned. */
					Placement_Automatic,

					/** The content is docked within a floating window with positioning specified by an FArea. */
					Placement_Specified
				};

			public:

				enum UE_DEPRECATED(5.8, "This enum will removed from the engine") EWindowPlacement
				{
					/** The content is docked within a primary window. */
					Placement_NoWindow,

					/** The content is docked within a floating window that is initially automatically positioned. */
					Placement_Automatic,

					/** The content is docked within a floating window with positioning specified by an FArea. */
					Placement_Specified
				};
	
				TSharedRef<FArea> Split( TSharedRef<FLayoutNode> InNode )
				{
					ChildNodes.Add(InNode);
					return SharedThis(this);
				}
				
				TSharedRef<FArea> SplitAt( int32 Index, TSharedRef<FLayoutNode> InNode )
				{
					check(Index >= 0);
					ChildNodes.Insert(InNode, FMath::Min(Index, ChildNodes.Num()));
					return SharedThis(this);
				}

				TSharedRef<FArea> SetOrientation( const EOrientation InOrientation )
				{
					Orientation = InOrientation;
					return SharedThis(this);
				}

				TSharedRef<FArea> SetWindowSize( FVector2D InSize )
				{
					UnscaledWindowSize = InSize;
					return SharedThis(this);
				}

				TSharedRef<FArea> SetWindow( FVector2D InPosition, bool IsMaximized )
				{
					WindowPlacement = EWindowPlacementInternal::Placement_Specified;
					UnscaledWindowPosition = InPosition;
					bIsMaximized = IsMaximized;
					return SharedThis(this);
				}

				TSharedRef<FArea> SetExtensionId( FName InExtensionId )
				{
					ExtensionId = InExtensionId;
					return SharedThis(this);
				}

				FName GetExtensionId() const
				{
					return ExtensionId;
				}

				virtual TSharedPtr<FArea> AsArea() override
				{
					return SharedThis(this);
				}

				virtual ~FArea() = default;

				UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
				TSharedRef<FArea> SetPanelDrawerActiveTab(FPanelDrawerTab&& InPanelDrawerTab)
				{
					ActivePanelDrawerTab = InPanelDrawerTab;
					return SharedThis(this);
				}

				UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
				TSharedRef<FArea> AddPanelDrawerInactiveTab(FPanelDrawerTab&& InPanelDrawerTab)
				{
					InactivePanelDrawerTabs.Add(InPanelDrawerTab);
					return SharedThis(this);
				}

				UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
				const TArray<FPanelDrawerTab>& GetPanelDrawerInactiveTabs() const
				{
					return InactivePanelDrawerTabs;
				}

			/**
			 * Returns true if this FArea defines a positionally specified window, else it returns false
			*/
			bool DefinesPositionallySpecifiedFloatingWindow()const
			{
				return WindowPlacement == EWindowPlacementInternal::Placement_Specified;
			}

			protected:
				FArea( const float InWidth, const float InHeight )
				: WindowPlacement(EWindowPlacementInternal::Placement_Automatic)
				, UnscaledWindowPosition(FVector2D(0.0, 0.0))
				, UnscaledWindowSize(InWidth, InHeight)
				, bIsMaximized( false )
				{
				}

				EWindowPlacementInternal WindowPlacement;
				FVector2D UnscaledWindowPosition;
				FVector2D UnscaledWindowSize;
				bool bIsMaximized;
				FName ExtensionId;
				FPanelDrawerTab ActivePanelDrawerTab;
				TArray<FPanelDrawerTab> InactivePanelDrawerTabs;
		};


		class FLayout : public TSharedFromThis<FLayout>
		{
				friend class FTabManager;
				
			public:

				static SLATE_API const TSharedRef<FTabManager::FLayout> NullLayout; /** A dummy layout meant to spawn nothing during (e.g., asset editor) initialization */

				TSharedRef<FLayout> AddArea( const TSharedRef<FArea>& InArea )
				{
					Areas.Add( InArea );
					return SharedThis(this);
				}

				const TWeakPtr<FArea>& GetPrimaryArea() const
				{
					return PrimaryArea;
				}

				TConstArrayView<TSharedRef<FArea>> GetAreas() const
				{
					return Areas;
				}
				
			public:
				static SLATE_API TSharedPtr<FTabManager::FLayout> NewFromString( const FString& LayoutAsText );
				static SLATE_API TSharedPtr<FTabManager::FLayout> NewFromJson( const TSharedPtr<FJsonObject>& LayoutAsJson );
				SLATE_API FName GetLayoutName() const;
				SLATE_API TSharedRef<FJsonObject> ToJson() const;
				SLATE_API FString ToString() const;

				SLATE_API void ProcessExtensions(const FLayoutExtender& Extender);

			protected:
				static SLATE_API TSharedRef<class FJsonObject> PersistToString_Helper(const TSharedRef<FLayoutNode>& NodeToPersist);
				static SLATE_API TSharedRef<FLayoutNode> NewFromString_Helper( TSharedPtr<FJsonObject> JsonObject );

				FLayout(const FName& InLayoutName)
					: LayoutName(InLayoutName)
				{}

				TWeakPtr< FArea > PrimaryArea;
				TArray< TSharedRef<FArea> > Areas;
				/** The layout will be saved into a config file with this name. E.g. LevelEditorLayout or MaterialEditorLayout */
				FName LayoutName;
		};	


		friend class FPrivateApi;
		class FPrivateApi
		{
			public:
				FPrivateApi( FTabManager& InTabManager )
				: TabManager( InTabManager )
				{
				}

				TSharedPtr<SWindow> GetParentWindow() const;
				void OnDockAreaCreated( const TSharedRef<SDockingArea>& NewlyCreatedDockArea );
				/** Notify the tab manager that a tab has been relocated. If the tab now lives in a new window, the NewOwnerWindow should be a valid pointer. */
				void OnTabRelocated( const TSharedRef<SDockTab>& RelocatedTab, const TSharedPtr<SWindow>& NewOwnerWindow );
				void OnTabOpening( const TSharedRef<SDockTab>& TabBeingOpened );
				void OnTabClosing( const TSharedRef<SDockTab>& TabBeingClosed );
				void OnDockAreaClosing( const TSharedRef<SDockingArea>& DockAreaThatIsClosing );
				void OnTabManagerClosing();
				bool CanTabLeaveTabWell(const TSharedRef<const SDockTab>& TabToTest) const;
				const TArray< TWeakPtr<SDockingArea> >& GetLiveDockAreas() const;
				/**
				 * Notify the tab manager that the NewForegroundTab was brought to front and the BackgroundedTab was send to the background as a result.
				 */
				void OnTabForegrounded( const TSharedPtr<SDockTab>& NewForegroundTab, const TSharedPtr<SDockTab>& BackgroundedTab );

				void ShowWindows();
				void HideWindows();

				void SetCanDoDeferredLayoutSave(bool bInCanDoDeferredLayoutSave);
				TSharedPtr<SDockingArea> GetDockingAreaForPanelDrawer(const TSharedPtr<SWindow>& InWindow) const;

			private:
				FTabManager& TabManager;
				
		};

		SLATE_API FTabManager::FPrivateApi& GetPrivateApi();

	public:
		class FSearchPreference
		{
		public:
			virtual TSharedPtr<SDockTab> Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const = 0;
			virtual ~FSearchPreference() = default;
		};

		class FRequireClosedTab : public FSearchPreference
		{
		public:
			SLATE_API virtual TSharedPtr<SDockTab> Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const override;
		};

		class FLiveTabSearch : public FSearchPreference
		{
		public:
			SLATE_API FLiveTabSearch(FName InSearchForTabId = NAME_None);

			SLATE_API virtual TSharedPtr<SDockTab> Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const override;

		private:
			FName SearchForTabId;
		};

		class FLastMajorOrNomadTab : public FSearchPreference
		{
		public:
			SLATE_API FLastMajorOrNomadTab(FName InFallbackTabId);

			SLATE_API virtual TSharedPtr<SDockTab> Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const override;
		private:
			FName FallbackTabId;
		};

	public:
		static TSharedRef<FLayout> NewLayout( const FName LayoutName )
		{
			return MakeShareable( new FLayout(LayoutName) );
		}
		
		static TSharedRef<FArea> NewPrimaryArea()
		{
			TSharedRef<FArea>Area = MakeShareable( new FArea(0,0) );
			Area->WindowPlacement = FArea::EWindowPlacementInternal::Placement_NoWindow;
			return Area;
		}

		static TSharedRef<FArea> NewArea( const float Width, const float Height )
		{
			return MakeShareable( new FArea( Width, Height ) );
		}

		static TSharedRef<FArea> NewArea( const FVector2D& WindowSize )
		{
			return MakeShareable( new FArea( UE_REAL_TO_FLOAT(WindowSize.X), UE_REAL_TO_FLOAT(WindowSize.Y) ) );
		}
		
		static TSharedRef<FStack> NewStack() 
		{
			return MakeShareable( new FStack() );
		}
		
		static TSharedRef<FSplitter> NewSplitter()
		{
			return MakeShareable( new FSplitter() );
		}

		static void RegisterDefaultTabWindowSize(const FTabId& TabName, const FVector2D DefaultSize)
		{
			DefaultTabWindowSizeMap.Add(TabName, DefaultSize);
		}

		static void UnregisterDefaultTabWindowSize(const FTabId& TabName)
		{
			DefaultTabWindowSizeMap.Remove(TabName);
		}

		SLATE_API void SetOnPersistLayout( const FOnPersistLayout& InHandler );

		/** Close all live areas and wipe all the persisted areas. */
		SLATE_API void CloseAllAreas();

		/** Gather the persistent layout */
		SLATE_API TSharedRef<FTabManager::FLayout> PersistLayout() const;

		/** Gather the persistent layout and execute the custom delegate for saving it to persistent storage (e.g. into config files) */
		SLATE_API void SavePersistentLayout();

		/** Request a deferred save of the layout. */
		SLATE_API void RequestSavePersistentLayout();

		/** Clears the request for a deferred save of the layout */
		SLATE_API void ClearPendingLayoutSave();

		/**
		 * Register a new tab spawner with the tab manager.  The spawner will be called when anyone calls
		 * InvokeTab().
		 * @param TabId The TabId to register the spawner for.
		 * @param OnSpawnTab The callback that will be used to spawn the tab.
		 * @param CanSpawnTab The callback that will be used to ask if spawning the tab is allowed
		 * @return The registration entry for the spawner.
		 */
		SLATE_API FTabSpawnerEntry& RegisterTabSpawner(const FName TabId, const FOnSpawnTab& OnSpawnTab, const FCanSpawnTab& CanSpawnTab = FCanSpawnTab());

		/**
		 * Unregisters the tab spawner matching the provided TabId.
		 * @param TabId The TabId to remove the spawner for.
		 * @return true if a spawner was found for this TabId, otherwise false.
		 */
		SLATE_API bool UnregisterTabSpawner( const FName TabId );

		/**
		 * Unregisters all tab spawners.
		 */
		SLATE_API void UnregisterAllTabSpawners();

		SLATE_API TSharedPtr<SWidget> RestoreFrom(const TSharedRef<FLayout>& Layout, const TSharedPtr<SWindow>& ParentWindow, const bool bEmbedTitleAreaContent = false,
			const EOutputCanBeNullptr RestoreAreaOutputCanBeNullptr = EOutputCanBeNullptr::Never);

		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API TSharedPtr<SWidget> RestorePanelDrawer(const TSharedRef<SWidget>& InContent, const TSharedRef<SWindow>& ParentWindow);

		SLATE_API void PopulateLocalTabSpawnerMenu( FMenuBuilder& PopulateMe );

		SLATE_API void PopulateTabSpawnerMenu(FMenuBuilder& PopulateMe, TSharedRef<FWorkspaceItem> MenuStructure);

		SLATE_API void PopulateTabSpawnerMenu( FMenuBuilder& PopulateMe, TSharedRef<FWorkspaceItem> MenuStructure, bool bIncludeOrphanedMenus);

		SLATE_API void PopulateTabSpawnerMenu( FMenuBuilder &PopulateMe, const FName& TabType );

		SLATE_API virtual void DrawAttention( const TSharedRef<SDockTab>& TabToHighlight );

		struct ESearchPreference
		{
			enum Type
			{
				PreferLiveTab,
				RequireClosedTab
			};
		};

		/** Insert a new UnmanagedTab document tab next to an existing tab (closed or open) that has the PlaceholdId. Give the New tab NewTabId */
		SLATE_API void InsertNewDocumentTab( FName PlaceholderId, FName NewTabId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab );

		/** Insert a new UnmanagedTab document tab next to an existing tab (closed or open) that has the PlaceholdId. */
		SLATE_API void InsertNewDocumentTab( FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab );
		
		/** Insert a new UnmanagedTab document tab next to an existing tab (closed or open) that has the PlaceholdId. */
		SLATE_API void InsertNewDocumentTab(FName PlaceholderId, ESearchPreference::Type SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab);

		/**
		 * Much like InsertNewDocumentTab, but the UnmanagedTab is not seen by the user as newly-created.
		 * e.g. Opening an restores multiple previously opened documents; these are not seen as new tabs.
		 */
		SLATE_API void RestoreDocumentTab(FName PlaceholderId, ESearchPreference::Type SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab);

		/**
		 * Try to open tab if it is closed at the last known location.  If it already exists, it will draw attention to the tab.
		 *
		 * @param TabId The tab identifier.
		 * @param bInvokeAsInactive	Leave the tab inactive instead of drawing attention to it
		 * @return The existing or newly spawned tab instance if successful.
		 */
		SLATE_API virtual TSharedPtr<SDockTab> TryInvokeTab(const FTabId& TabId, bool bInvokeAsInactive = false);

		/**
		 * Try to toggle a sidebar tab. 
		 *
		 * @param TabId The tab identifier.
		 * @return The toggled tab if toggling was sucessful.
		 */
		SLATE_API virtual TSharedPtr<SDockTab> TryToggleSidebarTab(const FTabId& TabId);

		/**
		 * Try to dissmiss a sidebar tab.
		 *
		 * @param TabId The tab identifier.
		 * @return The toggled tab if dissmissing was successful
		 */
		SLATE_API virtual TSharedPtr<SDockTab> TryDismissSidebarTab(const FTabId& TabId);

		/**
		 * Finds the first instance of an existing tab with the given tab id.
		 *
		 * @param TabId The tab identifier.
		 * @return The existing tab instance if found, otherwise null.
		 */
		SLATE_API TSharedPtr<SDockTab> FindExistingLiveTab(const FTabId& TabId) const;

		/**
		 * Try to open tab in the PanelDrawer. If it already exists elsewhere, it will draw attention to the tab.
		 * If the tab is open in another PanelDrawer it will transfer it to this drawer
		 * If no PanelDrawer is available in the manager we will just open a new standard tab
		 */
		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API TSharedPtr<SDockTab> TryOpenTabInPanelDrawer(const FTabId& TabId, const TSharedPtr<SWindow>& ParentWindow, bool bForceInDrawerPanel = false);

		/**
		 * Try to open tab in the PanelDrawer. If it already exists elsewhere, it will draw attention to the tab.
		 * If the PanelDrawer is already open with the tab it will close the drawer
		 * If the tab is open in another PanelDrawer it will transfer it to this drawer
		 * If no PanelDrawer is available in the manager we will just open a new standard tab
		 */
		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API TSharedPtr<SDockTab> TryToggleTabInPanelDrawer(const FTabId& TabId, const TSharedPtr<SWindow>& ParentWindow, bool bForceInDrawerPanel = false);

		SLATE_API virtual ~FTabManager();

		/** Sets whether or not this tab manager supports a custom menu bar for the active major tab that will be shown on top of the major tab area in the window this tab manager resides in. */
		SLATE_API void SetAllowWindowMenuBar(bool bInAllowWindowMenuBar);

		/** Whether or not this tab manager supports a custom menu bar for the active major tab that will be shown on top of the major tab area in the window this tab manager resides in. */
		bool AllowsWindowMenuBar() const { return bAllowPerWindowMenu; }

		/**
		 * Set the multi-box to use for generating a global menu bar.  The implementation is platform and setting specific
		 * On Mac the menu bar appears globally at the top of the desktop in all cases regardless of whether or not SetAllowWindowMenuBar is called.  On other desktop platforms the menu appears at the top of the window this tab manager is a part of only if SetAllowWindowMenuBar(true) is called.
		 * @param NewMenuMutliBox The multi-box to generate the global menu bar from.
		 */
		SLATE_API void SetMenuMultiBox(const TSharedPtr<FMultiBox> NewMenuMutliBox, const TSharedPtr<SWidget> MenuWidget);

		/**
		 * Update the native, global menu bar if it is being used.
		 * @param bForce Used to force an update even if the parent window doesn't contain the widget with keyboard focus.
		 */
		SLATE_API void UpdateMainMenu(TSharedPtr<SDockTab> ForTab, const bool bForce);

		/** Provide a tab that will be the main tab and cannot be closed.
		 *  Prefer using the version that takes in a TabID so you can set the MainTab before it is actually created
		 */
		SLATE_API void SetMainTab(const TSharedRef<const SDockTab>& InTab);

		SLATE_API void SetMainTab(const FTabId& InMainTabID);

		/** Is this Tab Manager in Read Only mode i.e all interactions with panels are disabled */
		SLATE_API bool IsReadOnly();

		SLATE_API void SetReadOnly(bool bInReadOnly);
	
		FOnReadOnlyModeChanged& GetOnReadOnlyModeChangedDelegate() { return OnReadOnlyModeChanged; }

		FOnToggleTemporarilySidebaredTabs& GetOnToggleTemporarilySidebaredTabsDelegate() { return OnToggleTemporarilySidebaredTabs; }

		/* Prevent or allow all tabs to be drag */
		void SetCanDoDragOperation(bool CanDoDragOperation) { bCanDoDragOperation = CanDoDragOperation; }
		
		/* Return true if we can do drag operation */
		bool GetCanDoDragOperation() { return bCanDoDragOperation; }

		/** @return if the provided tab can be closed. */
		SLATE_API bool IsTabCloseable(const TSharedRef<const SDockTab>& InTab) const;

		/** @return true if a tab is ever allowed in a sidebar */
		SLATE_API bool IsTabAllowedInSidebar(const FTabId TabId) const;

		/** @return how the given tab wants to behave in read only mode */
		TOptional<ETabReadOnlyBehavior> GetTabReadOnlyBehavior(const FTabId& TabId) const;

		/**
		 * Temporarily moves all open tabs in this tab manager to a sidebar or restores them from a temporary state
		 */
		SLATE_API void ToggleSidebarOpenTabs(const TSet<FTabId>& InExceptions = {});

		/** @return The local workspace menu root */
		SLATE_API const TSharedRef<FWorkspaceItem> GetLocalWorkspaceMenuRoot() const;

		/** Adds a category to the local workspace menu by name */
		SLATE_API TSharedRef<FWorkspaceItem> AddLocalWorkspaceMenuCategory( const FText& CategoryTitle );

		/** Adds an existing workspace item to the local workspace menu */
		SLATE_API void AddLocalWorkspaceMenuItem( const TSharedRef<FWorkspaceItem>& CategoryItem );

		/** Clears all categories in the local workspace menu */
		SLATE_API void ClearLocalWorkspaceMenuCategories();

		/** @return true if the tab has a factory registered for it that allows it to be spawned. */
		SLATE_API bool HasTabSpawner(FName TabId) const;

		/** Returns the owner tab (if it exists) */
		TSharedPtr<SDockTab> GetOwnerTab() { return OwnerTabPtr.Pin(); }

		/** Returns filter for additional control over available tabs */
		SLATE_API TSharedRef<FNamePermissionList>& GetTabPermissionList();

		SLATE_API FUIAction GetUIActionForTabSpawnerMenuEntry(TSharedPtr<FTabSpawnerEntry> InTabMenuEntry);

		/** Return an array of all tab spawners that should have menu items in the main menu. */
		SLATE_API TArray< TWeakPtr<FTabSpawnerEntry> > CollectSpawners();

		/** Find a tab spawner by name. */
		SLATE_API TSharedPtr<FTabSpawnerEntry> FindTabSpawnerFor(FName TabId);

		/** Find a tab spawner by name. */
		SLATE_API const TSharedPtr<const FTabSpawnerEntry> FindTabSpawnerFor(FName TabId) const;

	    /** Return whether there are any temporarily sidebarred tabs **/
		SLATE_API bool HasSidebaredTabs() const;
	
		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API bool HasPanelDrawer(const TSharedPtr<SWindow>& ParentWindow) const;

		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API bool IsPanelDrawerOpen(const TSharedPtr<SWindow>& ParentWindow) const;

		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API void ClosePanelDrawer(const TSharedPtr<SWindow>& ParentWindow);

		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnPanelDrawerStateChanged, const FTabManager&);

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API FDelegateHandle RegisterOnPanelDrawerStateChanges(FOnPanelDrawerStateChanged::FDelegate&& InDelegate);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

		UE_EXPERIMENTAL(5.7, "The panel drawer is still in devel1opment and might be removed. The api, implementation details and its form may change in near releases.")
		SLATE_API void UnregisterOnPanelDrawerStateChanges(FDelegateHandle InHandle);

	protected:
		SLATE_API void InvokeTabForMenu( FName TabId );

	protected:
		
		SLATE_API void InsertDocumentTab( FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab, bool bPlaySpawnAnim );
		SLATE_API void InsertDocumentTab( FName PlaceholderId, FName NewTabId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab, bool bPlaySpawnAnim );

		SLATE_API virtual void OpenUnmanagedTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab);
			
		SLATE_API void PopulateTabSpawnerMenu_Helper( FMenuBuilder& PopulateMe, struct FPopulateTabSpawnerMenu_Args Args );

		SLATE_API void MakeSpawnerMenuEntry( FMenuBuilder &PopulateMe, const TSharedPtr<FTabSpawnerEntry> &InSpawnerNode );

		SLATE_API TSharedPtr<SDockTab> InvokeTab_Internal(const FTabId& TabId, bool bInvokeAsInactive = false, bool bForceOpenWindowIfNeeded = false);

		/** Finds the last major or nomad tab in a particular window. */
		SLATE_API TSharedPtr<SDockTab> FindLastTabInWindow(TSharedPtr<SWindow> Window) const;

		SLATE_API TSharedPtr<SDockingTabStack> FindPotentiallyClosedTab( const FTabId& ClosedTabId );

		typedef TMap< FName, TSharedRef<FTabSpawnerEntry> > FTabSpawner;
		
		static TSharedRef<FTabManager> New( const TSharedPtr<SDockTab>& InOwnerTab, const TSharedRef<FTabSpawner>& InNomadTabSpawner )
		{
			return MakeShareable( new FTabManager(InOwnerTab, InNomadTabSpawner) );
		}

		SLATE_API FTabManager( const TSharedPtr<SDockTab>& InOwnerTab, const TSharedRef<FTabManager::FTabSpawner> & InNomadTabSpawner );

	SLATE_API TSharedPtr<SDockingArea> RestoreArea(
		const TSharedRef<FArea>& AreaToRestore, const TSharedPtr<SWindow>& InParentWindow, const bool bEmbedTitleAreaContent = false,
		const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::Never, bool bForceOpenWindowIfNeeded = false);

	SLATE_API TSharedPtr<class SDockingNode> RestoreArea_Helper(const TSharedRef<FLayoutNode>& LayoutNode, const TSharedPtr<SWindow>& ParentWindow, const bool bEmbedTitleAreaContent,
		FSidebarTabLists& OutSidebarTabs, const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::Never, bool bForceOpenWindowIfNeeded = false);

		/**
		 * Use CanRestoreSplitterContent + RestoreSplitterContent when the output of its internal RestoreArea_Helper can be a nullptr.
		 * Usage example:
		 *		TArray<TSharedRef<SDockingNode>> DockingNodes;
		 *		if (CanRestoreSplitterContent(DockingNodes, SplitterNode, ParentWindow, OutputCanBeNullptr))
		 *		{
		 *			// Create SplitterWidget only if it will be filled with at least 1 DockingNodes
		 *			TSharedRef<SDockingSplitter> SplitterWidget = SNew(SDockingSplitter, SplitterNode);
		 *			// Restore content
		 *			RestoreSplitterContent(DockingNodes, SplitterWidget);
		 *		}
		 */
		SLATE_API bool CanRestoreSplitterContent(TArray<TSharedRef<class SDockingNode>>& DockingNodes, const TSharedRef<FSplitter>& SplitterNode, const TSharedPtr<SWindow>& ParentWindow, FSidebarTabLists& OutSidebarTabs, const EOutputCanBeNullptr OutputCanBeNullptr);
		SLATE_API void RestoreSplitterContent(const TArray<TSharedRef<class SDockingNode>>& DockingNodes, const TSharedRef<class SDockingSplitter>& SplitterWidget);

		/**
		 * Use this standalone RestoreSplitterContent when the output of its internal RestoreArea_Helper cannot be a nullptr.
		 */
		SLATE_API void RestoreSplitterContent(const TSharedRef<FSplitter>& SplitterNode, const TSharedRef<class SDockingSplitter>& SplitterWidget, const TSharedPtr<SWindow>& ParentWindow, FSidebarTabLists& OutSidebarTabs);

		SLATE_API bool IsValidTabForSpawning( const FTab& SomeTab ) const;
		SLATE_API bool IsAllowedTab(const FTabId& TabId) const;
		SLATE_API bool IsAllowedTabType(const FName TabType) const;

		SLATE_API TSharedPtr<SDockTab> SpawnTab(const FTabId& TabId, const TSharedPtr<SWindow>& ParentWindow, const bool bCanOutputBeNullptr = false);

		SLATE_API TSharedPtr<class SDockingTabStack> FindTabInLiveAreas( const FTabMatcher& TabMatcher ) const;
		static SLATE_API TSharedPtr<class SDockingTabStack> FindTabInLiveArea( const FTabMatcher& TabMatcher, const TSharedRef<SDockingArea>& InArea );

		static bool HasAnyMatchingTabs( const TSharedRef<FTabManager::FLayoutNode>& SomeNode, const TFunctionRef<bool(const FTab&)>& Matcher );

	public:
		/**
		 * It searches for valid and open tabs on SomeNode.
		 * @return It returns true if there is at least a valid open tab in the input SomeNode.
		 */
		SLATE_API bool HasValidOpenTabs( const TSharedRef<FTabManager::FLayoutNode>& SomeNode ) const;

		/**
		 * Gets a TSharedPtr<FArea> with the given FTabId if present, else it return nullptr
		 *
		 * @param InTabIdToMatch the const &FTabId for which to find the FArea
		 */
	    SLATE_API TSharedPtr<FArea> GetAreaFromInitialLayoutWithTabType(const FTabId& InTabIdToMatch ) const;

protected:
		SLATE_API bool HasValidTabs( const TSharedRef<FTabManager::FLayoutNode>& SomeNode ) const;

		/**
		 * It sets the desired (or all) tabs in the FTabManager::FLayoutNode to the desired value.
		 * @param SomeNode The area whose tabs will be modified.
		 * @param NewTabState The new TabState value.
		 * @param OriginalTabState Only the tabs with this value will be modified. Use ETabState::AnyTab to modify them all.
		 */
		SLATE_API void SetTabsTo(const TSharedRef<FTabManager::FLayoutNode>& SomeNode, const ETabState::Type NewTabState, const ETabState::Type OriginalTabState) const;

		/**
		 * Notify the tab manager that the NewForegroundTab was brought to front and the BackgroundedTab was send to the background as a result.
		 */
		SLATE_API virtual void OnTabForegrounded( const TSharedPtr<SDockTab>& NewForegroundTab, const TSharedPtr<SDockTab>& BackgroundedTab );
		SLATE_API virtual void OnTabRelocated( const TSharedRef<SDockTab>& RelocatedTab, const TSharedPtr<SWindow>& NewOwnerWindow );
		SLATE_API virtual void OnTabOpening( const TSharedRef<SDockTab>& TabBeingOpened );
		SLATE_API virtual void OnTabClosing( const TSharedRef<SDockTab>& TabBeingClosed );
		/** Invoked when a tab manager is closing down. */
		SLATE_API virtual void OnTabManagerClosing();
		/** Check these all tabs to see if it is OK to close them. Ignore the TabsToIgnore */
		SLATE_API virtual bool CanCloseManager( const TSet< TSharedRef<SDockTab> >& TabsToIgnore = TSet< TSharedRef<SDockTab> >() );

		static SLATE_API void GetAllStacks( const TSharedRef<SDockingArea>& InDockArea, TArray< TSharedRef<SDockingTabStack> >& OutTabStacks );

		/** @return the stack that is under NodeToSearchUnder and contains TabIdToFind; Invalid pointer if not found. */
		static SLATE_API TSharedPtr<FTabManager::FStack> FindTabUnderNode( const FTabMatcher& Matcher, const TSharedRef<FTabManager::FLayoutNode>& NodeToSearchUnder );
		SLATE_API int32 FindTabInCollapsedAreas( const FTabMatcher& Matcher );
		SLATE_API void RemoveTabFromCollapsedAreas( const FTabMatcher& Matcher );

		/** Called when tab(s) have been added or windows created */
		SLATE_API virtual void UpdateStats();
		
		/** Called at the end of RestoreFrom for tab managers to complete any work after all tabs have been restored */
		virtual void FinishRestore() {};


		/*
		 * Search the docking area to see if we have one that is part of the window.
		 */
		TSharedPtr<SDockingArea> GetDockingAreaForWindow(const TSharedRef<SWindow>& InWindow) const;

		/*
		 * Search the docking area to see if we have one that is part of the window. With some customized Logic for the panel drawer.
		 */
		TSharedPtr<SDockingArea> GetDockingAreaForPanelDrawer(const TSharedPtr<SWindow>& InWindow) const;

	private:
		/** Checks all dock areas and adds up the number of open tabs and unique parent windows in the manager */
		SLATE_API void GetRecordableStats( int32& OutTabCount, TArray<TSharedPtr<SWindow>>& OutUniqueParentWindows ) const;

	/** If the HomeScreen tab is enabled this will force the primary area to contain the HomeScreen and the LevelEditor next to each other otherwise it will just force the LevelEditor to be the first tab */
	void FixLayoutLoadingPrimaryArea(const TSharedRef<FArea>& InPrimaryArea);

	TSharedPtr<SDockingTabStack> AttemptToOpenTab( const FTabId& ClosedTabId, bool bForceOpenWindowIfNeeded = false );

	/**
	 * Returns true if the given FLayoutNode contains a tab with the given FTabId.TabType
	 *
	 * @param InTabTypeToMatch the given FTabId.TabType to look to see if this tab manager manages it
	 * @param SomeNode the TSharedRef<FLayoutNode> in which to look for the tab with FTabId.TabType == InTabTypeToMatch
	 */
	bool HasAnyTabWithTabId( const TSharedRef<FTabManager::FLayoutNode>& SomeNode, const FName& InTabTypeToMatch ) const;

	/**
	 * Given FTabId returns the proper TSharedRef<FArea> to load it
	 *
	 * @param TabId the FTabId for which to find the FArea 
	 */
	TSharedRef<FArea> GetAreaForTabId(const FTabId& TabId);

	/** Based on if the tab can have a hidden display name it will return the correct Label to use for the tab */
	FText GetTabLabelBasedOnSpawner(const TSharedPtr<FTabSpawnerEntry>& InSpawnerEntry) const;

	/** Does the shared logic between InvokeTab_Internal and InvokeTabInPanelDrawer */
	bool InvokeTab_CanInvokeTab(const FTabId& TabId) const;
	TSharedPtr<SDockTab> InvokeTab_FindOrReuseExistingTab(const FTabId& TabId) const;
	void InvokeTab_DrawAttentionToTab(const TSharedPtr<SDockTab>& InvokedTab);

	TSharedPtr<SDockTab> InvokeTabInPanelDrawer(const FTabId& TabId, bool bToggleIfActiveInDrawer, const TSharedPtr<SWindow>& ParentWindow, bool bForceInDrawerPanel);

	/** Check if the tab can be invoked in a PanelDrawer, also return the tab since we need it to do some test */
	bool CanInvokeInPanelDrawer(const FTabId& TabId, const TSharedPtr<SDockingArea>& DockingArea, TSharedPtr<SDockTab>& OutTab, bool bForceInDrawerPanel);

	/** Find the right area to host the tab and setup its PanelDrawer to host the tab */
	void SetTabInPanelDrawer(const TSharedPtr<SDockTab>& InTab, const TSharedPtr<SDockingArea>& TargetDockingArea, const TSharedPtr<SWindow>& InWindow);

	/** Broadcasted when a PanelDrawer is opened or closed */
	void OnPanelDrawerStateChanged() const;

	/** Remove the tab from keep alive list of the PanelDrawers. Return true if found in the PanelDrawer keep alive */
	bool RemoveFromHiddenPanelDrawerTabs(TSharedPtr<SDockTab> ExistingTab);

	/** Transfer the KeepAlive tab to TabManager and clean the PanelDrawer data */
	void HandleClosingAreaPanelDrawerData(const TSharedPtr<SDockingArea>& ClosingDockingArea);

	protected:
		FTabSpawner TabSpawner;
		TSharedRef<FTabSpawner> NomadTabSpawner;

		SLATE_API bool HasTabSpawnerFor(FName TabId) const;

		TArray< TWeakPtr<SDockingArea> > DockAreas;
		/**
		 * CollapsedDockAreas refers to areas that were closed (e.g., by the user).
		 * We save its location so they can be re-opened in the same location if the user opens thems again.
		 */
		TArray< TSharedRef<FTabManager::FArea> > CollapsedDockAreas;
		/**
		 * InvalidDockAreas refers to areas that were not recognized by the Editor (e.g., LiveLink when its plugin its disabled).
		 * We save its location so they can be re-opened whenever they are recognized again (e.g., if its related plugin is re-enabled).
		 */
		TArray< TSharedRef<FTabManager::FArea> > InvalidDockAreas;

		/** The root for the local editor's tab spawner workspace menu */
		TSharedPtr<FWorkspaceItem> LocalWorkspaceMenuRoot;

		/** A Major tab that contains this TabManager's widgets. */
		TWeakPtr<SDockTab> OwnerTabPtr;

		/** The current menu multi-box for the tab, used to construct platform native main menus */
		TSharedPtr<FMultiBox> MenuMultiBox;
		TSharedPtr<SWidget> MenuWidget;

		/** Protected private API that must only be accessed by the docking framework internals */
		TSharedRef<FPrivateApi> PrivateApi;

		/** The name of the layout being used */
		FName ActiveLayoutName;

		/** Invoked when the tab manager is about to close */
		FOnPersistLayout OnPersistLayout_Handler;

		/**
		 * Instance ID for document tabs. Allows us to distinguish between different document tabs at runtime.
		 * This ID is never meant to be persisted, simply used to disambiguate between different documents, since most of thenm
		 * will have the same Tab Type (which is usually document).
		 */
		int32 LastDocumentUID;

		/** The fallback size for a window */
		SLATE_API const static FVector2D FallbackWindowSize;

		/** Default tab window sizes for newly-created tabs */
		static SLATE_API TMap<FTabId, FVector2D> DefaultTabWindowSizeMap;

		/** Returns the default window size for the TabId, or the fallback window size if it wasn't registered */
		static SLATE_API FVector2D GetDefaultTabWindowSize(const FTabId& TabId);

		/** The ID of the main tab, which cannot be closed*/
		FTabId MainNonCloseableTabID;

		/** To keep track of any pending tabs that want to be the main tab, but don't have their ID set yet */
		TSharedPtr<const SDockTab> PendingMainNonClosableTab;

		/** If true, this tab manager will always have one non-closable tab */
		bool bEnforceMainTab = false;

		/** The last window we docked a nomad or major tab into */
		TWeakPtr<SWindow> LastMajorDockWindow;

		/* Prevent or allow Drag operation. */
		bool bCanDoDragOperation = true;

		/** Whether or not this tab manager can perform deferred layout saves at this time; used internally to disable layout saves during drag operations */
		bool bCanDoDeferredLayoutSave = true;

		/** Whether or not this tab manager puts any registered menus in the windows menu bar area */
		bool bAllowPerWindowMenu = false;

		/** Handle to a pending layout save. */
		FTSTicker::FDelegateHandle PendingLayoutSaveHandle;

		/** Allow systems to dynamically hide tabs */
		TSharedRef<FNamePermissionList> TabPermissionList;

		/** Tabs which have been temporarily put in the a sidebar */
		TArray<TWeakPtr<SDockTab>> TemporarilySidebaredTabs;

		/** Whether this tab manager is in a read only mode (all tabs content disabled by default - overridable per tab) */
		bool bReadOnly = false;

		/** Delegate that broadcasts when the tab manager enters/leaves read only mode */
		FOnReadOnlyModeChanged OnReadOnlyModeChanged;

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		/** Delegate that broadcast when the PanelDrawer of the manager open or close */
		FOnPanelDrawerStateChanged OnPanelDrawerStateChangedDelegate;
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

		/** Keep alive the PanelDrawer tabs that have no area */
		TMap<FTabId, TSharedRef<SDockTab>> PanelDrawerActiveHiddenTabs;

		/** Delegate that broadcasts when the tab manager toggles between temporarily sidebared tabs */
		FOnToggleTemporarilySidebaredTabs OnToggleTemporarilySidebaredTabs;
};



class FProxyTabmanager;


class FGlobalTabmanager : public FTabManager
{
public:	

	static SLATE_API const TSharedRef<FGlobalTabmanager>& Get();

	/** Subscribe to notifications about the active tab changing */
	SLATE_API FDelegateHandle OnActiveTabChanged_Subscribe( const FOnActiveTabChanged::FDelegate& InDelegate );

	/** Unsubscribe to notifications about the active tab changing */
	SLATE_API void OnActiveTabChanged_Unsubscribe( FDelegateHandle Handle );

	/** Subscribe to notifications about a foreground tab changing */
	SLATE_API FDelegateHandle OnTabForegrounded_Subscribe(const FOnActiveTabChanged::FDelegate& InDelegate);

	/** Unsubscribe to notifications about a foreground tab changing */
	SLATE_API void OnTabForegrounded_Unsubscribe(FDelegateHandle Handle);

	/** @return the currently active tab; NULL pointer if there is no active tab */
	SLATE_API TSharedPtr<SDockTab> GetActiveTab() const;

	/** Can the manager activate this Tab as the new active tab? */
	SLATE_API bool CanSetAsActiveTab(const TSharedPtr<SDockTab>& Tab);

	/** Activate the NewActiveTab. If NewActiveTab is NULL, the active tab is cleared. */
	SLATE_API void SetActiveTab( const TSharedPtr<SDockTab>& NewActiveTab );

	/**
	 * Register a new nomad tab spawner with the tab manager. The spawner will be called when anyone calls InvokeTab().
	 * A nomad tab is a tab that can be placed with major tabs or minor tabs in any tab well.
	 * @param TabId The TabId to register the spawner for.
	 * @param OnSpawnTab The callback that will be used to spawn the tab.
	 * @param CanSpawnTab The callback that will be used to ask if spawning the tab is allowed.
	 * @return The registration entry for the spawner.
	 */
	SLATE_API FTabSpawnerEntry& RegisterNomadTabSpawner( const FName TabId, const FOnSpawnTab& OnSpawnTab, const FCanSpawnTab& CanSpawnTab = FCanSpawnTab());

	SLATE_API void UnregisterNomadTabSpawner( const FName TabId );

	SLATE_API void SetApplicationTitle( const FText& AppTitle );

	SLATE_API const FText& GetApplicationTitle() const;

	static TSharedRef<FGlobalTabmanager> New()
	{
		return MakeShareable( new FGlobalTabmanager() );
	}

	SLATE_API virtual bool CanCloseManager( const TSet< TSharedRef<SDockTab> >& TabsToIgnore = TSet< TSharedRef<SDockTab> >()) override;

	/** Gets the major tab for the manager */
	SLATE_API TSharedPtr<SDockTab> GetMajorTabForTabManager(const TSharedRef<FTabManager>& ChildManager);

	/** 
	 * Gets the tab manager that a major tab owns. The returned tab manager is the tab manager that manages the minor tabs for a major tab.
	 * Note: this is not the same as DockTab->GetTabManager(). That function returns the tab manager the major tab is in
	 */
	SLATE_API TSharedPtr<FTabManager> GetTabManagerForMajorTab(const TSharedPtr<SDockTab> DockTab) const;

	/**
	 * Get the tab manager that manage the minor tabs for a window if there is one
	 */
	SLATE_API TSharedPtr<FTabManager> GetSubTabManagerForWindow(const TSharedRef<SWindow>& InWindow) const;

	/** Draw the user's attention to a child tab manager */
	SLATE_API void DrawAttentionToTabManager( const TSharedRef<FTabManager>& ChildManager );

	SLATE_API TSharedRef<FTabManager> NewTabManager( const TSharedRef<SDockTab>& InOwnerTab );
	
	/**
	 * Update the native, global menu bar if it is being used for a specific tab managed by the global tab manager.
	 * @param ForTab The tab to update the main menu.
	 * @param bForce Used to force an update even if the parent window doesn't contain the widget with keyboard focus.
	 */
	SLATE_API void UpdateMainMenu(const TSharedRef<SDockTab>& ForTab, bool const bForce);

	/** Persist and serialize the layout of every TabManager and the custom visual state of every Tab. */
	SLATE_API void SaveAllVisualState();

	/** Provide a window under which all other windows in this application should nest. */
	SLATE_API void SetRootWindow( const TSharedRef<SWindow> InRootWindow );

	/** The window under which all other windows in our app nest; might be null */
	SLATE_API TSharedPtr<SWindow> GetRootWindow() const;

	/** Adds a legacy tab type to the tab type redirection map so tabs loaded with this type will be automatically converted to the new type */
	SLATE_API void AddLegacyTabType(FName InLegacyTabType, FName InNewTabType);

	/** Returns true if the specified tab type is registered as a legacy tab */
	SLATE_API bool IsLegacyTabType(FName InTabType) const;

	/** If the specified TabType is deprecated, returns the new replacement tab type. Otherwise, returns InTabType */
	SLATE_API FName GetTabTypeForPotentiallyLegacyTab(FName InTabType) const;

	/** Returns the highest number of tabs that were open simultaneously during this session */
	int32 GetMaximumTabCount() const { return AllTabsMaxCount; }

	/** Returns the highest number of parent windows that were open simultaneously during this session */
	int32 GetMaximumWindowCount() const { return AllAreasWindowMaxCount; }

	/** Sets whether or not any layouts ( across all tab managers ) can be saved at this time. */
	SLATE_API void SetCanSavePersistentLayouts(bool bInCanSavePersistentLayouts);

	/** Returns true if any layouts ( across all tab managers ) can be saved at this time. */
	SLATE_API bool CanSavePersistentLayouts() const;

	SLATE_API void SetProxyTabManager(TSharedPtr<FProxyTabmanager> InProxyTabManager);

	/** Get if the DockTab label should use the MiddleEllipsis overflow policy */
	SLATE_API bool GetShouldUseMiddleEllipsisForDockTabLabel() const;

	/** Set whether the DockTab label should use the MiddleEllipsis overflow policy */
	SLATE_API void SetShouldUseMiddleEllipsisForDockTabLabel(const bool bInShouldUseMiddleEllipsis);

	DECLARE_DELEGATE(FOnOverrideDockableAreaRestore);
	/** Used to override dockable area restoration behavior */
	FOnOverrideDockableAreaRestore OnOverrideDockableAreaRestore_Handler;

	/**
	 * Get all dockable windows managed by this tab manager
	 * @return Array of all windows that contain dock areas
	 */
	SLATE_API TArray<TSharedPtr<SWindow>> GetDockableWindows() const;

	struct FWindowMajorTabsInfo
	{
		/** The window these info belongs to */
		TSharedPtr<SWindow> Window;

		/** Ordered tabs docked in the window */
		TArray<TSharedPtr<SDockTab>> DockTabs;

		/** Active/foreground tab in the window */
		TSharedPtr<SDockTab> ActiveTab;
	};

	/**
	 * Get all major (top-level) tabs info for a specific window
	 * @param InWindow The window to query info from
	 * @return Window tabs info struct
	 */
	SLATE_API FWindowMajorTabsInfo GetDockableWindowMajorTabsInfo(const TSharedRef<SWindow>& InWindow) const;

protected:
	SLATE_API virtual void OnTabForegrounded( const TSharedPtr<SDockTab>& NewForegroundTab, const TSharedPtr<SDockTab>& BackgroundedTab ) override;
	SLATE_API virtual void OnTabRelocated( const TSharedRef<SDockTab>& RelocatedTab, const TSharedPtr<SWindow>& NewOwnerWindow ) override;
	SLATE_API virtual void OnTabClosing( const TSharedRef<SDockTab>& TabBeingClosed ) override;
	SLATE_API virtual void UpdateStats() override;

	SLATE_API virtual void OpenUnmanagedTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab) override;

	SLATE_API virtual void FinishRestore() override;

public:
	SLATE_API virtual void OnTabManagerClosing() override;

	/**
	 * Sets the initial layout shared pointer, which can be used later to get the current layout for tabs which were not
	 * spawned on initialization
	 *
	 * @param InLayout the FLayout which was loaded at layout load time
	 */
	SLATE_API void SetInitialLayoutSP(TSharedPtr<FTabManager::FLayout> InLayout);

	/**
	 * Gets the initial layout shared pointer, which can be used later to get the current layout for tabs which were not
	 * spawned on initialization
	 */
	SLATE_API TSharedPtr<FTabManager::FLayout> GetInitialLayoutSP();
	
private:

	using FTabAndManagerForDrawer = TPair<TSharedPtr<SDockTab>, TSharedPtr<FTabManager>>;
	void RelocatePanelDrawerNomadTab(FTabAndManagerForDrawer ForegroundTabAndSubManager, FTabAndManagerForDrawer BackgroundedTabAndSubManager);

	/** Whether tabs label should use MiddleEllipsis OverflowPolicy*/
	bool bShouldUseMiddleEllipsisForDockTabLabel = false;

	/** Pairs of Major Tab and the TabManager that manages tabs within it. */
	struct FSubTabManager
	{
		FSubTabManager( const TSharedRef<SDockTab>& InMajorTab, const TSharedRef<FTabManager>& InTabManager )
		: MajorTab( InMajorTab )
		, TabManager( InTabManager )
		{
		}

		TWeakPtr<SDockTab> MajorTab;
		TWeakPtr<FTabManager> TabManager;
	};

	struct FindByTab
	{
		FindByTab(const TSharedRef<SDockTab>& InTabToFind)
			: TabToFind(InTabToFind)
		{
		}

		bool operator()(const FGlobalTabmanager::FSubTabManager& TabManagerPair) const
		{
			return TabManagerPair.TabManager.IsValid() && TabManagerPair.MajorTab.IsValid() && TabManagerPair.MajorTab.Pin() == TabToFind;
		}

		const TSharedRef<SDockTab>& TabToFind;
	};

	struct FindByManager
	{
		FindByManager(const TSharedRef<FTabManager>& InManagerToFind)
			: ManagerToFind(InManagerToFind)
		{
		}

		bool operator()(const FGlobalTabmanager::FSubTabManager& TabManagerPair) const
		{
			return TabManagerPair.TabManager.IsValid() && TabManagerPair.MajorTab.IsValid() && TabManagerPair.TabManager.Pin() == ManagerToFind;
		}

		const TSharedRef<FTabManager>& ManagerToFind;
	};

	FGlobalTabmanager()
	: FTabManager( TSharedPtr<SDockTab>(), MakeShareable( new FTabSpawner() ) )
		, AllTabsMaxCount(0)
		, AllAreasWindowMaxCount(0)
		, bCanSavePersistentLayouts(true)
	{
	}

	TArray< FSubTabManager > SubTabManagers;

	/** The currently active tab; NULL if there is no active tab. */
	TWeakPtr<SDockTab> ActiveTabPtr;

	FOnActiveTabChanged OnActiveTabChanged;

	FOnActiveTabChanged TabForegrounded;

	FText AppTitle;

	/** A window under which all of the windows in this application will nest. */
	TWeakPtr<SWindow> RootWindowPtr;

	/** A map that correlates deprecated tab types to new tab types */
	TMap<FName, FName> LegacyTabTypeRedirectionMap;

	/** Keeps track of the running-maximum number of tabs in all dock areas and sub-managers during this session */
	int32 AllTabsMaxCount;

	/** Keeps track of the running-maximum number of unique parent windows in all dock areas and sub-managers during this session */
	int32 AllAreasWindowMaxCount;

	/** true if any layouts ( across all tab managers ) can be saved at this time. */
	bool bCanSavePersistentLayouts;

	/**  */
	TSharedPtr<FProxyTabmanager> ProxyTabManager;

	/**
	 * the initial layout shared pointer, which can be used to get the current layout for tabs which were not
	 * spawned on initialization
	 */
	TSharedPtr<FTabManager::FLayout> InitialLayoutSP;
};

//#HACK VREDITOR - Had to introduce the proxy tab manager to steal asset tabs.

DECLARE_MULTICAST_DELEGATE_TwoParams(FIsTabSupportedEvent, FTabId /* TabId */, bool& /* bOutIsSupported */ );
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTabEvent, TSharedPtr<SDockTab>);


class FProxyTabmanager : public FTabManager
{
public:

	FProxyTabmanager(TSharedRef<SWindow> InParentWindow)
		: FTabManager(TSharedPtr<SDockTab>(), MakeShareable(new FTabSpawner()))
		, ParentWindow(InParentWindow)
	{
		bCanDoDragOperation = false;
	}

	SLATE_API virtual void OpenUnmanagedTab(FName PlaceholderId, const FSearchPreference& SearchPreference, const TSharedRef<SDockTab>& UnmanagedTab) override;
	SLATE_API virtual void DrawAttention(const TSharedRef<SDockTab>& TabToHighlight) override;
	
	SLATE_API bool IsTabSupported( const FTabId TabId ) const;
	SLATE_API void SetParentWindow(TSharedRef<SWindow> InParentWindow);

public:
	FIsTabSupportedEvent OnIsTabSupported;
	FOnTabEvent OnTabOpened;
	FOnTabEvent OnAttentionDrawnToTab;

private:
	TWeakPtr<SWindow> ParentWindow;
};
