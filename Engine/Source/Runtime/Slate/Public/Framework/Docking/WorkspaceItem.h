// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "Misc/Attribute.h"

struct FTabSpawnerEntry;

class FWorkspaceItem : public TSharedFromThis<FWorkspaceItem>
{
protected:
	struct FWorkspaceItemSort
	{
		inline bool operator()( const TSharedRef<FWorkspaceItem> A, const TSharedRef<FWorkspaceItem> B ) const
		{
			if ( A->GetChildItems().Num() > 0 )
			{
				if ( B->GetChildItems().Num() == 0 )
				{
					return true;
				}
			}
			else if ( B->GetChildItems().Num() > 0 )
			{
				return false;
			}
			return ( A->GetDisplayName().CompareTo( B->GetDisplayName() ) == -1 );
		}
	};

public:
	static TSharedRef<FWorkspaceItem> NewGroup( const FText& DisplayName, const FSlateIcon& Icon = FSlateIcon(), const bool bSortChildren = false )
	{
		return MakeShared<FWorkspaceItem>(NAME_None, DisplayName, Icon, bSortChildren);
	}

	static TSharedRef<FWorkspaceItem> NewGroup( const FName& Name, const FText& DisplayName, const FSlateIcon& Icon = FSlateIcon(), const bool bSortChildren = false )
	{
		return MakeShared<FWorkspaceItem>(Name, DisplayName, Icon, bSortChildren);
	}

	static TSharedRef<FWorkspaceItem> NewGroup( const FText& DisplayName, const FText& TooltipText, const FSlateIcon& Icon = FSlateIcon(), const bool bSortChildren = false )
	{
		return MakeShared<FWorkspaceItem>(NAME_None, DisplayName, TooltipText, Icon, bSortChildren);
	}

	static TSharedRef<FWorkspaceItem> NewGroup( const FName& Name, const FText& DisplayName, const FText& TooltipText, const FSlateIcon& Icon = FSlateIcon(), const bool bSortChildren = false )
	{
		return MakeShared<FWorkspaceItem>(Name, DisplayName, TooltipText, Icon, bSortChildren);
	}

	TSharedRef<FWorkspaceItem> AddGroup( const FText& InDisplayName, const FSlateIcon& InIcon = FSlateIcon(), const bool InSortChildren = false )
	{
		TSharedRef<FWorkspaceItem> NewItem = FWorkspaceItem::NewGroup(NAME_None, InDisplayName, InIcon, InSortChildren);
		AddItem( NewItem );
		return NewItem;
	}

	TSharedRef<FWorkspaceItem> AddGroup( const FName& InName, const FText& InDisplayName, const FSlateIcon& InIcon = FSlateIcon(), const bool InSortChildren = false )
	{
		TSharedRef<FWorkspaceItem> NewItem = FWorkspaceItem::NewGroup(InName, InDisplayName, InIcon, InSortChildren);
		AddItem( NewItem );
		return NewItem;
	}

	TSharedRef<FWorkspaceItem> AddGroup( const FText& InDisplayName, const FText& InTooltipText, const FSlateIcon& InIcon = FSlateIcon(), const bool InSortChildren = false )
	{
		TSharedRef<FWorkspaceItem> NewItem = FWorkspaceItem::NewGroup(NAME_None, InDisplayName, InTooltipText, InIcon, InSortChildren);
		AddItem( NewItem );
		return NewItem;
	}

	TSharedRef<FWorkspaceItem> AddGroup( const FName& InName, const FText& InDisplayName, const FText& InTooltipText, const FSlateIcon& InIcon = FSlateIcon(), const bool InSortChildren = false )
	{
		TSharedRef<FWorkspaceItem> NewItem = FWorkspaceItem::NewGroup(InName, InDisplayName,InTooltipText, InIcon, InSortChildren);
		AddItem( NewItem );
		return NewItem;
	}

	const FName GetFName() const
	{
		return NameAttribute.Get();
	}

	const FText& GetDisplayName() const
	{
		return DisplayNameAttribute.Get();
	}
	
	const FText& GetTooltipText() const
	{
		return TooltipTextAttribute.Get();
	}

	const FSlateIcon& GetIcon() const
	{
		return Icon;
	}

	const TArray< TSharedRef<FWorkspaceItem> >& GetChildItems() const
	{
		return ChildItems;
	}

	void AddItem( const TSharedRef<FWorkspaceItem>& ItemToAdd )
	{
		ItemToAdd->ParentItem = SharedThis(this);
		ChildItems.Add( ItemToAdd );

		// If desired of this menu, sort the children
		if ( bSortChildren )
		{
			SortChildren();
		}

		// If this is our first child, our parent item may need sorting, resort it now
		if ( ChildItems.Num() == 1 && ParentItem.IsValid() && ParentItem.Pin()->bSortChildren )
		{
			ParentItem.Pin()->SortChildren();
		}
	}

	void RemoveItem( const TSharedRef<FWorkspaceItem>& ItemToRemove )
	{
		ChildItems.Remove(ItemToRemove);
	}

	void ClearItems()
	{
		ChildItems.Reset();
	}

	void SortChildren()
	{
		ChildItems.Sort( FWorkspaceItemSort() );
	}

	virtual TSharedPtr<FTabSpawnerEntry> AsSpawnerEntry()
	{
		return TSharedPtr<FTabSpawnerEntry>();
	}


 	TSharedPtr<FWorkspaceItem> GetParent() const
 	{
 		return ParentItem.Pin();
 	}

	bool HasChildrenIn( const TArray< TWeakPtr<FTabSpawnerEntry> >& AllowedSpawners )
	{
		// Spawner Entries are leaves. If this is a spawner entry and it allowed in this menu, then 
		// any group containing this node is populated.
		const TSharedPtr<FTabSpawnerEntry> ThisAsSpawnerEntry = this->AsSpawnerEntry();
		bool bIsGroupPopulated = ThisAsSpawnerEntry.IsValid() && AllowedSpawners.Contains(ThisAsSpawnerEntry.ToSharedRef());

		// Look through all the children of this node and see if any of them are populated
		for ( int32 ChildIndex=0; !bIsGroupPopulated && ChildIndex < ChildItems.Num(); ++ChildIndex )
		{
			const TSharedRef<FWorkspaceItem>& ChildItem = ChildItems[ChildIndex];
			if ( ChildItem->HasChildrenIn(AllowedSpawners) )
			{
				bIsGroupPopulated = true;
			}
		}

		return bIsGroupPopulated;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FWorkspaceItem() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	FWorkspaceItem( const FText& InDisplayName, const FSlateIcon& InIcon, const bool bInSortChildren )
		: Icon(InIcon)
		, NameAttribute(NAME_None)
		, DisplayNameAttribute(InDisplayName)
		, bSortChildren(bInSortChildren)
	{
	}

	FWorkspaceItem( const FName& InName, const FText& InDisplayName, const FSlateIcon& InIcon, const bool bInSortChildren )
		: Icon(InIcon)
		, NameAttribute(InName)
		, DisplayNameAttribute(InDisplayName)
		, bSortChildren(bInSortChildren)
	{
	}

	FWorkspaceItem( const FText& InDisplayName, const FText& InTooltipText, const FSlateIcon& InIcon, const bool bInSortChildren )
		: Icon(InIcon)
		, NameAttribute(NAME_None)
		, DisplayNameAttribute(InDisplayName)
		, TooltipTextAttribute(InTooltipText)
		, bSortChildren(bInSortChildren)
	{
	}

	FWorkspaceItem( const FName& InName, const FText& InDisplayName, const FText& InTooltipText, const FSlateIcon& InIcon, const bool bInSortChildren )
		: Icon(InIcon)
		, NameAttribute(InName)
		, DisplayNameAttribute(InDisplayName)
		, TooltipTextAttribute(InTooltipText)
		, bSortChildren(bInSortChildren)
	{
	}

	FSlateIcon Icon;
	TAttribute<FName> NameAttribute;
	TAttribute<FText> DisplayNameAttribute;
	TAttribute<FText> TooltipTextAttribute;
	UE_DEPRECATED(5.0, "Use DisplayNameAttribute instead.")
	FText DisplayName;
	UE_DEPRECATED(5.0, "Use TooltipTextAttribute instead.")
	FText TooltipText;
	bool bSortChildren;

	TArray< TSharedRef<FWorkspaceItem> > ChildItems;

	TWeakPtr<FWorkspaceItem> ParentItem;

	// Allows MakeShared with protected constructor.
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
};
