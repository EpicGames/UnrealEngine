// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
* An interface to allow Slate Widgets to expose scrollable functionality.
*/
class IScrollableWidget
{
public:
	virtual ~IScrollableWidget() = default;

	/** Gets the distance that user has scrolled into the control in normalized coordinates (0 - 1) */
	virtual FVector2D GetScrollDistance() = 0;

	/** Gets the distance that user has left to scroll in the control before reaching the end in normalized coordinates (0 - 1) */
	virtual FVector2D GetScrollDistanceRemaining() = 0;

	/** Returns the typed SWidget that implements this interface */
	virtual TSharedRef<class SWidget> GetScrollWidget() = 0;
};
