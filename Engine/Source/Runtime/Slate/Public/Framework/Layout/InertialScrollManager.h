// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"

/**
 * A helper class to calculate inertial scrolling.  This class combines a percentage of velocity lost
 * per second coupled with a static amount of velocity lost per second in order to achieve a quick decay
 * when the velocity grows small enough, and the percentage of friction lost prevents large velocities
 * from scrolling forever.
 */
class FInertialScrollManager
{

public:
	/**
	  * Constructor
	  * @param ScrollDecceleration	The acceleration against the velocity causing it to decay.
	  * @param SampleTimeout		Samples older than this amount of time will be discarded.
	  */
	SLATE_API FInertialScrollManager(double SampleTimeout = 0.1f);

	/** Adds a scroll velocity sample to help calculate a smooth velocity */
	SLATE_API void AddScrollSample(float Delta, double CurrentTime);

	/** Updates the current scroll velocity. Call every frame. */
	SLATE_API void UpdateScrollVelocity(const float InDeltaTime);

	/** 
	 * Stop the accumulation of inertial scroll. 
	 * @param bInShouldStopScrollNow true implies the scroll will stop instantly, else the list will scroll until any accumulated scroll offset is cleared.
	 */
	SLATE_API void ClearScrollVelocity(bool bInShouldStopScrollNow = false);

	/** Gets the calculated velocity of the scroll. */
	float GetScrollVelocity() const { return ScrollVelocity; }

	/** Gets the the value of bShouldStopScrollNow. */
	bool GetShouldStopScrollNow() const { return bShouldStopScrollNow; }

	/** Set the value of bShouldStopScrollNow to false. */
	void ResetShouldStopScrollNow() { bShouldStopScrollNow = false; }

private:
	struct FScrollSample
	{
		double Time;
		float Delta;

		FScrollSample(float InDelta, double InTime)
			: Time(InTime)
			, Delta(InDelta)
		{}
	};

	/** Used to calculate the appropriate scroll velocity over the last few frames while inertial scrolling */
	TArray<FScrollSample> ScrollSamples;

	/** The current velocity of the scroll */
	float ScrollVelocity;

	/** When true, the list will stop scrolling */
	bool bShouldStopScrollNow = false;

	/** Samples older than this amount of time will be discarded. */
	double SampleTimeout;
};
