// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Layout/Geometry.h"

struct FGeometry;

enum class EAllowOverscroll : uint8
{
	Yes,
	No
};

/**
 * Handles overscroll management.
 */
struct FOverscroll
{
public:

	/** The amount to scale the logarithm by to make it more loose */
	static SLATE_API float Looseness;
	/** The "max" used to perform the interpolation snap back, and make it faster the further away it is. */
	static SLATE_API float OvershootLooseMax;
	/** The bounce back rate when the overscroll stops. */
	static SLATE_API float OvershootBounceRate;

	SLATE_API FOverscroll();

	/** @return The Amount actually scrolled */
	SLATE_API float ScrollBy(const FGeometry& AllottedGeometry, float LocalDeltaScroll);

	/** How far the user scrolled above/below the beginning/end of the list. */
	SLATE_API float GetOverscroll(const FGeometry& AllottedGeometry) const;

	/** Ticks the overscroll manager so it can animate. */
	SLATE_API void UpdateOverscroll(float InDeltaTime);

	/**
	 * Should ScrollDelta be applied to overscroll or to regular item scrolling.
	 *
	 * @param bIsAtStartOfList  Are we at the very beginning of the list (i.e. showing the first item at the top of the view)?
	 * @param bIsAtEndOfList    Are we showing the last item on the screen completely?
	 * @param ScrollDelta       How much the user is trying to scroll in Slate Units.
	 *
	 * @return true if the user's scrolling should be applied toward overscroll.
	 */
	SLATE_API bool ShouldApplyOverscroll(const bool bIsAtStartOfList, const bool bIsAtEndOfList, const float ScrollDelta) const;
	
	/** Resets the overscroll amout. */
	SLATE_API void ResetOverscroll();
private:
	/** How much we've over-scrolled above/below the beginning/end of the list, stored in log form */
	float OverscrollAmount;
};
