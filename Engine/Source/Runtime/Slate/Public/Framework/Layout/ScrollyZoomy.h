// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Framework/Layout/InertialScrollManager.h"

class FSlateWindowElementList;
class SWidget;

/**
 * Interface for widgets that can be used with FScrollyZoomy.
 */
class IScrollableZoomable
{
public:

	/**
	 * Override this to scroll your widget's content.
	 *
	 * @param Offset The 2D offset to scroll by.
	 * @return True if anything was scrolled.
	 */
	virtual bool ScrollBy(const FVector2D& Offset) = 0;

	/**
	 * Override this to zoom your widget's content.
	 *
	 * @param Amount The amount to zoom by.
	 * @return True if anything was zoomed.
	 */
	virtual bool ZoomBy(const float Amount) = 0;
};


/**
 * Utility class that adds scrolling and zooming functionality to a widget.
 *
 * Derived your widget class from IScrollableZoomable, then embed an
 * instance of FScrollyZoomy as a widget member variable, and call this
 * class's event handlers from your own widget's event handler callbacks.
 */
class FScrollyZoomy
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InUseInertialScrolling Whether inertial scrolling should be used (default = true).
	 */
	SLATE_API FScrollyZoomy(const bool InUseInertialScrolling = true);

	/** 
	 * Should be called every frame to update simulation state.
	 *
	 * @param DeltaTime Time that's passed.
	 * @param ScrollableZoomable Interface to the widget to scroll/zoom.
	 */
	SLATE_API void Tick(const float DeltaTime, IScrollableZoomable& ScrollableZoomable);

	/**
	 * Should be called when a mouse button is pressed.
	 *
	 * @param MouseEvent The mouse event passed to a widget's OnMouseButtonDown function.
	 * @return If FReply::Handled is returned, that should be passed as the result of the calling function.
	 */
	SLATE_API FReply OnMouseButtonDown(const FPointerEvent& MouseEvent);

	/**
	 * Should be called when a mouse button is released.
	 *
	 * @param MyWidget Pointer to the widget that owns this object.
	 * @param MyGeometry Geometry of the widget we're scrolling.
	 * @param MouseEvent The mouse event passed to a widget's OnMouseButtonUp function.
	 * @return If FReply::Handled is returned, that should be passed as the result of the calling function.
	 */
	SLATE_API FReply OnMouseButtonUp(const TSharedRef<SWidget> MyWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
		
	/**
	 * Should be called when a mouse move event occurs.
	 *
	 * @param MyWidget Pointer to the widget that owns this object.
	 * @param ScrollableZoomable Interface to the widget to scroll/zoom.
	 * @param MyGeometry Geometry of the widget we're scrolling.
	 * @param MouseEvent The mouse event passed to a widget's OnMouseMove function.
	 * @return If FReply::Handled is returned, that should be passed as the result of the calling function.
	 */
	SLATE_API FReply OnMouseMove(const TSharedRef<SWidget> MyWidget, IScrollableZoomable& ScrollableZoomable, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * Should be called from your widget's OnMouseLeave override.
	 *
	 * @param MyWidget Pointer to the widget that owns this object.
	 * @param MouseEvent The mouse leave event that was passed to the widget's OnMouseLeave function.
	 */
	SLATE_API void OnMouseLeave(const TSharedRef<SWidget> MyWidget, const FPointerEvent& MouseEvent);
		
	/**
	 * Should be called by your widget when the mouse wheel is used
	 *
	 * @param MouseEvent	The event passed to your widget's OnMouseWheel function.
	 * @param ScrollableZoomable	Interface to the widget to scroll/zoom.
	 * @return If FReply::Handled is returned, that should be passed as the result of the calling function.
	 */
	SLATE_API FReply OnMouseWheel(const FPointerEvent& MouseEvent, IScrollableZoomable& ScrollableZoomable);

	/**
	 * Call this from your widget's OnCursorQuery function.
	 *
	 * @return The cursor reply to pass back.
	 */
	SLATE_API FCursorReply OnCursorQuery() const;

	/**
	 * Whether the user is actively scrolling.
	 *
	 * @return true if the user is scrolling, false otherwise.
	 */
	SLATE_API bool IsRightClickScrolling() const;

	/**
	 * Whether a software cursor should be rendered.
	 *
	 * This method should be called from your widget's OnPaint function.
	 *
	 * @return true if a software cursor should be rendered, false otherwise.
	 * @see GetSoftwareCursorPosition
	 */
	SLATE_API bool NeedsSoftwareCursor() const;

	/**
	 * Get the position of the software cursor (when NeedsSoftwareCursor is true).
	 *
	 * This method should be called from your widget's OnPaint function.
	 *
	 * @return Cursor position.
	 * @see NeedsSoftwareCursor
	 */
	SLATE_API UE::Slate::FDeprecateVector2DResult GetSoftwareCursorPosition() const;

	/**
	 * Call this from your widget's OnPaint to paint a software cursor, if needed 
	 *
	 * @param AllottedGeometry Widget geometry passed into OnPaint.
	 * @param MyCullingRect Widget clipping rect passed into OnPaint.
	 * @param OutDrawElements The draw element list.
	 * @param LayerId Layer identifier.
	 * @return New layer Identifier.
	 */
	SLATE_API int32 PaintSoftwareCursorIfNeeded(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

private:

	/** How much we scrolled while the right mouse button has been held. */
	float AmountScrolledWhileRightMouseDown;

	/** Whether the software cursor should be drawn in the view port. */
	bool bShowSoftwareCursor;

	/** The current position of the software cursor. */
	FVector2f SoftwareCursorPosition;

	/** Whether inertial scrolling is enabled. */
	bool UseInertialScrolling;

	/** Tracks simulation state for horizontal scrolling. */
	FInertialScrollManager HorizontalIntertia;

	/** Tracks simulation state for vertical scrolling. */
	FInertialScrollManager VerticalIntertia;
};
