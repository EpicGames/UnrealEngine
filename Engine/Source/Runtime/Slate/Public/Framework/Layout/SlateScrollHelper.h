// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Utility class to handle scrolling
 */
class FScrollHelper
{

public:

	/**
	 * Constructor
	 */
	FScrollHelper()
		: Position( 0.0f, 0.0f )
	{
	}


	/**
	 * Returns the current position
	 *
	 * @return  Current position
	 */
	const FVector2D& GetPosition() const
	{
		return Position;
	}


	/**
	 * Sets the scrolled position
	 */
	void SetPosition( const FVector2D& InNewPosition )
	{
		Position = InNewPosition;
	}


	/**
	 * Converts a coordinate to the scrollable area's space
	 *
	 * @param  InVec  Position to convert to the scrollable area's space
	 *
	 * @return  Returns the position transformed to the scrollable area's space
	 */
	FVector2D ToScrollerSpace( const FVector2D& InVec ) const
	{
		return InVec + Position;
	}


	/**
	 * Converts a coordinate back from the scrollable area's space
	 *
	 * @param  InVec  Position to convert from the scrollable area's space
	 *
	 * @return  Returns the position transformed back from the scrollable area's space
	 */
	FVector2D FromScrollerSpace( const FVector2D& InVec ) const
	{
		return ( InVec - Position ) ;
	}


	/**
	 * Converts a size to the scrollable area's space
	 *
	 * @param  InVec  Size to convert to the scrollable area's space
	 *
	 * @return  Returns the size transformed to the scrollable area's space
	 */
	FVector2D SizeToScrollerSpace( const FVector2D& InVec ) const
	{
		return InVec;
	}


	/**
	 * Converts a size back from the scrollable area's space
	 *
	 * @param  InVec  Size to convert from the scrollable area's space
	 *
	 * @return  Returns the size transformed back from the scrollable area's space
	 */
	FVector2D SizeFromScrollerSpace( const FVector2D& InVec ) const
	{
		return InVec;
	}


private:

	/** Current location offset */
	FVector2D Position;

};
