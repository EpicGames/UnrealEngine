// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/SlateRect.h"

/**
 * A convenient representation of a marquee selection
 */
struct FMarqueeRect
{
	/** Where the user began the marquee selection */
	FDeprecateSlateVector2D StartPoint;
	/** Where the user has dragged to so far */
	FDeprecateSlateVector2D EndPoint;

	/** Make a default marquee selection */
	FMarqueeRect( UE::Slate::FDeprecateVector2DParameter InStartPoint = FVector2f::ZeroVector )
		: StartPoint( InStartPoint )
		, EndPoint( InStartPoint )
	{
	}

	/**
	 * Update the location to which the user has dragged the marquee selection so far
	 *
	 * @param NewEndPoint   Where the user has dragged so far.
	 */
	void UpdateEndPoint( const UE::Slate::FDeprecateVector2DParameter& NewEndPoint )
	{
		EndPoint = NewEndPoint;
	}

	/** @return true if this marquee selection is not too small to be considered real */
	bool IsValid() const
	{
		return ! (EndPoint - StartPoint).IsNearlyZero();
	}

	/** @return the upper left point of the selection */
	UE::Slate::FDeprecateVector2DResult GetUpperLeft() const
	{
		return FVector2f( FMath::Min(StartPoint.X, EndPoint.X), FMath::Min( StartPoint.Y, EndPoint.Y ) );
	}

	/** @return the lower right of the selection */
	UE::Slate::FDeprecateVector2DResult GetLowerRight() const
	{
		return  FVector2f( FMath::Max(StartPoint.X, EndPoint.X), FMath::Max( StartPoint.Y, EndPoint.Y ) );
	}

	/** The size of the selection */
	UE::Slate::FDeprecateVector2DResult GetSize() const
	{
		FVector2f SignedSize = EndPoint - StartPoint;
		return FVector2f( FMath::Abs(SignedSize.X), FMath::Abs(SignedSize.Y) );
	}

	/** @return This marquee rectangle as a well-formed SlateRect */
	FSlateRect ToSlateRect() const
	{
		return FSlateRect( FVector2f(FMath::Min(StartPoint.X, EndPoint.X), FMath::Min(StartPoint.Y, EndPoint.Y)), FVector2f(FMath::Max(StartPoint.X, EndPoint.X), FMath::Max( StartPoint.Y, EndPoint.Y )) );
	}
};
