// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Types/ISlateMetaData.h"

class FDriverIdMetaData
	: public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FDriverIdMetaData, ISlateMetaData)

	FDriverIdMetaData(FName InId)
		: Id(MoveTemp(InId))
	{ }

	FName Id;
};
