// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ISlateMetaData;

/**
 * A static helper class which is used to easily construct various types of AutomationDriver specific SlateMetaData.
 */
class FDriverMetaData
{
public:

	/**
	 * @return the automation driver specific metadata type for specifying an Id for a widget
	 */
	static SLATE_API TSharedRef<ISlateMetaData> Id(FName Tag);
};
