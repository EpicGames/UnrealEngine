// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SLinkedBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SMenuOwner.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxDefs.h"

class FDropPreviewBlock;
class ITableRow;
class SClippingHorizontalBox;
class SHorizontalBox;
class SMultiBoxWidget;
class STableViewBase;
class SVerticalBox;
class UToolMenuBase;
class SUniformWrapPanel;
class FToolBarComboButtonBlock;

namespace MultiBoxConstants
{	
	/** The time that a mouse should be hovered over a sub-menu before it automatically opens */
	inline const float SubMenuOpenTime = 0.0f;

	/** When a sub-menu is already open (for at least SubMenuClobberMinLifetime), the time that a mouse
	    should be hovered over another sub-menu entry before dismissing the first menu and opening
		the new one; this doesn't apply to short-lived sub-menus, see below */
	inline const float SubMenuClobberTime = 0.5f;

	/** The time that a sub-menu needs to remain open in order for the SubMenuClobberTime to apply;
	    menus open shorter than this min lifetime will be instantly dismissed */
	inline const float SubMenuClobberMinLifetime = 0.5f;

	//inline const FName MenuItemFont = "MenuItem.Font";
	//inline const FName MenuBindingFont = "MenuItem.BindingFont";

	//inline const FName EditableTextFont = "MenuItem.Font";

	/** Minimum widget of an editable text box within a multi-box */
	inline const float EditableTextMinWidth = 30.0f;
}

/**
 * MultiBlock (abstract).  Wraps a "block" of useful UI functionality that can be added to a MultiBox.
 */
class FMultiBlock
	: public TSharedFromThis< FMultiBlock >		// Enables this->AsShared()
{

public:

	struct FMultiBlockParams
	{
		/** Direct processing of actions. Will use these actions if there is not UICommand associated with this block that handles actions*/
		FUIAction DirectActions;

		/** The action associated with this block (can be null for some actions) */
		const TSharedPtr< const FUICommandInfo > Action;

		/** The list of mappings from command info to delegates that should be called. This is here for quick access. Can be null for some widgets*/
		const TSharedPtr< const FUICommandList > ActionList;

		/** Optional extension hook which is used for debug display purposes, so users can see what hooks are where */
		FName ExtensionHook;

		/** Type of MultiBlock */
		EMultiBlockType Type = EMultiBlockType::None;

		/** Whether this block is part of the heading blocks for a section */
		bool bIsPartOfHeading = false;
	};

	/**
	 * Constructor
 	 *
	 * @param InCommand		The command info that describes what action to take when this block is activated
	 * @param InCommandList	The list of mappings from command info to delegates so we can find the delegates to process for the provided action
	 */
	FMultiBlock( const TSharedPtr< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, FName InExtensionHook = NAME_None, EMultiBlockType InType = EMultiBlockType::None, bool bInIsPartOfHeading = false)
		: Action( InCommand )
		, ActionList( InCommandList )
		, ExtensionHook( InExtensionHook )
		, Type( InType )
		, TutorialHighlightName( NAME_None )
		, bSearchable(true)
		, bIsPartOfHeading(bInIsPartOfHeading)
	{
	}

	/**
	 * Constructor
	 *
	 * @param InAction	UI action delegates that should be used in place of UI commands (dynamic menu items)
	 */
	FMultiBlock( const FUIAction& InAction,  FName InExtensionHook = NAME_None, EMultiBlockType InType = EMultiBlockType::None, bool bInIsPartOfHeading = false, TSharedPtr< const FUICommandList > InCommandList = nullptr )
		: DirectActions( InAction )
		, ActionList( InCommandList )
		, ExtensionHook( InExtensionHook )
		, Type( InType )
		, TutorialHighlightName( NAME_None )
		, bSearchable(true)
		, bIsPartOfHeading(bInIsPartOfHeading)
	{
	}

	/**
	 * Constructor
	 *
	 * @param InMultiBlockParams	Bundle of params for construction
	 */
	FMultiBlock(const FMultiBlockParams& InMultiBlockParams)
		: DirectActions(InMultiBlockParams.DirectActions)
		, Action(InMultiBlockParams.Action)
		, ActionList(InMultiBlockParams.ActionList)
		, ExtensionHook(InMultiBlockParams.ExtensionHook)
		, Type(InMultiBlockParams.Type)
		, TutorialHighlightName(NAME_None)
		, bSearchable(true)
		, bIsPartOfHeading(InMultiBlockParams.bIsPartOfHeading)
	{
	}

	virtual ~FMultiBlock() = default;

	/**
	 * Returns the action list associated with this block
	 * 
	 * @return The action list or null if one does not exist for this block
	 */
	TSharedPtr< const FUICommandList > GetActionList() const { return ActionList; }

	/**
	 * Returns the action associated with this block
	 * 
	 * @return The action for this block or null if one does not exist
	 */
	TSharedPtr< const FUICommandInfo> GetAction() const { return Action; }

	/**
	 * Returns the direct actions for this block.  Delegates may be unbound if this block has a UICommand
	 *
	 * @return DirectActions for this block
	 */
	const FUIAction& GetDirectActions() const { return DirectActions; }

	/** Creates a menu entry that is representative of this block */
	virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const { }

	/** Group blocks interface */
	virtual bool IsGroupStartBlock() const { return false; }
	virtual bool IsGroupEndBlock() const { return false; }
	virtual bool HasIcon() const { return false; }

	/** Set the tutorial highlight name for this menu entry */
	void SetTutorialHighlightName(FName InTutorialName)
	{
		TutorialHighlightName = InTutorialName;
	}

	/** Get the tutorial highlight name for this menu entry */
	FName GetTutorialHighlightName() const
	{
		return TutorialHighlightName;
	}

	/** Sets the style name which will be used for this block instead of the owning multibox's style */
	void SetStyleNameOverride(FName InStyleNameOverride)
	{
		StyleNameOverride = InStyleNameOverride;
	}

	/** Gets the style name which will be used for this block instead of the owning multibox's style */
	FName GetStyleNameOverride() const
	{
		return StyleNameOverride;
	}

	/**
	 * Creates a MultiBlock widget for this MultiBlock
	 *
	 * @param	InOwnerMultiBoxWidget	The widget that will own the new MultiBlock widget
	 * @param	InLocation				The location information for the MultiBlock widget
	 *
	 * @return  MultiBlock widget object
	 */
	SLATE_API TSharedRef<class IMultiBlockBaseWidget> MakeWidget(TSharedRef< class SMultiBoxWidget > InOwnerMultiBoxWidget, EMultiBlockLocation::Type InLocation, bool bSectionContainsIcons, TSharedPtr<SWidget> OptionsBlockWidget) const;

	/**
	 * Gets the type of this MultiBox
	 *
	 * @return	The MultiBlock's type
	 */
	const EMultiBlockType GetType() const
	{
		return Type;
	}

	/**
	 * Is this block a separator
	 *
	 * @return	True if block is a separator
	 */
	bool IsSeparator() const
	{
		return Type == EMultiBlockType::Separator;
	}

	/**
	 * Is this block a heading block or a block that belongs to a heading such as a separator
	 *
	 * @return	True if block is part of a section heading's blocks
	 */
	bool IsPartOfHeading() const
	{
		return (Type == EMultiBlockType::Heading) || bIsPartOfHeading;
	}

	/**
	* Sets the searchable state of this block
	*
	* @param	bSearchable		The searchable state to set
	*/
	SLATE_API void SetSearchable(bool bSearchable);

	/**
	* Gets the searchable state of this block
	*
	* @return	Whether this block is searchable
	*/
	SLATE_API bool GetSearchable() const;

	/** Gets the extension hook so users can see what hooks are where */
	FName GetExtensionHook() const { return ExtensionHook; }

	/** Get the visibility override for this block. */
	TAttribute<EVisibility> GetVisibilityOverride() const;

	/** Set the visibility override for this block. */
	void SetVisibilityOverride(TAttribute<EVisibility> InVisibilityOverride);

	/** Get the resize parameters which determine how this block behaves during toolbar resizing.*/
	const TAttribute<FMenuEntryResizeParams>& GetResizeParams() const;

	/** Set the resize parameters which determine how this block behaves during toolbar resizing.*/
	void SetResizeParams(const TAttribute<FMenuEntryResizeParams>& InResizeParams);

private:
	/**
	 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
	 *
	 * @return  MultiBlock widget object
	 */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const = 0;

	/**
 	 * Gets any alignment overrides for this block
	 *
	 * @param OutHorizontalAlignment	Horizontal alignment override
	 * @param OutVerticalAlignment	Vertical Alignment override 
	 * @param bOutAutoWidth		Fill or Auto width override
	 * @return true if overrides should be applied, false to use defaults 
 	 */
	UE_DEPRECATED(5.5, "Use the version that outputs FMenuEntryStyleParams.")
	virtual bool GetAlignmentOverrides(EHorizontalAlignment& OutHorizontalAlignment, EVerticalAlignment& OutVerticalAlignment, bool& bOutAutoWidth) const { return false; }

	/**
	  * Gets any alignment overrides for this block
	 *
	 * @param OutAlignmentParameters	Alignment parameters and overrides
	 * @return true if overrides should be applied, false to use defaults
	  */
	virtual bool GetAlignmentOverrides(FMenuEntryStyleParams& OutAlignmentParameters) const { return false; }
private:

	// We're friends with SMultiBoxWidget so that it can call MakeWidget() directly
	friend class SMultiBoxWidget;	

	/** Direct processing of actions. Will use these actions if there is not UICommand associated with this block that handles actions*/
	FUIAction DirectActions;

	/** The action associated with this block (can be null for some actions) */
	const TSharedPtr< const FUICommandInfo > Action;

	/** The list of mappings from command info to delegates that should be called. This is here for quick access. Can be null for some widgets*/
	const TSharedPtr< const FUICommandList > ActionList;

	/** Optional extension hook which is used for debug display purposes, so users can see what hooks are where */
	FName ExtensionHook;

	/** Type of MultiBlock */
	EMultiBlockType Type;

	/** Name to identify a widget for tutorials */
	FName TutorialHighlightName;

	FName StyleNameOverride;

	/** Whether this block can be searched */
	bool bSearchable;

	/** Whether this block is part of the heading blocks for a section */
	bool bIsPartOfHeading;

	/** Visibility override which, if set and returning anything but Visible, takes priority over other methods of determining visibility. */
	TAttribute<EVisibility> VisibilityOverride;

	/** Resize parameters which determine how this block behaves during toolbar resizing. */
	TAttribute<FMenuEntryResizeParams> ResizeParams;
};




/**
 * MultiBox.  Contains a list of MultiBlocks that provide various functionality.
 */
class FMultiBox
	: public TSharedFromThis< FMultiBox >		// Enables this->AsShared()
{

public:
	SLATE_API virtual ~FMultiBox();

	/**
	 * Creates a new multibox instance
	 */
	static SLATE_API TSharedRef<FMultiBox> Create( const EMultiBoxType InType,  FMultiBoxCustomization InCustomization, const bool bInShouldCloseWindowAfterMenuSelection );

	/**
	 * Gets the type of this MultiBox
	 *
	 * @return	The MultiBox's type
	 */
	const EMultiBoxType GetType() const
	{
		return Type;
	}

	/**
	 * Gets whether or not the window that contains this multibox should be destroyed after the user clicks on a menu item in this box
	 *
	 * @return	True if window should be closed automatically when the user clicks on a menu item, otherwise false
	 */
	bool ShouldCloseWindowAfterMenuSelection() const
	{
		return bShouldCloseWindowAfterMenuSelection;
	}

	/**
	 * returns the index of the item which was last selected in this box
	 */
	int32 GetLastSelectedCommandIndex() const;
	
	/**
	 * sets the index of the item which was last selected in this box
	 *
	 * @param LastSelectedCommandIndex the index of the last command that was selected for this box
	 */
	void SetLastSelectedCommandIndex( int32 LastSelectedCommandIndex );

	/**
	 * Adds a MultiBlock to this MultiBox, to the end of the list
	 */
	SLATE_API void AddMultiBlock( TSharedRef< const FMultiBlock > InBlock );

	/**
	 * Adds a MultiBlock to this MultiBox, to the front of the list
	 */
	SLATE_API void AddMultiBlockToFront(TSharedRef< const FMultiBlock > InBlock);

	/**
	 * Removes a MultiBlock from the list for user customization
	 */
	SLATE_API void RemoveCustomMultiBlock( TSharedRef< const FMultiBlock> InBlock );

	/**
	 * Inserts a MultiBlock to the list for user customization
	 */
	SLATE_API void InsertCustomMultiBlock( TSharedRef<const FMultiBlock> InBlock, int32 Index );

	DECLARE_DELEGATE_TwoParams( FOnMakeMultiBoxBuilderOverride, const TSharedRef<FMultiBox>&, const TSharedRef<SMultiBoxWidget>& );

	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedRef<SWidget>, FOnModifyBlockWidgetAfterMake, const TSharedRef<SMultiBoxWidget>&, const FMultiBlock&, const TSharedRef<SWidget>& );

	/**
	 * Allow further modifications to the block's widget after it has been made
	 */
	FOnModifyBlockWidgetAfterMake ModifyBlockWidgetAfterMake;

	/**
	 * Creates a MultiBox widget for this MultiBox
	 *
	 * @return  MultiBox widget object
	 */
	SLATE_API TSharedRef< class SMultiBoxWidget > MakeWidget( bool bSearchable, FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride = nullptr, TAttribute<float> InMaxHeight = TAttribute<float>() );


	/**
	 * Access this MultiBox's list of blocks
	 *
	 * @return	Our list of MultiBlocks
	 */
	const TArray< TSharedRef< const FMultiBlock > >& GetBlocks() const
	{
		return Blocks;
	}
	
	/** @return The style set used by the multibox widgets */
	const ISlateStyle* GetStyleSet() const { return StyleSet; }

	/** @return The style name used by the multibox widgets */
	const FName& GetStyleName() const { return StyleName; }

	/** Sets the style to use on the multibox widgets */
	void SetStyle( const ISlateStyle* InStyleSet, const FName& InStyleName )
	{
		StyleSet  = InStyleSet;
		StyleName = InStyleName;
	}

	/** @return The customization name for this box */
	SLATE_API FName GetCustomizationName() const;

	/**
	 * Creates a block from the provided command that is compatible with this box 
	 *
	 * @param Command	The UI command to create the block from
	 * @return The created multi block.  If null, this command could not be placed in this block
	 */
	SLATE_API TSharedPtr<FMultiBlock> MakeMultiBlockFromCommand( TSharedPtr<const FUICommandInfo> Command, bool bCommandMustBeBound ) const;

	/**
	 * Finds an existing block by name and type
	 *
	 * @param InName The name to search for
	  * @param InType The type to match during the search
	 */
	SLATE_API TSharedPtr<const FMultiBlock> FindBlockFromNameAndType(const FName InName, const EMultiBlockType InType) const;

	/** @return Is being editing */
	SLATE_API bool IsInEditMode() const;

	/** @return The tool menu associated with this multi box */
	SLATE_API UToolMenuBase* GetToolMenu() const;

	/**
	 * Only callable during edit mode
	 * @return Index of section heading or separator
	 */
	SLATE_API int32 GetSectionEditBounds(const int32 Index, int32& OutSectionEndIndex) const;

	DECLARE_DELEGATE_OneParam(FEditSelectionChangedDelegate, TSharedRef<const FMultiBlock>);

	/** Delegate to call while editing when selected block has changed */
	const FEditSelectionChangedDelegate& OnEditSelectionChanged() const { return EditSelectionChanged; }

	/** Delegate to call while editing when selected block has changed */
	FEditSelectionChangedDelegate& OnEditSelectionChanged() { return EditSelectionChanged; }

	/** Weak reference to tool menu that created this multibox */
	TWeakObjectPtr<UToolMenuBase> WeakToolMenu;

	/* Whether the MultiBox has a search widget */
	bool bHasSearchWidget;

	/** Whether the MultiBox can be focused. */
	bool bIsFocusable;

	/** Whether the WrapButton/Overflow can be shown (if applicable). Defaults to the style. */
	TOptional<bool> bAllowWrapButton;

	/* Returns the last command list used */
	const TSharedPtr<const FUICommandList> GetLastCommandList() const { return CommandLists.Num() > 0 ? CommandLists.Last() : nullptr; }

private:
	
	/**
	 * Constructor
	 *
	 * @param	InType	Type of MultiBox
	 * @param	bInShouldCloseWindowAfterMenuSelection	Sets whether or not the window that contains this multibox should be destroyed after the user clicks on a menu item in this box
	 */
	SLATE_API FMultiBox( const EMultiBoxType InType,  FMultiBoxCustomization InCustomization, const bool bInShouldCloseWindowAfterMenuSelection );

	/**
	 * @return true if this box can be customized by a user
	 */
	SLATE_API bool IsCustomizable() const;

private:

	/** All command lists in this box */
	TArray< TSharedPtr<const FUICommandList> > CommandLists;

	/** Ordered list of blocks */
	TArray< TSharedRef< const FMultiBlock > > Blocks;

	/** The style set to use with the widgets in the MultiBox */
	const ISlateStyle* StyleSet;

	/** The style name to use with the widgets in the MultiBox */
	FName StyleName;

	/** Type of MultiBox */
	EMultiBoxType Type;

	/** Delegate to call while editing when selected block has changed */
	FEditSelectionChangedDelegate EditSelectionChanged;

	/** True if window that owns any widgets created from this multibox should be closed automatically after the user commits to a menu choice */
	bool bShouldCloseWindowAfterMenuSelection;

	/** the Last selected command index. If unset this has a value of INDEX_NONE */
	int32 LastSelectedCommandIndex;

};



/**
 * MultiBlock Slate widget interface
 */
class IMultiBlockBaseWidget
{

public:

	/**
	 * Interprets this object as a SWidget
	 *
	 * @return  Widget reference
	 */
	virtual TSharedRef<SWidget> AsWidget() = 0;


	/**
	 * Interprets this object as a SWidget
	 *
	 * @return  Widget reference
	 */
	virtual TSharedRef<const SWidget> AsWidget() const = 0;

	/**
	 * Associates the owner MultiBox widget with this widget
	 *
	 * @param	InOwnerMultiBoxWidget		The MultiBox widget that owns us
	 */
	virtual void SetOwnerMultiBoxWidget(TSharedRef<SMultiBoxWidget> InOwnerMultiBoxWidget ) = 0;

	/**
	 * Associates this widget with a MultiBlock
	 *
	 * @param	InMultiBlock	The MultiBlock we'll be associated with
	 */
	virtual void SetMultiBlock(TSharedRef<const FMultiBlock> InMultiBlock) = 0;

	/**
	 * Adds a dropdown widget for options associated with this widget. The usage of this is block specific
	 *
	 * @param	InOptionsBlockWidget	The options block to associate with this widget
	 */
	virtual void SetOptionsBlockWidget(TSharedPtr<SWidget> InOptionsBlockWidget) = 0;

	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 *
	 * @param	StyleSet	The Slate style to use to build the widget
	 * @param	StyleName	The style name to use from the StyleSet
	 */
	virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) = 0;

	/**
	 * Sets the blocks location relative to the other blocks
	 * 
	 * @param InLocation	The MultiBlocks location
	 * @param bSectionContainsIcons Does the section contain icons?
	 */
	virtual void SetMultiBlockLocation(EMultiBlockLocation::Type InLocation, bool bSectionContainsIcons) = 0;

	/**
	 * Returns this MultiBlocks location
	 */
	UE_DEPRECATED(5.6, "Use the const version of GetMultiBlockLocation")
	virtual EMultiBlockLocation::Type GetMultiBlockLocation() = 0;
	
	/**
	 * Returns this MultiBlocks location
	 */
	virtual EMultiBlockLocation::Type GetMultiBlockLocation() const = 0;

	/**
	 * Returns true if editing this widget
	 */
	virtual bool IsInEditMode() const = 0;
};


template <>
struct TWidgetTypeTraits<class SMultiBlockBaseWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * MultiBlock Slate base widget (pure virtual).  You'll derive your own MultiBlock class from this base class.
 */
class SMultiBlockBaseWidget
	: public IMultiBlockBaseWidget,
	  public SCompoundWidget
{

public:
	TSharedPtr<const FMultiBlock> GetBlock() const { return MultiBlock; }

	/** IMultiBlockBaseWidget interface */
	SLATE_API virtual TSharedRef< SWidget > AsWidget() override;
	SLATE_API virtual TSharedRef< const SWidget > AsWidget() const override;
	SLATE_API virtual void SetOwnerMultiBoxWidget(TSharedRef<SMultiBoxWidget> InOwnerMultiBoxWidget) override;
	SLATE_API virtual void SetMultiBlock(TSharedRef<const FMultiBlock> InMultiBlock) override;
	SLATE_API virtual void SetOptionsBlockWidget(TSharedPtr<SWidget> InOptionsBlock) override;
	SLATE_API virtual void SetMultiBlockLocation(EMultiBlockLocation::Type InLocation, bool bInSectionContainsIcons) override;
	UE_DEPRECATED(5.6, "Use the const version of GetMultiBlockLocation")
	SLATE_API virtual EMultiBlockLocation::Type GetMultiBlockLocation() override;
	SLATE_API virtual EMultiBlockLocation::Type GetMultiBlockLocation() const override;
	SLATE_API virtual bool IsInEditMode() const override;

	/** SWidget Interface */
	SLATE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	SLATE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	SLATE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
protected:

	/** Weak reference back to the MultiBox widget that owns us */
	TWeakPtr<SMultiBoxWidget> OwnerMultiBoxWidget;

	/** The MultiBlock we're associated with */
	TSharedPtr<const FMultiBlock> MultiBlock;

	TSharedPtr<SWidget> OptionsBlockWidget;

	/** The MultiBlocks location relative to the other blocks in the set */
	EMultiBlockLocation::Type Location;

	/** Does the section this block resides in contain blocks with icons? */
	bool bSectionContainsIcons;
};


template <>
struct TWidgetTypeTraits<class SMultiBoxWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * MultiBox Slate widget
 */
class SMultiBoxWidget
	: public SMenuOwner
{

public:

	SLATE_BEGIN_ARGS( SMultiBoxWidget )
		: _ContentScale( FVector2D::UnitVector )
		{}

		/** Content scaling factor */
		SLATE_ATTRIBUTE( FVector2D, ContentScale )

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Associates this widget with a MultiBox
	 *
	 * @param	InMultiBox	The MultiBox we'll be associated with
	 */
	void SetMultiBox( const TSharedRef< FMultiBox >& InMultiBox )
	{
		MultiBox = InMultiBox;
	}

	/**
	* Sets new content for the SMultiBoxWidget
	*
	* @param	InContent	The new content to place in the ChildSlot
	*/
	void SetContent( const TSharedRef< SWidget >& InContent )
	{	
		ChildSlot
		[
			InContent 
		];
	}

	/**
	 * Access the MultiBox associated with this widget
	 */
	TSharedRef< const FMultiBox > GetMultiBox() const
	{
		return MultiBox.ToSharedRef();
	}

	/**
	* Sets the searchable state of this multibox
	*
	* @param	bSearchable		The searchable state to set
	*/
	SLATE_API void SetSearchable(bool bSearchable);
	/**
	* Gets the searchable state of this multibox
	*
	* @return	Whether this block is searchable
	*/
	SLATE_API bool GetSearchable() const;

	/**
	 * @return the the LinkedBoxManager
	 */
	TSharedRef<FLinkedBoxManager> GetLinkedBoxManager() { return LinkedBoxManager.ToSharedRef(); }

	/**
	* Sets optional maximum height of widget
	*/
	void SetMaxHeight(TAttribute<float> InMaxHeight) { MaxHeight = MoveTemp(InMaxHeight); }

	/**
	 * Builds this MultiBox widget up from the MultiBox associated with it
	 */
	SLATE_API void BuildMultiBoxWidget();

	/** Rebuilds this MultiBox widget using the input MultiBox */
	SLATE_API void UpdateMultiBoxWidget(TSharedRef<FMultiBox> InMultiBox);

	/** Generates the tiles for an STileView for button rows */
	SLATE_API TSharedRef<ITableRow> GenerateTiles(TSharedPtr<SWidget> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the maximum item width and height of the consituent widgets */
	SLATE_API float GetItemWidth() const;
	SLATE_API float GetItemHeight() const;

	/** Event handler for clicking the wrap button */
	SLATE_API TSharedRef<SWidget> OnWrapButtonClicked();

	const ISlateStyle* GetStyleSet() const { return MultiBox->GetStyleSet(); }
	const FName& GetStyleName() const { return MultiBox->GetStyleName(); }

	/**
	 * Called when a user drags a UI command into a multiblock in this widget
	 */
	SLATE_API void OnCustomCommandDragEnter( TSharedRef<const FMultiBlock> MultiBlock, const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent );

	/**
	 * Called when a user drags a UI command within multiblock in this widget
	 */
	SLATE_API void OnCustomCommandDragged( TSharedRef<const FMultiBlock> MultiBlock, const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent );

	/**
	 * Called when a user drops a UI command in this widget
	 */
	SLATE_API void OnCustomCommandDropped();
	
	/**
	 * Called after a drag was initiated from this box but was dropped elsewhere
	 */
	SLATE_API void OnDropExternal();

	/** Helper function used to transfer focus to the next/previous widget */
	static SLATE_API FReply FocusNextWidget(EUINavigation NavigationType);

	/** SWidget interface */
	SLATE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent ) override;
	SLATE_API virtual FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) override;
	SLATE_API virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent) override;

	/**
	* Starts a search and makes the search box visible
	*
	* @param	InChar		The first character typed in
	*/
	SLATE_API void BeginSearch(const TCHAR InChar);

	/**
	* Resets the search to be empty
	*/
	SLATE_API void ResetSearch();

	/**
	* Changes visibility of widgets in the multibox
	*/
	SLATE_API void FilterMultiBoxEntries();

	/**
	* Get the text to search by
	*
	* @return	The text currently being searched
	*/
	SLATE_API FText GetSearchText() const;

	/**
	* Get the SSearchText widget holding the search text
	*
	* @return	The widget to get
	*/
	SLATE_API TSharedPtr<SWidget> GetSearchTextWidget();

	/**
	* Set the block widget holding the search text
	*
	* @param	BlockWidget		The widget to set
	*/
	SLATE_API void SetSearchBlockWidget(TSharedPtr<SWidget>);

	/**
	* Adds a widget to SearchElement map, making the widget searchable
	*
	* @param	BlockWidget			The widget to add
	* @param	BlockDisplayText	The display text of the widget to search by
	*/
	UE_DEPRECATED(4.26, "AddSearchElement is deprecated as non-searchable elements also need to be stored, use AddElement instead")
	SLATE_API void AddSearchElement( TSharedPtr<SWidget>, FText );

	/**
	* Adds a widget to MultiBoxWidgets map, to access and modify its visibility based on search filters
	*
	* @param	BlockWidget			The widget to add
	* @param	BlockDisplayText	The display text of the widget to search by
	  @param    bSearchable			Whether the text is searchable
	*/
	SLATE_API void AddElement(TSharedPtr<SWidget> BlockWidget, FText BlockDisplayText, bool bInSearchable = true);

	/**
	 * @return True if the passed in block is being dragged
	 */
	SLATE_API bool IsBlockBeingDragged( TSharedPtr<const FMultiBlock> Block ) const;

	/**
	 * @return The visibility of customization widgets for a block
	 */
	SLATE_API EVisibility GetCustomizationVisibility(TWeakPtr<const FMultiBlock> BlockWeakPtr, TWeakPtr<SWidget> BlockWidgetWeakPtr) const;

	/**
	 * @return The visibility of the drop location indicator of a drag and drop for a block
	 */
	SLATE_API EVisibility GetCustomizationBorderDragVisibility(const FName InBlockName, const EMultiBlockType InBlockType, bool& bOutInsertAfter) const;

	/**
	 * Records the time that the multibox last summoned a menu
	 */
	SLATE_API void SetSummonedMenuTime(double InSummonedMenuTime);

	/**
	 * @return The last recorded time that the multibox summoned a menu
	 */
	SLATE_API double GetSummonedMenuTime() const;

	UE_DEPRECATED_FORGAME(5.5, "This function will be made private in the future.")
	bool ShouldShowMenuSearchField() const;

private:
	/** Adds a block Widget to this widget */
	SLATE_API void AddBlockWidget(const FMultiBlock& Block, TSharedPtr<SHorizontalBox> HorizontalBox, TSharedPtr<SVerticalBox> VerticalBox, TSharedPtr<SPanel> Panel, EMultiBlockLocation::Type InLocation, bool bSectionContainsIcons, TSharedPtr<const FToolBarComboButtonBlock> OptionsBlock);

	/**
	 * Updates the preview block being dragged.  The drag area is where the users dragged block will be dropped
	 */
	SLATE_API void UpdateDropAreaPreviewBlock( TSharedRef<const FMultiBlock> MultiBlock, TSharedPtr<FUICommandDragDropOp> DragDropContent, const FGeometry& DragArea, const FVector2D& DragPos );

	/** Creates the SearchTextWidget if the MultiBox has requested one */
	SLATE_API void CreateSearchTextWidget();

	SLATE_API EVisibility GetSearchBoxVisibility() const;

	/** Called when the search field text is committed (such as pressing ESC). */
	SLATE_API void OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type CommitType);
	
	/** Called when the search field text changes. */
	SLATE_API void OnFilterTextChanged(const FText& InFilterText);

	/**
	 * Walks the sub-menus and adds new searchable blocks representing the flattened structure of any nested sub-menus.
	 * Enables recursive search over sub-menus. Allows work of traversing sub-menus to be deferred until search is requested.
	 * 
	 * @param MaxRecursionLevels Maximum depth of nested sub-menus to walk when flattening
	 */
	SLATE_API void FlattenSubMenusRecursive(uint32 MaxRecursionLevels);

private:
	/** A preview of a block being dragged */
	struct FDraggedMultiBlockPreview
	{
		/** Name of entry being dragged */
		FName BlockName;
		/** Type of entry being dragged */
		EMultiBlockType BlockType;
		/** Preview block for the command */
		TSharedPtr<class FDropPreviewBlock> PreviewBlock;
		/** Index into the block list where the block will be added*/
		int32 InsertIndex;
		// Vertical for menus and vertical toolbars, horizontally otherwise
		EOrientation InsertOrientation;

		FDraggedMultiBlockPreview()
			: BlockType(EMultiBlockType::None)
			, InsertIndex( INDEX_NONE )
		{}

		void Reset()
		{
			BlockName = NAME_None;
			BlockType = EMultiBlockType::None;
			PreviewBlock.Reset();
			InsertIndex = INDEX_NONE;
		}

		bool IsSameBlockAs(const FName InName, const EMultiBlockType InType) const
		{
			return BlockName == InName && BlockType == InType;
		}

		bool IsValid() const { return BlockName != NAME_None && BlockType != EMultiBlockType::None && PreviewBlock.IsValid() && InsertIndex != INDEX_NONE; }
	};

	/** Contains information about sub-menu block widgets that were pulled and flatten in the parent menu to enable searching the entire menu tree from the top. */
	struct FFlattenSearchableBlockInfo
	{
		/** The block widget searchable text along with its ancestor menu searchable texts. Ex. ["Menu_X", "Sub_Menu_Y", "BlockText"] */
		TArray<FText> SearchableTextHierarchyComponents;

		/** The flatten widget wrapping the block widget that was built for the associated block. The flatten widget adds a 'hierarchy tip' widget to indicate the real location of this item in the hierarchy. */
		TSharedPtr<SWidget> Widget;

		/** Whether the hierarchy tip widget is visible in the flatten search result. This tip let the user know the real location of this item in the hierarchy. */
		EVisibility HierarchyTipVisibility = EVisibility::Collapsed;
	};

	/** The MultiBox we're associated with */
	TSharedPtr< FMultiBox > MultiBox;

	/** An array of widgets used for an STileView if used */
	TArray< TSharedPtr<SWidget> > TileViewWidgets;

	/** Box panel used for horizontally-oriented boxes, e.g., horizontal toolbar or menu bar */
	TSharedPtr<SHorizontalBox> MainHorizontalBox;

	/** Box panel used for vertically-oriented boxes, e.g., vertical toolbar or menu */
	TSharedPtr<SVerticalBox> MainVerticalBox;

	/** Box panel used for other boxes, e.g., horizontal wrapping toolbar */
	TSharedPtr<SPanel> MainPanelBox;

	/** Specialized box widget to handle clipping of toolbars and menubars */
	TSharedPtr<class SClippingHorizontalBox> ClippedHorizontalBox;
	
	/** Specialized box widget to handle clipping of vertical toolbars  */
	TSharedPtr<class SClippingVerticalBox> ClippedVerticalBox;

	/** Specialized box widget to handle clipping of toolbars and menubars */
	TSharedPtr<SUniformWrapPanel> UniformToolbarPanel;

	/** A preview of a block being dragged inside this box */
	FDraggedMultiBlockPreview DragPreview;

	/** The multibox widgets that are contained, linked to their searchable text hierarchies. */
	TMap<TSharedPtr<SWidget>, TArray<FText>> MultiBoxWidgets;

	/** The final multibox widgets as they were added to the UI, linked to their original MultiBlocks. */
	TMap<TSharedPtr<SWidget>, TSharedPtr<const FMultiBlock>> FinalMultiBoxWidgets;

	/** The set of searchable blocks found in this multibox sub-menus that were collected and flatten in this multibox to support recursively searching this multibox hierarchy. */
	TMap<TSharedPtr<const FMultiBlock>, TSharedPtr<FFlattenSearchableBlockInfo>> FlattenSearchableBlocks;

	/** Whether the set in FlattenSearchableBlocks has already been populated. */
	bool bDidFlattenSearchableBlocks = false;

	/** The set of searchable blocks widgets that were added from the list of flatten blocks collecteto display search results from this multibox hierarchy. */
	TMap<TSharedPtr<SWidget>, TSharedPtr<FFlattenSearchableBlockInfo>> FlattenSearchableWidgets;

	/** The list of visible hierarchy tips text (associated with flatten search widgets) in the search result. This is used to group sub-items matching the searched text and avoid showing the same tip several times. */
	TSet<FString> VisibleFlattenHierarchyTips;

	/* The search widget to be displayed at the top of the multibox */
	TSharedPtr<SSearchBox> SearchTextWidget;

	/* The search widget to be displayed at the top of the multibox */
	TSharedPtr<SWidget> SearchBlockWidget;

	/* The text being searched for */
	FText SearchText;

	/** Whether this multibox can be searched */
	bool bSearchable;

	/** The time when the multibox last summoned a menu */
	double SummonedMenuTime = 0.0;

	/** Optional maximum height of widget */
	TAttribute<float> MaxHeight;

	/** Allows Menu Elementes to size properly */
	TSharedPtr<FLinkedBoxManager> LinkedBoxManager;
};
