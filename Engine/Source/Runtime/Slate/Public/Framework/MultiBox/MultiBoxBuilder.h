// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBox.h"

class FUICommandInfo;
class FUICommandList;
struct FSlateIcon;
struct FUIAction;
struct FButtonArgs;

/** Delegate used by multi-box to call a user function to populate a new menu.  Used for spawning sub-menus and pull-down menus. */
DECLARE_DELEGATE_OneParam( FNewMenuDelegate, class FMenuBuilder& );


/**
 * MultiBox builder
 */
class FMultiBoxBuilder
{

public:

	/**
	 * Constructor
	 *
	 * @param	InType	Type of MultiBox
	 * @param	bInShouldCloseWindowAfterMenuSelection	Sets whether or not the window that contains this multibox should be destroyed after the user clicks on a menu item in this box
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget.  This can be modified after the MultiBox is created by calling the PushCommandList() and PopCommandList() methods.
	 * @param	InTutorialHighlightName	Optional name to identify this widget and highlight during tutorials
	 */
	SLATE_API FMultiBoxBuilder( const EMultiBoxType InType, FMultiBoxCustomization InCustomization, const bool bInShouldCloseWindowAfterMenuSelection, const TSharedPtr< const FUICommandList >& InCommandList, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>(), FName InTutorialHighlightName = NAME_None, FName InMenuName = NAME_None );

	virtual ~FMultiBoxBuilder() = default;

	/**
	 * Adds an editable text entry
	 *
	 * @param	InLabel				The label to display in the menu
	 * @param	InToolTip			The tool tip to display when the menu entry is hovered over
	 * @param	InIcon				The icon to display to the left of the label
	 * @param	InTextAttribute		The text string we're editing (often, a delegate will be bound to the attribute)
	 * @param	InOnTextCommitted	Called when the user commits their change to the editable text control
	 * @param	InOnTextChanged		Called when the text is changed interactively
	 * @param	bInReadOnly			Whether or not the text block is read only
	 */
	SLATE_API void AddEditableText( const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, const TAttribute< FText >& InTextAttribute, const FOnTextCommitted& InOnTextCommitted = FOnTextCommitted(), const FOnTextChanged& InOnTextChanged = FOnTextChanged(), bool bInReadOnly = false );

	/**
	 * Adds an editable text entry with a VerifyTextChanged delegate
	 *
	 * @param	InLabel					The label to display in the menu
	 * @param	InToolTip				The tool tip to display when the menu entry is hovered over
	 * @param	InIcon					The icon to display to the left of the label
	 * @param	InTextAttribute			The text string we're editing (often, a delegate will be bound to the attribute)
	 * @param	InOnVerifyTextChanged	Called to verify when the text is changed interactively
	 * @param	InOnTextCommitted		Called when the user commits their change to the editable text control
	 * @param	InOnTextChanged			Called when the text is changed interactively
	 * @param	bInReadOnly				Whether or not the text block is read only
	 */
	SLATE_API void AddVerifiedEditableText(const FText& InLabel, const FText& InToolTip, const FSlateIcon& InIcon, const TAttribute< FText >& InTextAttribute, const FOnVerifyTextChanged& InOnVerifyTextChanged, const FOnTextCommitted& InOnTextCommitted = FOnTextCommitted(), const FOnTextChanged& InOnTextChanged = FOnTextChanged(), bool bInReadOnly = false);

	/**
	 * Creates a widget for this MultiBox
	 *
	 * @return  New widget object
	 */
	SLATE_API virtual TSharedRef< class SWidget > MakeWidget( FMultiBox::FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride = nullptr);
	

	/** 
	 * Get the multi-box being built.
	 *
	 * @return The multi-box being built.
	 */
	SLATE_API TSharedRef< class FMultiBox > GetMultiBox();


	/**
	 * Pushes a new command list onto the stack.  This command list will be used for all subsequently-added multiblocks, until the command-list is popped.
	 *
	 * @param	CommandList		The new command list to use
	 */
	SLATE_API void PushCommandList( const TSharedRef< const FUICommandList > CommandList );

	SLATE_API void SetLastSelectedCommandIndex( int32 InLastSelectedCommandIndex );


	/**
	 * Pops the current command list.
	 */
	SLATE_API void PopCommandList();
	
	/**
	 * @return The top command list
	 */
	SLATE_API TSharedPtr<const FUICommandList> GetTopCommandList();

	/**
	 * Pushes a new extender onto the stack. This extender will be used for all subsequently-added multiblocks, until the extender is popped.
	 *
	 * @param	InExtender	The new extender to use
	 */
	SLATE_API void PushExtender( TSharedRef< FExtender > InExtender );


	/**
	 * Pops the current extender.
	 */
	SLATE_API void PopExtender();

	/** @return The style set used by the multibox widgets */
	SLATE_API const ISlateStyle* GetStyleSet() const;

	/** @return The style name used by the multibox widgets */
	SLATE_API const FName& GetStyleName() const;

	/** the override for the checkbox style */
	SLATE_API void SetCheckBoxStyle(FName InCheckBoxStyle);

	/** Sets the style to use on the multibox widgets */
	SLATE_API void SetStyle( const ISlateStyle* InStyleSet, const FName& InStyleName );

	/** @return  The customization settings for the box being built */
	SLATE_API FMultiBoxCustomization GetCustomization() const;

	/** Sets extender support */
	void SetExtendersEnabled(bool bEnabled) { bExtendersEnabled = bEnabled; }

	/** @return True if extenders are enabled */
	bool ExtendersEnabled() const { return bExtendersEnabled; }

protected:
	/** Applies any potential extension hooks at the current place */
	virtual void ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition) = 0;
	
	/** Applies the beginning of a section, including the header and relevant separator */
	virtual void ApplySectionBeginning() {}

protected:

	/** The MultiBox we're building up */
	TSharedRef< class FMultiBox > MultiBox;

	/** A stack of command lists which map command infos to delegates that should be called.  New multi blocks will always use
		the command-list at the top of the stack at the time they are added. */
	TArray< TSharedPtr< const FUICommandList > > CommandListStack;

	/** The extender stack holding all the possible extensions for this menu builder */
	TArray< TSharedPtr<class FExtender> > ExtenderStack;

	/** Name to identify this widget and highlight during tutorials */
	FName TutorialHighlightName;

	/** Name of the menu */
	FName MenuName;

	/** the override for the checkbox style for this menu */
	FName CheckBoxStyle;

	/** If extenders are enabled */
	bool bExtendersEnabled;
};

/** Helper struct that holds FMenuEntry params for construction */
struct FMenuEntryParams : public FMultiBlock::FMultiBlockParams
{
	FMenuEntryParams()
	{
		Type = EMultiBlockType::MenuEntry;
	}

	/** Optional overridden text label for this menu entry.  If not set, then the action's label will be used instead. */
	TAttribute<FText> LabelOverride;

	/** Optional overridden tool tip for this menu entry.  If not set, then the action's tool tip will be used instead. */
	TAttribute<FText> ToolTipOverride;

	/** Optional overridden input binding text for this menu entry.  If not set, then the UI action's binding will be used if available. */
	TAttribute<FText> InputBindingOverride;

	/** Set in case this entry needs to be dynamically hidden */
	TAttribute<EVisibility> Visibility;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	FSlateIcon IconOverride;

	/** Optional menu entry builder associated with this entry for building sub-menus and pull down menus */
	FNewMenuDelegate EntryBuilder;

	/** Delegate that returns an entire menu */
	FOnGetContent MenuBuilder;

	/** Widget to be added to the menu */
	TSharedPtr<SWidget> EntryWidget;

	/** True if this menu entry opens a sub-menu */
	bool bIsSubMenu = false;

	/** True if the search algorithm should walk down this menu sub menus. Usually true, unless the menu has circular/infinite expansion (happens in some menus generated on the fly by reflection). */
	bool bIsRecursivelySearchable = true;;

	/** True if this menu entry opens a sub-menu by clicking on it only */
	bool bOpenSubMenuOnClick = false;

	/** In the case where a command is not bound, the user interface action type to use.  If a command is bound, we
		simply use the action type associated with that command. */
	EUserInterfaceActionType UserInterfaceActionType;

	/** True if the menu should close itself and all its children or the entire open menu stack */
	bool bCloseSelfOnly = false;

	/** An extender that this menu entry should pass down to its children, so they get extended properly */
	TSharedPtr<FExtender> Extender;

	/** For submenus, whether the menu should be closed after something is selected */
	bool bShouldCloseWindowAfterMenuSelection = true;

	/** Display name for tutorials */
	FName TutorialHighlightName;
};

/**
 * Base menu builder
 */
class FBaseMenuBuilder : public FMultiBoxBuilder
{

public:

	/**
	 * Constructor
	 *
	 * @param	InType	Type of MultiBox
	 * @param	bInShouldCloseWindowAfterMenuSelection	Sets whether or not the window that contains this multibox should be destroyed after the user clicks on a menu item in this box
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 * @param	bInCloseSelfOnly	True if clicking on a menu entry closes itself only and its children but not the entire stack 
	 * @param	InTutorialHighlightName	Optional name to identify this widget and highlight during tutorials
	 */
	SLATE_API FBaseMenuBuilder( const EMultiBoxType InType, const bool bInShouldCloseWindowAfterMenuSelection, TSharedPtr< const FUICommandList > InCommandList, bool bInCloseSelfOnly, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>(), const ISlateStyle* InStyleSet = &FCoreStyle::Get(), FName InTutorialHighlightName = NAME_None, FName InMenuName = NAME_None );

	/**
	 * Adds a menu entry
	 *
	 * @param	InCommand			The command associated with this menu entry
	 * @param	InExtensionHook		The section hook. Can be NAME_None
	 * @param	InLabelOverride		Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride	Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride		Optional name of the slate brush to use for the tool bar image.  If omitted, then the command's icon will be used instead.
	 * @param	InTutorialHighlightName	Optional name to identify this widget and highlight during tutorials
	 * @param   InVisibility Optional Visibility Override. Can be used to show/hide this entry dynamically.
	 */
	SLATE_API void AddMenuEntry( const TSharedPtr< const FUICommandInfo > InCommand, FName InExtensionHook = NAME_None, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const FSlateIcon& InIconOverride = FSlateIcon(), FName InTutorialHighlightName = NAME_None, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());

	/**
	 * Adds a menu entry without the use of a command
	 *
	 * @param	InLabel		Label to show in the menu entry
	 * @param	InToolTip	Tool tip used when hovering over the menu entry
	 * @param	InIcon		The icon to use		
	 * @param	UIAction	Actions to execute on this menu item.
	 * @param	InExtensionHook			The section hook. Can be NAME_None
	 * @param	UserInterfaceActionType	Type of interface action
	 * @param	InTutorialHighlightName	Optional name to identify this widget and highlight during tutorials
	 * @param	InInputBindingOverride  Optional overridden input binding text for this menu entry.  If not set, then the UI action's binding will be used if available.
	 * @param   InVisibility Optional Visibility Override. Can be used to show/hide this entry dynamically.
	 */
	SLATE_API void AddMenuEntry( const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FSlateIcon& InIcon, const FUIAction& UIAction, FName InExtensionHook = NAME_None, const EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::Button, FName InTutorialHighlightName = NAME_None, const TAttribute<FText>& InInputBindingOverride = TAttribute<FText>(), const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());
	
	/**
	 * Adds a menu entry with a custom widget
	 *
	 * @param	UIAction				Actions to execute on this menu item.
	 * @param	Contents				Custom widget to display
	 * @param	InExtensionHook			The section hook. Can be NAME_None
	 * @param	InToolTip				Tool tip used when hovering over the menu entry
	 * @param	UserInterfaceActionType	Type of interface action
	 * @param	InTutorialHighlightName	Optional name to identify this widget and highlight during tutorials
	 * @param   InVisibility Optional Visibility Override. Can be used to show/hide this entry dynamically.
	 */
	SLATE_API void AddMenuEntry( const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const FName& InExtensionHook = NAME_None, const TAttribute<FText>& InToolTip = TAttribute<FText>(), const EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::Button, FName InTutorialHighlightName = NAME_None, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());

	/** Adds a menu entry with given param struct */
	SLATE_API void AddMenuEntry( const FMenuEntryParams& InMenuEntryParams );

protected:
	/** True if clicking on a menu entry closes itself only and its children and not the entire stack */
	bool bCloseSelfOnly;
};


/**
 * Vertical menu builder
 */
class FMenuBuilder : public FBaseMenuBuilder
{
public:

	/**
	 * Constructor
	 *
	 * @param	bInShouldCloseWindowAfterMenuSelection	Sets whether or not the window that contains this multibox should be destroyed after the user clicks on a menu item in this box
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 * @param	bInCloseSelfOnly	True if clicking on a menu entry closes itself only and its children but not the entire stack
	 * @param	bInSearchable	True if the menu is searchable
	 * @param	bInRecursivelySearchable True if search algorithm should go down the sub-menus when searching. If false, the search will not scan the sub-menus. Recursive search is usually disabled on
	 *                                   menus that are automatically generated in a way that the menu can expand indefinitely in a circular fashion. For example, the Blueprint API binding reflected on
	 *                                   function return type can do that. Think about a function like "A* A::GetParent()". If a root menu expands "A" functions and expands on the function return types, then
	 *                                   selecting "GetParent()" will expand another "A" menu. Without simulation, the reflection API don't know when the recursion finishes, nor does the recursive search algorithm.
	 */
	FMenuBuilder( const bool bInShouldCloseWindowAfterMenuSelection, TSharedPtr< const FUICommandList > InCommandList, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>(), const bool bInCloseSelfOnly = false, const ISlateStyle* InStyleSet = &FCoreStyle::Get(), bool bInSearchable = true, FName InMenuName = NAME_None, bool bInRecursivelySearchable = true)
		: FBaseMenuBuilder( EMultiBoxType::Menu, bInShouldCloseWindowAfterMenuSelection, InCommandList, bInCloseSelfOnly, InExtender, InStyleSet, NAME_None, InMenuName )
		, bSectionNeedsToBeApplied(false)
		, bSearchable(bInSearchable)
		, bRecursivelySearchable(bInRecursivelySearchable)
		, bIsEditing(false)
	{
		if(bSearchable)
		{
			AddSearchWidget();
		}
	}

	/**
	* Creates a widget for this MultiBox
	*
	* @return  New widget object
	*/
	SLATE_API virtual TSharedRef< class SWidget > MakeWidget( FMultiBox::FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride = nullptr) override;
	SLATE_API virtual TSharedRef< class SWidget > MakeWidget( FMultiBox::FOnMakeMultiBoxBuilderOverride* InMakeMultiBoxBuilderOverride, uint32 MaxHeight);

public:
	/**
	 * Adds a separator
	 */
	SLATE_API void AddMenuSeparator(FName InExtensionHook = NAME_None, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());
	SLATE_API void AddSeparator(FName InExtensionHook = NAME_None, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());

	/**
	 * Starts a section on to the extender section hook stack
	 * 
	 * @param InExtensionHook	The section hook. Can be NAME_None
	 * @param InHeadingText		The heading text to use. If none, only a separator is used
	 * @param InVisibility Optional Visibility Override. Can be used to show/hide this Section dynamically.
	 * @param InResizeParams	Optional resize parameters and overrides, @see FMenuEntryResizeParams
	 */
	SLATE_API void BeginSection( FName InExtensionHook, const TAttribute< FText >& InHeadingText = TAttribute<FText>(), const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>(), const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>() );

	/**
	 * Ends the current section
	 */
	SLATE_API void EndSection();


	/**
	 * Adds a sub-menu which is a menu within a menu
	 * 
	 * @param	InMenuLabel			The text that should be shown for the menu
	 * @param	InToolTip			The tooltip that should be shown when the menu is hovered over
	 * @param	InSubMenu			Sub-Menu object which creates menu entries for the sub-menu
	 * @param	InUIAction			Actions to execute on this menu item.
	 * @param	InExtensionHook		The section hook. Can be NAME_None
	 * @param	InUserInterfaceActionType	Type of interface action
	 * @param bInOpenSubMenuOnClick Sub-menu will open only if the sub-menu entry is clicked
	 * @param	InIcon				The icon to use
	 * @param	bInShouldCloseWindowAfterMenuSelection	Whether the submenu should close after an item is selected
	 * @param   InVisibility Optional Visibility Override. Can be used to show/hide this entry dynamically.
	 */
	SLATE_API void AddSubMenu( const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InSubMenu, const FUIAction& InUIAction, FName InExtensionHook, const EUserInterfaceActionType InUserInterfaceActionType, const bool bInOpenSubMenuOnClick = false, const FSlateIcon& InIcon = FSlateIcon(), const bool bInShouldCloseWindowAfterMenuSelection = true, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>(), const TAttribute<FText>& InInputBindingOverride = TAttribute<FText>() );

	SLATE_API void AddSubMenu( const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InSubMenu, const bool bInOpenSubMenuOnClick = false, const FSlateIcon& InIcon = FSlateIcon(), const bool bInShouldCloseWindowAfterMenuSelection = true, FName InExtensionHook = NAME_None, FName InTutorialHighlightName = NAME_None, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>() );

	SLATE_API void AddSubMenu( const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InSubMenu, const bool bInOpenSubMenuOnClick = false, const bool bInShouldCloseWindowAfterMenuSelection = true, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>() );

	SLATE_API void AddSubMenu( const FUIAction& UIAction, const TSharedRef< SWidget > Contents, const FNewMenuDelegate& InSubMenu, const bool bInShouldCloseWindowAfterMenuSelection = true, const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>() );

	/**
	 * Adds any widget to the menu
	 * 
	 * @param	InWidget			The widget that should be shown in the menu
	 * @param	InLabel				Optional label text to be added to the left of the content
	 * @param	bInNoIndent			If true, removes the padding from the left of the widget that lines it up with other menu items (default == false)
	 * @param	bInSearchable		If true, widget will be searchable (default == true)
	 * @param	InToolTipText		Optional tooltip text to be added to the widget and label
	 */
	SLATE_API void AddWidget( TSharedRef<SWidget> InWidget, const FText& InLabel, bool bInNoIndent = false, bool bInSearchable = true, const TAttribute<FText>&  InToolTipText = FText());

	/**
	 * Adds any widget to the menu
	 *
	 * @param	InWidget			The widget that should be shown in the menu
	 * @param	InLabel				Optional label text to be added to the left of the content
	 * @param	InStyleParams		Optional style parameters and overrides, @see FMenuEntryStyleParams
	 * @param	bInSearchable		If true, widget will be searchable (default == true)
	 * @param	InToolTipText		Optional tooltip text to be added to the widget and label
	 * @param	InIcon				Optional icon to be shown to the left of the label/content
	 */
	SLATE_API void AddWidget(const TSharedRef<SWidget>& InWidget, const FText& InLabel, const FMenuEntryStyleParams& InStyleParams, bool bInSearchable = true, const TAttribute<FText>& InToolTipText = FText(), const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());

	/**
	 * Adds any widget to the menu
	 *
	 * @param	InWidget			The widget that should be shown in the menu
	 * @param	InLabel				Optional label text to be added to the left of the content
	 * @param	InStyleParams		Optional style parameters and overrides, @see FMenuEntryStyleParams
	 * @param	InResizeParams		Optional resize parameters and overrides, @see FMenuEntryResizeParams
	 * @param	bInSearchable		If true, widget will be searchable (default == true)
	 * @param	InToolTipText		Optional tooltip text to be added to the widget and label
	 * @param	InIcon				Optional icon to be shown to the left of the label/content
	 */
	SLATE_API void AddWidget(const TSharedRef<SWidget>& InWidget, const FText& InLabel, const FMenuEntryStyleParams& InStyleParams, const TAttribute<FMenuEntryResizeParams>& InResizeParams, bool bInSearchable = true, const TAttribute<FText>& InToolTipText = FText(), const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const TAttribute<EVisibility>& InVisibility = TAttribute<EVisibility>());


	/**
	* Adds the widget the multibox will use for searching
	*/
	SLATE_API void AddSearchWidget();

	void SetIsEditing(bool bInIsEditing) { bIsEditing = bInIsEditing; }

protected:
	/** FMultiBoxBuilder interface */
	SLATE_API virtual void ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition) override;
	SLATE_API virtual void ApplySectionBeginning() override;

public:
	/**
	 * Adds a sub-menu which is a menu within a menu
	 * 
	 * @param	InMenuLabel			The text that should be shown for the menu
	 * @param	InToolTip			The tooltip that should be shown when the menu is hovered over
	 * @param	InSubMenu			Sub-Menu object which creates the sub-menu
	 */
	SLATE_API void AddWrapperSubMenu( const FText& InMenuLabel, const FText& InToolTip, const FOnGetContent& InSubMenu, const FSlateIcon& InIcon );

	SLATE_API void AddWrapperSubMenu( const FText& InMenuLabel, const FText& InToolTip, const FOnGetContent& InSubMenu, const FSlateIcon& InIcon, const FUIAction& UIAction );

	/**
	 * Adds a sub-menu which is a menu within a menu
	 * 
	 * @param	InMenuLabel			The text that should be shown for the menu
	 * @param	InToolTip			The tooltip that should be shown when the menu is hovered over
	 * @param	InSubMenu			Sub-Menu object
	 */
	SLATE_API void AddWrapperSubMenu( const FText& InMenuLabel, const FText& InToolTip, const TSharedPtr<SWidget>& InSubMenu, const FSlateIcon& InIcon );

	void SetSearchable(bool bIsSearchable) { bSearchable = bIsSearchable; };

private:
	/** Current extension hook name for sections to determine where sections begin and end */
	FName CurrentSectionExtensionHook;
	
	/** Any pending section's heading text */
	FText CurrentSectionHeadingText;

	/** Keep track of current section visibility, in case the section needs to be dynamically hidden */
	TAttribute<EVisibility> CurrentSectionVisibility;

	/** Resize parameters for the current section */
	TAttribute<FMenuEntryResizeParams> CurrentSectionResizeParams;

	/** True if there is a pending section that needs to be applied */
	bool bSectionNeedsToBeApplied;

	/** Whether this menu is searchable */
	bool bSearchable;

	/** Whether the search algorithm should walk down this menu sub-menu(s) (if the menu is searchable in first place). */
	bool bRecursivelySearchable;

	/** Whether menu is currently being edited */
	bool bIsEditing;
};



/**
 * Menu bar builder
 */
class FMenuBarBuilder : public FBaseMenuBuilder
{

public:

	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	FMenuBarBuilder( TSharedPtr< const FUICommandList > InCommandList, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>(), const ISlateStyle* InStyleSet = &FCoreStyle::Get(), FName InMenuName = NAME_None)
		: FBaseMenuBuilder( EMultiBoxType::MenuBar, false, InCommandList, false, InExtender, InStyleSet, NAME_None, InMenuName )
	{
		MultiBox->SetStyle(InStyleSet, "WindowMenuBar");
	}


	/**
	 * Adds a pull down menu
	 *
	 * @param	InMenuLabel			The text that should be shown for the menu
	 * @param	InToolTip			The tooltip that should be shown when the menu is hovered over
	 * @param	InPullDownMenu		Pull down menu object for the menu to add.
	 */
	SLATE_API void AddPullDownMenu( const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FNewMenuDelegate& InPullDownMenu, FName InExtensionHook = NAME_None, FName InTutorialHighlightName = NAME_None);

	/**
	 * Adds a pull down menu
	 *
	 * @param	InMenuLabel				The text that should be shown for the menu
	 * @param	InToolTip				The tooltip that should be shown when the menu is hovered over
	 * @param	InMenuContentGenerator	Delegate that generates a widget for this pulldown menu's content.  Called when the menu is summoned.
	 */
	SLATE_API void AddPullDownMenu(const TAttribute<FText>& InMenuLabel, const TAttribute<FText>& InToolTip, const FOnGetContent& InMenuContentGenerator, FName InExtensionHook = NAME_None, FName InTutorialHighlightName = NAME_None);

protected:
	/** FMultiBoxBuilder interface */
	SLATE_API virtual void ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition) override;
};



/**
 * Tool bar builder
 */
class FToolBarBuilder : public FMultiBoxBuilder
{
	friend class UToolMenus;
public:

	UE_DEPRECATED(4.26, "FToolBarBuilder constructor that takes in an EOrientation is deprecated.  Use one of the specific per-type FToolbarBuilder overrides instead.")
	FToolBarBuilder(TSharedPtr< const FUICommandList > InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender, EOrientation Orientation, const bool InForceSmallIcons = false, const bool bUniform = false)
		: FMultiBoxBuilder(bUniform ? EMultiBoxType::UniformToolBar : (Orientation == Orient_Horizontal) ? EMultiBoxType::ToolBar : EMultiBoxType::VerticalToolBar, InCustomization, false, InCommandList, InExtender)
		, bSectionNeedsToBeApplied(false)
		, bSectionShouldHaveSeparator(true)
		, bIsFocusable(true)
		, bForceSmallIcons(InForceSmallIcons)
	{
	}

	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	FToolBarBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)
		: FMultiBoxBuilder(EMultiBoxType::ToolBar, InCustomization, false, InCommandList, InExtender)
		, bSectionNeedsToBeApplied(false)
		, bSectionShouldHaveSeparator(true)
		, bIsFocusable(true)
		, bForceSmallIcons(InForceSmallIcons)
	{
		MultiBox->bIsFocusable = bIsFocusable;
	}

	void SetLabelVisibility( EVisibility InLabelVisibility ) { LabelVisibility  = InLabelVisibility ; }

	SLATE_API void SetIsFocusable(bool bInIsFocusable);

	/** Whether to allow the wrap button/overflow menu, if applicable. Defaults to the style. */
	SLATE_API void SetAllowWrapButton(const TOptional<bool>& bInAllowWrapButton) const;

	/**
	 * Adds a tool bar button
	 *
	 * @param	ButtonArgs The Parameters object which will provide the data to initialize the button
	 */
	SLATE_API virtual void AddToolBarButton(const FButtonArgs& ButtonArgs);

	/**
	 * Adds a tool bar button
	 *
	 * @param	InCommand				The command associated with this tool bar button
	 * @param	InExtensionHook			The section hook. Can be NAME_None.
	 * @param	InLabelOverride			Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride		Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride			Optional name of the slate brush to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 * @param	InTutorialHighlightName	Name to identify this widget and highlight during tutorials
	 * @param	InCustomMenuDelegate	Optional custom menu delegate for cases where the toolbar is compressed into a menu
	 * @param	InVisibilityOverride	Optional visibility override which, if set and returning anything but Visible, overrides the visibility of the toolbar button.
	 * @param	InToolbarLabelOverride	Optional label override for when this block appears in a toolbar. If omitted, then the action's label will be used instead.
	 * @param	InResizeParams			Optional resize parameters to control how this entry behaves during toolbar resizing.
	 */
	SLATE_API TSharedPtr<FToolBarButtonBlock> AddToolBarButton(
		const TSharedPtr<const FUICommandInfo> InCommand,
		FName InExtensionHook = NAME_None,
		const TAttribute<FText>& InLabelOverride = TAttribute<FText>(),
		const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(),
		const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(),
		FName InTutorialHighlightName = NAME_None,
		FNewMenuDelegate InCustomMenuDelegate = FNewMenuDelegate(),
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>(),
		TAttribute<FText> InToolbarLabelOverride = TAttribute<FText>(),
		const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>()
	);

	/**
	 * Adds a tool bar button
	 *
	 * @param	InAction				Actions to execute on this menu item.
	 * @param	InExtensionHook			The section hook. Can be NAME_None.
	 * @param	InLabelOverride			Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride		Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride			Optional icon to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 * @param	UserInterfaceActionType	Type of interface action
	 * @param	InTutorialHighlightName	Name to identify this widget and highlight during tutorials
	 * @param	InVisibilityOverride	Optional visibility override which, if set and returning anything but Visible, overrides the visibility of the toolbar button.
	 * @param	InToolbarLabelOverride	Optional label override for when this block appears in a toolbar. If omitted, then the action's label will be used instead.
	 * @param	InResizeParams			Optional resize parameters to control how this entry behaves during toolbar resizing.
	 */
	SLATE_API TSharedPtr<FToolBarButtonBlock> AddToolBarButton(
		const FUIAction& InAction,
		FName InExtensionHook = NAME_None,
		const TAttribute<FText>& InLabelOverride = TAttribute<FText>(),
		const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(),
		const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(),
		const EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::Button,
		FName InTutorialHighlightName = NAME_None,
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>(),
		TAttribute<FText> InToolbarLabelOverride = TAttribute<FText>(),
		const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>()
	);

	/**
	 * Adds a combo button
	 *
	 * @param	InAction					UI action that sets the enabled state for this combo button
	 * @param	InMenuContentGenerator		Delegate that generates a widget for this combo button's menu content. Called when the menu is summoned.
	 * @param	InLabelOverride				Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride			Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride				Optional icon to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 * @param	bInSimpleComboBox			If true, the icon and label won't be displayed
	 * @param	InTutorialHighlightName		Name to identify this widget and highlight during tutorials
	 * @param	InVisibilityOverride		Optional visibility override which, if set and returning anything but Visible, overrides the visibility of the combo button.
	 * @param	InToolbarLabelOverride		Optional label override for when this block appears in a toolbar.  If omitted, then the action's label will be used instead.
	 * @param	InPlacementOverride			Optional override placement to customize menu placement once opened via e.g. a toolbar menu button.
	 * @param	InUserInterfaceActionType	Optional user interface action type to use if a command isn't bound. If a command is bound, we use the action type associated with the command instead.
	 * @param	InResizeParams			Optional resize parameters to control how this entry behaves during toolbar resizing.
	 */
	SLATE_API TSharedPtr<class FToolBarComboButtonBlock> AddComboButton(
		const FUIAction& InAction,
		const FOnGetContent& InMenuContentGenerator,
		const TAttribute<FText>& InLabelOverride = TAttribute<FText>(),
		const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(),
		const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(),
		bool bInSimpleComboBox = false,
		FName InTutorialHighlightName = NAME_None,
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>(),
		TAttribute<FText> InToolbarLabelOverride = TAttribute<FText>(),
		TAttribute<EMenuPlacement> InPlacementOverride = TAttribute<EMenuPlacement>(),
		const EUserInterfaceActionType InUserInterfaceActionType = EUserInterfaceActionType::Button,
		const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>()
	);

	/**
	 * Adds a tool bar stack button
	 *
	 * @param	InCommand				The command associated with this tool bar button
	 * @param	InTutorialHighlightName	Name to identify this widget and highlight during tutorials
	 */
	SLATE_API void AddToolbarStackButton(const TSharedPtr< const FUICommandInfo > InCommand, FName InTutorialHighlightName = NAME_None);

	/**
	 * Adds any widget to the toolbar
	 * 
	 * @param	InWidget				The widget that should be shown in the toolbar
	 * @param	InLabel                 Optional Label.  
	 * @param	InTutorialHighlightName	Name to identify this widget and highlight during tutorials
	 * @param	bInSearchable			If true, widget will be searchable (default == true)
	 * @param	InToolTip				Optional tool tip 
	 */
	SLATE_API void AddToolBarWidget(TSharedRef<SWidget> InWidget, const TAttribute<FText>& InLabel = TAttribute<FText>(), FName InTutorialHighlightName = NAME_None, bool bInSearchable = true, const TAttribute<FText>& InToolTip = TAttribute<FText>());

	/**
	 * Adds any widget to the toolbar
	 *
	 * @param	InWidget				The widget that should be shown in the toolbar
	 * @param	InTutorialHighlightName	Name to identify this widget and highlight during tutorials
	 * @param	bInSearchable			If true, widget will be searchable (default == true)
	 * @param	InAlignment				Horizontal alignment for the widget inside the toolbar
	 * @param	InCustomMenuDelegate	Optional custom menu delegate for cases where the toolbar is compressed into a menu
	 * @param	InVisibilityOverride	Optional visibility override which, if set and returning anything but Visible, overrides the visibility of the widget.
	 */
	SLATE_API void AddWidget(
		TSharedRef<SWidget> InWidget,
		FName InTutorialHighlightName = NAME_None,
		bool bInSearchable = true,
		EHorizontalAlignment InAlignment = HAlign_Fill,
		FNewMenuDelegate InCustomMenuDelegate = FNewMenuDelegate(),
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>()
	);

	/**
	 * Adds any widget to the toolbar
	 *
	 * @param	InWidget				The widget that should be shown in the toolbar
	 * @param	InStyleParams			Style parameters for the widget inside the toolbar
	 * @param	InTutorialHighlightName	Name to identify this widget and highlight during tutorials
	 * @param	bInSearchable			If true, widget will be searchable (default == true)
	 * @param	InCustomMenuDelegate	Optional custom menu delegate for cases where the toolbar is compressed into a menu
	 * @param	InVisibilityOverride	Optional visibility override which, if set and returning anything but Visible, overrides the visibility of the widget.
	 * @param	InResizeParams			Optional resize parameters to control how this entry behaves during toolbar resizing.
	 */
	SLATE_API void AddWidget(
		TSharedRef<SWidget> InWidget,
		const FMenuEntryStyleParams& InStyleParams,
		FName InTutorialHighlightName = NAME_None,
		bool bInSearchable = true,
		FNewMenuDelegate InCustomMenuDelegate = FNewMenuDelegate(),
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>(),
		const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>()
	);

	/**
	 * Adds a toolbar separator
	 *
	 * @param	InExtensionHook			Optional extensibility hook.
	 * @param	InVisibilityOverride	Optional visibility override which, if set and returning anything but Visible,
	 * overrides the visibility of the separator.
	 * @param	InResizeParams			Optional resize parameters to control how this entry behaves during toolbar resizing.
	 */
	SLATE_API void AddSeparator(
		FName InExtensionHook = NAME_None,
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>(),
		const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>()
	);

	/**
	 * Starts a section on to the extender section hook stack
	 *
	 * @param InExtensionHook				The section hook. Can be NAME_None
	 * @param bInSectionShouldHaveSeparator	Whether this section should have a separator to its left (this doesn't apply
	 * @param InResizeParams				Optional resize parameters to control how this section behaves during toolbar resizing. Note that this currently only supports wrapping options.
	 * to the first section added, because that never gets a separator).
	 */
	SLATE_API void BeginSection(FName InExtensionHook, bool bInSectionShouldHaveSeparator = true, const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>());

	/**
	 * Ends the current section
	 */
	SLATE_API void EndSection();

	/** 
	 * Starts a new Group block, must be used in conjunction with EndBlockGroup
	 */
	SLATE_API void BeginBlockGroup();
	
	/** 
	 * End a group block, must be used in conjunction with BeginBlockGroup.
	 */
	SLATE_API void EndBlockGroup();

	/**
	 * Overrides the style being used by this toolbar with a different one for the
	 * The override will be used for any added blocks until EndStyleOverride is called
	 */
	SLATE_API void BeginStyleOverride(FName StyleOverrideName);
	SLATE_API void EndStyleOverride();
protected:

	FToolBarBuilder(EMultiBoxType InType, TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>(), const bool InForceSmallIcons = false)
		: FMultiBoxBuilder(InType, InCustomization, false, InCommandList, InExtender)
		, bSectionNeedsToBeApplied(false)
		, bSectionShouldHaveSeparator(true)
		, bIsFocusable(false)
		, bForceSmallIcons(InForceSmallIcons)
	{
	}


	/** FMultiBoxBuilder interface */
	SLATE_API virtual void ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition) override;
	SLATE_API virtual void ApplySectionBeginning() override;

	SLATE_API void InitializeToolBarButtonBlock(TSharedPtr<FToolBarButtonBlock> ButtonRowBlock, const FButtonArgs& ButtonArgs);

	/** Handles AddWidget calls */
	void AddWidgetInternal(
		TSharedRef<SWidget> InWidget,
		const FMenuEntryStyleParams& InStyleParams,
		FName InTutorialHighlightName = NAME_None,
		bool bInSearchable = true,
		FNewMenuDelegate InCustomMenuDelegate = FNewMenuDelegate(),
		TAttribute<EVisibility> InVisibilityOverride = TAttribute<EVisibility>(),
		const TAttribute<FMenuEntryResizeParams>& InResizeParams = TAttribute<FMenuEntryResizeParams>()
	);

private:
	/** Current extension hook name for sections to determine where sections begin and end */
	FName CurrentSectionExtensionHook;

	/** Resize parameters for the current section */
	TAttribute<FMenuEntryResizeParams> CurrentSectionResizeParams;

	FName CurrentStyleOverride;

	TOptional< EVisibility > LabelVisibility;

	/** True if there is a pending section that needs to be applied */
	bool bSectionNeedsToBeApplied;

	/** Whether the current section should be preceded by a separator */
	bool bSectionShouldHaveSeparator;

	/** Whether the buttons created can receive keyboard focus */
	bool bIsFocusable;

	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;
};


class FVerticalToolBarBuilder : public FToolBarBuilder
{
public:
	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	FVerticalToolBarBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)
		: FToolBarBuilder(EMultiBoxType::VerticalToolBar, InCommandList, InCustomization, InExtender, InForceSmallIcons)
	{
		this->SetStyle(&FAppStyle::Get(), "FVerticalToolBar");
	}
};

class FUniformToolBarBuilder : public FToolBarBuilder
{
public:
	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	FUniformToolBarBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)
		: FToolBarBuilder(EMultiBoxType::UniformToolBar, InCommandList, InCustomization, InExtender, InForceSmallIcons)
	{
	}
};

class FSlimHorizontalToolBarBuilder : public FToolBarBuilder
{
public:
	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	FSlimHorizontalToolBarBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)
		: FToolBarBuilder(EMultiBoxType::SlimHorizontalToolBar, InCommandList, InCustomization, InExtender, InForceSmallIcons)
	{
	}
};

/**
 * Button grid builder
 */
class FButtonRowBuilder : public FMultiBoxBuilder
{
public:
	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	FButtonRowBuilder(TSharedPtr< const FUICommandList > InCommandList, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>())
		: FMultiBoxBuilder(EMultiBoxType::ButtonRow, FMultiBoxCustomization::None, false, InCommandList, InExtender)
	{
	}


	/**
	 * Adds a button to a row
	 *
	 * @param	InCommand				The command associated with this tool bar button
	 * @param	InLabelOverride			Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride		Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride			Optional icon to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 */
	SLATE_API void AddButton(const TSharedPtr< const FUICommandInfo > InCommand, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const FSlateIcon& InIconOverride = FSlateIcon());

	/**
	 * Adds a button to a row
	 *
	 * @param	InLabel					The button label to display
	 * @param	InToolTip				The tooltip for the button
	 * @param	InUIAction				Action to execute when the button is clicked or when state should be checked
	 * @param	InIcon					The icon for the button
	 * @param	InUserInterfaceActionType	The style of button to display
	 */
	SLATE_API void AddButton(const FText& InLabel, const FText& InToolTip, const FUIAction& InUIAction, const FSlateIcon& InIcon = FSlateIcon(), const EUserInterfaceActionType InUserInterfaceActionType = EUserInterfaceActionType::Button);

protected:
	/** FMultiBoxBuilder interface */
	virtual void ApplyHook(FName InExtensionHook, EExtensionHook::Position HookPosition) override {}
};

class FSlimHorizontalUniformToolBarBuilder : public FToolBarBuilder
{
public:
	/**
	 * Constructor
	 *
	 * @param	InCommandList	The action list that maps command infos to delegates that should be called for each command associated with a multiblock widget
	 */
	SLATE_API FSlimHorizontalUniformToolBarBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false);

	SLATE_API virtual void AddToolBarButton(const FButtonArgs& ButtonArgs) override;

};
