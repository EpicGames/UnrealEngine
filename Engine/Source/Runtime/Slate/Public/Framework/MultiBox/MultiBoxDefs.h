// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/ISlateStyle.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Types/SlateStructs.h"
#include "MultiBoxDefs.generated.h"

class SToolTip;
class SWidget;

/** Types specific to SlimWrappingToolBar (and it's variants). */
namespace UE::Slate::PrioritizedWrapBox
{
	enum class EWrapMode : uint8
	{
		Preferred = 0,					// Wraps at the line length specified in PreferredSize
		Parent = 1,						// Wraps at the wrap box's resulting/actual size
	};

	/**
	 * The behavior when a slot's desired height exceeds the allotted/available size (including MaxLineHeight when specified)
	 * By default this Clips.
	 */
	enum class EVerticalOverflowBehavior : uint8
	{
		Clip = 0,						// Always Clip to the line height as it's been calculated so far, excluding this slots height from affecting the line height
		ExpandProportional = 1,			// Uses the desired height of the slot, maintaining the total area after clamping the width to the available space

		Default = Clip,
	};
}

/**
 * Types of MultiBoxes
 */
UENUM(BlueprintType)
enum class EMultiBoxType : uint8
{
	/** Horizontal menu bar */
	MenuBar,

	/** Horizontal tool bar */
	ToolBar,

	/** Vertical tool bar */
	VerticalToolBar,

	/** Toolbar which is a slim version of the toolbar that aligns an icon and a text element horizontally */
	SlimHorizontalToolBar,

	/** A toolbar that tries to arrange all toolbar items uniformly (supports only horizontal toolbars for now) */
	UniformToolBar,

	/** Vertical menu (pull-down menu, or context menu) */
	Menu,

	/** Buttons arranged in rows, with a maximum number of buttons per row, like a toolbar but can have multiple rows*/
	ButtonRow,

	/** A toolbar with horizontally-oriented buttons that tries to arrange all toolbar items uniformly */
	SlimHorizontalUniformToolBar,

	/** Horizontal tool bar that can (optionally) wrap to subsequent rows */
	SlimWrappingToolBar,
};


/**
 * Types of MultiBlocks
 */
UENUM(BlueprintType)
enum class EMultiBlockType : uint8
{
	None = 0,
	ButtonRow,
	EditableText,
	Heading,
	MenuEntry,
	Separator,
	ToolBarButton,
	ToolBarComboButton,
	Widget,
};


class FMultiBoxSettings
{
public:

	DECLARE_DELEGATE_RetVal_FourParams( TSharedRef< SToolTip >, FConstructToolTip, const TAttribute<FText>& /*ToolTipText*/, const TSharedPtr<SWidget>& /*OverrideContent*/, const TSharedPtr<const FUICommandInfo>& /*Action*/, bool /*ShowActionShortcut*/);

	/** Access to whether multiboxes use small icons or default sized icons */
	static SLATE_API TAttribute<bool> UseSmallToolBarIcons;
	static SLATE_API TAttribute<bool> DisplayMultiboxHooks;
	static SLATE_API FConstructToolTip ToolTipConstructor;
	
	static SLATE_API TAttribute<int> MenuSearchFieldVisibilityThreshold;

	SLATE_API FMultiBoxSettings();

	static SLATE_API TSharedRef< SToolTip > ConstructDefaultToolTip( const TAttribute<FText>& ToolTipText, const TSharedPtr<SWidget>& OverrideContent, const TSharedPtr<const FUICommandInfo>& Action, bool ShowActionShortcut = true );

	static SLATE_API void ResetToolTipConstructor();
};

struct FMultiBoxCustomization
{
	static SLATE_API const FMultiBoxCustomization None;

	static FMultiBoxCustomization AllowCustomization( FName InCustomizationName )
	{
		ensure( InCustomizationName != NAME_None );
		return FMultiBoxCustomization( InCustomizationName );
	}

	FName GetCustomizationName() const { return CustomizationName; }

	FMultiBoxCustomization( FName InCustomizationName )
		: CustomizationName( InCustomizationName )
	{}

private:
	/** The Name of the customization that uniquely identifies the multibox for saving and loading users data*/
	FName CustomizationName;
};

/** 
 * Block location information
 */
namespace EMultiBlockLocation
{
	enum Type
	{
		/** Default, either no other blocks in group or grouping style is disabled */
		None = -1,
		
		/** Denotes the beginning of a group, currently left most first */
		Start,

		/** Denotes a middle block(s) of a group */
		Middle,

		/** Denotes the end of a group, currently the right most */
		End,
	};

	/** returns the passed in style with the addition of the location information */
	static FName ToName(FName StyleName, Type InLocation)
	{
		switch(InLocation)
		{
		case Start:
			return ISlateStyle::Join(StyleName, ".Start");
		case Middle:
			return ISlateStyle::Join(StyleName, ".Middle");
		case End:
			return ISlateStyle::Join(StyleName, ".End");
		}
		return StyleName;
	}

	/**
	 * Trims a provided margin based on the block location. Grouped multiblocks want to be
	 * directly next to each other so that they can be presented as a single visual item.
	 * This function allows for defining a single "padding between items" value and have the
	 * appropriate sides of that padding zeroed out. 
	 * @param InBoxType The type of box the group is within. Used to determine horizontal/vertical flow.
	 * @param InLocation The location of the block within the group. 
	 * @param Margin The margin around a group or individual block.
	 * @return The margin for the block at the given location.
	 */
	static FMargin ToHorizontalMargin(EMultiBoxType InBoxType, Type InLocation, FMargin Margin)
	{
		if (InBoxType == EMultiBoxType::VerticalToolBar || InBoxType == EMultiBoxType::Menu)
		{
			// Vertical groups cut off the tops & bottoms
			switch (InLocation)
			{
			case Start:
				return FMargin(Margin.Left, Margin.Top, Margin.Right, 0.f);
			case Middle:
				return FMargin(Margin.Left, 0.f, Margin.Right, 0.f);
			case End:
				return FMargin(Margin.Left, 0.f, Margin.Right, Margin.Bottom);
			}
			return Margin;
		}
		
		// Horizontal groups cut off the left & right
		switch (InLocation)
		{
		case Start:
			return FMargin(Margin.Left, Margin.Top, 0.f, Margin.Bottom);
		case Middle:
			return FMargin(0.f, Margin.Top, 0.f, Margin.Bottom);
		case End:
			return FMargin(0.f, Margin.Top, Margin.Right, Margin.Bottom);
		}
		return Margin;
	}
}

/** Contains various Style parameters and overrides. Not all are applicable to a given entry */
struct FMenuEntryStyleParams
{
	// Workaround for clang deprecation warnings for deprecated members
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMenuEntryStyleParams() = default;
	FMenuEntryStyleParams(FMenuEntryStyleParams&&) = default;
	FMenuEntryStyleParams(const FMenuEntryStyleParams&) = default;
	FMenuEntryStyleParams& operator=(FMenuEntryStyleParams&&) = default;
	FMenuEntryStyleParams& operator=(const FMenuEntryStyleParams&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** If true, removes the padding from the left of the widget that lines it up with other menu items */
	bool bNoIndent = false;

	/** Horizontal alignment for this widget in its parent container. Note: only applies to toolbars */
	EHorizontalAlignment HorizontalAlignment = HAlign_Fill;

	/** (Optional) Vertical alignment for this widget in its parent container */
	TOptional<EVerticalAlignment> VerticalAlignment;

	/** (Optionally) override the size rule, where the default is generally Auto */
	TOptional<FSizeParam::ESizeRule> SizeRule;

	/** (Optionally) override the minimum size. This will apply to the width or height, depending on the menu orientation */
	UE_DEPRECATED(5.6, "Use MinimumSize instead")
	TOptional<float> MinSize;

	/** (Optionally) override the maximum size. This will apply to the width or height, depending on the menu orientation */
	UE_DEPRECATED(5.6, "Use MaximumSize instead")
	TOptional<float> MaxSize;

	/** (Optionally) set the desired width override */
	UE_DEPRECATED(5.6, "Use DesiredWidthOverride instead")
	TOptional<float> DesiredWidth;

	/** (Optionally) set the desired height override */
	UE_DEPRECATED(5.6, "Use DesiredHeightOverride instead")
	TOptional<float> DesiredHeight;

	/** (Optionally) override the fill proportion when the SizeRule is Stretch or StretchContent, defaults to 1.0f */
	TOptional<float> FillSize;

	/** (Optionally) override the minimum fill proportion when the SizeRule is StretchContent, defaults to FillSize/1.0f */
	TOptional<float> FillSizeMin;

	/** (Optionally) override the minimum size. This will apply to the width or height, depending on the menu orientation */
	TAttribute<float> MinimumSize;

	/** (Optionally) override the maximum size. This will apply to the width or height, depending on the menu orientation */
	TAttribute<float> MaximumSize;

	/** (Optionally) set the desired width override */
	TAttribute<FOptionalSize> DesiredWidthOverride;

	/** (Optionally) set the desired height override */
	TAttribute<FOptionalSize> DesiredHeightOverride;
};

struct FMenuEntryResizeParams
{
	// Keep default values separate from the attributes. This allows the attributes to act like unset optionals by default.
	static constexpr int32 DefaultClippingPriority = 0;
	static constexpr bool DefaultAllowClipping = true;
	static constexpr bool DefaultVisibleInOverflow = true;

	/** The priority of this entry during resizing (default is 0). A higher priority relative to other entries keeps
	 * the entry visible for longer as size is constrained during toolbar resizing. */
	TAttribute<int32> ClippingPriority;

	/** If true (default), allow this entry to be clipped during resizing of toolbars. If false, this entry will never
	 * be clipped and always stay in the toolbar. */
	TAttribute<bool> AllowClipping;

	/** If true (default), this entry will be visible in a toolbar overflow menu. If false, this entry will disappear when it overflows. */
	TAttribute<bool> VisibleInOverflow;

	struct FWrappingParams
	{
		// Keep default values separate from the attributes. This allows the attributes to act like unset optionals by default.
		static constexpr bool DefaultAllowWrapping = true;
		static constexpr int32 DefaultPriority = 0;
		static constexpr UE::Slate::PrioritizedWrapBox::EWrapMode DefaultMode = UE::Slate::PrioritizedWrapBox::EWrapMode::Parent;
		static constexpr UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior DefaultVerticalOverflowBehavior = UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior::Default;
		static constexpr bool DefaultForceNewLine = false;

		/** If true (default), allow this entry to be wrapped to the next line during resizing of toolbars. If false, this entry will never be wrapped. */
		TAttribute<bool> Allow;

		/** Override to specify a wrap priority, where a higher priority means the entry will be wrapped to the next line first. */
		TAttribute<int32> Priority;

		/** Override to specify the wrap mode. By default, this is "Preferred". */
		TAttribute<UE::Slate::PrioritizedWrapBox::EWrapMode> Mode;

		/** Override to specify the vertical overflow behavior. By default, this is "ClipOrExpandProportional". */
		TOptional<UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior> VerticalOverflowBehavior;

		/** If true (default is false), the entry should always be placed on a new line. Other entries can appear to it's right, but never to it's left. */
		TOptional<bool> ForceNewLine;
	} Wrapping;
};
