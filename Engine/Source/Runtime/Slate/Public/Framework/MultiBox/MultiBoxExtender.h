// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FExtensionBase;
class FMenuBarBuilder;
class FMenuBuilder;
class FToolBarBuilder;
class FUICommandList;

/** Called on your extension to add new pull-down menus to a menu bar */
DECLARE_DELEGATE_OneParam( FMenuBarExtensionDelegate, class FMenuBarBuilder& );

/** Called on your extension to add new menu items and sub-menus to a pull-down menu or context menu */
DECLARE_DELEGATE_OneParam( FMenuExtensionDelegate, class FMenuBuilder& );

/** Called on your extension to add new toolbar items to your toolbar */
DECLARE_DELEGATE_OneParam( FToolBarExtensionDelegate, class FToolBarBuilder& );


/** Where in relation to an extension hook should you apply your extension */
namespace EExtensionHook
{
	enum Position
	{
		/** Inserts the extension before the element or section. */
		Before,
		/** Inserts the extension after the element or section. */
		After,
		/** Sections only. Inserts the extension at the beginning of the section. */
		First,
	};
}


/** Forward declare FExtensionBase. This type is used as a dumb handle to allow you to remove a previously added extension; its implementation is deliberately hidden */
class FExtensionBase;


class FExtender
{
public:
	/**
	 * Extends a menu bar at the specified extension point
	 *
	 * @param	ExtensionHook			Part of the menu to extend.  You can extend the same point multiple times, and extensions will be applied in the order they were registered.
	 * @param	HookPosition			Where to apply hooks in relation to the extension hook
	 * @param	CommandList				The UI command list responsible for handling actions for the menu items you'll be extending the menu with
	 * @param	MenuExtensionDelegate	Called to populate the part of the menu you're extending
	 *
	 * @return	Pointer to the new extension object.  You can use this later to remove the extension.
	 */
	SLATE_API TSharedRef< const FExtensionBase > AddMenuBarExtension( FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr< FUICommandList >& CommandList, const FMenuBarExtensionDelegate& MenuBarExtensionDelegate );


	/**
	 * Extends a menu at the specified extension point
	 *
	 * @param	ExtensionHook			Part of the menu to extend.  You can extend the same point multiple times, and extensions will be applied in the order they were registered.
	 * @param	HookPosition			Where to apply hooks in relation to the extension hook
	 * @param	CommandList				The UI command list responsible for handling actions for the menu items you'll be extending the menu with
	 * @param	MenuExtensionDelegate	Called to populate the part of the menu you're extending
	 *
	 * @return	Pointer to the new extension object.  You can use this later to remove the extension.
	 */
	SLATE_API TSharedRef< const FExtensionBase > AddMenuExtension( FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr< FUICommandList >& CommandList, const FMenuExtensionDelegate& MenuExtensionDelegate );

	
	/**
	 * Extends a tool bar at the specified extension point
	 *
	 * @param	ExtensionHook			Part of the menu to extend.  You can extend the same point multiple times, and extensions will be applied in the order they were registered.
	 * @param	HookPosition			Where to apply hooks in relation to the extension hook
	 * @param	CommandList				The UI command list responsible for handling actions for the toolbar items you'll be extending the menu with
	 * @param	ToolbarExtensionDelegate	Called to populate the part of the toolbar you're extending
	 *
	 * @return	Pointer to the new extension object.  You can use this later to remove the extension.
	 */
	SLATE_API TSharedRef< const FExtensionBase > AddToolBarExtension( FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr< FUICommandList >& CommandList, const FToolBarExtensionDelegate& ToolBarExtensionDelegate );


	/**
	 * Removes an existing extension.
	 *
	 * @param	Extension	The extension to remove
	 */
	SLATE_API void RemoveExtension( const TSharedRef< const FExtensionBase >& Extension );

	/**
	 * Applies any menu bar extensions at the specified extension point
	 *
	 * @param	ExtensionHook		The point we're apply extensions to right now.  Only extensions that match this point will be applied.
	 * @param	HookPosition		Where in relation to the hook will extensions be applied to
	 * @param	MenuBarBuilder		The menu bar that will be modified with the extensions in-place
	 */
	SLATE_API void Apply( FName ExtensionHook, EExtensionHook::Position HookPosition, FMenuBarBuilder& MenuBarBuilder ) const;


	/**
	 * Applies any extensions at the specified extension point
	 *
	 * @param	ExtensionHook		The point we're apply extensions to right now.  Only extensions that match this point will be applied.
	 * @param	HookPosition		Where in relation to the hook will extensions be applied to
	 * @param	MenuBuilder			The menu that will be modified with the extensions in-place
	 */
	SLATE_API void Apply( FName ExtensionHook, EExtensionHook::Position HookPosition, FMenuBuilder& MenuBuilder ) const;
	

	/**
	 * Applies any extensions at the specified extension point
	 *
	 * @param	ExtensionHook		The point we're apply extensions to right now.  Only extensions that match this point will be applied.
	 * @param	HookPosition		Where in relation to the hook will extensions be applied to
	 * @param	ToolBarBuilder		The toolbar that will be modified with the extensions in-place
	 */
	SLATE_API void Apply( FName ExtensionHook, EExtensionHook::Position HookPosition, FToolBarBuilder& ToolBarBuilder ) const;


	/**
	 * Consolidates an array of FExtenders into a single FExtender
	 * 
	 * @param Extenders The array of FExtenders to consolidate
	 * @return A new FExtender pointer that has all of the extensions of the input
	 */
	static SLATE_API TSharedRef<FExtender> Combine(const TArray< TSharedPtr<FExtender> >& Extenders);

	/** Returns the number of extensions in the extender */
	int32 NumExtensions() const { return Extensions.Num(); }

private:

	/** List of extensions to our menu.  The order of the list must be maintained. */
	TArray< TSharedPtr< const FExtensionBase > > Extensions;
};
