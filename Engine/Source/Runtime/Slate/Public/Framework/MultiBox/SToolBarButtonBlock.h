// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

class SBorder;

/** Delegate used by multi-box to call a user function to populate a new menu.  Used for spawning sub-menus and pull-down menus. */
DECLARE_DELEGATE_OneParam( FNewMenuDelegate, class FMenuBuilder& );

/**  Delegate that takes the default toolbar button created by this as a parameter, adds any decorator needed and returns the new widget*/
DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<SWidget>, FGetDecoratedButtonDelegate, TSharedRef<SWidget> );

struct FButtonArgs : public TSharedFromThis<FButtonArgs>
{
	TSharedPtr< const FUICommandInfo > Command;
	TSharedPtr< const FUICommandList > CommandList;
	FName ExtensionHook;
	TAttribute<FText> LabelOverride;
	TAttribute<FText> ToolbarLabelOverride;
	TAttribute<FText> ToolTipOverride;
	TAttribute<FSlateIcon> IconOverride;
	FName TutorialHighlightName;
	FName BorderBrushName;
	EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::None;
	FUIAction Action;
	
	FNewMenuDelegate CustomMenuDelegate;
	FOnGetContent OnGetMenuContent;
	FGetDecoratedButtonDelegate GetDecoratedButtonDelegate;

	TAttribute<EVisibility> VisibilityOverride;

	explicit FButtonArgs() {}
};


/**
 * Tool bar button MultiBlock
 */
class FToolBarButtonBlock
	: public FMultiBlock
{

public:

	
	SLATE_API FToolBarButtonBlock( FButtonArgs ButtonArgs );

	/**
	 * Constructor
	 *
	 * @param	InCommand				The command associated with this tool bar button
	 * @param	InCommandList			The list of commands that are mapped to delegates so that we know what to execute for this button
	 * @param	InLabelOverride			Optional label override.  If omitted, then the action's label will be used instead.
	 * @param	InToolTipOverride		Optional tool tip override.	 If omitted, then the action's label will be used instead.
	 * @param	InIconOverride			Optional icon to use for the tool bar image.  If omitted, then the action's icon will be used instead.
	 * @param	InToolbarLabelOverride	Optional label to use when the block appears in a toolbar. If omitted, then the label override or command name will be used instead.
	 */
	SLATE_API FToolBarButtonBlock(
		const TSharedPtr<const FUICommandInfo> InCommand,
		TSharedPtr<const FUICommandList> InCommandList,
		const TAttribute<FText>& InLabelOverride = TAttribute<FText>(),
		const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(),
		const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(),
		TAttribute<FText> InToolbarLabelOverride = TAttribute<FText>()
	);

	/**
	 * Constructor
	 *
	 * @param	InLabel						The label to display in the menu
	 * @param	InToolTip					The tool tip to display when the menu entry is hovered over
	 * @param	InIcon						The icon to display to the left of the label
	 * @param	InUIAction					UI action to take when this menu item is clicked as well as to determine if
	 * the menu entry can be executed or appears "checked"
	 * @param	InUserInterfaceActionType	Type of interface action
	 * @param	InToolbarLabelOverride	Optional label to use when the block appears in a toolbar. If omitted, then the
	 * label override or command name will be used instead.
	 *
	 */
	SLATE_API FToolBarButtonBlock(
		const TAttribute<FText>& InLabel,
		const TAttribute<FText>& InToolTip,
		const TAttribute<FSlateIcon>& InIcon,
		const FUIAction& InUIAction,
		const EUserInterfaceActionType InUserInterfaceActionType,
		TAttribute<FText> InToolbarLabelOverride = TAttribute<FText>()
	);

	void SetLabelVisibility( EVisibility InLabelVisibility ) { LabelVisibility = InLabelVisibility ; }

	void SetIsFocusable( bool bInIsFocusable ) { bIsFocusable = bInIsFocusable; }

	/** Set whether this toolbar should always use small icons, regardless of the current settings */
	void SetForceSmallIcons( const bool InForceSmallIcons ) { bForceSmallIcons = InForceSmallIcons; }

	/** Set the border brush for this toolbar button block */
	void SetBorderBrushName( const FName InBorderBrushName ) { BorderBrushName = InBorderBrushName; }

	/** FMultiBlock interface */
	SLATE_API virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const override;
	SLATE_API virtual bool HasIcon() const override;

	/** Set optional delegate to customize when a menu appears instead of the widget, such as in toolbars */
	SLATE_API void SetCustomMenuDelegate(const FNewMenuDelegate& InOnFillMenuDelegate);
	
	SLATE_API void SetCustomWidget(const TSharedPtr<SWidget>& InCustomWidget);

	SLATE_API void SetOnGetMenuContent(const FOnGetContent& OnGetMenuContent);

	/**
	 *  Delegate that takes the default toolbar button created by this as a parameter, adds any decorator needed and returns the new widget
	 *
	 * @param InGetDecoratedButtonDelegate the delegate that handles decorating the button
	 */
	void SetGetDecoratedButtonDelegate( const FGetDecoratedButtonDelegate& InGetDecoratedButtonDelegate );

protected:
	
	SLATE_API bool GetIsFocusable() const;

private:

	/** FMultiBlock private interface */
	SLATE_API virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;

private:

	// Friend our corresponding widget class
	friend class SToolBarButtonBlock;

	/** Optional overridden text label for this tool bar button.  If not set, then the action's label will be used instead. */
	TAttribute<FText> LabelOverride;

	/** Optional overridden text label for when this tool bar button appears in a toolbar. If not set, then the label override or the action's label will be used instead. */
	TAttribute<FText> ToolbarLabelOverride;

	/** Optional overridden tool tip for this tool bar button.  If not set, then the action's tool tip will be used instead. */
	TAttribute<FText> ToolTipOverride;
	
	/** The name of the border brush style, if present */
	TAttribute<FName> BorderBrushName;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	TAttribute<FSlateIcon> IconOverride;

	TOptional< EVisibility > LabelVisibility;
	
	TSharedPtr<SWidget> CustomWidget;
	
	/** In the case where a command is not bound, the user interface action type to use.  If a command is bound, we
		simply use the action type associated with that command. */
	EUserInterfaceActionType UserInterfaceActionType;

	/** Whether ToolBar will have Focusable buttons */
	bool bIsFocusable;

	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;

	/** Optional delegate to customize when a menu appears instead of the widget, such as in toolbars */
	FNewMenuDelegate CustomMenuDelegate;

	/** Delegate to execute to get the menu content of this button */
	FOnGetContent OnGetMenuContent;

	/**  Delegate that takes the default toolbar button created by this as a parameter, adds any decorator needed and returns the new widget*/
	FGetDecoratedButtonDelegate GetDecoratedButtonDelegate;
};


template <>
struct TWidgetTypeTraits<class SToolBarButtonBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Tool bar button MultiBlock widget
 */
class SToolBarButtonBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SToolBarButtonBlock )
		: _IsFocusable(false)
		, _ForceSmallIcons(false)
		, _TutorialHighlightName(NAME_None)
		{}

		SLATE_ARGUMENT( TOptional<EVisibility>, LabelVisibility )
		SLATE_ARGUMENT( bool, IsFocusable )
		SLATE_ARGUMENT( bool, ForceSmallIcons )
		SLATE_ARGUMENT( FName, TutorialHighlightName )
	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

protected:

	/**
	 * Called by Slate when this tool bar button's button is clicked
	 */
	SLATE_API FReply OnClicked();
	
	/**
	 * Called by Slate when this tool bar check box button is toggled
	 */
	SLATE_API void OnCheckStateChanged( const ECheckBoxState NewCheckedState );

	/**
	 * Called by slate to determine if this button should appear checked
	 *
	 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
	 */
	SLATE_API ECheckBoxState GetCheckState() const;

	/**
	 * Called by Slate to determine if this button is enabled
	 * 
	 * @return True if the menu entry is enabled, false otherwise
	 */
	SLATE_API bool IsEnabled() const;

	/**
	 * Called by Slate to determine if this button is visible
	 *
	 * @return EVisibility::Visible or EVisibility::Collapsed, depending on if the button should be displayed
	 */
	SLATE_API EVisibility GetBlockVisibility() const;

private:
	/** Called by Slate to determine whether icons/labels are visible */
	EVisibility GetIconVisibility(bool bIsASmallIcon) const;

	/** Gets the icon brush for the toolbar block widget */
	const FSlateBrush* GetIconBrush() const;

	const FSlateBrush* GetOverlayIconBrush() const;

	/** Gets the icon brush for the toolbar block widget */
	const FSlateBrush* GetNormalIconBrush() const;
	const FSlateBrush* GetSmallIconBrush() const;

	FSlateColor GetIconForegroundColor() const;

	/** Gets the brush for the left side options block (the side with the clickable toolbar button */
	const FSlateBrush* GetOptionsBlockLeftBrush() const;

	/** Gets the brush for the right side options block (the side with the options dropdown */
	const FSlateBrush* GetOptionsBlockRightBrush() const;

	EVisibility GetOptionsSeparatorVisibility() const;
private:
	/** Overrides the visibility of the of label. This is used to set up the LabelVisibility attribute */
	TOptional<EVisibility> LabelVisibilityOverride;

	/** Controls the visibility of the of label, defaults to GetIconVisibility */
	TAttribute<EVisibility> LabelVisibility;

	TSharedPtr<SBorder> ButtonBorder;
	TSharedPtr<SBorder> OptionsBorder;

	/** Whether ToolBar will have Focusable buttons. */
	bool bIsFocusable;

	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;

	/** Name to identify a widget for tutorials */
	FName TutorialHighlightName;

};
