// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"

class SCheckBox;
class SComboButton;

/**
 * Tool bar combo button MultiBlock
 */
class FToolBarComboButtonBlock
	: public FMultiBlock
{

public:
	/**
	 * Constructor
	 *
	 * @param	InAction					UI action that sets the enabled state for this combo button
	 * @param	InMenuContentGenerator		Delegate that generates a widget for this combo button's menu content.
	 *										Called when the menu is summoned.
	 * @param	InLabel						Optional label for this combo button.
	 * @param	InToolTip					Tool tip string (required!)
	 * @param	InIcon						Optional icon to use for the tool bar image
	 * @param	bInSimpleComboBox			If true, the icon and label won't be displayed
	 * @param	InToolbarLabelOverride		Optional label to use when the block appears in a toolbar. If omitted, then the
	 *										label override or command name will be used instead.
	 * @param	InPlacementOverride			Optional override placement to customize menu placement once opened via e.g. a toolbar menu button.
	 * @param	InUserInterfaceActionType	Optional type of user interface action to use. If a command is bound, the user interface action type associated with the command will be used instead.
	 */
	FToolBarComboButtonBlock(
		const FUIAction& InAction,
		const FOnGetContent& InMenuContentGenerator,
		const TAttribute<FText>& InLabel = TAttribute<FText>(),
		const TAttribute<FText>& InToolTip = TAttribute<FText>(),
		const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(),
		bool bInSimpleComboBox = false,
		TAttribute<FText> InToolbarLabelOverride = TAttribute<FText>(),
		TAttribute<EMenuPlacement> InPlacementOverride = MenuPlacement_ComboBox,
		const EUserInterfaceActionType InUserInterfaceActionType = EUserInterfaceActionType::Button
	);

	/** FMultiBlock interface */
	virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const override;
	virtual bool HasIcon() const override;

	/** 
	 * Sets the visibility of the blocks label
	 *
	 * @param InLabelVisibility		Visibility setting to use for the label
	 */
	void SetLabelVisibility( EVisibility InLabelVisibility ) { LabelVisibility = InLabelVisibility; }

	/** Set whether this toolbar should always use small icons, regardless of the current settings */
	void SetForceSmallIcons( const bool InForceSmallIcons ) { bForceSmallIcons = InForceSmallIcons; }

	bool IsSimpleComboBox() const { return bSimpleComboBox; }

	void SetCustomWidget(const TSharedPtr<SWidget>& InCustomWidget) { CustomWidget = InCustomWidget; }

private:

	/**
	 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
	 *
	 * @return  MultiBlock widget object
	 */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;


private:

	// Friend our corresponding widget class
	friend class SToolBarComboButtonBlock;

	/** Delegate that generates a widget for this combo button's menu content.  Called when the menu is summoned. */
	FOnGetContent MenuContentGenerator;

	/** Optional overridden text label for this tool bar button.  If not set, then the action's label will be used instead. */
	TAttribute<FText> Label;

	/** Optional overridden text label for when this tool bar button appears in a toolbar. If not set, then the label override or the action's label will be used instead. */
	TAttribute<FText> ToolbarLabelOverride;

	/** Optional overridden tool tip for this tool bar button.  If not set, then the action's tool tip will be used instead. */
	TAttribute<FText> ToolTip;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	TAttribute<FSlateIcon> Icon;

	/** Optional overriden setting for handling placement */
	TAttribute<EMenuPlacement> PlacementOverride;

	/** Controls the Labels visibility, defaults to GetIconVisibility if no override is provided */
	TOptional< EVisibility > LabelVisibility;

	/** Optional custom widget to use instead of the standard button content */
	TSharedPtr<SWidget> CustomWidget;

	/** In the case where a command is not bound, the user interface action type to use. If a command is bound, we
		simply use the action type associated with that command. */
	EUserInterfaceActionType UserInterfaceActionType;

	/** If true, the icon and label won't be displayed */
	bool bSimpleComboBox;

	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;
};



template <>
struct TWidgetTypeTraits<class SToolBarComboButtonBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Tool bar button MultiBlock widget
 */
class SToolBarComboButtonBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SToolBarComboButtonBlock )
		: _ForceSmallIcons( false )
	{}

		/** Controls the visibility of the blocks label */
		SLATE_ARGUMENT( TOptional< EVisibility >, LabelVisibility )
		
		/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
		SLATE_ATTRIBUTE( FSlateIcon, Icon );

		/** Whether this toolbar should always use small icons, regardless of the current settings */
		SLATE_ARGUMENT( bool, ForceSmallIcons )

	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

protected:

	/**
	 * Called by Slate when content for this button's menu needs to be generated
	 *
	 * @return	The widget to use for the menu content
	 */
	SLATE_API TSharedRef<SWidget> OnGetMenuContent();

	/**
	 * Called by Slate when this tool bar button's button is clicked
	 */
	SLATE_API FReply OnClicked();

	/**
	 * Called by Slate when this tool bar check box button is toggled
	 */
	SLATE_API void OnCheckStateChanged(const ECheckBoxState NewCheckedState);

	/**
	 * Called by slate to determine if this button should appear checked
	 *
	 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
	 */
	SLATE_API ECheckBoxState GetCheckState() const;

	/**
	 * Called by Slate to determine if this button is enabled
	 * 
	 * @return True if the menu entry is enabled, false otherwise
	 */
	SLATE_API bool IsEnabled() const;

	/** True if we have an action bound */
	bool HasAction() const;
	
	/** True if we have IsChecked bound */ 
	bool HasCheckedState() const;

	/**
	 * @return Gets the effective checkbox style for this widget. 
	 */
	const FCheckBoxStyle* GetCheckBoxStyle(const ISlateStyle* StyleSet, const FName& StyleName, bool bIsSplitButton) const;
	
	/**
	 * Called by Slate to determine if this button is visible
	 *
	 * @return EVisibility::Visible or EVisibility::Collapsed, depending on if the button should be displayed
	 */
	SLATE_API EVisibility GetVisibility() const;
private:
	/** @return True if this toolbar button is using a dynamically set icon */
	bool HasDynamicIcon() const;

	/** Gets the icon brush for the toolbar block widget */
	const FSlateBrush* GetIconBrush() const;

	/** @return The icon for the toolbar button; may be dynamic, so check HasDynamicIcon */
	const FSlateBrush* GetNormalIconBrush() const;

	/** @return The small icon for the toolbar button; may be dynamic, so check HasDynamicIcon */
	const FSlateBrush* GetSmallIconBrush() const;

	/** Called by Slate to determine whether icons/labels are visible */
	EVisibility GetIconVisibility(bool bIsASmallIcon) const;

	FSlateColor GetIconForegroundColor() const;

	const FSlateBrush* GetOverlayIconBrush() const;

	FSlateColor OnGetForegroundColor() const;

	const FSlateBrush* GetBorderImage() const;

private:
	/** Overrides the visibility of the of label. This is used to set up the LabelVisibility attribute */
	TOptional<EVisibility> LabelVisibilityOverride;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	TAttribute<FSlateIcon> Icon;
	
	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;

	TSharedPtr<SWidget> LeftHandSideWidget; // Widget that comes before the combo button
	TSharedPtr<SComboButton> ComboButtonWidget;

	/** The foreground color for button when the combo button is open */
	FSlateColor OpenForegroundColor;

	/** The checkbox style to be used for simulating checkbox foreground color. */
	const FCheckBoxStyle* CheckBoxStyle = nullptr;

	/** The hovered style for the entire block used when the block has multiple widgets */
	const FSlateBrush* BlockHovered = nullptr;
};
