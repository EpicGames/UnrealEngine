// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/Commands/Contexts/UIIdentifierContext.h"
#include "Framework/Commands/Contexts/UIContentContext.h"

class SCheckBoxStack;

/**
 * Tool bar combo button MultiBlock, but toggleable. Right & long-press clicks spawn menu.
 * Designed to hold a stack of tools in one block.
 */
class FToolBarStackButtonBlock
	: public FMultiBlock
{

public:

	/**
	 * Constructor
	 *
	 * @param	InCommand			The command associated with this tool bar button
	 * @param	InCommandList		The list of commands that are mapped to delegates so that we know what to execute for this button
	 * @param	bInSimpleStackBox	If true, the icon and label won't be displayed
	 */
	FToolBarStackButtonBlock(const TSharedRef< const FUICommandInfo > InCommand, TSharedPtr< const FUICommandList > InCommandList, bool bInSimpleStackBox = false);

	/** FMultiBlock interface */
	virtual void CreateMenuEntry(class FMenuBuilder& MenuBuilder) const override;

	/** 
	 * Sets the visibility of the blocks label
	 *
	 * @param InLabelVisibility		Visibility setting to use for the label
	 */
	void SetLabelVisibility( EVisibility InLabelVisibility ) { LabelVisibility = InLabelVisibility; }

	/** Set whether this toolbar should always use small icons, regardless of the current settings */
	void SetForceSmallIcons( const bool InForceSmallIcons ) { bForceSmallIcons = InForceSmallIcons; }
protected:

	/**
	 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
	 *
	 * @return  MultiBlock widget object
	 */
	virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;

	/** Provides dynamic, icons, labels, & descriptions for this stack, caches on get */
	const TSharedPtr<FUIIdentifierContext> GetStackIdentifier() const;

	/** Provides context menu content stack, caches on get */
	const TSharedPtr<FUIContentContext> GetStackContent() const;

protected:

	// Friend our corresponding widget class
	friend class SToolBarStackButtonBlock;

	/** Delegate that generates a widget for this combo button's menu content.  Called when the menu is summoned. */
	FOnGetContent MenuContentGenerator;

	/** Controls the Labels visibility, defaults to GetIconVisibility if no override is provided */
	TOptional< EVisibility > LabelVisibility;

	/** If true, the icon and label won't be displayed */
	bool bSimpleComboBox;

	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;

	/** To avoid re-searching the FUICommandList, we cache the last identifier context */
	mutable TWeakPtr<FUIIdentifierContext> CachedStackIdentifier;

	/** To avoid re-searching the FUICommandList, we cache the last content context */
	mutable TWeakPtr<FUIContentContext> CachedStackContent;
};



template <>
struct TWidgetTypeTraits<class SToolBarStackButtonBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Tool bar button MultiBlock widget
 */
class SToolBarStackButtonBlock
	: public SMultiBlockBaseWidget
{

public:

	SLATE_BEGIN_ARGS( SToolBarStackButtonBlock )
		: _ForceSmallIcons( false )
	{}

		/** Controls the visibility of the blocks label */
		SLATE_ARGUMENT( TOptional< EVisibility >, LabelVisibility )

		/** Whether this toolbar should always use small icons, regardless of the current settings */
		SLATE_ARGUMENT( bool, ForceSmallIcons )

	SLATE_END_ARGS()


	/**
	 * Builds this MultiBlock widget up from the MultiBlock associated with it
	 */
	SLATE_API virtual void BuildMultiBlockWidget(const ISlateStyle* StyleSet, const FName& StyleName) override;


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

protected:

	/**
	 * Called by Slate when this tool bar check box button is toggled
	 */
	SLATE_API void OnCheckStateChanged(const ECheckBoxState NewCheckedState);

	/**
	 * Called by slate to determine if this button should appear checked
	 *
	 * @return ECheckBoxState::Checked if it should be checked, ECheckBoxState::Unchecked if not.
	 */
	SLATE_API ECheckBoxState GetCheckState() const;

	/**
	 * Called by Slate when content for this button's menu needs to be generated
	 *
	 * @return	The widget to use for the menu content
	 */
	SLATE_API TSharedRef<SWidget> OnGetMenuContent();

	/**
	 * Called by Slate to determine if this button is enabled
	 * 
	 * @return True if the menu entry is enabled, false otherwise
	 */
	SLATE_API bool IsEnabled() const;

	/**
	 * Called by Slate to determine if this button is visible
	 *
	 * @return EVisibility::Visible or EVisibility::Collapsed, depending on if the button should be displayed
	 */
	SLATE_API EVisibility GetVisibility() const;
protected:

	/** Gets the icon brush for the toolbar block widget */
	SLATE_API const FSlateBrush* GetIconBrush() const;

	/** Gets the label for the toolbar block widget */
	SLATE_API FText GetLabel() const;

	/** Gets the description for the toolbar block widget */
	SLATE_API FText GetDescription() const;

	/** @return The icon for the toolbar button; may be dynamic, so check HasDynamicIcon */
	SLATE_API const FSlateBrush* GetNormalIconBrush() const;

	/** @return The small icon for the toolbar button; may be dynamic, so check HasDynamicIcon */
	SLATE_API const FSlateBrush* GetSmallIconBrush() const;

	/** Called by Slate to determine whether icons/labels are visible */
	SLATE_API EVisibility GetIconVisibility(bool bIsASmallIcon) const;

	SLATE_API FSlateColor GetIconForegroundColor() const;

	SLATE_API const FSlateBrush* GetOverlayIconBrush() const;

protected:
	/** Overrides the visibility of the of label. This is used to set up the LabelVisibility attribute */
	TOptional<EVisibility> LabelVisibilityOverride;

	/** Controls the visibility of the of label, defaults to GetIconVisibility */
	TAttribute< EVisibility > LabelVisibility;

	/** Optional overridden icon for this tool bar button.  IF not set, then the action's icon will be used instead. */
	TAttribute<FSlateIcon> Icon;

	TSharedPtr<SCheckBoxStack> StackButtonWidget;

	/** The foreground color for button when the combo button is open */
	FSlateColor OpenForegroundColor;

	/** Whether this toolbar should always use small icons, regardless of the current settings */
	bool bForceSmallIcons;
};
