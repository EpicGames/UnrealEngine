// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/SToolBarButtonBlock.h"

struct FButtonArgs;
/**
 * Horizontal Button Row MultiBlock
 */
class FUniformToolbarButtonBlock
	: public FToolBarButtonBlock
{

public:

	/**
	 * Constructor
	 *
	 * @param	ButtonArgs The Parameters object which will provide the data to initialize the button
	 */
	SLATE_API FUniformToolbarButtonBlock( FButtonArgs ButtonArgs);

	
	/**
	 * Allocates a widget for this type of MultiBlock.  Override this in derived classes.
	 *
	 * @return  MultiBlock widget object
	 */
	 SLATE_API virtual TSharedRef< class IMultiBlockBaseWidget > ConstructWidget() const override;

};

class SUniformToolbarButtonBlock
	: public SToolBarButtonBlock
{
	
};

