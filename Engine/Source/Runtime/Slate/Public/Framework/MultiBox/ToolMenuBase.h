// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/NamePermissionList.h"

#include "ToolMenuBase.generated.h"

class FMultiBox;

UENUM()
enum class ECustomizedToolMenuVisibility
{
	None,
	Visible,
	Hidden
};

USTRUCT()
struct FCustomizedToolMenuEntry
{
	GENERATED_BODY()

	FCustomizedToolMenuEntry() :
		Visibility(ECustomizedToolMenuVisibility::None)
	{
	}

	UPROPERTY()
	ECustomizedToolMenuVisibility Visibility;
};

USTRUCT()
struct FCustomizedToolMenuSection
{
	GENERATED_BODY()

	FCustomizedToolMenuSection() :
		Visibility(ECustomizedToolMenuVisibility::None)
	{
	}

	UPROPERTY()
	ECustomizedToolMenuVisibility Visibility;
};

USTRUCT()
struct FCustomizedToolMenuNameArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FName> Names;
};

/*
 * A menu profile is a way for systems to modify instances of a menu by showing/hiding specific items. You can have multiple profiles active on
 * a single menu at the same time.
 */ 
USTRUCT()
struct FToolMenuProfile
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	TMap<FName, FCustomizedToolMenuEntry> Entries;

	UPROPERTY()
	TMap<FName, FCustomizedToolMenuSection> Sections;

	UPROPERTY()
	TArray<FName> SuppressExtenders;

	FNamePermissionList MenuPermissions;

	SLATE_API FCustomizedToolMenuEntry* FindEntry(const FName InEntryName);
	SLATE_API const FCustomizedToolMenuEntry* FindEntry(const FName InEntryName) const;
	SLATE_API FCustomizedToolMenuEntry* AddEntry(const FName InEntryName);
	SLATE_API ECustomizedToolMenuVisibility GetEntryVisiblity(const FName InSectionName) const;
	SLATE_API bool IsEntryHidden(const FName InEntryName) const;

	SLATE_API FCustomizedToolMenuSection* FindSection(const FName InSectionName);
	SLATE_API const FCustomizedToolMenuSection* FindSection(const FName InSectionName) const;
	SLATE_API FCustomizedToolMenuSection* AddSection(const FName InSectionName);
	SLATE_API ECustomizedToolMenuVisibility GetSectionVisiblity(const FName InSectionName) const;
	SLATE_API bool IsSectionHidden(const FName InSectionName) const;

	SLATE_API void SetSuppressExtenders(const FName InOwnerName, const bool bInSuppress);
	SLATE_API bool IsSuppressExtenders() const;

};

/*
 * A menu customization is a specialization of menu profiles - that allows for advanced behavior such as modifying the order of sections/entries
 * A menu can only have one customization active at a time
 */
USTRUCT()
struct FCustomizedToolMenu : public FToolMenuProfile
{
	GENERATED_BODY()

	SLATE_API FName GetEntrySectionName(const FName InEntryName) const;
	
	UPROPERTY()
	TMap<FName, FCustomizedToolMenuNameArray> EntryOrder;

	UPROPERTY()
	TArray<FName> SectionOrder;
};

/*
 * Structure to describe a menu profile for the whole hierarchy of a menu
 */
struct FToolMenuProfileHierarchy
{
	SLATE_API FToolMenuProfile GenerateFlattenedMenuProfile() const;

	TArray<const FToolMenuProfile*> ProfileHierarchy;
	TArray<const FToolMenuProfile*> RuntimeProfileHierarchy;
}; 

/*
 * Structure to describe the menu customization for the whole hierarchy of a menu
 */
struct FCustomizedToolMenuHierarchy : public FToolMenuProfileHierarchy
{
	SLATE_API FName GetEntrySectionName(const FName InEntryName) const;
	SLATE_API bool IsEntryHidden(const FName InEntryName) const;
	SLATE_API bool IsSectionHidden(const FName InSectionName) const;
	SLATE_API bool IsSuppressExtenders() const;
	SLATE_API FCustomizedToolMenu GenerateFlattened() const;

	TArray<const FCustomizedToolMenu*> Hierarchy;
	TArray<const FCustomizedToolMenu*> RuntimeHierarchy;
};

UCLASS(Abstract, MinimalAPI)
class UToolMenuBase : public UObject
{
	GENERATED_BODY()

public:

	virtual bool IsEditing() const { return false; }
	virtual FName GetSectionName(const FName InEntryName) const { return NAME_None; }

	virtual bool ContainsSection(const FName InName) const { return false; }
	virtual bool ContainsEntry(const FName InName) const { return false; }

	virtual FCustomizedToolMenu* FindMenuCustomization() const { return nullptr; }
	virtual FCustomizedToolMenu* AddMenuCustomization() const { return nullptr; }
	virtual FCustomizedToolMenuHierarchy GetMenuCustomizationHierarchy() const { return FCustomizedToolMenuHierarchy(); }

	virtual FToolMenuProfile* FindMenuProfile(const FName& ProfileName) const { return nullptr; }
	virtual FToolMenuProfile* AddMenuProfile(const FName& ProfileName) const { return nullptr; }
	virtual FToolMenuProfileHierarchy GetMenuProfileHierarchy(const FName& ProfileName) const { return FToolMenuProfileHierarchy(); }

	virtual void UpdateMenuCustomizationFromMultibox(const TSharedRef<const FMultiBox>& InMultiBox) {}
	virtual void OnMenuDestroyed() {}
};

