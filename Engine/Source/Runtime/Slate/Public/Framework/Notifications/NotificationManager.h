// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Containers/LockFreeList.h"
#include "Layout/SlateRect.h"
#include "Widgets/SWindow.h"

#include "HAL/IConsoleManager.h"
#include "Misc/CoreAsyncTaskNotificationImpl.h"

struct FNotificationInfo;

template <class T> class TLockFreePointerListLIFO;

/** 
 * Handle to an active progress notification.
 * Used to update the notification
 */
struct FProgressNotificationHandle
{
	friend class FSlateNotificationManager;

	FProgressNotificationHandle()
		: Id(INDEX_NONE)
	{}

	void Reset() { Id = INDEX_NONE; }
	bool IsValid() const { return Id != INDEX_NONE; }

	bool operator==(const FProgressNotificationHandle& OtherHandle) const
	{
		return Id == OtherHandle.Id;
	}
private:
	FProgressNotificationHandle(int32 InId)
		: Id(InId)
	{}

	int32 Id;
};

/** Base class for any handlers that display progress bars for progres notifications */
class IProgressNotificationHandler
{
public:
	virtual ~IProgressNotificationHandler() = default;

	/**
	 * Called when a progress notification begins
 	 * @param Handle			Handle to the notification
	 * @param DisplayText		Display text used to describe the type of work to the user
	 * @param TotalWorkToDo		Arbitrary number of work units to perform. 
	 */
	virtual void StartProgressNotification(FProgressNotificationHandle Handle, FText DisplayText, int32 TotalWorkToDo) = 0;

	/**
	 * Called when a notification should be updated. 
	 * @param InHandle				Handle to the notification that was previously created with StartProgressNotification
	 * @param TotalWorkDone			The total number of work units done for the notification.
 	 * @param UpdatedTotalWorkToDo	UpdatedTotalWorkToDo. This value will be 0 if the total work did not change
	 * @param UpdatedDisplayText	Updated display text of the notification. This value will be empty if the text did not change
	 */
	virtual void UpdateProgressNotification(FProgressNotificationHandle Handle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo, FText UpdatedDisplayText) = 0;

	/**
	 * Called when a notification should be cancelled
	 */
	virtual void CancelProgressNotification(FProgressNotificationHandle Handle) = 0;
};
/**
 * A class which manages a group of notification windows                 
 */
class FSlateNotificationManager
{
	friend class SNotificationList;

public:
	/**
	 * Gets the instance of this manager                   
	 */
	static SLATE_API FSlateNotificationManager& Get();

	/** Update the manager */
	SLATE_API void Tick();

	/** Provide a window under which all notifications should nest. */
	SLATE_API void SetRootWindow( const TSharedRef<SWindow> InRootWindow );

	/**
	 * Adds a floating notification
	 * @param Info 		Contains various settings used to initialize the notification
	 */
	SLATE_API TSharedPtr<SNotificationItem> AddNotification(const FNotificationInfo& Info);

	/**
	 * Thread safe method of queuing a notification for presentation on the next tick
	 * @param Info 		Pointer to heap allocated notification info. Released by FSlateNotificationManager once the notification is displayed;
	 */
	SLATE_API void QueueNotification(FNotificationInfo* Info);

	/**
	 * Begin a progress notification. These notifications should be used for background work not blocking work. Use SlowTask for blocking work
	 * @param DisplayText		Display text used to describe the type of work to the user
	 * @param TotalWorkToDo		Arbitrary number of work units to perform. If more than one progress notification is active this work is summed with any other active notifications. 
	 *				This value usually represents the number of steps to be completed
 	 *
	 * @return NotificationHandle	A handle that can be used to update the progress or cancel the notification
	 */
	SLATE_API FProgressNotificationHandle StartProgressNotification(FText DisplayText, int32 TotalWorkToDo);

	/**
	 * Updates a progress notification. 
	 * @param InHandle		Handle to the notification that was previously created with StartProgressNotification
	 * @param TotalWorkDone		The total number of work uits done for the notification. This is NOT an incremental value. 
 	 * @param UpdatedTotalWorkToDo	(Optional) If the total work to do changes you can update the value here. Note that if the total work left increases but the total work done does not, the progress bar may go backwards
	 * @param UpdatedDisplayText	(Optional) Updated display text of the notification
	 * @return NotificationHandle	A handle that can be used to update the progress or cancel the notification
	 */
	SLATE_API void UpdateProgressNotification(FProgressNotificationHandle InHandle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo = 0, FText UpdatedDisplayText = FText::GetEmpty());

	/**
	 * Cancels an active notification
	 */
	SLATE_API void CancelProgressNotification(FProgressNotificationHandle InHandle);

	/**
	 * Sets the progress notification handler for the current application. Only one handler is used at a time. 
	 * This handler is not managed in any way. If your handler is being destroyed call SetProgressNotificationHandler(nullptr)
	 */
	SLATE_API void SetProgressNotificationHandler(IProgressNotificationHandler* NewHandler);

	/**
	 * Called back from the SlateApplication when a window is activated/resized
	 * We need to keep notifications topmost in the z-order so we manage it here directly
	 * as there isn't a cross-platform OS-level way of making a 'topmost child'.
	 * @param ActivateEvent 	Information about the activation event
	 */
	SLATE_API void ForceNotificationsInFront(const TSharedRef<SWindow> InWindow);

	/**
	 * Gets all the windows that represent notifications
	 * @returns an array of windows that contain notifications.
	 */
	SLATE_API void GetWindows(TArray< TSharedRef<SWindow> >& OutWindows) const;

	/**
	 * Sets whether notifications should be displayed at all. Note, notifications can be
	 * disabled via console variable Slate.bAllowNotificationWidget.
	 *
	 * @param	bShouldAllow	Whether notifications should be enabled.  It defaults to on.
	 */
	void SetAllowNotifications( const bool bShouldAllow )
	{
		this->bAllowNotifications = bShouldAllow;
	}

	/**
	 * Checks whether notifications are currently enabled.
	 *
	 *@return	true if allowed, false otherwise.
	 */
	bool AreNotificationsAllowed() const
	{
		return this->bAllowNotifications;
	}

	/**
	 * Register a Staged Async Notification, allowing the NotificationManager to keep a reference to it
	 */
	SLATE_API void RegisterStagedNotification(TSharedPtr<IAsyncTaskNotificationImpl> InNotification);

	/**
	 * Unregister a previously added Staged Notification
	 */
	SLATE_API void UnregisterStagedNotification(TSharedPtr<IAsyncTaskNotificationImpl> InNotification);

protected:
	/** Protect constructor as this is a singleton */
	SLATE_API FSlateNotificationManager();

	/** Create a notification list for the specified screen rectangle */
	SLATE_API TSharedRef<SNotificationList> CreateStackForArea(const FSlateRect& InRectangle, TSharedPtr<SWindow> Window);

	/** FCoreDelegates::OnPreExit shutdown callback */
	SLATE_API void ShutdownOnPreExit();

private:

	/** A list of notifications, bound to a particular region */
	struct FRegionalNotificationList
	{
		/** Constructor */
		FRegionalNotificationList(const FSlateRect& InRectangle);

		/** Arranges the notifications in a stack */
		void Arrange();

		/** Remove any dead notifications */
		void RemoveDeadNotifications();

		/** The notification list itself (one per notification) */
		TArray<TSharedRef<SNotificationList>> Notifications;

		/** The rectangle we use to determine the anchor point for this list */
		FSlateRect Region;

		/** When notifications must be parented (pure Wayland), the window this list is anchored to;
		 *  Arrange() positions toasts in its lower-right client area instead of the monitor work area. */
		TWeakPtr<SWindow> AnchorWindow;
	};

	/** Handle to a system which updates progress notifications (e.g a status bar) */
	IProgressNotificationHandler* ProgressNotificationHandler = nullptr;

	/** A window under which all of the notification windows will nest. */
	TWeakPtr<SWindow> RootWindowPtr;

	/** An array of notification lists grouped by work area regions */
	TArray<FRegionalNotificationList> RegionalLists;

	/** Thread safe queue of notifications to display */
	TLockFreePointerListLIFO<FNotificationInfo> PendingNotifications;

	/** An array of staged notifications */
	TArray<TSharedPtr<IAsyncTaskNotificationImpl>> StagedNotifications;

	/** Critical Section to guard access to StagedNotifications */
	FCriticalSection StagedNotificationCS;

	/** Counter used to create progress handles */
	static SLATE_API int32 ProgressHandleCounter;

	/** Whether notifications should be displayed or not.  This can be used to globally suppress notification pop-ups */
	bool bAllowNotifications = true;

	/** CVar allowing us to control the display of notifications. */
	FAutoConsoleVariableRef CVarAllowNotifications;
	
	// On Linux there is a hack in SDL to already delayed the showing of notification that is not interacting well with this optimization.
	static constexpr bool bShowNotificationImmediately = PLATFORM_LINUX;
};
