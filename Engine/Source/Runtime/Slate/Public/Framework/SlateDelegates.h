// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"

class ITableRow;

/** Notification when user clicks outside a specified region. */
DECLARE_DELEGATE(FOnClickedOutside)


/**
 * A delegate that is invoked when widgets want to notify a user that they have been clicked.
 * Intended for use by buttons and other button-like widgets.
 */
DECLARE_DELEGATE_RetVal( 
	FReply, 
	FOnClicked )

/** Allows for loose coupling for OnDragDetect event handling. */
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDragDetected,
	const FGeometry&,
	const FPointerEvent&)


/** Allows for loose coupling for OnDragEnter event handling. */
DECLARE_DELEGATE_TwoParams(FOnDragEnter,
	const FGeometry&,
	const FDragDropEvent&)

/** Allows for loose coupling for OnDragLeave event handling. */
DECLARE_DELEGATE_OneParam(FOnDragLeave,
	const FDragDropEvent&)

/** Allows for loose coupling for OnDragOver event handling. */
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDragOver,
	const FGeometry&,
	const FDragDropEvent&)

/** Allows for loose coupling for OnDrop event handling. */
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDrop,
	const FGeometry&,
	const FDragDropEvent&)


/** Delegate type for handling OnDrop events */
DECLARE_DELEGATE_RetVal_TwoParams(
FReply, FDropEventHandler,
	/** The geometry of the widget*/
	const FGeometry&,
	/** The Mouse Event that we are processing */
	const FDragDropEvent& )

/** Delegate type for handling OnDrop events */
DECLARE_DELEGATE_TwoParams(
	FDragEventHandler,
	/** The geometry of the widget*/
	const FGeometry&,
	/** The Mouse Event that we are processing */
	const FDragDropEvent& )

/**
 * Sometimes widgets ask for content to display; at those times they rely on this delegate.
 * For example, the content of a popup is usually driven by code, so it is usually not known
 * until the popup is opening. At that time, OnGetContent is invoked.
 */
DECLARE_DELEGATE_RetVal(
	/** return a widget */
	TSharedRef<SWidget>,
	FOnGetContent
)

/**
 * Delegate to call before a context menu is opened.
 * The user returns the menu content to display or null if a context menu should not be opened 
 */
DECLARE_DELEGATE_RetVal(
	/** return a widget */
	TSharedPtr<SWidget>,
	FOnContextMenuOpening )

/** Delegate for hooking up to an inline editable text block 'IsSelected' check. */
DECLARE_DELEGATE_RetVal( bool, FIsSelected );

/** Delegate for hooking up to an editable text box's 'OnBeginTextEdit' */
DECLARE_DELEGATE_OneParam(FOnBeginTextEdit, const FText&)

/** Delegate for hooking up to an editable text box's 'OnTextChanged' */
DECLARE_DELEGATE_OneParam( FOnTextChanged, const FText& );

/** Delegate for validating typed-in characters in SEditableText. Only invoked for typed characters */
DECLARE_DELEGATE_RetVal_OneParam( bool, FOnIsTypedCharValid, const TCHAR /*TypedCharacter*/ );


/** Delegate for hooking up to an editable text box's 'OnTextCommitted' 

	The first parameter (NewText) is the new text string.

	The second parameter contains information about how the text was committed.
*/
DECLARE_DELEGATE_TwoParams( FOnTextCommitted, const FText&, ETextCommit::Type );

/** Notification for float value change */
DECLARE_DELEGATE_OneParam( FOnFloatValueChanged, float );

/** Notification for int32 value change */
DECLARE_DELEGATE_OneParam( FOnInt32ValueChanged, int32 );

/** Notification for bool value change */
DECLARE_DELEGATE_OneParam( FOnBooleanValueChanged, bool );

/** Notification for float value committed */
DECLARE_DELEGATE_TwoParams( FOnFloatValueCommitted, float, ETextCommit::Type);

/** Notification for int32 value committed */
DECLARE_DELEGATE_TwoParams( FOnInt32ValueCommitted, int32, ETextCommit::Type);

/** Notification for FLinearColor value change */
DECLARE_DELEGATE_OneParam( FOnLinearColorValueChanged, FLinearColor )

/** Notification for FVector4 value change */
DECLARE_DELEGATE_OneParam(FOnVector4ValueChanged, FVector4&)

/** Invoked when a list view has reached its target offset.  */
DECLARE_DELEGATE(FOnFinishedScrolling);

/** Invoked when the touch has started.  */
DECLARE_DELEGATE(FOnTouchStart);

/** Invoked when the touch has moved.  */
DECLARE_DELEGATE(FOnTouchMove);

/** Invoked when the touch has ended.  */
DECLARE_DELEGATE(FOnTouchEnd);

DECLARE_DELEGATE(FOnTableViewBadState);

template< typename ArgumentType >
class TSlateDelegates
{
public:
	/**
	 * Useful for debugging issues with table views.
	 */
	DECLARE_DELEGATE_RetVal_OneParam (
	/** return: String representation of the item for debugging purposes. */
	FString,
		FOnItemToString_Debug,
		/** param: An item to turn into a string for debug purposes. */
		ArgumentType);


	/** A delegate type for OnGenerateWidget handler. Given a data item, the handler should return a Widget visualizing that item */
	DECLARE_DELEGATE_RetVal_OneParam (
	/** return: The Widget visualization of the item */
	TSharedRef<SWidget>,
		FOnGenerateWidget,
		/** param: An item to visualize */
		ArgumentType );

	/**
	 * The advanced version of OnGenerateWidget.
	 * You are given the reference to the owning list/tree and asked to return the appropriate container widget.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams (
	/** return: The Widget visualization of the item */
	TSharedRef<class ITableRow>,
		FOnGenerateRow,
		/** param: An item to visualize */
		ArgumentType,
		/** param: The owning widget */
		const TSharedRef< class STableViewBase >& );

	/** A delegate to be invoked to update an existing item (or its containing widget) when the owning view is invalidated. */
	DECLARE_DELEGATE_OneParam(
		FOnRefreshRow,
		/** param: An item to update */
		ArgumentType);

	/** A delegate to be invoked when an item has come into view after it was requested to come into view. */
	DECLARE_DELEGATE_TwoParams (
		FOnItemScrolledIntoView,
		/** param: The item which just scrolled into view after it was requested. */
		ArgumentType,
		/** param: The widget representing the item. */
		const TSharedPtr<ITableRow>&
	)

	/** A delegate for the OnGetChildren handler. Given a data item, populate an output array with its children if it has any. */
	DECLARE_DELEGATE_TwoParams (
		FOnGetChildren,
		/** params: A data item and an array to populate with the item's children. */
		ArgumentType, TArray<ArgumentType>& );

	/** A delegate for the OnSetExpansionRecursive handler. Given a data item, recursively expand/collapse its children. */
	DECLARE_DELEGATE_TwoParams (
		FOnSetExpansionRecursive,
		/** params: A data item and whether we should be expanding or collapsing. */
		ArgumentType, bool );

	/** A delegate type invoked when a selection changes somewhere. */
	DECLARE_DELEGATE_TwoParams(
		FOnSelectionChanged,
		/** param: The newly selected value */
		ArgumentType,
		ESelectInfo::Type
	)

	/** A delegate type to control if a item is selectable or navigable. */
	DECLARE_DELEGATE_RetVal_OneParam(
		bool,
		FIsSelectableOrNavigable,
		/** param: The newly selected value */
		ArgumentType
	)

	DECLARE_DELEGATE_TwoParams(
		FOnExpansionChanged,
		ArgumentType,
		bool
	)

	/** Called when the user clicks on an item in a tree or list */
	DECLARE_DELEGATE_OneParam(
		FOnMouseButtonClick,
			/** param: The item that was double-clicked on */
			ArgumentType
			)

	/** Called when the user double-clicks on an item in a tree or list */
	DECLARE_DELEGATE_OneParam(
		FOnMouseButtonDoubleClick,
		/** param: The item that was double-clicked on */
		ArgumentType
	)

	/** Invoked when someone clicks on a hyperlink. */
	DECLARE_DELEGATE_OneParam(
		FOnNavigate,
		/** param: The payload stored in the hyperlink; e.g. a URL */
		const ArgumentType&
	)
};


/** Notification for when a key event occurs */
DECLARE_DELEGATE_RetVal_OneParam( FReply, FOnKeyEvent, const FKeyEvent& )

/** A Delegate for passing along a string of a source code location to access */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FAccessSourceCode, const FString& /*FileName*/, int32 /*InLineNumber*/, int32 /*InColumnNumber*/);

/** A Delegate for an asset object to access */
DECLARE_DELEGATE_RetVal_OneParam(bool, FAccessAsset, UObject* /*InAsset*/);

/** A Delegate for an debug object to access */
DECLARE_DELEGATE_RetVal_OneParam(bool, FAccessDebugObject, UObject* /*InObject*/);

/** A delegate for a callback during OnKeyDown from an SWidget */
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnKeyDown, const FGeometry&, const FKeyEvent&);

/** A delegate for a callback during OnKeyChar from an SWidget */
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnKeyChar, const FGeometry&, const FCharacterEvent&);

/** A delegate for a callback whenever text is changed programmatically or interactively by the user, and has to be verified */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnVerifyTextChanged, const FText&, FText&)