// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateTypes.h"
#include "CheckBoxWidgetStyle.generated.h"

/**
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UCheckBoxWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:
	/** The actual data describing the button's appearance. */
	UPROPERTY(Category=Appearance, EditAnywhere, meta=(ShowOnlyInnerProperties))
	FCheckBoxStyle CheckBoxStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast< const struct FSlateWidgetStyle* >( &CheckBoxStyle );
	}
};
