// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateTypes.h"
#include "ProgressWidgetStyle.generated.h"

/**
 */
UCLASS(BlueprintType, hidecategories=Object, MinimalAPI)
class UProgressWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:
	/** The actual data describing the button's appearance. */
	UPROPERTY(Category="Style", EditAnywhere, BlueprintReadWrite, meta=(ShowOnlyInnerProperties))
	FProgressBarStyle ProgressBarStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast< const struct FSlateWidgetStyle* >( &ProgressBarStyle );
	}
};
