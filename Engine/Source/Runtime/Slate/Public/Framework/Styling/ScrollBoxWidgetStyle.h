// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateTypes.h"
#include "ScrollBoxWidgetStyle.generated.h"

/**
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UScrollBoxWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:
	/** The actual data describing the scrollbox's appearance. */
	UPROPERTY(Category=Appearance, EditAnywhere, meta=(ShowOnlyInnerProperties))
	FScrollBoxStyle ScrollBoxStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast< const struct FSlateWidgetStyle* >( &ScrollBoxStyle );
	}
};
