// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateTypes.h"
#include "SpinBoxWidgetStyle.generated.h"

/**
*/
UCLASS(hidecategories = Object, MinimalAPI)
class USpinBoxWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:
	/** The actual data describing the button's appearance. */
	UPROPERTY(Category = Appearance, EditAnywhere, meta = (ShowOnlyInnerProperties))
	FSpinBoxStyle SpinBoxStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast< const struct FSlateWidgetStyle* >(&SpinBoxStyle);
	}
};
