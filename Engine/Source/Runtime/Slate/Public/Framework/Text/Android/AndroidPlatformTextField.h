// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// some android platforms disable this but not defined otherwise if AndroidPlatform.h not included first
#ifndef USE_ANDROID_JNI
	#define USE_ANDROID_JNI							1
#endif

#if USE_ANDROID_JNI

#include "CoreMinimal.h"
#include "Framework/Application/IPlatformTextField.h"

class IVirtualKeyboardEntry;

class FAndroidPlatformTextField : public IPlatformTextField
{
public:
	virtual void ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) override;
	virtual bool AllowMoveCursor() override;
private:
	bool ShouldUseAutocorrect(TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) const;
	bool EnableNewKeyboardConfig() const;
	//	SlateTextField* TextField;
};

typedef FAndroidPlatformTextField FPlatformTextField;

#else

#include "Framework/Text/GenericPlatformTextField.h"

#endif
