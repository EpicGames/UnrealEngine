// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/ITextLayoutMarshaller.h"

/**
 * Base class implementing some common functionality for all text marshallers
 */
class FBaseTextLayoutMarshaller : public ITextLayoutMarshaller
{
public:

	//~ Begin ITextLayoutMarshaller Interface
	virtual bool RequiresLiveUpdate() const override
	{
		return false;
	}

	virtual void MakeDirty() override
	{
		bIsDirty = true;
	}

	virtual void ClearDirty() override
	{
		bIsDirty = false;
	}

	virtual bool IsDirty() const override
	{
		return bIsDirty;
	}
	//~ End ITextLayoutMarshaller Interface

protected:

	FBaseTextLayoutMarshaller()
		: bIsDirty(false)
	{
	}

	virtual ~FBaseTextLayoutMarshaller() = default;

private:

	bool bIsDirty;

};
