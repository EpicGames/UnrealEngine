// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CharRangeList.generated.h"


/** Represents a range of characters, specified by the Unicode code point of the first and last characters in the range, both included.
*   If you need a single-character range, simply use the same character for both the first and last characters. */
USTRUCT()
struct FCharRange
{
	GENERATED_USTRUCT_BODY()

	/** Initializes this range from the given substring. See ToString for an overview on the required format.
	    *OutLastIndex will contain the index of the character following the parsed range. */
	bool InitializeFromString(const FString& InDefinitionString, int32 InFromIndex, int32* OutLastIndex);
	/** Returns a string that represents this range, as "a-z" if the range contains multiple characters or simply "p" if it contains a single character. */
	FString ToString() const;

	/** Unicode code point of the first character in the range (inclusive). Only characters within the Basic Multilingual Plane are supported.
	*   Note: The code point must be entered in decimal, not hexadecimal. */
	UPROPERTY(EditAnywhere, Category = "Char Range")
	uint16 First = 0;

	/** Unicode code point of the last character in the range (inclusive). Only characters within the Basic Multilingual Plane are supported.
	*   Note: The code point must be entered in decimal, not hexadecimal. */
	UPROPERTY(EditAnywhere, Category = "Char Range")
	uint16 Last = 0;
};

/** Represents a list of character ranges. */
USTRUCT()
struct FCharRangeList
{
	GENERATED_USTRUCT_BODY()

	/** Initializes this instance with the character ranges represented by the passed definition string.
	*   A definition string contains characters and ranges of characters, one after another with no special separators between them.
	*   Characters - and \ must be escaped like this: \- and \\
	* 
	*   Examples:
	*       "aT._" <-- Letters 'a' and 'T', dot and underscore.
	*       "a-zT._" <-- All letters from 'a' to 'z', letter 'T', dot and underscore.
	*       "a-zA-Z0-9._" <-- All lowercase and uppercase letters, all digits, dot and underscore.
	*       "a-zA-Z0-9\-\\._" <-- All lowercase and uppercase letters, all digits, minus sign, backslash, dot and underscore.
	*/
	SLATE_API bool InitializeFromString(const FString& InDefinitionString);
	/** Returns a definition string that represents all the character ranges in this instance. */
	SLATE_API FString ToString() const;

	/** Empties this instance. */
	SLATE_API void Empty();

	/** Returns true if this instance does not contain any characters, or false otherwise. */
	SLATE_API bool IsEmpty() const;

	/** Returns true if the given char is included in any of the char ranges, or false otherwise. */
	SLATE_API bool IsCharIncluded(TCHAR InChar) const;
	/** Returns true if all the characters in the given string are included in any of the char ranges, or false otherwise. */
	SLATE_API bool AreAllCharsIncluded(const FString& InString) const;
	/** Finds all the characters in the given string that are not included in any of the char ranges. */
	SLATE_API TSet<TCHAR> FindCharsNotIncluded(const FString& InString) const;

	UPROPERTY(EditAnywhere, Category = "Char Range List")
	TArray<FCharRange> Ranges;
};
