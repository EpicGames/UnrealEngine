// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ILayoutBlock.h"

class FDefaultLayoutBlock : public ILayoutBlock
{
public:

	static TSharedRef< FDefaultLayoutBlock > Create( const TSharedRef< IRun >& InRun, const FTextRange& InRange, FVector2D InSize, const FLayoutBlockTextContext& InTextContext, const TSharedPtr< IRunRenderer >& InRenderer )
	{
		return MakeShareable( new FDefaultLayoutBlock( InRun, InRange, InSize, InTextContext, InRenderer ) );
	}

	virtual ~FDefaultLayoutBlock() = default;

	virtual TSharedRef< IRun > GetRun() const override { return Run; }
	virtual FTextRange GetTextRange() const override { return Range; }
	virtual FVector2D GetSize() const override { return Size; }
	virtual FLayoutBlockTextContext GetTextContext() const override { return TextContext; }
	virtual TSharedPtr< IRunRenderer > GetRenderer() const override { return Renderer; }

	virtual void SetLocationOffset( const FVector2D& InLocationOffset ) override { LocationOffset = InLocationOffset; }
	virtual FVector2D GetLocationOffset() const override { return LocationOffset; }

private:

	static TSharedRef< FDefaultLayoutBlock > Create( const FDefaultLayoutBlock& Block )
	{
		return MakeShareable( new FDefaultLayoutBlock( Block ) );
	}

	FDefaultLayoutBlock( const TSharedRef< IRun >& InRun, const FTextRange& InRange, FVector2D InSize, const FLayoutBlockTextContext& InTextContext, const TSharedPtr< IRunRenderer >& InRenderer )
		: Run( InRun )
		, Range( InRange )
		, Size( InSize )
		, LocationOffset( ForceInitToZero )
		, TextContext( InTextContext )
		, Renderer( InRenderer )
	{

	}

	FDefaultLayoutBlock( const FDefaultLayoutBlock& Block )
		: Run( Block.Run )
		, Range( Block.Range )
		, Size( Block.Size )
		, LocationOffset( ForceInitToZero )
		, TextContext( Block.TextContext )
		, Renderer( Block.Renderer )
	{

	}

private:

	TSharedRef< IRun > Run;

	FTextRange Range;
	FVector2D Size;
	FVector2D LocationOffset;
	FLayoutBlockTextContext TextContext;
	TSharedPtr< IRunRenderer > Renderer;
};
