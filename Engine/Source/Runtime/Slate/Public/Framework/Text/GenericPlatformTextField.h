// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/IPlatformTextField.h"

class IVirtualKeyboardEntry;

class FGenericPlatformTextField : public IPlatformTextField
{
public:
	virtual void ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) override {};
	virtual bool AllowMoveCursor() override { return true; };
};

typedef FGenericPlatformTextField FPlatformTextField;
