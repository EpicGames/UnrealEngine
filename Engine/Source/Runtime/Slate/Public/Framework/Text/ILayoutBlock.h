// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/IRunRenderer.h"

class ILayoutBlock
{
public:

	virtual ~ILayoutBlock() = default;

	virtual TSharedRef< class IRun > GetRun() const = 0;
	virtual FTextRange GetTextRange() const = 0;
	virtual FVector2D GetSize() const = 0;
	virtual FLayoutBlockTextContext GetTextContext() const = 0;
	virtual TSharedPtr< class IRunRenderer > GetRenderer() const = 0;

	virtual void SetLocationOffset( const FVector2D& InLocationOffset ) = 0;
	virtual FVector2D GetLocationOffset() const = 0;
};
