// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Application/IPlatformTextField.h"
#include "Internationalization/Text.h"
#include "IOSView.h"

#import <UIKit/UIKit.h>

class IVirtualKeyboardEntry;

@class SlateTextField;
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
@class SlateTextField2;
#endif

class FIOSPlatformTextField : public IPlatformTextField
{
public:
	FIOSPlatformTextField();
	virtual ~FIOSPlatformTextField();

	virtual void ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) override;
	virtual bool AllowMoveCursor() override;

private:
	SlateTextField* TextField;
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	SlateTextField2* TextField2;
#endif
};

typedef FIOSPlatformTextField FPlatformTextField;

@interface SlateTextField : UIAlertController
{
	TWeakPtr<IVirtualKeyboardEntry> TextWidget;
	FText TextEntry;
    
    bool bTransitioning;
    bool bWantsToShow;
    NSString* CachedTextContents;
    NSString* CachedPlaceholderContents;
    FKeyboardConfig CachedKeyboardConfig;
    
    UIAlertController* AlertController;
}

-(void)show:(TSharedPtr<IVirtualKeyboardEntry>)InTextWidget text:(NSString*)TextContents placeholder:(NSString*)PlaceholderContents keyboardConfig:(FKeyboardConfig)KeyboardConfig;
-(void)hide;
-(void)updateToDesiredState;
-(bool)hasTextWidget;

@end

#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
@interface SlateTextField2 : NSObject <UITextFieldDelegate>
{
	TWeakPtr<IVirtualKeyboardEntry> TextWidget;

	UITextField* InputTextField;
}

-(void)show:(TSharedPtr<IVirtualKeyboardEntry>)InTextWidget text:(NSString*)TextContents placeholder:(NSString*)PlaceholderContents keyboardConfig:(FKeyboardConfig)KeyboardConfig;
-(void)hide;
-(bool)hasTextWidget;

@end
#endif
