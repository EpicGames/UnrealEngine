// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Framework/Text/ITextDecorator.h"

#if WITH_FANCY_TEXT

/**
 */
class IRichTextMarkupParser
{
public:

	/**
	 * Virtual destructor
	 */
	virtual ~IRichTextMarkupParser() = default;

	/**
	 * Processes the provided Input string producing a set of FTextLineParseResults and a output string stripped of any markup.
	 */
	virtual void Process(TArray<FTextLineParseResults>& Results, const FString& Input, FString& Output) = 0;

};


#endif //WITH_FANCY_TEXT
