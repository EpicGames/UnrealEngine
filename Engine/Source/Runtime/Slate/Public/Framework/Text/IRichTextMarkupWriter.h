// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Framework/Text/IRun.h"

#if WITH_FANCY_TEXT

/**
 */
class IRichTextMarkupWriter
{
public:

	/** A single run in a rich-text line, contains the meta-information that was parsed out of it, as well as the lexical contents of the run */
	struct FRichTextRun
	{
		FRichTextRun(const FRunInfo& InInfo, FString InText)
			: Info(InInfo)
			, Text(MoveTemp(InText))
		{
		}

		FRunInfo Info;
		FString Text;
	};

	/** A single line in a rich-text document, comprising of a number of styled runs */
	struct FRichTextLine
	{
		TArray<FRichTextRun> Runs;
	};

	/**
	 * Virtual destructor
	 */
	virtual ~IRichTextMarkupWriter() = default;

	/**
	 * Write the provided array of line and run info, producing an output string containing the markup needed to persist the run layouts
	 */
	virtual void Write(const TArray<FRichTextLine>& InLines, FString& Output) = 0;

};

#endif //WITH_FANCY_TEXT
