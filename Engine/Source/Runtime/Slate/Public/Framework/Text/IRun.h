// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/ShapedTextCacheFwd.h"
#include "Misc/EnumClassFlags.h"

class ILayoutBlock;
class IRunRenderer;
enum class ETextHitPoint : uint8;
enum class ETextShapingMethod : uint8;
class FFreeTypeFace;

struct FRunInfo
{
	FRunInfo()
		: Name()
		, MetaData()
	{
	}

	FRunInfo( FString InName )
		: Name( MoveTemp(InName) )
		, MetaData()
	{

	}

	FString Name;
	TMap< FString, FString > MetaData;
};

/** Attributes that a run can have */
enum class ERunAttributes : uint8
{
	/**
	 * This run has no special attributes
	 */
	None = 0,

	/**
	 * This run supports text, and can have new text inserted into it
	 * Note that even a run which doesn't support text may contain text (likely a breaking space character), however that text should be considered immutable
	 */
	SupportsText = 1<<0,
};
ENUM_CLASS_FLAGS(ERunAttributes);

/** The basic data needed when shaping a run of text */
struct FShapedTextContext
{
	FShapedTextContext(const ETextShapingMethod InTextShapingMethod, const TextBiDi::ETextDirection InBaseDirection)
		: TextShapingMethod(InTextShapingMethod)
		, BaseDirection(InBaseDirection)
	{
	}

	inline bool operator==(const FShapedTextContext& Other) const
	{
		return TextShapingMethod == Other.TextShapingMethod
			&& BaseDirection == Other.BaseDirection;
	}

	inline bool operator!=(const FShapedTextContext& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FShapedTextContext& Key)
	{
		uint32 KeyHash = 0;
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.TextShapingMethod));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.BaseDirection));
		return KeyHash;
	}

	/** The method used to shape the text within this layout */
	ETextShapingMethod TextShapingMethod;

	/** The base direction of the current line of text */
	TextBiDi::ETextDirection BaseDirection;
};

/** The context needed when performing text operations on a run of text */
struct FRunTextContext : public FShapedTextContext
{
	FRunTextContext(const ETextShapingMethod InTextShapingMethod, const TextBiDi::ETextDirection InBaseDirection, FShapedTextCacheRef InShapedTextCache)
		: FShapedTextContext(InTextShapingMethod, InBaseDirection)
		, ShapedTextCache(InShapedTextCache)
		, LoadingFontFaces(nullptr)
	{
	}

	FRunTextContext(const ETextShapingMethod InTextShapingMethod, const TextBiDi::ETextDirection InBaseDirection, FShapedTextCacheRef InShapedTextCache, TArray<TWeakPtr<FFreeTypeFace>>* InLoadingFontFaces)
		: FShapedTextContext(InTextShapingMethod, InBaseDirection)
		, ShapedTextCache(InShapedTextCache)
		, LoadingFontFaces(InLoadingFontFaces)
	{
	}
	/** The shaped text cache that should be used by this line of text */
	FShapedTextCacheRef ShapedTextCache;

	TArray<TWeakPtr<FFreeTypeFace>>* LoadingFontFaces;
};

/** The context needed when creating a block from a run of a text */
struct FLayoutBlockTextContext : public FRunTextContext
{
	FLayoutBlockTextContext(const FRunTextContext& InRunTextContext, const TextBiDi::ETextDirection InTextDirection)
		: FRunTextContext(InRunTextContext)
		, TextDirection(InTextDirection)
	{
	}

	inline bool operator==(const FLayoutBlockTextContext& Other) const
	{
		return FRunTextContext::operator==(Other)
			&& TextDirection == Other.TextDirection;
	}

	inline bool operator!=(const FLayoutBlockTextContext& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FLayoutBlockTextContext& Key)
	{
		uint32 KeyHash = GetTypeHash(static_cast<const FRunTextContext&>(Key));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.TextDirection));
		return KeyHash;
	}

	/** The reading direction of the text contained within this block */
	TextBiDi::ETextDirection TextDirection;
};

class IRun
{
public:

	virtual ~IRun() = default;

	virtual FTextRange GetTextRange() const = 0;
	virtual void SetTextRange( const FTextRange& Value ) = 0;

	virtual int16 GetBaseLine( float Scale ) const = 0;
	virtual int16 GetMaxHeight( float Scale ) const = 0;

	virtual FVector2D Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext) const = 0;
	virtual int8 GetKerning(int32 CurrentIndex, float Scale, const FRunTextContext& TextContext) const = 0;
	virtual FVector2D GetOutlineSize(int32 StartIndex, int32 EndIndex, float Scale) const { return FVector2d::ZeroVector; }
	virtual FVector2D GetShadowSize(int32 StartIndex, int32 EndIndex, float Scale) const { return FVector2d::ZeroVector; }

	virtual TSharedRef< class ILayoutBlock > CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< class IRunRenderer >& Renderer ) = 0;

	virtual int32 GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint = nullptr ) const = 0;
	
	virtual FVector2D GetLocationAt(const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale) const = 0;

	virtual void BeginLayout() = 0;
	virtual void EndLayout() = 0;

	virtual void Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange) = 0;
	virtual TSharedRef<IRun> Clone() const = 0;

	virtual void AppendTextTo(FString& Text) const = 0;
	virtual void AppendTextTo(FString& Text, const FTextRange& Range) const = 0;

	virtual const FRunInfo& GetRunInfo() const = 0;

	virtual ERunAttributes GetRunAttributes() const = 0;

};
