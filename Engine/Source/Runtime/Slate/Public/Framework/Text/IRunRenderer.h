// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IRunRenderer
{
public:

	virtual ~IRunRenderer() = default;

};
