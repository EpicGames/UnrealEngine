// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ILineHighlighter.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;
struct FTextBlockStyle;

class ISlateLineHighlighter : public ILineHighlighter
{
public:
	virtual ~ISlateLineHighlighter() = default;

	UE_DEPRECATED(5.6, "Please use OnPaint with Offset (FVector2D) instead of OffsetX (float).")
	virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const float OffsetX, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const final
	{
		return OnPaint(Args, Line, FVector2D(OffsetX, 0), Width, DefaultStyle, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
	
	virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const = 0;
};
