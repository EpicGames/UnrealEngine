// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ILayoutBlock.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
struct FTextBlockStyle;

struct FTextArgs
{
	FTextArgs(const FTextLayout::FLineView& InLine, const TSharedRef<ILayoutBlock>& InBlock, const FTextBlockStyle& InDefaultStyle, ETextOverflowPolicy InOverflowPolicy, ETextOverflowDirection InOverflowDirection)
		: Line(InLine)
		, Block(InBlock)
		, DefaultStyle(InDefaultStyle)
		, OverflowPolicy(InOverflowPolicy)
		, OverflowDirection(InOverflowDirection)
		, bIsLastVisibleBlock(false)
		, bIsNextBlockClipped(false)
	{}

	const FTextLayout::FLineView& Line;
	const TSharedRef< ILayoutBlock >& Block;
	const FTextBlockStyle& DefaultStyle;
	ETextOverflowPolicy OverflowPolicy;
	ETextOverflowDirection OverflowDirection;
	bool bIsLastVisibleBlock;
	bool bIsNextBlockClipped;

};

class ISlateRun : public IRun
{
public:
	virtual int32 OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const = 0;

	virtual const TArray< TSharedRef<SWidget> >& GetChildren() = 0;

	virtual void ArrangeChildren(const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const = 0;
};
