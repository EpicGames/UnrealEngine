// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/IRunRenderer.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/ISlateRun.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FTextBlockStyle;

class ISlateRunRenderer : public IRunRenderer
{
public:
	virtual ~ISlateRunRenderer() = default;

	virtual int32 OnPaint( const FPaintArgs& Args, const FTextLayout::FLineView& Line, const TSharedRef< ISlateRun >& Run, const TSharedRef< ILayoutBlock >& Block, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const = 0;
};
