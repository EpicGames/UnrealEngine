// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ISlateRun.h"

class ISlateStyle;

#if WITH_FANCY_TEXT

struct FTextRunParseResults
{
	FTextRunParseResults( FString InName, const FTextRange& InOriginalRange)
		: Name( InName )
		, OriginalRange( InOriginalRange )
		, MetaData()
	{

	}

	FTextRunParseResults( FString InName, const FTextRange& InOriginalRange, const FTextRange& InContentRange)
		: Name( InName )
		, OriginalRange( InOriginalRange )
		, ContentRange( InContentRange )
		, MetaData()
	{

	}

	FString Name;
	FTextRange OriginalRange;
	FTextRange ContentRange;
	TMap< FString, FTextRange > MetaData; 
};

struct FTextLineParseResults
{
public:

	FTextLineParseResults()
		: Range( )
		, Runs()
	{

	}

	FTextLineParseResults(const FTextRange& InRange)
		: Range( InRange )
		, Runs()
	{

	}

	FTextRange Range;
	TArray< FTextRunParseResults > Runs;
};

struct FTextRunInfo : FRunInfo
{
	FTextRunInfo( FString InName, const FText& InContent )
		: FRunInfo( MoveTemp(InName) )
		, Content( InContent )
	{

	}

	FText Content;
};

class ITextDecorator
{
public:

	virtual ~ITextDecorator() = default;

	virtual bool Supports( const FTextRunParseResults& RunInfo, const FString& Text ) const = 0;

	virtual TSharedRef< ISlateRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunInfo, const FString& OriginalText, const TSharedRef< FString >& ModelText, const ISlateStyle* Style ) = 0;
};

#endif //WITH_FANCY_TEXT
