// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FTextLayout;

/**
 * Interface used to get/set the raw text to/from a text layout
 */
class ITextLayoutMarshaller
{
public:

	virtual ~ITextLayoutMarshaller() = default;

	/**
	 * Populate the text layout from the string
	 */
	virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) = 0;

	/**
	 * Populate the string from the text layout
	 */
	virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) = 0;

	/**
	 * Return true if the marshaller requires the text be updated "live" (eg, because it inserts formatting directly into the source text)
	 * Returning true will cause SetText to be called every time the source text is changed, which is costly, but required for things like syntax highlighting
	 */
	virtual bool RequiresLiveUpdate() const = 0;

	/**
	 * Mark this text layout as dirty (eg, because some settings have changed), so that it will cause SetText to be called on the next Tick
	 */
	virtual void MakeDirty() = 0;

	/**
	 * Mark this text layout as clean once it's been updated
	 */
	virtual void ClearDirty() = 0;

	/**
	 * Is this text layout dirty? (eg, because some settings have changed)
	 * If so, you should call SetText on it to update the text layout
	 */
	virtual bool IsDirty() const = 0;

};
