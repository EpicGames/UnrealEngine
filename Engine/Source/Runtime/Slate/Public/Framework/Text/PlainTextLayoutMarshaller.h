// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Framework/Text/BaseTextLayoutMarshaller.h"

class FTextLayout;

/**
 * Get/set the raw text to/from a text layout as plain text
 */
class FPlainTextLayoutMarshaller : public FBaseTextLayoutMarshaller
{
public:

	static SLATE_API TSharedRef< FPlainTextLayoutMarshaller > Create();

	SLATE_API virtual ~FPlainTextLayoutMarshaller();
	
	SLATE_API void SetIsPassword(const TAttribute<bool>& InIsPassword);

	// ITextLayoutMarshaller
	SLATE_API virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
	SLATE_API virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) override;

protected:

	SLATE_API FPlainTextLayoutMarshaller();

	/** This this marshaller displaying a password? */
	TAttribute<bool> bIsPassword;

};
