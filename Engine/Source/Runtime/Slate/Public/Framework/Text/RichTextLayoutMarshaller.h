// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ITextDecorator.h"
#if WITH_FANCY_TEXT
	#include "Framework/Text/BaseTextLayoutMarshaller.h"
#endif

class FSlateTextUnderlineLineHighlighter;
class FSlateTextStrikeLineHighlighter;
class IRichTextMarkupParser;
class IRichTextMarkupWriter;
class ISlateStyle;
struct FTextBlockStyle;

#if WITH_FANCY_TEXT

/**
 * Get/set the raw text to/from a text layout as rich text
 */
class FRichTextLayoutMarshaller : public FBaseTextLayoutMarshaller
{
public:

	static SLATE_API TSharedRef< FRichTextLayoutMarshaller > Create(TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet);
	static SLATE_API TSharedRef< FRichTextLayoutMarshaller > Create(TSharedPtr< IRichTextMarkupParser > InParser, TSharedPtr< IRichTextMarkupWriter > InWriter, TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet);

	SLATE_API virtual ~FRichTextLayoutMarshaller();

	// ITextLayoutMarshaller
	SLATE_API virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
	SLATE_API virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) override;

	/** Set the Multiplier applied on the font size of the text*/
	SLATE_API void SetFontSizeMultiplier(const float NewFontSizeMultiplier);

	/**
	 * Replace all decorators on this marshaller
	 */
	inline void SetDecorators(TArrayView<TSharedRef<ITextDecorator>> InDecorators)
	{
		Decorators = InDecorators;
	}

	/**
	 * Append an inline decorator to this marshaller
	 */
	inline FRichTextLayoutMarshaller& AppendInlineDecorator(const TSharedRef< ITextDecorator >& DecoratorToAdd)
	{
		InlineDecorators.Add(DecoratorToAdd);
		return *this;
	}

	inline void SetDecoratorStyleSet(const ISlateStyle* InStyleSet)
	{
		DecoratorStyleSet = InStyleSet;
	}

protected:

	SLATE_API FRichTextLayoutMarshaller(TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet);
	SLATE_API FRichTextLayoutMarshaller(TSharedPtr< IRichTextMarkupParser > InParser, TSharedPtr< IRichTextMarkupWriter > InWriter, TArray< TSharedRef< ITextDecorator > > InDecorators, const ISlateStyle* const InDecoratorStyleSet);

	SLATE_API TSharedPtr< ITextDecorator > TryGetDecorator(const FString& Line, const FTextRunParseResults& TextRun) const;

	SLATE_API virtual void AppendRunsForText(
		const int32 LineIndex,
		const FTextRunParseResults& TextRun, 
		const FString& ProcessedString, 
		const FTextBlockStyle& DefaultTextStyle,
		const TSharedRef<FString>& InOutModelText, 
		FTextLayout& TargetTextLayout,
		TArray<TSharedRef<IRun>>& Runs,
		TArray<FTextLineHighlight>& LineHighlights,
		TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextUnderlineLineHighlighter>>& CachedUnderlineHighlighters,
		TMap<const FTextBlockStyle*, TSharedPtr<FSlateTextStrikeLineHighlighter>>& CachedStrikeLineHighlighters
		);

	/** The parser used to resolve any markup used in the provided string. */
	TSharedPtr< IRichTextMarkupParser > Parser;

	/** The writer used to recreate any markup needed to preserve the styled runs. */
	TSharedPtr< IRichTextMarkupWriter > Writer;

	/** Any decorators that should be used while parsing the text. */
	TArray< TSharedRef< ITextDecorator > > Decorators;

	/** Additional decorators can be appended inline. Inline decorators get precedence over decorators not specified inline. */
	TArray< TSharedRef< ITextDecorator > > InlineDecorators;

	/** The style set used for looking up styles used by decorators */
	const ISlateStyle* DecoratorStyleSet;

	/** Multiplier applied on font */
	float FontSizeMultiplier = 1.0f;

};

#endif //WITH_FANCY_TEXT
