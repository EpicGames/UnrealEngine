// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Framework/Text/ITextDecorator.h"
#include "Internationalization/Regex.h"
#include "Framework/Text/IRichTextMarkupParser.h"
#include "Framework/Text/IRichTextMarkupWriter.h"

#if WITH_FANCY_TEXT

class FDefaultRichTextMarkupParser : public IRichTextMarkupParser
{
public:
	static SLATE_API TSharedRef< FDefaultRichTextMarkupParser > Create();
	static SLATE_API TSharedRef< FDefaultRichTextMarkupParser > GetStaticInstance();

public:
	SLATE_API virtual void Process(TArray<FTextLineParseResults>& Results, const FString& Input, FString& Output) override;

private:
	FDefaultRichTextMarkupParser();

	void ParseLineRanges(const FString& Input, const TArray<FTextRange>& LineRanges, TArray<FTextLineParseResults>& LineParseResultsArray) const;
	void HandleEscapeSequences(const FString& Input, TArray<FTextLineParseResults>& LineParseResultsArray, FString& ConcatenatedUnescapedLines) const;

	FRegexPattern EscapeSequenceRegexPattern;
	FRegexPattern ElementRegexPattern;
	FRegexPattern AttributeRegexPattern;
};

class FDefaultRichTextMarkupWriter : public IRichTextMarkupWriter
{
public:
	static SLATE_API TSharedRef< FDefaultRichTextMarkupWriter > Create();
	static SLATE_API TSharedRef< FDefaultRichTextMarkupWriter > GetStaticInstance();

public:
	SLATE_API virtual void Write(const TArray<FRichTextLine>& InLines, FString& Output) override;

private:
	FDefaultRichTextMarkupWriter() {}
	static void EscapeText(FString& TextToEscape);
};

#endif //WITH_FANCY_TEXT
