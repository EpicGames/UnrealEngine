// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"

enum class ETextHitPoint : uint8;

namespace RunUtils
{

/**
 * Calculate the correct hit-point for the given index, based on the given range and text direction
 *
 * @param InIndex				The index to calculate the hit-point for
 * @param InTextRange			The range of text that the index was taken from
 * @param InTextDirection		The text direction of all the text within the given range
 *
 * @return The calculated hit-point
 */
SLATE_API ETextHitPoint CalculateTextHitPoint(const int32 InIndex, const FTextRange& InTextRange, const TextBiDi::ETextDirection InTextDirection);

/**
 * Calculate the range that will measure to the given index, based on the given range and text direction
 *
 * @param InOffset				The offset value to calculate the range for
 * @param InTextRange			The range of text to apply the offset to
 * @param InTextDirection		The text direction of all the text within the given range
 *
 * @return The calculated measure range
 */
SLATE_API FTextRange CalculateOffsetMeasureRange(const int32 InOffset, const FTextRange& InTextRange, const TextBiDi::ETextDirection InTextDirection);

}
