// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Text/IRun.h"

class FSlateFontCache;

/** Information representing a piece of shaped text */
struct FCachedShapedTextKey
{
public:
	FCachedShapedTextKey(const FTextRange& InTextRange, const float InScale, const FShapedTextContext& InTextContext, const FSlateFontInfo& InFontInfo)
		: TextRange(InTextRange)
		, Scale(InScale)
		, TextContext(InTextContext)
		, FontInfo(InFontInfo)
	{
	}

	inline bool operator==(const FCachedShapedTextKey& Other) const
	{
		return TextRange == Other.TextRange
			&& Scale == Other.Scale
			&& TextContext == Other.TextContext
			&& FontInfo.IsIdenticalTo(Other.FontInfo);
	}

	inline bool operator!=(const FCachedShapedTextKey& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FCachedShapedTextKey& Key)
	{
		uint32 KeyHash = 0;
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.TextRange));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.Scale));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.TextContext));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.FontInfo));
		return KeyHash;
	}

	FTextRange TextRange;
	float Scale;
	FShapedTextContext TextContext;
	FSlateFontInfo FontInfo;
};

/** Cache of shaped text */
class FShapedTextCache
{
public:
	/** Create a new shaped text cache */
	static FShapedTextCacheRef Create(const TSharedRef<FSlateFontCache>& InFontCache)
	{
		return MakeShared<FShapedTextCache>(FPrivateToken{}, InFontCache);
	}

	/**
	 * Try and find an existing shaped text instance
	 *
	 * @param InKey					The key identifying the shaped text instance to find
	 *
	 * @return The shaped text instance, or null if it wasn't found or was stale
	 */
	SLATE_API FShapedGlyphSequencePtr FindShapedText(const FCachedShapedTextKey& InKey) const;

	/**
	 * Add the given shaped text instance to the cache, or generate a new instance and add that based on the parameters provided
	 *
	 * @param InKey					The key identifying the shaped text instance to add
	 * @param InText				The text to shape. InKey may specify a sub-section of the entire text
	 * @param InTextDirection		The text direction of all of the text to be shaped. If present we do a unidirectional shape, otherwise we do a bidirectional shape
	 * @param InShapedText			The shaped text instance to add
	 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a shaped text during the shaping algorithm.  
	 *
	 * @return The shaped text instance
	 */
	SLATE_API FShapedGlyphSequenceRef AddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);
	SLATE_API FShapedGlyphSequenceRef AddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, const TextBiDi::ETextDirection InTextDirection, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);
	SLATE_API FShapedGlyphSequenceRef AddShapedText(const FCachedShapedTextKey& InKey, FShapedGlyphSequenceRef InShapedText);

	/**
	 * Try and find an existing shaped text instance, or add a new entry to the cache if one cannot be found
	 *
	 * @param InKey					The key identifying the shaped text instance to find or add
	 * @param InText				The text to shape if we can't find the shaped text in the cache. InKey may specify a sub-section of the entire text
	 * @param InTextDirection		The text direction of all of the text to be shaped. If present we do a unidirectional shape, otherwise we do a bidirectional shape
	 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a shaped text during the shaping algorithm.  
	 *
	 * @return The shaped text instance
	 */
	SLATE_API FShapedGlyphSequenceRef FindOrAddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);
	SLATE_API FShapedGlyphSequenceRef FindOrAddShapedText(const FCachedShapedTextKey& InKey, const TCHAR* InText, const TextBiDi::ETextDirection InTextDirection, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);

	/**
	 * Try and find an existing overflow shaped text instance, or add a new entry to the cache if one cannot be found.
	 * The overflow sequence is used to replace characters that are clipped.
	 *
	 * @return The shaped text instance
	 */
	SLATE_API FShapedGlyphSequenceRef FindOrAddOverflowEllipsisText(const float InScale, const FShapedTextContext& InTextContext, const FSlateFontInfo& InFontInfo);

	/**
	 * Clear this cache
	 */
	SLATE_API void Clear();

	/**
	 * Get the font cache used by this instance
	 */
	TSharedPtr<FSlateFontCache> GetFontCache() const
	{
		return FontCachePtr.Pin();
	}

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	/** Constructor */
	FShapedTextCache(FPrivateToken, const TSharedRef<FSlateFontCache>& InFontCache)
		: FontCachePtr(InFontCache)
	{
	}

private:
	/** Font cache to use when shaping text */
	TWeakPtr<FSlateFontCache> FontCachePtr;

	/** Mapping between a cache key and the corresponding shaped text */
	TMap<FCachedShapedTextKey, FShapedGlyphSequencePtr> CachedShapedText;
};

/** Utility functions that can provide efficient caching of common operations */
namespace ShapedTextCacheUtil
{

/**
 * Measure a sub-section of a run of text
 *
 * @param InShapedTextCache		The shaped text cache to use
 * @param InRunKey				The key identifying the cached shaped text for the run
 * @param InMeasureRange		The range of text that should be measured
 * @param InText				The text to shape if we can't find the shaped text in the cache. InMeasureRange may specify a sub-section of the entire text
 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a text during shaping.
 *
 * @return The measured size of the shaped text
 */
SLATE_API FVector2D MeasureShapedText(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const FTextRange& InMeasureRange, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);

/**
 * Find the character index at the specified position in pixels along the string horizontally
 *
 * @param InShapedTextCache		The shaped text cache to use
 * @param InRunKey				The key identifying the cached shaped text for the run
 * @param InTextRange			The range of text that should be extracted into its own shaped glyph sequence
 * @param InText				The text to shape if we can't find the shaped text in the cache. InTextRange may specify a sub-section of the entire text
 * @param InHorizontalOffset	The horizontal offset to get the character index for
 *
 * @return The index of the character closest to the specified horizontal offset
 */
SLATE_API int32 FindCharacterIndexAtOffset(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const FTextRange& InTextRange, const TCHAR* InText, const int32 InHorizontalOffset);

/**
 * Get the kerning between two shaped glyphs
 *
 * @param InShapedTextCache		The shaped text cache to use
 * @param InRunKey				The key identifying the cached shaped text for the run
 * @param InGlyphIndex			The index of the glyph to get the kerning for (will get it between the given glyph, and it's next glyph)
 * @param InText				The text to shape if we can't find the shaped text in the cache. (InGlyphIndex, InGlyphIndex+1) will be the range used
 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a text during shaping.
 *
 * @return The kerning
 */
SLATE_API int8 GetShapedGlyphKerning(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const int32 InGlyphIndex, const TCHAR* InText, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);

/**
 * Extract a sub-section of a run of text into its own shaped glyph sequence
 *
 * @param InShapedTextCache		The shaped text cache to use
 * @param InRunKey				The key identifying the cached shaped text for the run
 * @param InTextRange			The range of text that should be extracted into its own shaped glyph sequence
 * @param InText				The text to shape if we can't find the shaped text in the cache. InTextRange may specify a sub-section of the entire text
 * @param InTextDirection		The text direction of all of the text to be shaped
 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a text during shaping.
 *
 * @return The shaped text
 */
SLATE_API FShapedGlyphSequenceRef GetShapedTextSubSequence(const FShapedTextCacheRef& InShapedTextCache, const FCachedShapedTextKey& InRunKey, const FTextRange& InTextRange, const TCHAR* InText, const TextBiDi::ETextDirection InTextDirection, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr);

}
