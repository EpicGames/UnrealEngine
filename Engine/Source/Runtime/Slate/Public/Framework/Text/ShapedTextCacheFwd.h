// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FShapedTextCache;

typedef TSharedPtr<FShapedTextCache> FShapedTextCachePtr;
typedef TSharedRef<FShapedTextCache> FShapedTextCacheRef;
