// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Types/ISlateMetaData.h"
#include "SlateAsyncFontLoadingTypes.generated.h"


/** The policy for how to handle painting the text in text widgets if there are font faces that are still async loading while painting.*/
UENUM()
enum class EFontFacesLoadingPaintPolicy : uint8
{
	/**
	 * The text will not be painted if any of the associated font faces are async loading.
	 * This means the text widgets can appear blank until all of the font faces are finished async loading. 
	 */
	DoNotPaint,
	/**
	 * All glyphs supported by loaded font faces will be painted.
	 * All glyphs that are missing will have a either a fallback character empty space painted while the font face is async loading.
	 * This is best efffort until all font faces are finished loading and can result in visual artifacts. 
	 */
	PaintAvailableGlyphs
};

/** Metadata that holds the delegates if a widget wants to handle async font loading. */
class FSlateTextLayoutPendingFontFacesLoadingMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FSlateTextLayoutPendingFontFacesLoadingMetaData, ISlateMetaData)

	
	FSimpleDelegate OnAllFontFacesFinishLoading;
};
