// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Widgets/IToolTip.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/ISlateRun.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class FWidgetViewModel;
enum class ETextHitPoint : uint8;

#if WITH_FANCY_TEXT

class FSlateHyperlinkRun : public ISlateRun, public TSharedFromThis< FSlateHyperlinkRun >
{
public:

	typedef TMap< FString, FString > FMetadata;
	DECLARE_DELEGATE_OneParam( FOnClick, const FMetadata& /*Metadata*/ );
	DECLARE_DELEGATE_RetVal_OneParam( FText, FOnGetTooltipText, const FMetadata& /*Metadata*/ );
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<IToolTip>, FOnGenerateTooltip, const FMetadata& /*Metadata*/ );

public:

	class FWidgetViewModel
	{
	public:

		bool IsPressed() const { return bIsPressed; }
		bool IsHovered() const { return bIsHovered; }

		void SetIsPressed( bool Value ) { bIsPressed = Value; }
		void SetIsHovered( bool Value ) { bIsHovered = Value; }

	private:

		bool bIsPressed;
		bool bIsHovered;
	};

public:

	static SLATE_API TSharedRef< FSlateHyperlinkRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate );
																																	 
	static SLATE_API TSharedRef< FSlateHyperlinkRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange );

public:

	virtual ~FSlateHyperlinkRun() = default;

	SLATE_API virtual FTextRange GetTextRange() const override;

	SLATE_API virtual void SetTextRange( const FTextRange& Value ) override;

	SLATE_API virtual int16 GetBaseLine( float Scale ) const override;

	SLATE_API virtual int16 GetMaxHeight( float Scale ) const override;

	SLATE_API virtual FVector2D Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const override;

	SLATE_API virtual int8 GetKerning(int32 CurrentIndex, float Scale, const FRunTextContext& TextContext) const override;

	SLATE_API virtual FVector2d GetOutlineSize(int32 StartIndex, int32 EndIndex, float Scale) const override;
	
	SLATE_API virtual FVector2d GetShadowSize(int32 StartIndex, int32 EndIndex, float Scale) const override;

	SLATE_API virtual TSharedRef< ILayoutBlock > CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer ) override;

	SLATE_API virtual int32 OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	SLATE_API virtual const TArray< TSharedRef<SWidget> >& GetChildren() override;

	SLATE_API virtual void ArrangeChildren( const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	SLATE_API virtual int32 GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint = nullptr ) const override;

	SLATE_API virtual FVector2D GetLocationAt( const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale ) const override;

	virtual void BeginLayout() override { Children.Empty(); }
	virtual void EndLayout() override {}

	SLATE_API virtual void Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange) override;
	SLATE_API virtual TSharedRef<IRun> Clone() const override;

	SLATE_API virtual void AppendTextTo(FString& AppendToText) const override;
	SLATE_API virtual void AppendTextTo(FString& AppendToText, const FTextRange& PartialRange) const override;

	SLATE_API virtual const FRunInfo& GetRunInfo() const override;

	SLATE_API virtual ERunAttributes GetRunAttributes() const override;

protected:

	SLATE_API FSlateHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate );
																										 
	SLATE_API FSlateHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange );

	SLATE_API FSlateHyperlinkRun( const FSlateHyperlinkRun& Run );

protected:

	SLATE_API void OnNavigate();

protected:

	FRunInfo RunInfo;
	TSharedRef< const FString > Text;
	FTextRange Range;
	FHyperlinkStyle Style;
	FOnClick NavigateDelegate;
	FOnGenerateTooltip TooltipDelegate;
	FOnGetTooltipText TooltipTextDelegate;

	TSharedRef< FWidgetViewModel > ViewModel;
	TArray< TSharedRef<SWidget> > Children;
};

#endif //WITH_FANCY_TEXT
