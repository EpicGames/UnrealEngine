// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/SlateTextRun.h"

class FPaintArgs;
class FSlateWindowElementList;
enum class ETextHitPoint : uint8;

class FSlatePasswordRun : public FSlateTextRun
{
public:

	static SLATE_API TSharedRef< FSlatePasswordRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style );

	static SLATE_API TSharedRef< FSlatePasswordRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& Style, const FTextRange& InRange );

public:

	virtual ~FSlatePasswordRun() = default;

	SLATE_API virtual FVector2D Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const override;
	SLATE_API virtual int8 GetKerning(int32 CurrentIndex, float Scale, const FRunTextContext& TextContext) const override;

	SLATE_API virtual int32 OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	SLATE_API virtual int32 GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint = nullptr ) const override;

	SLATE_API virtual FVector2D GetLocationAt(const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale) const override;

	SLATE_API virtual TSharedRef<IRun> Clone() const override;

protected:

	SLATE_API FSlatePasswordRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle );

	SLATE_API FSlatePasswordRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FTextBlockStyle& InStyle, const FTextRange& InRange );

	SLATE_API FSlatePasswordRun( const FSlatePasswordRun& Run );

	static SLATE_API TCHAR GetPasswordChar();
	static SLATE_API FString BuildPasswordString(const int32 InLength);
};
