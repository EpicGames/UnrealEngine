// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ISlateRunRenderer.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FTextBlockStyle;

class FSlateTextHighlightRunRenderer : public ISlateRunRenderer
{
public:

	static SLATE_API TSharedRef< FSlateTextHighlightRunRenderer > Create();

	virtual ~FSlateTextHighlightRunRenderer() = default;

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FTextLayout::FLineView& Line, const TSharedRef< ISlateRun >& Run, const TSharedRef< ILayoutBlock >& Block, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

private:

	SLATE_API FSlateTextHighlightRunRenderer();

};
