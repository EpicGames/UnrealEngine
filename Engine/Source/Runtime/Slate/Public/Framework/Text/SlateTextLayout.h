// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"


class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
enum class EFontFacesLoadingPaintPolicy : uint8;

class FSlateTextLayout : public FTextLayout
{
public:

	static SLATE_API TSharedRef< FSlateTextLayout > Create(SWidget* InOwner, FTextBlockStyle InDefaultTextStyle);

	SLATE_API FChildren* GetChildren();

	SLATE_API virtual void ArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const;

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;

	SLATE_API virtual void EndLayout() override;

	SLATE_API void SetDefaultTextStyle(FTextBlockStyle InDefaultTextStyle);
	SLATE_API const FTextBlockStyle& GetDefaultTextStyle() const;

	SLATE_API void SetIsPassword(const TAttribute<bool>& InIsPassword);

	/** Sets the delegate that is Called when all font faces for the text have finished loading asynchronously. */
	SLATE_API void SetOnAllFontFacesFinishLoading(const FSimpleDelegate& InAllFontFacesFinishLoading);

	/** Gets the paint policy to be used for text if the font faces are still async loading. */
	SLATE_API EFontFacesLoadingPaintPolicy GetFontFacesLoadingPaintPolicy() const;

	/** Sets the paint policy to use for text if the font faces are still async loading. */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
protected:

	SLATE_API FSlateTextLayout(SWidget* InOwner, FTextBlockStyle InDefaultTextStyle);

	SLATE_API virtual int32 OnPaintHighlights(const FPaintArgs& Args, const FTextLayout::FLineView& LineView, const TArray<FLineViewHighlight>& Highlights, const FTextBlockStyle& DefaultTextStyle, const FGeometry& AllottedGeometry, const FSlateRect& ClippingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	SLATE_API virtual void AggregateChildren();

	SLATE_API virtual TSharedRef<IRun> CreateDefaultTextRun(const TSharedRef<FString>& NewText, const FTextRange& NewRange) const override;

	virtual void HandleAllFontFacesFinishLoading() override;
protected:
	/** Default style used by the TextLayout */
	FTextBlockStyle DefaultTextStyle;

private:

	TSlotlessChildren< SWidget > Children;

	/** This this layout displaying a password? */
	TAttribute<bool> bIsPassword;

	/** The paint policy to use for the text layout if any of the font faces are still async loading. */
	EFontFacesLoadingPaintPolicy FontFacesLoadingPaintPolicy;
	friend class FSlateTextLayoutFactory;
};
