// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateLineHighlighter.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
struct FGeometry;
struct FTextBlockStyle;

/** Run highlighter used to draw lines */
class ISlateTextLineHighlighter : public ISlateLineHighlighter
{
public:
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	static const int32 DefaultZIndex = 1;

protected:
	SLATE_API ISlateTextLineHighlighter(const FSlateBrush& InLineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity);

	virtual void GetLineMetrics(const float InFontScale, int16& OutLinePos, int16& OutLineThickness) const = 0;

	/** Brush used to draw the line */
	FSlateBrush LineBrush;

	/** Font the underline is associated with */
	FSlateFontInfo FontInfo;

	/** The color to draw the underline (typically matches the text its associated with) */
	FSlateColor ColorAndOpacity;

	/** Offset at which to draw the shadow (if any) */
	UE::Slate::FDeprecateVector2DResult ShadowOffset;

	/** The color to draw the shadow */
	FLinearColor ShadowColorAndOpacity;
};

/** Run highlighter used to draw underlines */
class FSlateTextUnderlineLineHighlighter : public ISlateTextLineHighlighter
{
public:
	static SLATE_API TSharedRef<FSlateTextUnderlineLineHighlighter> Create(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity);

protected:
	SLATE_API FSlateTextUnderlineLineHighlighter(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity);

	SLATE_API virtual void GetLineMetrics(const float InFontScale, int16& OutLinePos, int16& OutLineThickness) const override;
};

/** Run highlighter used to draw strikes */
class FSlateTextStrikeLineHighlighter : public ISlateTextLineHighlighter
{
public:
	static SLATE_API TSharedRef<FSlateTextStrikeLineHighlighter> Create(const FSlateBrush& InUnderlineBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity);

protected:
	SLATE_API FSlateTextStrikeLineHighlighter(const FSlateBrush& InStrikeBrush, const FSlateFontInfo& InFontInfo, const FSlateColor InColorAndOpacity, const UE::Slate::FDeprecateVector2DParameter InShadowOffset, const FLinearColor InShadowColorAndOpacity);

	SLATE_API virtual void GetLineMetrics(const float InFontScale, int16& OutLinePos, int16& OutLineThickness) const override;
};
