// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

enum class ETextOverflowPolicy : uint8;

namespace SlateTextUtils
{
	// Whether the given InOverflowPolicy is a valid Ellipsis policy
	SLATE_API bool IsEllipsisPolicy(const ETextOverflowPolicy& InOverflowPolicy);
}
