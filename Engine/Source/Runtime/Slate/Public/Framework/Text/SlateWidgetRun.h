// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Widgets/SWidget.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/ISlateRun.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
struct FTextBlockStyle;
enum class ETextHitPoint : uint8;

#if WITH_FANCY_TEXT

class FSlateWidgetRun : public ISlateRun, public TSharedFromThis< FSlateWidgetRun >
{
public:

	struct FWidgetRunInfo
	{
		TSharedRef< SWidget > Widget;
		TAttribute< int16 > Baseline;
		TOptional< FVector2D > Size;

		FWidgetRunInfo( const TSharedRef< SWidget >& InWidget, const TAttribute< int16 >& InBaseline, TOptional< FVector2D > InSize = TOptional< FVector2D >() )
			: Widget( InWidget )
			, Baseline( InBaseline )
			, Size( InSize )
		{

		}
	};

	static SLATE_API TSharedRef< FSlateWidgetRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo);

	static SLATE_API TSharedRef< FSlateWidgetRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo, const FTextRange& InRange);

	virtual ~FSlateWidgetRun() = default;

	SLATE_API virtual FTextRange GetTextRange() const override;
	SLATE_API virtual void SetTextRange( const FTextRange& Value ) override;
	SLATE_API virtual int16 GetBaseLine( float Scale ) const override;
	SLATE_API virtual int16 GetMaxHeight( float Scale ) const override;
	SLATE_API virtual FVector2D Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const override;
	SLATE_API virtual int8 GetKerning(int32 CurrentIndex, float Scale, const FRunTextContext& TextContext) const override;
	SLATE_API virtual TSharedRef< ILayoutBlock > CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer ) override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& PaintArgs, const FTextArgs& TextArgs, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual const TArray< TSharedRef<SWidget> >& GetChildren() override;
	SLATE_API virtual void ArrangeChildren( const TSharedRef< ILayoutBlock >& Block, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual int32 GetTextIndexAt( const TSharedRef< ILayoutBlock >& Block, const FVector2D& Location, float Scale, ETextHitPoint* const OutHitPoint = nullptr ) const override;
	SLATE_API virtual FVector2D GetLocationAt( const TSharedRef< ILayoutBlock >& Block, int32 Offset, float Scale ) const override;
	virtual void BeginLayout() override {}
	virtual void EndLayout() override {}
	SLATE_API virtual void Move(const TSharedRef<FString>& NewText, const FTextRange& NewRange) override;
	SLATE_API virtual TSharedRef<IRun> Clone() const override;
	SLATE_API virtual void AppendTextTo(FString& Text) const override;
	SLATE_API virtual void AppendTextTo(FString& AppendToText, const FTextRange& PartialRange) const override;
	SLATE_API virtual const FRunInfo& GetRunInfo() const override;
	SLATE_API virtual ERunAttributes GetRunAttributes() const override;

protected:

	SLATE_API FSlateWidgetRun(const TSharedRef<class FTextLayout>& TextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo);

	SLATE_API FSlateWidgetRun(const TSharedRef<class FTextLayout>& TextLayout, const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FWidgetRunInfo& InWidgetInfo, const FTextRange& InRange);

	SLATE_API FSlateWidgetRun( const FSlateWidgetRun& Run );

private:

	TWeakPtr<class FTextLayout> TextLayout;
	FRunInfo RunInfo;
	TSharedRef< const FString > Text;
	FTextRange Range;
	FWidgetRunInfo Info;
	TArray< TSharedRef<SWidget> > Children;

	mutable FVector2D WidgetSize;

};
#endif //WITH_FANCY_TEXT
