// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

#if WITH_FANCY_TEXT
	#include "Framework/Text/PlainTextLayoutMarshaller.h"
	#include "Framework/Text/SyntaxTokenizer.h"
#endif

class FTextLayout;

#if WITH_FANCY_TEXT

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting
 */
class FSyntaxHighlighterTextLayoutMarshaller : public FPlainTextLayoutMarshaller
{
public:

	SLATE_API virtual ~FSyntaxHighlighterTextLayoutMarshaller();

	// ITextLayoutMarshaller
	SLATE_API virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
	SLATE_API virtual bool RequiresLiveUpdate() const override;

	SLATE_API void EnableSyntaxHighlighting(const bool bEnable);
	SLATE_API bool IsSyntaxHighlightingEnabled() const;

protected:

	virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) = 0;

	SLATE_API FSyntaxHighlighterTextLayoutMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer);

	/** Tokenizer used to style the text */
	TSharedPtr< ISyntaxTokenizer > Tokenizer;

	/** True if syntax highlighting is enabled, false to fallback to plain text */
	bool bSyntaxHighlightingEnabled;

};

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting for our rich-text markup
 */
class FRichTextSyntaxHighlighterTextLayoutMarshaller : public FSyntaxHighlighterTextLayoutMarshaller
{
public:

	struct FSyntaxTextStyle
	{
		FSyntaxTextStyle()
			: NormalTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.Normal"))
			, NodeTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.Node"))
			, NodeAttributeKeyTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.NodeAttributeKey"))
			, NodeAttribueAssignmentTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.NodeAttribueAssignment"))
			, NodeAttributeValueTextStyle(FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.NodeAttributeValue"))
		{
		}

		FSyntaxTextStyle(const FTextBlockStyle& InNormalTextStyle, const FTextBlockStyle& InNodeTextStyle, const FTextBlockStyle& InNodeAttributeKeyTextStyle, const FTextBlockStyle& InNodeAttribueAssignmentTextStyle, const FTextBlockStyle& InNodeAttributeValueTextStyle)
			: NormalTextStyle(InNormalTextStyle)
			, NodeTextStyle(InNodeTextStyle)
			, NodeAttributeKeyTextStyle(InNodeAttributeKeyTextStyle)
			, NodeAttribueAssignmentTextStyle(InNodeAttribueAssignmentTextStyle)
			, NodeAttributeValueTextStyle(InNodeAttributeValueTextStyle)
		{
		}

		FTextBlockStyle NormalTextStyle;
		FTextBlockStyle NodeTextStyle;
		FTextBlockStyle NodeAttributeKeyTextStyle;
		FTextBlockStyle NodeAttribueAssignmentTextStyle;
		FTextBlockStyle NodeAttributeValueTextStyle;
	};

	static SLATE_API TSharedRef< FRichTextSyntaxHighlighterTextLayoutMarshaller > Create(const FSyntaxTextStyle& InSyntaxTextStyle);

	SLATE_API virtual ~FRichTextSyntaxHighlighterTextLayoutMarshaller();

protected:

	SLATE_API virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;

	SLATE_API FRichTextSyntaxHighlighterTextLayoutMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	/** Styles used to display the text */
	FSyntaxTextStyle SyntaxTextStyle;

};

#endif //WITH_FANCY_TEXT
