// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class ISyntaxTokenizer
{
public:
	/** Denotes the type of token */
	enum class ETokenType : uint8
	{
		/** Syntax token matching the tokenizing rules provided */
		Syntax,
		/** String token containing some literal text */
		Literal,
	};

	/** A token referencing a range in the original text */
	struct FToken
	{
		FToken(const ETokenType InType, const FTextRange& InRange)
			: Type(InType)
			, Range(InRange)
		{
		}

		ETokenType Type;
		FTextRange Range;
	};

	/** A line containing a series of tokens */
	struct FTokenizedLine
	{
		FTextRange Range;
		TArray<FToken> Tokens;
	};

	virtual ~ISyntaxTokenizer() = default;
	
	virtual void Process(TArray<FTokenizedLine>& OutTokenizedLines, const FString& Input) = 0;
};

/**
 * Tokenize the text based upon the given rule set
 */
class FSyntaxTokenizer : public ISyntaxTokenizer
{
public:
	/** Rule used to match syntax token types */
	struct FRule
	{
		FRule(FString InMatchText)
			: MatchText(MoveTemp(InMatchText))
		{
		}

		FString MatchText;
	};

	/** 
	 * Create a new tokenizer which will use the given rules to match syntax tokens
	 * @param InRules Rules to control the tokenizer, processed in-order so the most greedy matches must come first
	 */
	static SLATE_API TSharedRef< FSyntaxTokenizer > Create(TArray<FRule> InRules);

	SLATE_API virtual ~FSyntaxTokenizer();

	SLATE_API virtual void Process(TArray<FTokenizedLine>& OutTokenizedLines, const FString& Input) override;

private:

	FSyntaxTokenizer(TArray<FRule> InRules);

	void TokenizeLineRanges(const FString& Input, const TArray<FTextRange>& LineRanges, TArray<FTokenizedLine>& OutTokenizedLines);

	/** Rules to control the tokenizer, processed in-order so the most greedy matches must come first */
	TArray<FRule> Rules;

};
