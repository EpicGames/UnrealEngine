// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/ITextDecorator.h"
#include "Framework/Text/SlateWidgetRun.h"

class ISlateStyle;

#if WITH_FANCY_TEXT

class FWidgetDecorator : public ITextDecorator
{
public:

	DECLARE_DELEGATE_RetVal_TwoParams( FSlateWidgetRun::FWidgetRunInfo, FCreateWidget, const FTextRunInfo& /*RunInfo*/, const ISlateStyle* /*Style*/ )

public:

	static SLATE_API TSharedRef< FWidgetDecorator > Create( FString InRunName, const FCreateWidget& InCreateWidgetDelegate );
	virtual ~FWidgetDecorator() = default;

public:

	SLATE_API virtual bool Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const override;

	SLATE_API virtual TSharedRef< ISlateRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override;

private:

	SLATE_API FWidgetDecorator( FString InRunName, const FCreateWidget& InCreateWidgetDelegate );

private:

	FString RunName;
	FCreateWidget CreateWidgetDelegate;
};

class FImageDecorator : public ITextDecorator
{
public:

	static SLATE_API TSharedRef< FImageDecorator > Create( FString InRunName, const ISlateStyle* const InOverrideStyle = NULL );
	virtual ~FImageDecorator() = default;

public:

	SLATE_API virtual bool Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const override;

	SLATE_API virtual TSharedRef< ISlateRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override;

private:

	SLATE_API FImageDecorator( FString InRunName, const ISlateStyle* const InOverrideStyle );

private:

	FString RunName;
	const ISlateStyle* OverrideStyle;
};

class FHyperlinkDecorator : public ITextDecorator
{
public:

	static SLATE_API TSharedRef< FHyperlinkDecorator > Create( FString Id, const FSlateHyperlinkRun::FOnClick& NavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate = FSlateHyperlinkRun::FOnGetTooltipText(), const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate = FSlateHyperlinkRun::FOnGenerateTooltip() );
	virtual ~FHyperlinkDecorator() = default;

public:

	SLATE_API virtual bool Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const override;

	SLATE_API virtual TSharedRef< ISlateRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override;

protected:

	SLATE_API FHyperlinkDecorator( FString InId, const FSlateHyperlinkRun::FOnClick& InNavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate );

protected:

	FSlateHyperlinkRun::FOnClick NavigateDelegate;

protected:
	FString Id;
	FSlateHyperlinkRun::FOnGetTooltipText ToolTipTextDelegate;
	FSlateHyperlinkRun::FOnGenerateTooltip ToolTipDelegate;
};


#endif //WITH_FANCY_TEXT
