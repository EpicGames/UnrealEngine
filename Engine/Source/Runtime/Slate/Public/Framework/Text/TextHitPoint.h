// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/** Describes logically how a line was hit when performing a screen-space -> view model conversion */
enum class ETextHitPoint : uint8
{
	/** The hit-point was within the text range */
	WithinText,
	/** The hit-point was logically to the left of the text range (the visually left gutter for left-to-right text, the visually right gutter for right-to-left text) */
	LeftGutter,
	/** The hit-point was logically to the right of the text range (the visually right gutter for left-to-right text, the visually left gutter for right-to-left text) */
	RightGutter,
};
