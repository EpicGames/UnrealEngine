// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Framework/Text/TextRunRenderer.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/IRun.h"
#include "Styling/SlateTypes.h"

#include "TextLayout.generated.h"

#define TEXT_LAYOUT_DEBUG 0

class IBreakIterator;
class ILayoutBlock;
class ILineHighlighter;
class IRunRenderer;
enum class ETextHitPoint : uint8;
enum class ETextShapingMethod : uint8;

UENUM( BlueprintType )
namespace ETextJustify
{
	enum Type : int
	{
		/**
		 * Justify the text logically to the left.
		 * When text is flowing left-to-right, this will align text visually to the left.
		 * When text is flowing right-to-left, this will align text visually to the right.
		 */
		Left,

		/**
		 * Justify the text in the center.
		 * Text flow direction has no impact on this justification mode.
		 */
		Center,

		/**
		 * Justify the text logically to the right.
		 * When text is flowing left-to-right, this will align text visually to the right.
		 * When text is flowing right-to-left, this will align text visually to the left.
		 */
		Right,

		/**
		 * Always justify the text to the left, regardless of the flow direction of the current culture.
		 */
		InvariantLeft,

		/**
		 * Always justify the text to the right, regardless of the flow direction of the current culture.
		 */
		InvariantRight,
	};
}


/** 
 * The different methods that can be used if a word is too long to be broken by the default line-break iterator.
 */
UENUM( BlueprintType )
enum class ETextWrappingPolicy : uint8
{
	/** No fallback, just use the given line-break iterator */
	DefaultWrapping = 0,

	/** Fallback to per-character wrapping if a word is too long */
	AllowPerCharacterWrapping,
};

/** 
 * The different directions that text can flow within a paragraph of text.
 * @note If you change this enum, make sure and update CVarDefaultTextFlowDirection and GetDefaultTextFlowDirection.
 */
UENUM( BlueprintType )
enum class ETextFlowDirection : uint8
{
	/** Automatically detect the flow direction for each paragraph from its text */
	Auto = 0,

	/** Force text to be flowed left-to-right */
	LeftToRight,

	/** Force text to be flowed right-to-left */
	RightToLeft,

	/** Uses the set culture to determine if text should flow left-to-right or right-to-left.  By comparison, Auto will use the text itself to determine it. */
	Culture,
};

/** Get the default text flow direction (from the "Slate.DefaultTextFlowDirection" CVar) */
SLATE_API ETextFlowDirection GetDefaultTextFlowDirection();

/** Location within the text model. */
struct FTextLocation
{
public:
	FTextLocation( const int32 InLineIndex = 0, const int32 InOffset = 0 )
		: LineIndex( InLineIndex )
		, Offset( InOffset )
	{

	}

	FTextLocation( const FTextLocation& InLocation, const int32 InOffset )
		: LineIndex( InLocation.GetLineIndex() )
		, Offset(FMath::Max(InLocation.GetOffset() + InOffset, 0))
	{

	}

	bool operator==( const FTextLocation& Other ) const
	{
		return
			LineIndex == Other.LineIndex &&
			Offset == Other.Offset;
	}

	bool operator!=( const FTextLocation& Other ) const
	{
		return
			LineIndex != Other.LineIndex ||
			Offset != Other.Offset;
	}

	bool operator<( const FTextLocation& Other ) const
	{
		return this->LineIndex < Other.LineIndex || (this->LineIndex == Other.LineIndex && this->Offset < Other.Offset);
	}

	int32 GetLineIndex() const { return LineIndex; }
	int32 GetOffset() const { return Offset; }
	bool IsValid() const { return LineIndex != INDEX_NONE && Offset != INDEX_NONE; }

	friend inline uint32 GetTypeHash(const FTextLocation& InSubject)
	{
		return HashCombine(InSubject.LineIndex, InSubject.Offset);
	}

private:
	int32 LineIndex;
	int32 Offset;
};

class FTextSelection
{

public:

	FTextLocation LocationA;

	FTextLocation LocationB;

public:

	FTextSelection()
		: LocationA(INDEX_NONE)
		, LocationB(INDEX_NONE)
	{
	}

	FTextSelection(const FTextLocation& InLocationA, const FTextLocation& InLocationB)
		: LocationA(InLocationA)
		, LocationB(InLocationB)
	{
	}

	const FTextLocation& GetBeginning() const
	{
		if (LocationA.GetLineIndex() == LocationB.GetLineIndex())
		{
			if (LocationA.GetOffset() < LocationB.GetOffset())
			{
				return LocationA;
			}

			return LocationB;
		}
		else if (LocationA.GetLineIndex() < LocationB.GetLineIndex())
		{
			return LocationA;
		}

		return LocationB;
	}

	const FTextLocation& GetEnd() const
	{
		if (LocationA.GetLineIndex() == LocationB.GetLineIndex())
		{
			if (LocationA.GetOffset() > LocationB.GetOffset())
			{
				return LocationA;
			}

			return LocationB;
		}
		else if (LocationA.GetLineIndex() > LocationB.GetLineIndex())
		{
			return LocationA;
		}

		return LocationB;
	}
};

class FTextLayout
	: public TSharedFromThis<FTextLayout>
{
public:

	struct FBlockDefinition
	{
		/** Range inclusive of trailing whitespace, as used to visually display and interact with the text */
		FTextRange ActualRange;
		/** The render to use with this run (if any) */
		TSharedPtr< IRunRenderer > Renderer;
	};

	struct FBreakCandidate
	{
		/** Range inclusive of trailing whitespace, as used to visually display and interact with the text */
		FTextRange ActualRange;
		/** Range exclusive of trailing whitespace, as used to perform wrapping on a word boundary */
		FTextRange TrimmedRange;
		/** Measured size inclusive of trailing whitespace, as used to visually display and interact with the text */
		FVector2D ActualSize;
		/** Measured width exclusive of trailing whitespace, as used to perform wrapping on a word boundary */
		float TrimmedWidth;
		/** If this break candidate has trailing whitespace, this is the width of the first character of the trailing whitespace */
		float FirstTrailingWhitespaceCharWidth;

		int16 MaxAboveBaseline;
		int16 MaxBelowBaseline;

		int8 Kerning;

#if TEXT_LAYOUT_DEBUG
		FString DebugSlice;
#endif
	};

	class FRunModel
	{
	public:

		SLATE_API FRunModel( const TSharedRef< IRun >& InRun);

	public:

		SLATE_API TSharedRef< IRun > GetRun() const;;

		SLATE_API void BeginLayout();
		SLATE_API void EndLayout();

		SLATE_API FTextRange GetTextRange() const;
		SLATE_API void SetTextRange( const FTextRange& Value );
		
		SLATE_API int16 GetBaseLine( float Scale ) const;
		SLATE_API int16 GetMaxHeight( float Scale ) const;

		SLATE_API FVector2D Measure( int32 BeginIndex, int32 EndIndex, float Scale, const FRunTextContext& InTextContext );

		SLATE_API int8 GetKerning( int32 CurrentIndex, float Scale, const FRunTextContext& InTextContext );

		static SLATE_API int32 BinarySearchForBeginIndex( const TArray< FTextRange >& Ranges, int32 BeginIndex );

		static SLATE_API int32 BinarySearchForEndIndex( const TArray< FTextRange >& Ranges, int32 RangeBeginIndex, int32 EndIndex );

		SLATE_API TSharedRef< ILayoutBlock > CreateBlock( const FBlockDefinition& BlockDefine, float InScale, const FLayoutBlockTextContext& InTextContext ) const;

		SLATE_API void ClearCache();

		SLATE_API void AppendTextTo(FString& Text) const;
		SLATE_API void AppendTextTo(FString& Text, const FTextRange& Range) const;

	private:

		TSharedRef< IRun > Run;
		TArray< FTextRange > MeasuredRanges;
		TArray< FVector2D > MeasuredRangeSizes;
	};

	struct ELineModelDirtyState
	{
		typedef uint8 Flags;
		enum Enum
		{
			None = 0,
			WrappingInformation = 1<<0,
			TextBaseDirection = 1<<1, 
			ShapingCache = 1<<2,
			All = WrappingInformation | TextBaseDirection | ShapingCache,
		};
	};

	struct FLineModel
	{
	public:

		SLATE_API FLineModel( const TSharedRef< FString >& InText );

		TSharedRef< FString > Text;
		FShapedTextCacheRef ShapedTextCache;
		TextBiDi::ETextDirection TextBaseDirection;
		TArray< FRunModel > Runs;
		TArray< FBreakCandidate > BreakCandidates;
		TArray< FTextRunRenderer > RunRenderers;
		TArray< FTextLineHighlight > LineHighlights;
		ELineModelDirtyState::Flags DirtyFlags;
	};

	struct FLineViewHighlight
	{
		/** Offset for this highlight, relative to the FLineView::Offset that contains it */
		FVector2D Offset;
		/** Width for this highlight, the height will be either FLineView::Size.Y or FLineView::TextHeight depending on whether you want to highlight the entire line, or just the text within the line */
		float Width;
		/** Custom highlighter implementation used to do the painting */
		TSharedPtr< ILineHighlighter > Highlighter;
	};

	struct FLineView
	{
		TArray< TSharedRef< ILayoutBlock > > Blocks;
		TArray< FLineViewHighlight > UnderlayHighlights;
		TArray< FLineViewHighlight > OverlayHighlights;
		FVector2D Offset;
		FVector2D Size;
		float TextHeight;
		float JustificationWidth;
		FTextRange Range;
		TextBiDi::ETextDirection TextBaseDirection;
		int32 ModelIndex;
	};

	/** A mapping between the offsets into the text as a flat string (with line-breaks), and the internal lines used within a text layout */
	struct FTextOffsetLocations
	{
	friend class FTextLayout;

	public:
		SLATE_API int32 TextLocationToOffset(const FTextLocation& InLocation) const;
		SLATE_API FTextLocation OffsetToTextLocation(const int32 InOffset) const;
		SLATE_API int32 GetTextLength() const;

	private:
		struct FOffsetEntry
		{
			FOffsetEntry(const int32 InFlatStringIndex, const int32 InDocumentLineLength)
				: FlatStringIndex(InFlatStringIndex)
				, DocumentLineLength(InDocumentLineLength)
			{
			}

			/** Index in the flat string for this entry */
			int32 FlatStringIndex;

			/** The length of the line in the document (not including any trailing \n character) */
			int32 DocumentLineLength;
		};

		/** This array contains one entry for each line in the document; 
			the array index is the line number, and the entry contains the index in the flat string that marks the start of the line, 
			along with the length of the line (not including any trailing \n character) */
		TArray<FOffsetEntry> OffsetData;
	};

public:

	SLATE_API virtual ~FTextLayout();

	inline const TArray< FTextLayout::FLineView >& GetLineViews() const { return LineViews; }
	inline const TArray< FTextLayout::FLineModel >& GetLineModels() const { return LineModels; }

	/**
	 * Get the size of the text layout, including any lines which extend beyond the wrapping boundaries (eg, lines with lots of trailing whitespace, or lines with no break candidates)
	 * @note This value is unscaled
	 */
	SLATE_API FVector2D GetSize() const;

	/**
	 * Get the size of the text layout that can actually be seen from the parent widget
	 */
	SLATE_API FVector2D GetViewSize() const;

	/**
	 * Get the size of the text layout, including any lines which extend beyond the wrapping boundaries (eg, lines with lots of trailing whitespace, or lines with no break candidates)
	 * @note This value is scaled
	 */
	SLATE_API FVector2D GetDrawSize() const;

	/**
	 * Get the size of the text layout after the text has been wrapped, and including the first piece of trailing whitespace for any given soft-wrapped line
	 * @note This value is unscaled
	 */
	SLATE_API FVector2D GetWrappedSize() const;

	/**
	 * Get the size of the text layout after the text has been wrapped, and including the first piece of trailing whitespace for any given soft-wrapped line
	 * @note This value is scaled
	 */
	SLATE_API FVector2D GetWrappedDrawSize() const;

	inline float GetWrappingWidth() const { return WrappingWidth; }
	SLATE_API void SetWrappingWidth( float Value );

	inline ETextWrappingPolicy GetWrappingPolicy() const { return WrappingPolicy; }
	SLATE_API void SetWrappingPolicy(ETextWrappingPolicy Value);

	inline float GetLineHeightPercentage() const { return LineHeightPercentage; }
	SLATE_API void SetLineHeightPercentage( float Value );

	inline bool GetApplyLineHeightToBottomLine() const { return ApplyLineHeightToBottomLine; }
	SLATE_API void SetApplyLineHeightToBottomLine( bool Value );

	inline ETextJustify::Type GetJustification() const { return Justification; }
	SLATE_API void SetJustification( ETextJustify::Type Value );

	/** Get the visual justification for this document (based on the visual justification used by the first line of text) */
	SLATE_API ETextJustify::Type GetVisualJustification() const;

	/** @note This option is destructive to the model text, so changing it requires refreshing the text layout from its marshaller */
	inline ETextTransformPolicy GetTransformPolicy() const { return TransformPolicy; }
	SLATE_API void SetTransformPolicy(ETextTransformPolicy Value);

	inline float GetScale() const { return Scale; }
	SLATE_API void SetScale( float Value );

	inline ETextShapingMethod GetTextShapingMethod() const { return TextShapingMethod; }
	SLATE_API void SetTextShapingMethod( const ETextShapingMethod InTextShapingMethod );

	inline ETextFlowDirection GetTextFlowDirection() const { return TextFlowDirection; }
	SLATE_API void SetTextFlowDirection( const ETextFlowDirection InTextFlowDirection );

	SLATE_API void SetTextOverflowPolicy(const TOptional<ETextOverflowPolicy> InTextOverflowPolicy);

	inline FMargin GetMargin() const { return Margin; }
	SLATE_API void SetMargin( const FMargin& InMargin );

	SLATE_API void SetVisibleRegion( const FVector2D& InViewSize, const FVector2D& InScrollOffset );

	/** Set the iterator to use to detect appropriate soft-wrapping points for lines (or null to go back to using the default) */
	SLATE_API void SetLineBreakIterator( TSharedPtr<IBreakIterator> InLineBreakIterator );

	/** Set the information used to help identify who owns this text layout in the case of an error */
	SLATE_API void SetDebugSourceInfo(const TAttribute<FString>& InDebugSourceInfo);

	SLATE_API void ClearLines();

	struct FNewLineData
	{
		FNewLineData(TSharedRef<FString> InText, TArray<TSharedRef<IRun>> InRuns)
			: Text(MoveTemp(InText))
			, Runs(MoveTemp(InRuns))
		{
		}

		TSharedRef<FString> Text;
		TArray<TSharedRef<IRun>> Runs;
	};

	SLATE_API void AddLine( const FNewLineData& NewLine );

	SLATE_API void AddLines( const TArray<FNewLineData>& NewLines );

	/**
	* Clears all run renderers
	*/
	SLATE_API void ClearRunRenderers();

	/**
	* Replaces the current set of run renderers with the provided renderers.
	*/
	SLATE_API void SetRunRenderers( const TArray< FTextRunRenderer >& Renderers );

	/**
	* Adds a single run renderer to the existing set of renderers.
	*/
	SLATE_API void AddRunRenderer( const FTextRunRenderer& Renderer );

	/**
	* Removes a single run renderer to the existing set of renderers.
	*/
	SLATE_API void RemoveRunRenderer( const FTextRunRenderer& Renderer );

	/**
	* Clears all line highlights
	*/
	SLATE_API void ClearLineHighlights();

	/**
	* Replaces the current set of line highlights with the provided highlights.
	*/
	SLATE_API void SetLineHighlights( const TArray< FTextLineHighlight >& Highlights );

	/**
	* Adds a single line highlight to the existing set of highlights.
	*/
	SLATE_API void AddLineHighlight( const FTextLineHighlight& Highlight );

	/**
	* Removes a single line highlight to the existing set of highlights.
	*/
	SLATE_API void RemoveLineHighlight( const FTextLineHighlight& Highlight );

	/**
	* Updates the TextLayout's if any changes have occurred since the last update.
	*/
	SLATE_API virtual void UpdateIfNeeded();

	SLATE_API virtual void UpdateLayout();

	SLATE_API virtual void UpdateHighlights();

	SLATE_API void DirtyRunLayout(const TSharedRef<const IRun>& Run);

	SLATE_API void DirtyLayout();

	SLATE_API bool IsLayoutDirty() const;

	SLATE_API int32 GetLineViewIndexForTextLocation(const TArray< FTextLayout::FLineView >& LineViews, const FTextLocation& Location, const bool bPerformInclusiveBoundsCheck) const;

	/**
	 * 
	 */
	SLATE_API FTextLocation GetTextLocationAt( const FVector2D& Relative, ETextHitPoint* const OutHitPoint = nullptr ) const;

	SLATE_API FTextLocation GetTextLocationAt( const FLineView& LineView, const FVector2D& Relative, ETextHitPoint* const OutHitPoint = nullptr ) const;

	SLATE_API FVector2D GetLocationAt( const FTextLocation& Location, const bool bPerformInclusiveBoundsCheck ) const;

	SLATE_API bool SplitLineAt(const FTextLocation& Location);

	SLATE_API bool JoinLineWithNextLine(int32 LineIndex);

	SLATE_API bool InsertAt(const FTextLocation& Location, TCHAR Character);

	SLATE_API bool InsertAt(const FTextLocation& Location, const FString& Text);

	SLATE_API bool InsertAt(const FTextLocation& Location, TSharedRef<IRun> InRun, const bool bAlwaysKeepRightRun = false);

	SLATE_API bool RemoveAt(const FTextLocation& Location, int32 Count = 1);

	SLATE_API bool RemoveLine(int32 LineIndex);

	SLATE_API bool IsEmpty() const;

	SLATE_API int32 GetLineCount() const;

	SLATE_API void GetAsText(FString& DisplayText, FTextOffsetLocations* const OutTextOffsetLocations = nullptr) const;

	SLATE_API void GetAsText(FText& DisplayText, FTextOffsetLocations* const OutTextOffsetLocations = nullptr) const;

	/** Constructs an array containing the mappings between the text that would be returned by GetAsText, and the internal FTextLocation points used within this text layout */
	SLATE_API void GetTextOffsetLocations(FTextOffsetLocations& OutTextOffsetLocations) const;

	SLATE_API void GetSelectionAsText(FString& DisplayText, const FTextSelection& Selection) const;

	SLATE_API FTextSelection GetGraphemeAt(const FTextLocation& Location) const;

	SLATE_API FTextSelection GetWordAt(const FTextLocation& Location) const;

	/** Returns true if there are any font faces that are currently loading for the text layout . Else returns false. */
	SLATE_API bool AreAnyFontFacesLoading() const;
protected:

	SLATE_API FTextLayout();

	/**
	 * Calculates the text direction for each line based upon the current shaping method and document flow direction
	 * When changing the shaping method, or document flow direction, all the lines need to be dirtied (see DirtyAllLineModels(ELineModelDirtyState::TextBaseDirection))
	 */
	SLATE_API void CalculateTextDirection();

	/**
	 * Calculates the text direction for the given line based upon the current shaping method and document flow direction
	 */
	SLATE_API void CalculateLineTextDirection(FLineModel& LineModel) const;

	/**
	 * Calculates the visual justification for the given line view
	 */
	SLATE_API ETextJustify::Type CalculateLineViewVisualJustification(const FLineView& LineView) const;

	/**
	* Create the wrapping cache for the current text based upon the current scale
	* Each line keeps its own cached state, so needs to be cleared when changing the text within a line
	* When changing the scale, all the lines need to be cleared (see DirtyAllLineModels(ELineModelDirtyState::WrappingInformation))
	*/
	SLATE_API void CreateWrappingCache();

	/**
	* Create the wrapping cache for the given line based upon the current scale
	*/
	SLATE_API void CreateLineWrappingCache(FLineModel& LineModel);

	/**
	 * Flushes the text shaping cache for each line
	 */
	SLATE_API void FlushTextShapingCache();

	/**
	 * Flushes the text shaping cache for the given line
	 */
	SLATE_API void FlushLineTextShapingCache(FLineModel& LineModel);

	/**
	 * Set the given dirty flags on all line models in this layout
	 */
	SLATE_API void DirtyAllLineModels(const ELineModelDirtyState::Flags InDirtyFlags);

	/**
	* Clears the current layouts view information.
	*/
	SLATE_API void ClearView();

	/**
	 * Transform the given line model text based on the active transform policy.
	 * @note This is destructive to the model text!
	 */
	SLATE_API void TransformLineText(FLineModel& LineModel) const;

	/**
	* Notifies all Runs that we are beginning to generate a new layout.
	*/
	SLATE_API virtual void BeginLayout();

	/**
	* Notifies all Runs on the given line is beginning to have a new layout generated.
	*/
	SLATE_API void BeginLineLayout(FLineModel& LineModel);

	/**
	* Notifies all Runs that the layout has finished generating.
	*/
	SLATE_API virtual void EndLayout();

	/**
	* Notifies all Runs on the given line has finished having a new layout generated.
	*/
	SLATE_API void EndLineLayout(FLineModel& LineModel);

	/**
	 * Called to generate a new empty text run for this text layout
	 */
	virtual TSharedRef<IRun> CreateDefaultTextRun(const TSharedRef<FString>& NewText, const FTextRange& NewRange) const = 0;

	/**
	 * Called when it is detected during a layout pass that all async loading font faces for this text layout have finished loading.
	 */
	virtual void HandleAllFontFacesFinishLoading() {}
private:

	SLATE_API float GetWrappingDrawWidth() const;

	SLATE_API void FlowLayout();
	SLATE_API void MarginLayout();

	SLATE_API void FlowLineLayout(const int32 LineModelIndex, const float WrappingDrawWidth, TArray<TSharedRef<ILayoutBlock>>& SoftLine);

	SLATE_API void FlowHighlights();

	SLATE_API void JustifyLayout();

	SLATE_API void CreateLineViewBlocks( int32 LineModelIndex, const int32 StopIndex, const float WrappedLineWidth, const TOptional<float>& JustificationWidth, int32& OutRunIndex, int32& OutRendererIndex, int32& OutPreviousBlockEnd, TArray< TSharedRef< ILayoutBlock > >& OutSoftLine );

	SLATE_API FBreakCandidate CreateBreakCandidate( int32& OutRunIndex, FLineModel& Line, int32 PreviousBreak, int32 CurrentBreak );

	SLATE_API void GetAsTextAndOffsets(FString* const OutDisplayText, FTextOffsetLocations* const OutTextOffsetLocations) const;

	friend class FSlateEditableTextLayout;
	SLATE_API void GetAsTextAndOffsets(FString* const OutDisplayText, FTextOffsetLocations* const OutTextOffsetLocations, const TCHAR* LineTerminatorOverride) const;

protected:
	
	struct ETextLayoutDirtyState
	{
		typedef uint8 Flags;
		enum Enum
		{
			None = 0,
			Layout = 1<<0,
			Highlights = 1<<1, 
			All = Layout | Highlights,
		};
	};

	struct FTextLayoutSize
	{
		FTextLayoutSize()
			: DrawWidth(0.0f)
			, WrappedWidth(0.0f)
			, Height(0.0f)
		{
		}

		FVector2D GetDrawSize() const
		{
			return FVector2D(DrawWidth, Height);
		}

		FVector2D GetWrappedSize() const
		{
			return FVector2D(WrappedWidth, Height);
		}

		/** Width of the text layout, including any lines which extend beyond the wrapping boundaries (eg, lines with lots of trailing whitespace, or lines with no break candidates) */
		float DrawWidth;

		/** Width of the text layout after the text has been wrapped, and including the first piece of trailing whitespace for any given soft-wrapped line */
		float WrappedWidth;

		/** Height of the text layout */
		float Height;
	};

	/** The indices for all of the line views that require justification. */
	TSet< int32 > LineViewsToJustify;

	/** Information given to use by our an external source (typically our owner widget) to help identify who owns this text layout in the case of an error */
	TAttribute<FString> DebugSourceInfo;

	/** The models for the lines of text. A LineModel represents a single string with no manual breaks. */
	TArray< FLineModel > LineModels;

	/** The views for the lines of text. A LineView represents a single visual line of text. Multiple LineViews can map to the same LineModel, if for example wrapping occurs. */
	TArray< FLineView > LineViews;

	/** The array of unique font faces still loading for the text. */
	TArray<TWeakPtr<class FFreeTypeFace>> LoadingFontFaces;

	/** The size of the text layout that can actually be seen from the parent widget */
	FVector2D ViewSize;

	/** The scroll offset of the text layout from the parent widget */
	FVector2D ScrollOffset;

	/** The size of the margins to put about the text. This is an unscaled value. */
	FMargin Margin;

	/** The iterator to use to detect appropriate soft-wrapping points for lines */
	TSharedPtr<IBreakIterator> LineBreakIterator;

	/** The iterator to use to detect grapheme cluster boundaries */
	TSharedRef<IBreakIterator> GraphemeBreakIterator;

	/** The iterator to use to detect word boundaries */
	TSharedRef<IBreakIterator> WordBreakIterator;

	/** The final size of the text layout on screen. */
	FTextLayoutSize TextLayoutSize;

	/** How the text should be aligned with the margin. */
	ETextJustify::Type Justification;

	/** Unicode BiDi text detection */
	TUniquePtr<TextBiDi::ITextBiDi> TextBiDiDetection;

	int32 CachedLayoutGeneration = 0;

	/** The scale to draw the text at */
	float Scale;

	/** The width that the text should be wrap at. If 0 or negative no wrapping occurs. */
	float WrappingWidth;

	/** The percentage to modify a line height by. */
	float LineHeightPercentage;

	/** Extra height of the last line due to line height. */
	float OverHeight;

	/** Override for the text overflow policy. If unset, the style is used */
	TOptional<ETextOverflowPolicy> TextOverflowPolicyOverride;

	/** Whether parameters on the layout have changed which requires the view be updated. */
	ETextLayoutDirtyState::Flags DirtyFlags;

	/** The method used to shape the text within this layout */
	ETextShapingMethod TextShapingMethod;

	/** How should the text within this layout be flowed? */
	ETextFlowDirection TextFlowDirection;

	/** The wrapping policy used by this text layout. */
	ETextWrappingPolicy WrappingPolicy;

	/** The transform policy used by this text layout. */
	ETextTransformPolicy TransformPolicy;

	/** Whether or not line height should be applied to the last line. */
	bool ApplyLineHeightToBottomLine : 1;

	/** True if there were any font faces that were detected to be async loading in the previous layout pass. Else false. */
	bool bLoadingFontFacesOnPreviousLayout : 1;
};
