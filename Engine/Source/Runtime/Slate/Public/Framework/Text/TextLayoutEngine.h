// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

// Generic Text Layout
#include "Framework/Text/TextRunRenderer.h"
#include "Framework/Text/TextLineHighlight.h"

#include "Framework/Text/TextHitPoint.h"
#include "Framework/Text/ShapedTextCacheFwd.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/IRunRenderer.h"
#include "Framework/Text/ILineHighlighter.h"
#include "Framework/Text/ILayoutBlock.h"

#include "Framework/Text/TextLayout.h"
#include "Framework/Text/DefaultLayoutBlock.h"
#include "Framework/Text/WidgetLayoutBlock.h"


// Slate Specific Text Layout
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ISlateRunRenderer.h"
#include "Framework/Text/ISlateLineHighlighter.h"

#include "Framework/Text/SlateTextLayout.h"

#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/SlateImageRun.h"
#include "Framework/Text/SlateWidgetRun.h"
