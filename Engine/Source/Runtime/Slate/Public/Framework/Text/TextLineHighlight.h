// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class ILineHighlighter;

struct FTextLineHighlight
{
	FTextLineHighlight( int32 InLineIndex, const FTextRange& InRange, int32 InZOrder, const TSharedRef< ILineHighlighter >& InHighlighter )
		: LineIndex( InLineIndex )
		, Range( InRange )
		, ZOrder( InZOrder )
		, Highlighter( InHighlighter )
	{

	}

	bool operator==(const FTextLineHighlight& Other) const
	{
		return LineIndex == Other.LineIndex
			&& Range == Other.Range
			&& ZOrder == Other.ZOrder
			&& Highlighter == Other.Highlighter;
	}

	bool operator!=(const FTextLineHighlight& Other) const
	{
		return LineIndex != Other.LineIndex
			|| Range != Other.Range
			|| ZOrder != Other.ZOrder
			|| Highlighter != Other.Highlighter;
	}

	int32 LineIndex;
	FTextRange Range;
	int32 ZOrder;
	TSharedRef< ILineHighlighter > Highlighter;
};
