// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "Containers/UnrealString.h"

UE_DEPRECATED(4.23, "Including TextRange.h has been deprecated. Please include UnrealString.h in the Core module instead.")
inline void TextRangeHeaderDeprecatedWarning()
{
}

inline void TriggerTextRangeHeaderDeprecatedWarning()
{
	TextRangeHeaderDeprecatedWarning();
}
