// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IRunRenderer;

struct FTextRunRenderer
{
	FTextRunRenderer( int32 InLineIndex, const FTextRange& InRange, const TSharedRef< IRunRenderer >& InRenderer )
		: LineIndex( InLineIndex )
		, Range( InRange )
		, Renderer( InRenderer )
	{

	}

	bool operator==(const FTextRunRenderer& Other) const
	{
		return LineIndex == Other.LineIndex
			&& Range == Other.Range
			&& Renderer == Other.Renderer;
	}

	bool operator!=(const FTextRunRenderer& Other) const
	{
		return LineIndex != Other.LineIndex
			|| Range != Other.Range
			|| Renderer != Other.Renderer;
	}

	int32 LineIndex;
	FTextRange Range;
	TSharedRef< IRunRenderer > Renderer;
};
