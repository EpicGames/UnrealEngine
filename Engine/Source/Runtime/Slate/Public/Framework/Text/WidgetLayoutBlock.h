// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ILayoutBlock.h"

class FWidgetLayoutBlock : public ILayoutBlock
{
public:

	static TSharedRef< FWidgetLayoutBlock > Create( const TSharedRef< IRun >& InRun, const TSharedRef< SWidget >& InWidget, const FTextRange& InRange, FVector2D InSize, const FLayoutBlockTextContext& InTextContext, const TSharedPtr< IRunRenderer >& InRenderer )
	{
		return MakeShareable( new FWidgetLayoutBlock( InRun, InWidget, InRange, InSize, InTextContext, InRenderer ) );
	}

	virtual ~FWidgetLayoutBlock() = default;

	virtual TSharedRef< IRun > GetRun() const override { return Run; }

	virtual FTextRange GetTextRange() const override { return Range; }

	virtual FVector2D GetSize() const override { return Size; }

	virtual FLayoutBlockTextContext GetTextContext() const override { return TextContext; }

	virtual TSharedPtr< IRunRenderer > GetRenderer() const override { return Renderer; }

	TSharedRef< SWidget > GetWidget() const { return Widget; }

	virtual void SetLocationOffset( const FVector2D& InLocationOffset ) override { LocationOffset = InLocationOffset; }
	virtual FVector2D GetLocationOffset() const override { return LocationOffset; }

private:

	static TSharedRef< FWidgetLayoutBlock > Create( const FWidgetLayoutBlock& Block )
	{
		return MakeShareable( new FWidgetLayoutBlock( Block ) );
	}

	FWidgetLayoutBlock( const TSharedRef< IRun >& InRun, const TSharedRef< SWidget >& InWidget, const FTextRange& InRange, FVector2D InSize, const FLayoutBlockTextContext& InTextContext, const TSharedPtr< IRunRenderer >& InRenderer )
		: Run( InRun )
		, Widget( InWidget )
		, Range( InRange )
		, Size( InSize )
		, LocationOffset( ForceInitToZero )
		, TextContext( InTextContext )
		, Renderer( InRenderer )
	{

	}

	FWidgetLayoutBlock( const FWidgetLayoutBlock& Block )
		: Run( Block.Run )
		, Widget( Block.Widget )
		, Range( Block.Range )
		, Size( Block.Size )
		, LocationOffset( ForceInitToZero )
		, TextContext( Block.TextContext )
		, Renderer( Block.Renderer )
	{

	}

private:

	TSharedRef< IRun > Run;
	TSharedRef< SWidget > Widget;

	FTextRange Range;
	FVector2D Size;
	FVector2D LocationOffset;
	FLayoutBlockTextContext TextContext;
	TSharedPtr< IRunRenderer > Renderer;
};
