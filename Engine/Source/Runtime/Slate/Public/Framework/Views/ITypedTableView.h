// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "ITypedTableView.generated.h"

class ITableRow;

UENUM()
namespace ESelectionMode
{
	enum Type : int
	{
		/** Nothing can be selected and there is no hover cue for selection.  You can still handle mouse button events though. */
		None,

		/** A single item can be selected at once, or no item may be selected. */
		Single,

		/** A single item can be selected at once, or no item may be selected.  You can click the item to toggle selection on and off. */
		SingleToggle,

		/** Multiple items can be selected at the same time. */
		Multi
	};
}


/** Lists can exist as regular lists or as backing widgets for a tree. */
UENUM()
namespace ETableViewMode
{
	enum Type : int
	{
		List,
		Tile,
		Tree
	};
}


/** Interface for container widgets to talk to the owner list, tree, or grid. */
template<typename ItemType>
class ITypedTableView
{
public:
	/**
	 * Returns whether the table view uses selector focus.
	 *
	 * @return true if the table view uses selector focus; false otherwise.
	 */
	virtual bool Private_UsesSelectorFocus() const = 0;	

	/**
	 * Test if the item passed in has selector focus.
	 *
	 * @param InItem The item to test.
	 *
	 * @return true if the item has selector focus in this list; false otherwise.
	 */
	virtual bool Private_HasSelectorFocus( const ItemType& TheItem ) const = 0;

	/**
	 * Test if the current item is selected.
	 *
	 * @param InItem The item to test.
	 *
	 * @return true if the item is selected in this list; false otherwise.
	 */
	virtual bool Private_IsItemSelected( const ItemType& TheItem ) const = 0;

	/**
	 * Test if the current item can be selected or navigated to.
	 *
	 * @param InItem The item to test.
	 *
	 * @return true if the item is selectable or navigable; false otherwise.
	 */
	virtual bool Private_IsItemSelectableOrNavigable(const ItemType& TheItem) const = 0;

	/**
	 * Test if the current item should be highlighted. This is separate from hover highlights.
	 *
	 * @param TheItem The item to test.
	 *
	 * @return true if the item is marked as highlighted in this list; false otherwise.
	 */
	virtual bool Private_IsItemHighlighted(const ItemType& TheItem) const = 0;
	
	/**
	 * Set the selection state of an item. Does not cause an OnSelectionChanged()!
	 *
	 * @param InItem			The Item whose selection state to modify
	 * @param bSelected			true to select the item; false to unselect
	 * @param bWasUserDirected	true if the user specifically, directly selected this item; false otherwise
	 */
	virtual void Private_SetItemSelection( ItemType TheItem, bool bShouldBeSelected, bool bWasUserDirected = false ) = 0;

	/**
	 * Empty the selection set.
	 */
	virtual void Private_ClearSelection() = 0;

	/**
	 * Select a range of items from the current RangeSelectionStart to the SelectorItem
	 * Does not cause an OnSelectionChanged()!
	 * 
	 * @param SelectorItem  The endpoint item at which the selection should stop.
	 */
	virtual void Private_SelectRangeFromCurrentTo( ItemType SelectorItem ) = 0;

	/**
	 * Notify the ItemsWidget that we changed its selection.
	 * Triggers as OnSelectionChanged().
	 *
	 * @param SelectInfo Provides context on how the selection changed
	 */
	virtual void Private_SignalSelectionChanged( ESelectInfo::Type SelectInfo ) = 0;

	/**
	 * Given a Widget, find the corresponding data item.
	 * 
	 * @param TheWidget  A widget generated by the list view for some data item.
	 *
	 * @return The data item from which TheWidget was generated
	 */
	virtual const TObjectPtrWrapTypeOf<ItemType>* Private_ItemFromWidget( const ITableRow* TheWidget ) const = 0;

	/**
	 * Set whether some data item is expanded or not.
	 * 
	 * @param InItem         The item whose expansion state to control
	 * @param InExpandItem   If true the item should be expanded; otherwise collapsed
	 */
	virtual void Private_SetItemExpansion( ItemType TheItem, bool bShouldBeExpanded ) = 0;

	/**
	 * Called when the expander arrow for the given item is shift+clicked.
	 * 
	 * @param InItem         The item whose expansion state to control
	 * @param InExpandItem   If true the item should be expanded; otherwise collapsed
	 */
	virtual void Private_OnExpanderArrowShiftClicked( ItemType TheItem, bool bShouldBeExpanded ) = 0;

	/** 
	 * @param InItem   The data item whose expansion state to query
	 *
	 * @return true if the item is expanded; false otherwise.
	 */
	virtual bool Private_IsItemExpanded( const ItemType& TheItem ) const = 0;
	
	/**
	 * @param ItemIndexInList  The index of the data item in the linearized array.
	 *
	 * @return true if the item has children
	 */
	virtual bool Private_DoesItemHaveChildren( int32 ItemIndexInList ) const = 0;
	
	/** @return the number of items that are selected */
	virtual int32 Private_GetNumSelectedItems() const = 0;

	/**
	* Enable a soft highlight on the element. This is useful for explaining parent/child relationships without actually modifying selection. Unrelated to any hover highlights.
	*
	* @param TheItem         		The item whose highlighted state to change.
	* @param bShouldBeHighlighted   Enables a highlight on this item if true, otherwise disables the highlight.
	*/
	virtual void Private_SetItemHighlighted(ItemType TheItem, bool bShouldBeHighlighted) = 0;

	/**
	* Empty the set of highlighted items.
	*/
	virtual void Private_ClearHighlightedItems() = 0;

	/**
	 * @param ItemIndexInList  The index of the data item in the linearized array.
	 *
	 * @return Nesting level within the tree: 0 is root-level, 1 is children of root, etc.
	 */
	virtual int32 Private_GetNestingDepth( int32 ItemIndexInList ) const = 0;

	/**
	 * @return a TBitArray in which each true bit indicates that a
	 *           vertical wire is needed at this depth of the tree view.
	 */
	virtual const TBitArray<>& Private_GetWiresNeededByDepth( int32 ItemIndexInList ) const = 0;

	/** @return true if this item is the last direct descendant of its parent. */
	virtual bool Private_IsLastChild(int32 ItemIndexInList) const = 0;

	/** Called when the item is right-clicked when not right mouse scrolling */
	virtual void Private_OnItemRightClicked( ItemType TheItem, const FPointerEvent& MouseEvent ) = 0;

	/** Called when the item is clicked, and returns whether or not the event was handled */
	virtual bool Private_OnItemClicked(ItemType TheItem) = 0;

	/** Called when the item is double-clicked, and returns whether or not the event was handled */
	virtual bool Private_OnItemDoubleClicked( ItemType TheItem ) = 0;

	/** @return the selection mode of this TableView */
	virtual ESelectionMode::Type Private_GetSelectionMode() const = 0;

	/** @return the orientation of this TableView */
	virtual EOrientation Private_GetOrientation() const = 0;

	/** Is the list pending a refresh? */
	virtual bool Private_IsPendingRefresh() const = 0;

	/** @return All currently selected items in the table view */
	virtual TArray<ItemType> GetSelectedItems() const = 0;

	/**
	 * Find a widget for this item if it has already been constructed.
	 *
	 * @param InItem  The item for which to find the widget.
	 *
	 * @return A pointer to the corresponding widget if it exists; otherwise nullptr.
	*/
	virtual TSharedPtr<ITableRow> WidgetFromItem(const ItemType& InItem) const = 0;

	/** @return Is the tableview a tree or a list? */
	virtual ETableViewMode::Type GetTableViewMode() const = 0;
	
	/** @return a shared reference to the widgets; required for forwarding focus. */
	virtual TSharedRef<SWidget> AsWidget() = 0;

};
