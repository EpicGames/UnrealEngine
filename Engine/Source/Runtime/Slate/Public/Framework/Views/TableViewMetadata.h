// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/ISlateMetaData.h"
#include "ITypedTableView.h"

class ITableViewMetadata : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(ITableViewMetadata, ISlateMetaData);

	/** Returns all currently constructed rows that represent selected items */
	virtual TArray<TSharedPtr<ITableRow>> GatherSelectedRows() const = 0;
};

template <typename ItemType>
class TTableViewMetadata : public ITableViewMetadata
{
public:
	SLATE_METADATA_TYPE(TTableViewMetadata, ITableViewMetadata);

	TTableViewMetadata(TSharedRef<ITypedTableView<ItemType>> InOwnerTableView)
		: OwnerTableView(InOwnerTableView)
	{}

	virtual TArray<TSharedPtr<ITableRow>> GatherSelectedRows() const override
	{
		TArray<TSharedPtr<ITableRow>> SelectedRows;
		if (TSharedPtr<ITypedTableView<ItemType>> PinnedTableView = OwnerTableView.Pin())
		{
			for (const ItemType& Item : PinnedTableView->GetSelectedItems())
			{
				if (TSharedPtr<ITableRow> TableRow = PinnedTableView->WidgetFromItem(Item))
				{
					SelectedRows.Add(TableRow);
				}
			}
		}
		return SelectedRows;
	}

private:
	TWeakPtr<ITypedTableView<ItemType>> OwnerTableView;
};