// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectBase.h"
#include "UObject/GCObject.h"
#include "UObject/FieldPath.h"

class ITableRow;
struct FSparseItemInfo;

/**
 * Lists/Trees only work with shared pointer types, and UObjbectBase*.
 * Type traits to ensure that the user does not accidentally make a List/Tree of value types.
 */
template <typename T, typename Enable = void>
struct TIsValidListItem
{
	enum
	{
		Value = false
	};
};
template <typename T>
struct TIsValidListItem<TSharedRef<T, ESPMode::NotThreadSafe>>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<TSharedRef<T, ESPMode::ThreadSafe>>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<TSharedPtr<T, ESPMode::NotThreadSafe>>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<TSharedPtr<T, ESPMode::ThreadSafe>>
{
	enum
	{
		Value = true
	};
};
template <typename T, ESPMode Mode>
struct TIsValidListItem<TWeakPtr<T, Mode>>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<T*, typename TEnableIf<TPointerIsConvertibleFromTo<T, UObjectBase>::Value>::Type>
{
	enum
	{
		Value = true
	};
};

template <typename T>
struct TIsValidListItem<const T*, typename TEnableIf<TPointerIsConvertibleFromTo<T, UObjectBase>::Value>::Type>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<TObjectPtr<T>>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<TWeakObjectPtr<T>>
{
	enum
	{
		Value = true
	};
};
template <typename T>
struct TIsValidListItem<TFieldPath<T>>
{
	enum
	{
		Value = true
	};
};
template <>
struct TIsValidListItem<FName>
{
	enum
	{
		Value = true
	};
};

/**
 * Furthermore, ListViews of TSharedPtr<> work differently from lists of UObject*.
 * ListTypeTraits provide the specialized functionality such as pointer testing, resetting,
 * and optional serialization for UObject garbage collection.
 */
template <typename T, typename Enable=void> struct TListTypeTraits
{
	static_assert(TIsValidListItem<T>::Value, "Item type T must be a UObjectBase pointer, TFieldPath, TSharedRef, TSharedPtr or FName.");
};


/**
 * Pointer-related functionality (e.g. setting to nullptr, testing for nullptr) specialized for SharedPointers.
 */
template <typename T> struct TListTypeTraits< TSharedPtr<T, ESPMode::NotThreadSafe> >
{
public:
	typedef TSharedPtr<T> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TSharedPtr<T, ESPMode::NotThreadSafe>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TSharedPtr<T, ESPMode::NotThreadSafe>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<TSharedPtr<T, ESPMode::NotThreadSafe>>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TSharedPtr<T> >&, 
		TSet< TSharedPtr<T> >&, 
		TMap< const U*, TSharedPtr<T> >& )
	{
	}

	static bool IsPtrValid( const TSharedPtr<T>& InPtr )
	{
		return InPtr.IsValid();
	}

	static void ResetPtr( TSharedPtr<T>& InPtr )
	{
		InPtr.Reset();
	}

	static TSharedPtr<T> MakeNullPtr()
	{
		return TSharedPtr<T>(nullptr);
	}

	static TSharedPtr<T> NullableItemTypeConvertToItemType( const TSharedPtr<T>& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump( TSharedPtr<T> InPtr )
	{
		return InPtr.IsValid() ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)InPtr.Get()) : FString("nullptr");
	}

	class SerializerType{};
};


template <typename T> struct TListTypeTraits< TSharedPtr<T, ESPMode::ThreadSafe> >
{
public:
	typedef TSharedPtr<T, ESPMode::ThreadSafe> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TSharedPtr<T, ESPMode::ThreadSafe>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TSharedPtr<T, ESPMode::ThreadSafe>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<TSharedPtr<T, ESPMode::ThreadSafe>>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TSharedPtr<T, ESPMode::ThreadSafe> >&, 
		TSet< TSharedPtr<T, ESPMode::ThreadSafe> >&, 
		TMap< const U*, TSharedPtr<T, ESPMode::ThreadSafe> >& WidgetToItemMap)
	{
	}

	static bool IsPtrValid( const TSharedPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		return InPtr.IsValid();
	}

	static void ResetPtr( TSharedPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		InPtr.Reset();
	}

	static TSharedPtr<T, ESPMode::ThreadSafe> MakeNullPtr()
	{
		return TSharedPtr<T, ESPMode::ThreadSafe>(nullptr);
	}

	static TSharedPtr<T, ESPMode::ThreadSafe> NullableItemTypeConvertToItemType( const TSharedPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump( TSharedPtr<T, ESPMode::ThreadSafe> InPtr )
	{
		return InPtr.IsValid() ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)InPtr.Get()) : FString("nullptr");
	}

	class SerializerType{};
};


template <typename T> struct TListTypeTraits< TSharedRef<T, ESPMode::NotThreadSafe> >
{
public:
	typedef TSharedPtr<T> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TSharedRef<T, ESPMode::NotThreadSafe>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TSharedRef<T, ESPMode::NotThreadSafe>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<TSharedRef<T, ESPMode::NotThreadSafe>>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TSharedRef<T> >&, 
		TSet< TSharedRef<T> >&, 
		TMap< const U*, TSharedRef<T> >& )
	{
	}

	static bool IsPtrValid( const TSharedPtr<T>& InPtr )
	{
		return InPtr.IsValid();
	}

	static void ResetPtr( TSharedPtr<T>& InPtr )
	{
		InPtr.Reset();
	}

	static TSharedPtr<T> MakeNullPtr()
	{
		return TSharedPtr<T>(nullptr);
	}

	static TSharedRef<T> NullableItemTypeConvertToItemType( const TSharedPtr<T>& InPtr )
	{
		return InPtr.ToSharedRef();
	}

	static FString DebugDump( TSharedRef<T> InPtr )
	{
		return FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)&InPtr.Get());
	}

	class SerializerType{};
};


template <typename T> struct TListTypeTraits< TSharedRef<T, ESPMode::ThreadSafe> >
{
public:
	typedef TSharedPtr<T, ESPMode::ThreadSafe> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TSharedRef<T, ESPMode::ThreadSafe>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TSharedRef<T, ESPMode::ThreadSafe>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<TSharedRef<T, ESPMode::ThreadSafe>>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TSharedRef<T, ESPMode::ThreadSafe> >&, 
		TSet< TSharedRef<T, ESPMode::ThreadSafe> >&,
		TMap< const U*, TSharedRef<T, ESPMode::ThreadSafe> >&)
	{
	}

	static bool IsPtrValid( const TSharedPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		return InPtr.IsValid();
	}

	static void ResetPtr( TSharedPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		InPtr.Reset();
	}

	static TSharedPtr<T, ESPMode::ThreadSafe> MakeNullPtr()
	{
		return TSharedPtr<T, ESPMode::ThreadSafe>(nullptr);
	}

	static TSharedRef<T, ESPMode::ThreadSafe> NullableItemTypeConvertToItemType( const TSharedPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		return InPtr.ToSharedRef();
	}

	static FString DebugDump( TSharedRef<T, ESPMode::ThreadSafe> InPtr )
	{
		return FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)&InPtr.Get());
	}

	class SerializerType{};
};

template <typename T> struct TListTypeTraits< TWeakPtr<T, ESPMode::NotThreadSafe> >
{
public:
	typedef TWeakPtr<T> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TWeakPtr<T, ESPMode::NotThreadSafe>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TWeakPtr<T, ESPMode::NotThreadSafe>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<TWeakPtr<T, ESPMode::NotThreadSafe>>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TWeakPtr<T> >&, 
		TSet< TWeakPtr<T> >&, 
		TMap< const U*, TWeakPtr<T> >& )
	{
	}

	static bool IsPtrValid( const TWeakPtr<T>& InPtr )
	{
		return InPtr.Pin().IsValid();
	}

	static void ResetPtr( TWeakPtr<T>& InPtr )
	{
		InPtr = nullptr;
	}

	static TWeakPtr<T> MakeNullPtr()
	{
		return TWeakPtr<T>();
	}

	static TWeakPtr<T> NullableItemTypeConvertToItemType( const TWeakPtr<T>& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump( TWeakPtr<T> InPtr )
	{
		return InPtr.IsValid() ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)InPtr.Get()) : FString("nullptr");
	}

	class SerializerType{};
};


template <typename T> struct TListTypeTraits< TWeakPtr<T, ESPMode::ThreadSafe> >
{
public:
	typedef TWeakPtr<T, ESPMode::ThreadSafe> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TWeakPtr<T, ESPMode::ThreadSafe>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TWeakPtr<T, ESPMode::ThreadSafe>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<TWeakPtr<T, ESPMode::ThreadSafe>>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TWeakPtr<T, ESPMode::ThreadSafe> >&, 
		TSet< TWeakPtr<T, ESPMode::ThreadSafe> >&, 
		TMap< const U*, TWeakPtr<T, ESPMode::ThreadSafe> >& WidgetToItemMap)
	{
	}

	static bool IsPtrValid( const TWeakPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		return InPtr.Pin().IsValid();
	}

	static void ResetPtr( TWeakPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		InPtr = nullptr;
	}

	static TWeakPtr<T, ESPMode::ThreadSafe> MakeNullPtr()
	{
		return TWeakPtr<T, ESPMode::ThreadSafe>(nullptr);
	}

	static TWeakPtr<T, ESPMode::ThreadSafe> NullableItemTypeConvertToItemType( const TWeakPtr<T, ESPMode::ThreadSafe>& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump( TWeakPtr<T, ESPMode::ThreadSafe> InPtr )
	{
		return InPtr.IsValid() ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)InPtr.Pin().Get()) : FString("nullptr");
	}

	class SerializerType{};
};

/**
 * Pointer-related functionality (e.g. setting to nullptr, testing for nullptr) specialized for SharedPointers.
 */
template <typename T> struct TListTypeTraits< TWeakObjectPtr<T> >
{
public:
	typedef TWeakObjectPtr<T> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TWeakObjectPtr<T>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TWeakObjectPtr<T>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs< TWeakObjectPtr<T> >;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< TWeakObjectPtr<T> >&,
		TSet< TWeakObjectPtr<T> >&,
		TMap< const U*, TWeakObjectPtr<T> >&)
	{
	}

	static bool IsPtrValid( const TWeakObjectPtr<T>& InPtr )
	{
		return InPtr.IsValid();
	}

	static void ResetPtr( TWeakObjectPtr<T>& InPtr )
	{
		InPtr.Reset();
	}

	static TWeakObjectPtr<T> MakeNullPtr()
	{
		return nullptr;
	}

	static TWeakObjectPtr<T> NullableItemTypeConvertToItemType( const TWeakObjectPtr<T>& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump( TWeakObjectPtr<T> InPtr )
	{
		T* ObjPtr = InPtr.Get();
		return ObjPtr ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT " [%s]"), (UPTRINT)ObjPtr, *ObjPtr->GetName()) : FString("nullptr");
	}

	class SerializerType{};
};


/**
 * Pointer-related functionality for TObjectPtr<T>.
 */
template <typename T> struct TListTypeTraits< TObjectPtr<T> >
{
public:
	typedef TObjectPtr<T> NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<TObjectPtr<T>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TObjectPtr<T>, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs< TObjectPtr<T> >;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector& Collector,
		TArray<TObjectPtr<T>>& ItemsWithGeneratedWidgets,
		TSet<TObjectPtr<T>>& SelectedItems,
		TMap< const U*, TObjectPtr<T> >& WidgetToItemMap)
	{
		// Serialize generated items
		Collector.AddReferencedObjects(ItemsWithGeneratedWidgets);

		// Serialize the map Value. We only do it for the WidgetToItemMap because we know that both maps are updated at the same time and contains the same objects
		// Also, we cannot AddReferencedObject to the Keys of the ItemToWidgetMap or we end up with keys being set to 0 when the UObject is destroyed which generate an invalid id in the map.
		for (auto& It : WidgetToItemMap)
		{
			Collector.AddReferencedObject(It.Value);
		}

		// Serialize the selected items
		Collector.AddReferencedObjects(SelectedItems);
	}

	static bool IsPtrValid( const TObjectPtr<T>& InPtr )
	{
		return InPtr != nullptr;
	}

	static void ResetPtr(TObjectPtr<T>& InPtr )
	{
		InPtr = nullptr;
	}

	static TObjectPtr<T> MakeNullPtr()
	{
		return nullptr;
	}

	static TObjectPtr<T> NullableItemTypeConvertToItemType( const TObjectPtr<T>& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump(TObjectPtr<T> InPtr)
	{
		return InPtr ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT " [%s]"), (UPTRINT)InPtr.Get(), *InPtr->GetName()) : FString("nullptr");
	}

	typedef FGCObject SerializerType;
};


/**
 * Lists of pointer types only work if the pointers are deriving from UObject*.
 * In addition to testing and setting the pointers to nullptr, Lists of UObjects
 * will serialize the objects they are holding onto.
 */
template <typename T>
struct TListTypeTraits<T*, typename TEnableIf<TPointerIsConvertibleFromTo<T, UObjectBase>::Value>::Type>
{
public:
	typedef T* NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<T*, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<T*, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<T*>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector& Collector, 
		TArray<TObjectPtr<T>>& ItemsWithGeneratedWidgets, 
		TSet<TObjectPtr<T>>& SelectedItems, 
		TMap< const U*, TObjectPtr<T> >& WidgetToItemMap)
	{
		// Serialize generated items
		Collector.AddReferencedObjects(ItemsWithGeneratedWidgets);
		
		// Serialize the map Value. We only do it for the WidgetToItemMap because we know that both maps are updated at the same time and contains the same objects
		// Also, we cannot AddReferencedObject to the Keys of the ItemToWidgetMap or we end up with keys being set to 0 when the UObject is destroyed which generate an invalid id in the map.
		for (auto& It : WidgetToItemMap)
		{
			Collector.AddReferencedObject(It.Value);
		}

		// Serialize the selected items
		Collector.AddReferencedObjects(SelectedItems);
	}

	static bool IsPtrValid( T* InPtr ) { return InPtr != nullptr; }

	static void ResetPtr( T*& InPtr ) { InPtr = nullptr; }

	static T* MakeNullPtr() { return nullptr; }

	static T* NullableItemTypeConvertToItemType( T* InPtr ) { return InPtr; }

	static FString DebugDump( T* InPtr )
	{
		return InPtr ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT " [%s]"), (UPTRINT)InPtr, *InPtr->GetName()) : FString("nullptr");
	}

	typedef FGCObject SerializerType;
};

template <typename T>
struct TListTypeTraits<const T*, typename TEnableIf<TPointerIsConvertibleFromTo<T, UObjectBase>::Value>::Type>
{
public:
	typedef const T* NullableType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<const T*, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<const T*, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<const T*>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector& Collector, 
		TArray<TObjectPtr<const T>>& ItemsWithGeneratedWidgets, 
		TSet<TObjectPtr<const T>>& SelectedItems,
		TMap< const U*, TObjectPtr<const T> >& WidgetToItemMap)
	{
		// Serialize generated items
		Collector.AddReferencedObjects(ItemsWithGeneratedWidgets);

		// Serialize the map Value. We only do it for the WidgetToItemMap because we know that both maps are updated at the same time and contains the same objects
		// Also, we cannot AddReferencedObject to the Keys of the ItemToWidgetMap or we end up with keys being set to 0 when the UObject is destroyed which generate an invalid id in the map.
		for (auto& It : WidgetToItemMap)
		{
			Collector.AddReferencedObject(It.Value);
		}

		// Serialize the selected items
		Collector.AddReferencedObjects(SelectedItems);
	}

	static bool IsPtrValid( const T* InPtr ) { return InPtr != nullptr; }

	static void ResetPtr( const T*& InPtr ) { InPtr = nullptr; }

	static const T* MakeNullPtr() { return nullptr; }

	static const T* NullableItemTypeConvertToItemType( const T* InPtr ) { return InPtr; }

	static FString DebugDump( const T* InPtr )
	{
		return InPtr ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT " [%s]"), (UPTRINT)InPtr, *InPtr->GetName()) : FString("nullptr");
	}

	typedef FGCObject SerializerType;
};


/**
 * Pointer-related functionality (e.g. setting to nullptr, testing for nullptr) specialized for TFieldPaths.
 */
template <typename T> struct TListTypeTraits< TFieldPath<T> >
{
public:
	typedef TFieldPath<T> NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<TFieldPath<T>, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<TFieldPath<T>, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs< TFieldPath<T> >;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector& Collector,
		TArray< TFieldPath<T> >& ItemsWithGeneratedWidgets,
		TSet< TFieldPath<T> >& SelectedItems,
		TMap< const U*, TFieldPath<T> >& WidgetToItemMap)
	{
		for (TFieldPath<T>& Item : ItemsWithGeneratedWidgets)
		{
			if (Item != nullptr)
			{
				Item->AddReferencedObjects(Collector);
			}
		}

		// Serialize the map Value. We only do it for the WidgetToItemMap because we know that both maps are updated at the same time and contains the same objects
		// Also, we cannot AddReferencedObject to the Keys of the ItemToWidgetMap or we end up with keys being set to 0 when the UObject is destroyed which generate an invalid id in the map.
		for (auto& It : WidgetToItemMap)
		{
			if (It.Value != nullptr)
			{
				It.Value->AddReferencedObjects(Collector);
			}
		}

		// Serialize the selected items
		for (TFieldPath<T>& Item : SelectedItems)
		{
			if (Item != nullptr)
			{
				Item->AddReferencedObjects(Collector);
			}
		}
	}

	static bool IsPtrValid(const TFieldPath<T>& InPtr)
	{
		return !!InPtr.Get();
	}

	static void ResetPtr(TFieldPath<T>& InPtr)
	{
		InPtr.Reset();
	}

	static TFieldPath<T> MakeNullPtr()
	{
		return nullptr;
	}

	static TFieldPath<T> NullableItemTypeConvertToItemType(const TFieldPath<T>& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(TFieldPath<T> InPtr)
	{
		T* ObjPtr = InPtr.Get();
		return ObjPtr ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT " [%s]"), (UPTRINT)ObjPtr, *ObjPtr->GetName()) : FString("nullptr");
	}

	class SerializerType {};
};

/**
 * Functionality (e.g. setting the value, testing invalid value) specialized for FName.
 */
template <>
struct TListTypeTraits<FName>
{
public:
	typedef FName NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<FName, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<FName, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<FName>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector& Collector,
		TArray<FName>& ItemsWithGeneratedWidgets,
		TSet<FName>& SelectedItems,
		TMap<const U*, FName>& WidgetToItemMap)
	{
	}

	static bool IsPtrValid(const FName& InValue)
	{
		return !InValue.IsNone();
	}

	static void ResetPtr(FName& InValue)
	{
		InValue = FName();
	}

	static FName MakeNullPtr()
	{
		return FName();
	}

	static FName NullableItemTypeConvertToItemType(const FName& InValue)
	{
		return InValue;
	}

	static FString DebugDump(FName InValue)
	{
		return InValue.ToString();
	}

	class SerializerType {};
};
