// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/IFilter.h"
#include "Widgets/Views/STreeView.h"

/** Handles filtering and expanding a TreeView using an IFilter implementation. */
template<typename ItemType>
class TreeFilterHandler
{
public:
	typedef typename TSlateDelegates< ItemType >::FOnGetChildren FOnGetChildren;

public:

	TreeFilterHandler()
	{
		TreeView = nullptr;
		Filter = nullptr;
		RootItems = nullptr;
		TreeRootItems = nullptr;
		bIsEnabled = false;
		bWasEnabled = false;
	}

	/** Sets the TreeView to be filtered. */
	void SetTreeView(STreeView<ItemType>* InTreeView)
	{
		TreeView = InTreeView;
	}

	/** Sets the filter which will be used to filter the items in the TreeView. */
	void SetFilter(IFilter<ItemType>* InFilter)
	{
		Filter = InFilter;
	}

	/** 
	 * Sets the root data arrays for the tree.
	 * @param RootItems		An array of source root items to be displayed by the tree.
	 * @param TreeRootItems	The array of root items which are set as the ItemsSource of the tree. This array will be modified to reflect any filtering.
	 */
	void SetRootItems(TArray<ItemType>* InRootItems, TArray<ItemType>* InTreeRootItems)
	{
		RootItems = InRootItems;
		TreeRootItems = InTreeRootItems;
	}

	/**
	 * Sets the delegate which should be used for traversing the children of the nodes in
	 * the tree.
	 */
	void SetGetChildrenDelegate(FOnGetChildren InGetChildrenDelegate)
	{
		GetChildrenDelegate = InGetChildrenDelegate;
	}

	/** Gets whether or not filtering is enabled for the TreeView. */
	bool GetIsEnabled()
	{
		return bIsEnabled;
	}

	/** 
	 * Sets whether or not filtering the tree is enabled. Calling this DOES NOT refresh the tree,
	 * this must be done by calling RefreshAndFilterTree.
	 */
	void SetIsEnabled(bool bInIsEnabled)
	{
		bIsEnabled = bInIsEnabled;
	}

	/** Removes any cached references to an item. */
	void RemoveCachedItem(ItemType ItemToRemove)
	{
		PrefilterExpandedItems.Remove(ItemToRemove);
	}

	/**
	 * Refreshes the associated TreeView, filtering items if the filter is enabled. When filtering,
	 * the expanded state of the items is saved, and when the filter is cleared the pre-filter expand
	 * state is restored.
	 */
	void RefreshAndFilterTree()
	{
		if (TreeView && Filter && RootItems && TreeRootItems && GetChildrenDelegate.IsBound())
		{
			TreeRootItems->Empty();
			ItemsPassingFilter.Empty();
			if (bIsEnabled)
			{
				if (!bWasEnabled)
				{
					PrefilterExpandedItems.Empty();
					TreeView->GetExpandedItems(PrefilterExpandedItems);
				}
				for (auto RootItem : *RootItems)
				{
					if (ExpandAndCacheMatches(RootItem))
					{
						TreeRootItems->Add(RootItem);
					}
				}
			}
			else
			{
				if (bWasEnabled)
				{
					TreeView->ClearExpandedItems();
					for (auto PrefilterExpandedItem : PrefilterExpandedItems)
					{
						TreeView->SetItemExpansion(PrefilterExpandedItem, true);
					}
					PrefilterExpandedItems.Empty();
				}
				for (auto RootItem : *RootItems)
				{
					TreeRootItems->Add(RootItem);
				}
			}

			TreeView->RequestTreeRefresh();
			bWasEnabled = bIsEnabled;
		}
	}

	/**
	 * Gets the children for a node in the tree which pass the filter if filtering is enabled, or
	 * gets all children if the filter is not enabled. In order for filtering to work correctly this
	 * function must be used for the TreeView's OnGetChildren delegate.
	 */
	void OnGetFilteredChildren(ItemType Parent, TArray<ItemType>& FilteredChildren)
	{
		if (bIsEnabled)
		{
			TArray<ItemType> AllChildren;
			GetChildrenDelegate.Execute(Parent, AllChildren);
			for (auto Child : AllChildren)
			{
				if (ItemsPassingFilter.Contains(Child))
				{
					FilteredChildren.Add(Child);
				}
			}
		}
		else
		{
			GetChildrenDelegate.Execute(Parent, FilteredChildren);
		}
	}

private:
	/**
	 * Recursively checks for nodes in the tree which match the filter using depth first
	 * traversal. Any nodes which match are cached, and all nodes leading to matches are expanded. 
	 */
	bool ExpandAndCacheMatches(ItemType Parent)
	{
		bool bChildMatches = false;
		TArray<ItemType> AllChildren;
		GetChildrenDelegate.Execute(Parent, AllChildren);
		for (auto Child : AllChildren)
		{
			bChildMatches |= ExpandAndCacheMatches(Child);
		}

		bool bParentMatches = Filter->PassesFilter(Parent);

		if (bChildMatches || bParentMatches)
		{
			TreeView->SetItemExpansion(Parent, true);
			ItemsPassingFilter.Add(Parent);
			return true;
		}

		return false;
	}

private:
	/** The tree view to be filtered. */
	STreeView<ItemType>* TreeView;

	/** The filter to use for filtering. */
	IFilter<ItemType>* Filter;

	/** The source collection of root items for the tree. */
	TArray<ItemType>* RootItems;

	/** The collection of root items which is being displayed by the tree. */
	TArray<ItemType>* TreeRootItems;

	/** A delegate to get the children for a node in the tree. */
	FOnGetChildren GetChildrenDelegate;

	/** A cache of the items which passed the filter. */
	TSet<ItemType> ItemsPassingFilter;

	/** A set of the items which were expanded in the tree before any filtering. */
	TSet<ItemType> PrefilterExpandedItems;

	/** Whether or not the collection will be filtered on refresh. */
	bool bIsEnabled;

	/** Whether or not the tree was filtered the last time it was refreshed. */
	bool bWasEnabled;
};