// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_NAVIGATION_DEBUGGING
#include "Input/HittestGrid.h"
#include "Styling/SlateStyle.h"

class FSlateUser;
class FSlateWindowElementList;
class FWidgetPath;
class SWidget;
class SWindow;

class FDelegateHandle;
struct FGeometry;

class FNavigationDebuggingStyle : public FSlateStyleSet
{
public:
	enum class EArrowType
	{
		Focus,
		Candidate,
		Target
	};

	FLinearColor GetFocusedWidgetColor() const;
	FLinearColor GetTargetWidgetColor() const;

	const FSlateBrush* GetArrowBrush(const EUINavigation InNavigation, const EArrowType InArrow) const;

	static FNavigationDebuggingStyle& Get();

private:
	FNavigationDebuggingStyle();
	FNavigationDebuggingStyle(const FNavigationDebuggingStyle&) = delete;
	FNavigationDebuggingStyle& operator=(const FNavigationDebuggingStyle&) = delete;

	virtual ~FNavigationDebuggingStyle() override;

private:
	TMap<EUINavigation, TMap<EArrowType, const FSlateBrush*>> NavigationArrowBrushes;
};

struct FSlateUserNavigationData
{
	FSlateUserNavigationData() = delete;
	FSlateUserNavigationData(const FSlateUserNavigationData&) = delete;
	FSlateUserNavigationData& operator=(const FSlateUserNavigationData&) = delete;

	FSlateUserNavigationData(FSlateUser& InSlateUser);

public:
	bool CanPaint() const;

public:
	TSharedRef<FWidgetPath> FocusedWidgetPath;
	TSharedPtr<SWindow> WindowWidget;
	TSharedPtr<SWidget> ViewportWidget;
	TSharedPtr<SWidget> FocusedWidget;
	FGeometry WindowGeometry;
};

struct FSlateNavigationDebugger
{
	SLATE_API static FSlateNavigationDebugger& Get();

	SLATE_API int32 Paint(FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

private:
	FSlateNavigationDebugger() = default;
	FSlateNavigationDebugger(const FSlateNavigationDebugger&) = delete;
	FSlateNavigationDebugger& operator=(const FSlateNavigationDebugger&) = delete;

private:
	int32 PaintForUser(FSlateUser& InSlateUser, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	int32 PaintCandidateWidgets(const FSlateUserNavigationData& InNavigationData, const TArray<FHittestGrid::FDebuggingFindNextFocusableWidgetArgs>& InCandidateWidgets, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	int32 PaintFocusTargetArrowsAndLines(const TPair<EUINavigation, TSharedRef<SWidget>>& FocusedToDestination, const FSlateUserNavigationData& InNavigationData, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	void PaintGeometryOutline(const FGeometry& InGeometry, const FLinearColor& InColor, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	void PaintDashedLine(const FSlateUserNavigationData& InNavigationData, TArray<FVector2f>&& PointsToConnect, const FLinearColor& InColor, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	// Returns Arrow Geometry
	FGeometry PaintOnScreenWidgetArrow(const FSlateUserNavigationData& InNavigationData, const EUINavigation InNavDirection, const FGeometry& InWidgetGeometry, const FNavigationDebuggingStyle::EArrowType InArrowType, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	// Returns Arrow Geometry
	FGeometry PaintOffScreenWidgetArrow(const FSlateUserNavigationData& InNavigationData, const EUINavigation InNavDirection, TSharedRef<const SWidget> InWidget, const FNavigationDebuggingStyle::EArrowType InArrowType, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

private:
	TMap<EUINavigation, TSharedRef<SWidget>> SimulateNavigationDirection(const FSlateUserNavigationData& InNavigationData, const int32 SlateUserIndex) const;
	void SetListenToHitTestGrid(const bool bBind, TArray<FHittestGrid::FDebuggingFindNextFocusableWidgetArgs>& InContainer, FDelegateHandle& InHandle) const;
};

#endif // WITH_SLATE_NAVIGATION_DEBUGGING