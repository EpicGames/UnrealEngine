// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "Misc/Timespan.h"
#include "Core.h"
#include "CoreUObject.h"
#include "Json.h"
#include "SlateCore.h"
#include "SlateOptMacros.h"

#include "Framework/Text/PlatformTextField.h"

#include "InputCore.h"


#include "Framework/SlateDelegates.h"
#include "SlateFwd.h"

// Application
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/NavigationConfig.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"

// Commands
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"

// Legacy
#include "Widgets/SWeakWidget.h"
#include "Framework/Text/TextRunRenderer.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/TextHitPoint.h"
#include "Framework/Text/ShapedTextCacheFwd.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/IRunRenderer.h"
#include "Framework/Text/ILineHighlighter.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/DefaultLayoutBlock.h"
#include "Framework/Text/WidgetLayoutBlock.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ISlateRunRenderer.h"
#include "Framework/Text/ISlateLineHighlighter.h"
#include "Framework/Text/SlateTextLayout.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/SlateImageRun.h"
#include "Framework/Text/SlateWidgetRun.h"
#include "Framework/Text/TextLayoutEngine.h"
#include "Widgets/SPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SSpinningImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Text/ITextDecorator.h"
#include "Framework/Text/TextDecorators.h"
#include "Framework/Text/SlateTextLayoutFactory.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/Application/IMenu.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/Layout/SMenuOwner.h"
////
#include "Framework/MultiBox/MultiBox.h"
////
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Widgets/Text/ISlateEditableTextWidget.h"
#endif
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SScrollBarTrack.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Notifications/SErrorHint.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Widgets/Layout/SSplitter.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Framework/Views/ITypedTableView.h"
#include "Framework/Layout/InertialScrollManager.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SViewport.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"

///
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SComboBox.h"

// Docking Framework
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutService.h"

