// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/**
 * This file exists for backward compatibility. Slate.h has been split from one monolithic header
 * into SlateBasics.h and SlateExtras.h. Over time more files will be shifted from SlateBasic.h to
 * SlateExtras.h so older code using Slate.h will not be affected.
 * 
 * However, no one should include SlateExtras.h. It exists purely to assist migration of legacy projects.
 */

#include "SlateOptMacros.h"
#include "Framework/Application/IPlatformTextField.h"
#include "Framework/SlateDelegates.h"
#include "SlateFwd.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/NavigationConfig.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/SWeakWidget.h"
#include "Framework/Text/TextRunRenderer.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/TextHitPoint.h"
#include "Framework/Text/ShapedTextCacheFwd.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/IRunRenderer.h"
#include "Framework/Text/ILineHighlighter.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/DefaultLayoutBlock.h"
#include "Framework/Text/WidgetLayoutBlock.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ISlateRunRenderer.h"
#include "Framework/Text/ISlateLineHighlighter.h"
#include "Framework/Text/SlateTextLayout.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/SlateImageRun.h"
#include "Framework/Text/SlateWidgetRun.h"
#include "Framework/Text/TextLayoutEngine.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SSpinningImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Text/ITextDecorator.h"
#include "Framework/Text/TextDecorators.h"
#include "Framework/Text/SlateTextLayoutFactory.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/Application/IMenu.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/Layout/SMenuOwner.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Widgets/Text/ISlateEditableTextWidget.h"
#endif
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SScrollBarTrack.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Notifications/SErrorHint.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Widgets/Layout/SSplitter.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Framework/Views/ITypedTableView.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SViewport.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SComboBox.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutService.h"
#include "SlateBasics.h"

// Legacy
#include "Widgets/Layout/SMissingWidget.h"
#include "Framework/Layout/SlateScrollHelper.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Commands/UICommandDragDropOp.h"
#include "Framework/Layout/InertialScrollManager.h"
#include "Framework/Commands/GenericCommands.h"

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SVolumeControl.h"
#include "Widgets/Colors/SColorSpectrum.h"
#include "Widgets/Colors/SColorWheel.h"
///
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
///
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SRichTextHyperlink.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Input/STextComboPopup.h"
#include "Widgets/Input/SExpandableButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableComboBox.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SVirtualKeyboardEntry.h"
#include "Framework/Layout/ScrollyZoomy.h"
#include "Widgets/Layout/SSafeZone.h"
#include "Framework/MarqueeRect.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SVirtualJoystick.h"

// Docking Framework
#include "Widgets/Docking/SDockTab.h"