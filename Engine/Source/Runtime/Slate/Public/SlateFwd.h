// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector.h"

class SSearchBox;
class SVolumeControl;
class SColorSpectrum;
class SColorWheel;

class FToolBarButtonBlock;
class SToolBarButtonBlock;
class FToolBarComboButtonBlock;
class SToolBarComboButtonBlock;
class FToolBarStackButtonBlock;
class SToolBarStackButtonBlock;
class SHyperlink;
class SRichTextHyperlink;
class SThrobber;
class SCircularThrobber;
class STextEntryPopup;
class STextComboPopup;
class SExpandableButton;
class SExpandableArea;
class SNotificationItem;
class SNotificationList;
class SWidgetSwitcher;
class SSuggestionTextBox;
template <typename ItemType> class SBreadcrumbTrail;
class STextComboBox;
template<typename NumericType> class SNumericEntryBox;
template<typename OptionType> class SEditableComboBox;
class FSlateNotificationManager;
class SDPIScaler;
class SInlineEditableTextBlock;
class SVirtualKeyboardEntry;
class SSafeZone;
struct FMarqueeRect;
struct FButtonArgs;

template <typename NumericType, typename VectorType, int32 NumberOfComponent> class SNumericVectorInputBox;
using SVectorInputBox = SNumericVectorInputBox<float, UE::Math::TVector<float>, 3>;
template <typename NumericType> class SNumericRotatorInputBox;
using SRotatorInputBox = SNumericRotatorInputBox<float>;
class SVirtualJoystick;

enum ETabActivationCause : uint8;
enum ETabRole : uint8;
class SDockTab;

// Legacy
class SMissingWidget;
class FSlateStyleRegistry;
class FUICommandDragDropOp;
class FInertialScrollManager;
class FGenericCommands;

class SPopup;

