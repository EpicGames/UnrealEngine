// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Materials/MaterialInterface.h"
#include "Framework/Application/SlateApplication.h"

/**
 * Dynamic brush for referencing a UMaterial.  
 *
 * Note: This brush nor the slate renderer holds a strong reference to the material.  You are responsible for maintaining the lifetime of the brush and material object.
 */
struct FSlateMaterialBrush : public FSlateBrush
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InMaterial The material to use.
	 * @param InImageSize The material's dimensions.
	 */
	FSlateMaterialBrush( class UMaterialInterface& InMaterial, const UE::Slate::FDeprecateVector2DParameter& InImageSize )
		: FSlateBrush( ESlateBrushDrawType::Image, NAME_None, FMargin(0), ESlateBrushTileType::NoTile, ESlateBrushImageType::FullColor, InImageSize, FLinearColor::White, &InMaterial )
	{
		ResourceName = FName( *InMaterial.GetFullName() );
	}

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InImageSize The material's dimensions.
	 */
	FSlateMaterialBrush(const UE::Slate::FDeprecateVector2DParameter& InImageSize)
		: FSlateBrush(ESlateBrushDrawType::Image, FName("None"), FMargin(0), ESlateBrushTileType::NoTile, ESlateBrushImageType::FullColor, InImageSize, FLinearColor::White)
	{}

	/** Sets the material to use. */
	void SetMaterial(UMaterialInterface* InMaterial)
	{
		SetResourceObject(InMaterial);
		ResourceName = InMaterial != nullptr ? FName(*InMaterial->GetFullName()) : NAME_None;
	}
}; 
