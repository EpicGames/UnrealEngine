// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION UE_DISABLE_OPTIMIZATION_SHIP
#define END_SLATE_FUNCTION_BUILD_OPTIMIZATION   UE_ENABLE_OPTIMIZATION_SHIP
