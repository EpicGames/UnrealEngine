// Copyright Epic Games, Inc. All Rights Reserved.

// Can't be #pragma once because we want this to be interpreted differently depending on the module that is referencing the including file

// SLATE_MODULE is defined private to the module in Slate.build.cs
// This allows us to establish a scope that is public within the Slate module itself, but protected from all consumers of the module

// [[ IncludeTool: Inline ]] // Markup to tell IncludeTool that this file is state changing and cannot be optimized out.

#ifdef SLATE_MODULE
#define SLATE_SCOPE public
#else
#define SLATE_SCOPE protected
#endif