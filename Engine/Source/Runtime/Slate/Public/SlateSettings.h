// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SlateSettings.generated.h"

/**
* Settings that control Slate functionality
*/
UCLASS(config=Engine, defaultconfig, meta=(DisplayName="Slate"), MinimalAPI)
class USlateSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** 
	 * Allow children of SConstraintCanvas to share render layers. Children must set explicit ZOrder on their slots to control render order. 
	 * Recommendation: Enable for mobile platforms.
	 */
	UPROPERTY(config, EditAnywhere, Category="ConstraintCanvas")
	bool bExplicitCanvasChildZOrder;
};
