// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreUObjectSharedPCH.h"

// From Json:
#include "Policies/JsonPrintPolicy.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"

// From InputCore:
#include "InputCoreTypes.h"

// From SlateCore:
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Application/SlateApplicationBase.h"
#include "Application/SlateWindowHelper.h"
#include "Application/ThrottleManager.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/CursorReply.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "Input/NavigationReply.h"
#include "Input/PopupMethodReply.h"
#include "Input/Reply.h"
#include "Input/ReplyBase.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/LayoutGeometry.h"
#include "Layout/LayoutUtils.h"
#include "Layout/Margin.h"
#include "Layout/PaintGeometry.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Layout/WidgetPath.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderer.h"
#include "Rendering/SlateRenderTransform.h"
#include "SlateGlobals.h"
#include "SlotBase.h"
#include "Sound/SlateSound.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateWidgetStyleContainerInterface.h"
#include "Styling/StyleDefaults.h"
#include "Styling/WidgetStyle.h"
#include "Textures/SlateIcon.h"
#include "Textures/SlateShaderResource.h"
#include "Textures/SlateTextureData.h"
#include "Types/ISlateMetaData.h"
#include "Types/PaintArgs.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/IToolTip.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

// From Slate:
#include "Framework/Application/IMenu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Layout/InertialScrollManager.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Framework/Layout/Overscroll.h"
#include "Framework/MarqueeRect.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ShapedTextCacheFwd.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/TextRunRenderer.h"
#include "Framework/Views/ITypedTableView.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "SlateFwd.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SToolTip.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Widgets/Text/ISlateEditableTextWidget.h"
#endif
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"