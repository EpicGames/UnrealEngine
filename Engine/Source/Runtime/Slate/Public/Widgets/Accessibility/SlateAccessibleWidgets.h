// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "CoreMinimal.h"
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"

// SButton
class FSlateAccessibleButton
	: public FSlateAccessibleWidget
	, public IAccessibleActivatable
{
public:
	FSlateAccessibleButton(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::Button) {}
	virtual ~FSlateAccessibleButton() = default;

	virtual IAccessibleActivatable* AsActivatable() override { return this; }

	// IAccessibleActivatable
	SLATE_API virtual void Activate() override;
	// ~
};
// ~

// SCheckBox
class FSlateAccessibleCheckBox
	: public FSlateAccessibleWidget
	, public IAccessibleActivatable
	, public IAccessibleProperty
{
public:
	FSlateAccessibleCheckBox(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::CheckBox) {}
	virtual ~FSlateAccessibleCheckBox() = default;

	virtual IAccessibleActivatable* AsActivatable() override { return this; }
	virtual IAccessibleProperty* AsProperty() override { return this; }

	// IAccessibleActivatable
	SLATE_API virtual void Activate() override;
	SLATE_API virtual bool IsCheckable() const override;
	SLATE_API virtual bool GetCheckedState() const override;
	// ~
	
	// IAccessibleProperty
	SLATE_API virtual FString GetValue() const override;
	SLATE_API virtual FVariant GetValueAsVariant() const override;
	// ~
};
// ~

// SEditableText
class FSlateAccessibleEditableText
	: public FSlateAccessibleWidget
	, public IAccessibleText
	, public IAccessibleProperty
{
public:
	FSlateAccessibleEditableText(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::TextEdit) {}

	// IAccessibleWidget
	virtual IAccessibleText* AsText() override { return this; }
	virtual IAccessibleProperty* AsProperty() override { return this; }
	// ~

	// IAccessibleText
	SLATE_API virtual const FString& GetText() const override;
	// ~

	// IAccessibleProperty
	SLATE_API virtual bool IsReadOnly() const override;
	SLATE_API virtual bool IsPassword() const override;
	SLATE_API virtual FString GetValue() const override;
	SLATE_API virtual FVariant GetValueAsVariant() const override;
	SLATE_API virtual void SetValue(const FString& Value) override;
	// ~
};
// ~

// SEditableTextBox
class FSlateAccessibleEditableTextBox
	: public FSlateAccessibleWidget
	, public IAccessibleText
	, public IAccessibleProperty
{
public:
	FSlateAccessibleEditableTextBox(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::TextEdit) {}

	// IAccessibleWidget
	virtual IAccessibleText* AsText() override { return this; }
	virtual IAccessibleProperty* AsProperty() override { return this; }
	// ~

	// IAccessibleText
	SLATE_API virtual const FString& GetText() const override;
	// ~

	// IAccessibleProperty
	SLATE_API virtual bool IsReadOnly() const override;
	SLATE_API virtual bool IsPassword() const override;
	SLATE_API virtual FString GetValue() const override;
	SLATE_API virtual FVariant GetValueAsVariant() const override;
	SLATE_API virtual void SetValue(const FString& Value) override;
	// ~
};
// ~

// SHyperlink
class FSlateAccessibleHyperlink
	: public FSlateAccessibleButton
{
public:
	FSlateAccessibleHyperlink(TWeakPtr<SWidget> InWidget) : FSlateAccessibleButton(InWidget) { WidgetType = EAccessibleWidgetType::Hyperlink; }
	
	// todo: add way to get URL for external hyperlinks? may need to go with SHyperlinkLaunchURL
};
// ~

// Layouts
class FSlateAccessibleLayout
	: public FSlateAccessibleWidget
{
public:
	FSlateAccessibleLayout(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::Layout) {}
};
// ~

// SSlider
class FSlateAccessibleSlider
	: public FSlateAccessibleWidget
	, public IAccessibleProperty
{
public:
	FSlateAccessibleSlider(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::Slider) {}

	// IAccessibleWidget
	virtual IAccessibleProperty* AsProperty() override { return this; }
	// ~

	// IAccessibleProperty
	SLATE_API virtual bool IsReadOnly() const override;
	SLATE_API virtual float GetStepSize() const override;
	SLATE_API virtual float GetMaximum() const override;
	SLATE_API virtual float GetMinimum() const override;
	SLATE_API virtual FString GetValue() const override;
	SLATE_API virtual FVariant GetValueAsVariant() const override;
	SLATE_API virtual void SetValue(const FString& Value) override;
	// ~
};
// ~

// STextBlock
class FSlateAccessibleTextBlock
	: public FSlateAccessibleWidget
	//, public IAccessibleText // Disabled until we have better support for text. JAWS will not read these properly as-is.
{
public:
	FSlateAccessibleTextBlock(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::Text) {}

	// IAccessibleWidget
	//virtual IAccessibleText* AsText() override { return this; }
	// ~

	// IAccessibleText
	SLATE_API virtual const FString& GetText() const /*override*/;
	// ~
};
// ~

#endif
