// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/AppStyle.h"

class FPaintArgs;
class FSlateWindowElementList;

enum class EColorBlockAlphaDisplayMode : uint8
{
	// Draw a single block that draws color and opacity as one. I.E the entire block will be semi-transparent if opacity < 1
	Combined,
	// The color block is split into in half. The left half draws the color with opacity and the right half draws without any opacity
	Separate,
	// The color block is split into in half. The left half draws the color without any opacity and the right half draws with opacity
	SeparateReverse,
	// Alpha is omitted from display
	Ignore,
};


template <>
struct TWidgetTypeTraits<class SColorBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SColorBlock : public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SColorBlock, SLeafWidget, SLATE_API)

public:

	SLATE_BEGIN_ARGS(SColorBlock)
		: _Color(FLinearColor::White)
		, _AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
		, _CornerRadius(0.0f)
		, _ColorIsHSV(false)
		, _ShowBackgroundForAlpha(false)
		, _UseSRGB(true)
		, _AlphaDisplayMode(EColorBlockAlphaDisplayMode::Combined)
		, _Size(FVector2D(16, 16))
		, _OnMouseButtonDown()
	{}

		/** The color to display for this color block */
		SLATE_ATTRIBUTE(FLinearColor, Color)

		/** Background to display for when there is a color with transparency. Irrelevant if ignoring alpha */
		SLATE_ATTRIBUTE(const FSlateBrush*, AlphaBackgroundBrush)

		/** Rounding to apply to the corners of the block */
		SLATE_ATTRIBUTE(FVector4, CornerRadius)

		/** Whether the color displayed is HSV or not */
		SLATE_ATTRIBUTE(bool, ColorIsHSV)

		/** Whether to display a background for viewing opacity. Irrelevant if ignoring alpha */
		SLATE_ATTRIBUTE(bool, ShowBackgroundForAlpha)

		/** Whether to display sRGB color */
		SLATE_ATTRIBUTE(bool, UseSRGB)

		/** How the color block displays color and opacity */
		SLATE_ATTRIBUTE(EColorBlockAlphaDisplayMode, AlphaDisplayMode)

		/** How big should this color block be? */
		SLATE_ATTRIBUTE(FVector2D, Size)

		/** A handler to activate when the mouse is pressed. */
		SLATE_EVENT(FPointerEventHandler, OnMouseButtonDown)

	SLATE_END_ARGS()

public:
	SLATE_API SColorBlock();
	SLATE_API ~SColorBlock();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);

private:
	// SWidget overrides
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	void MakeSection(TArray<FSlateGradientStop>& OutGradientStops, FVector2D StartPt, FVector2D EndPt, FLinearColor Color, const FWidgetStyle& InWidgetStyle, bool bIgnoreAlpha) const;

private:
	/** The color to display for this color block */
	TSlateAttribute<FLinearColor> Color;

	TSlateAttribute<const FSlateBrush*> AlphaBackgroundBrush;

	TSlateAttribute<FVector4> GradientCornerRadius;

	TSlateAttribute<FVector2D> ColorBlockSize;

	/** A handler to activate when the mouse is pressed. */
	FPointerEventHandler MouseButtonDownHandler;

	/** Whether to ignore alpha entirely from the input color */
	TSlateAttribute<EColorBlockAlphaDisplayMode> AlphaDisplayMode;

	/** Whether the color displayed is HSV or not */
	TSlateAttribute<bool> ColorIsHSV;

	/** Whether to display a background for viewing opacity. Irrelevant if ignoring alpha */
	TSlateAttribute<bool> ShowBackgroundForAlpha;

	/** Whether to display sRGB color */
	TSlateAttribute<bool> bUseSRGB;
};
