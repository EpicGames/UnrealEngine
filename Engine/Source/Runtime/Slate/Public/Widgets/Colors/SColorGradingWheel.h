// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/SlateDelegates.h"

/**
 * Implements the color wheel widget.
 */
class UE_DEPRECATED(5.5, "Color grading widgets and associated types have moved to the AdvancedWidgets module.") SColorGradingWheel
	: public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SColorGradingWheel, SLeafWidget, SLATE_API)

public:

	DECLARE_DELEGATE_OneParam(FOnColorGradingWheelMouseCapture, const FLinearColor&);
	DECLARE_DELEGATE_OneParam(FOnColorGradingWheelValueChanged, const FLinearColor&);

	SLATE_BEGIN_ARGS(SColorGradingWheel)
		: _SelectedColor()
		, _DesiredWheelSize()
		, _ExponentDisplacement()
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
	{ }
	
		/** The current color selected by the user. */
		SLATE_ATTRIBUTE(FLinearColor, SelectedColor)
		
		SLATE_ATTRIBUTE(int32, DesiredWheelSize)

		SLATE_ATTRIBUTE(float, ExponentDisplacement)
		
		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FOnColorGradingWheelMouseCapture, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FOnColorGradingWheelMouseCapture, OnMouseCaptureEnd)

		/** Invoked when a new value is selected on the color wheel. */
		SLATE_EVENT(FOnColorGradingWheelValueChanged, OnValueChanged)

	SLATE_END_ARGS()
	
public:
	SLATE_API SColorGradingWheel();
	SLATE_API virtual ~SColorGradingWheel();

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	SLATE_API void Construct(const FArguments& InArgs);

public:

	// SWidget overrides

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	SLATE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	
protected:

	/**
	 * Calculates the position of the color selection indicator.
	 *
	 * @return The position relative to the widget.
	 */
	SLATE_API UE::Slate::FDeprecateVector2DResult CalcRelativePositionFromCenter() const;

	/**
	 * Performs actions according to mouse click / move
	 *
	 * @return	True if the mouse action occurred within the color wheel radius
	 */
	SLATE_API bool ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bProcessWhenOutsideColorWheel);

	/** */
	SLATE_API void SetSelectedColorAttribute(TAttribute<FLinearColor> InSelectedColor);

	/** */
	SLATE_API void SetDesiredWheelSizeAttribute(TAttribute<int32> InDesiredWheelSize);

	/** */
	SLATE_API void SetExponentDisplacementAttribute(TAttribute<float> InExponentDisplacement);

	/** @return an attribute reference of SelectedColor */
	TSlateAttributeRef<FLinearColor> GetSelectedColorAttribute() const { return TSlateAttributeRef<FLinearColor>(SharedThis(this), SelectedColorAttribute); }

	/** @return an attribute reference of DesiredWheelSize */
	TSlateAttributeRef<int32> GetDesiredWheelSizeAttribute() const { return TSlateAttributeRef<int32>(SharedThis(this), DesiredWheelSizeAttribute); }

	/** @return an attribute reference of ExponentDisplacement */
	TSlateAttributeRef<float> GetExponentDisplacementAttribute() const { return TSlateAttributeRef<float>(SharedThis(this), ExponentDisplacementAttribute); }

	/** The color wheel image to show. */
	const FSlateBrush* Image;

	/** The color selector image to show. */
	const FSlateBrush* SelectorImage;

	/** Invoked when the mouse is pressed and a capture begins. */
	FOnColorGradingWheelMouseCapture OnMouseCaptureBegin;

	/** Invoked when the mouse is let up and a capture ends. */
	FOnColorGradingWheelMouseCapture OnMouseCaptureEnd;

	/** Invoked when a new value is selected on the color wheel. */
	FOnColorGradingWheelValueChanged OnValueChanged;

private:
	/** The current color selected by the user. */
	TSlateAttribute<FLinearColor> SelectedColorAttribute;

	TSlateAttribute<int32> DesiredWheelSizeAttribute;
	TSlateAttribute<float> ExponentDisplacementAttribute;

	/** Flags used to check if the SlateAttribute is set. */
	union
	{
		struct
		{
			uint8 bIsAttributeDesiredWheelSizeSet : 1;
			uint8 bIsAttributeExponentDisplacementSet : 1;
		};
		uint8 Union_IsAttributeSet;
	};
};
