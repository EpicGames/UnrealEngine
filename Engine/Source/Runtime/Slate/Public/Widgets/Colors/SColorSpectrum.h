// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/SlateDelegates.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FSlateBrush;

/**
 * Implements the color spectrum widget.
 */
class SColorSpectrum
	: public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SColorSpectrum)
		: _SelectedColor()
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		, _CtrlMultiplier(0.1f)
		, _PreventThrottling(false)
		, _ProcessMouseMoveImmediate(false)
		{ }
	
		/** The current color selected by the user */
		SLATE_ATTRIBUTE(FLinearColor, SelectedColor)
		
		/** Invoked when the mouse is pressed and a capture begins */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when a new value is selected on the color spectrum */
		SLATE_EVENT(FOnLinearColorValueChanged, OnValueChanged)

		/** Multiplier to use when ctrl is held down */
		SLATE_ATTRIBUTE(float, CtrlMultiplier)

		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)

		/** If mouse inputs should be processed immediately. This should be used when spawned in a Modal. */
		SLATE_ARGUMENT(bool, ProcessMouseMoveImmediate)

	SLATE_END_ARGS()
	
public:
	SLATE_API SColorSpectrum();
	SLATE_API virtual ~SColorSpectrum();

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	SLATE_API void Construct( const FArguments& InArgs );

public:

	// SWidget overrides

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnFinishedPointerInput() override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

protected:

	/**
	 * Calculates the position of the color selection indicator.
	 *
	 * @return The position relative to the widget.
	 */
	SLATE_API FVector2D CalcRelativeSelectedPosition( ) const;

	/**
	 * Performs actions according to mouse click / move
	 */
	SLATE_API void ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

private:

	// The color spectrum image to show.
	const FSlateBrush* Image;
	
	// The current color selected by the user.
	TSlateAttribute<FLinearColor, EInvalidateWidgetReason::Paint> SelectedColor;

	// Mouse sensitivity multiplier to use when dragging the selector on the color spectrum, applied when the ctrl modifier key is pressed
	TAttribute<float> CtrlMultiplier;

	// The color selector image to show.
	const FSlateBrush* SelectorImage;

	// Whether the user is dragging the slider
	bool bDragging = false;

	/** Whether the user requires high precision mode */
	bool bHighPrecision = false;

	// Cached mouse position to restore after dragging.
	FVector2f LastSpectrumPosition;

	/** If refresh requests for the viewport should happen for all value changes **/
	bool bPreventThrottling = false;

	/** If mouse inputs should be processed immediately. This should be used when spawned in a Modal. */
	bool bProcessMouseMoveImmediate = false;

	/** Last Geometry to use after dragging. */
	FGeometry LastGeometry;

	/** Last mouse event to use after dragging. */
	FPointerEvent LastMouseEvent;

private:

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

	// Holds a delegate that is executed when a new value is selected on the color spectrum.
	FOnLinearColorValueChanged OnValueChanged;
};
