// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/SlateDelegates.h"
#include "Styling/CoreStyle.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FSlateBrush;

/**
 * Implements the color wheel widget.
 */
class SColorWheel
	: public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SColorWheel)
		: _SelectedColor()
		, _ColorWheelBrush(FCoreStyle::Get().GetBrush("ColorWheel.HueValueCircle"))
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		, _CtrlMultiplier(0.1f)
		, _PreventThrottling(false)
		, _ProcessMouseMoveImmediate(false)
	{ }
	
		/** The current color selected by the user. */
		SLATE_ATTRIBUTE(FLinearColor, SelectedColor)

		/** ColorWheelBrush to use. */
		SLATE_ATTRIBUTE(const FSlateBrush*, ColorWheelBrush)
		
		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when a new value is selected on the color wheel. */
		SLATE_EVENT(FOnLinearColorValueChanged, OnValueChanged)

		/** Multiplier to use when ctrl is held down */
		SLATE_ATTRIBUTE(float, CtrlMultiplier)

		/** If refresh requests for the viewport should happen for all value changes. */
		SLATE_ARGUMENT(bool, PreventThrottling)

		/** If mouse inputs should be processed immediately. This should be used when spawned in a Modal. */
		SLATE_ARGUMENT(bool, ProcessMouseMoveImmediate)

	SLATE_END_ARGS()
	
public:
	SLATE_API SColorWheel();
	SLATE_API virtual ~SColorWheel();

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	/**
	 * Set selector visibility.
	 *
	 * @param bShow
	 */
	void ShowSelector(bool bShow = true) { bShouldDrawSelector = bShow; }

public:

	//~ SWidget overrides

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	SLATE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual void OnFinishedPointerInput() override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

protected:

	/**
	 * Calculates the position of the color selection indicator.
	 *
	 * @return The position relative to the widget.
	 */
	SLATE_API UE::Slate::FDeprecateVector2DResult CalcRelativePositionFromCenter() const;

	/**
	 * Performs actions according to mouse click / move
	 *
	 * @return	True if the mouse action occurred within the color wheel radius
	 */
	SLATE_API bool ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bProcessWhenOutsideColorWheel);

	/** If refresh requests for the viewport should happen for all value changes **/
	bool bPreventThrottling = false;

	/** If mouse inputs should be processed immediately. This should be used when spawned in a Modal. */
	bool bProcessMouseMoveImmediate = false;

private:

	/** The color wheel image to show. */
	const FSlateBrush* Image;
	
	/** The current color selected by the user. */
	TSlateAttribute<FLinearColor, EInvalidateWidgetReason::Paint> SelectedColor;

	/** Mouse sensitivity multiplier to use when dragging the selector on the color wheel, applied when the ctrl modifier key is pressed */
	TAttribute<float> CtrlMultiplier;

	/** The color selector image to show. */
	const FSlateBrush* SelectorImage;

	/** Flag to show/hide color selector */
	bool bShouldDrawSelector;

	/** Whether the user is dragging the slider */
	bool bDragging = false;

	/** Whether the user requires high precision mode */
	bool bHighPrecision = false;

	/** Cached mouse position to restore after dragging. */
	FVector2f LastWheelPosition;

	/** Last Geometry to use after dragging. */
	FGeometry LastGeometry;

	/** Last mouse event to use after dragging. */
	FPointerEvent LastMouseEvent;

private:

	/** Invoked when the mouse is pressed and a capture begins. */
	FSimpleDelegate OnMouseCaptureBegin;

	/** Invoked when the mouse is let up and a capture ends. */
	FSimpleDelegate OnMouseCaptureEnd;

	/** Invoked when a new value is selected on the color wheel. */
	FOnLinearColorValueChanged OnValueChanged;
};
