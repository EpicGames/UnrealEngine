// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * Implements a Slate widget that renders a color gradient consisting of multiple stops.
 */
class SComplexGradient
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SComplexGradient)
		: _GradientColors()
		, _HasAlphaBackground(false)
		, _Orientation(Orient_Vertical)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

		/** The colors used in the gradient */
		SLATE_ATTRIBUTE(TArray<FLinearColor>, GradientColors)

		/** Whether a checker background is displayed for alpha viewing */
		SLATE_ARGUMENT(bool, HasAlphaBackground)
		
		/** Horizontal or vertical gradient */
		SLATE_ARGUMENT(EOrientation, Orientation)

		/** When specified use this as the gradients desired size */
		SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSizeOverride)

	SLATE_END_ARGS()
	
	SLATE_API SComplexGradient();
	SLATE_API virtual ~SComplexGradient();

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	SLATE_API void Construct( const FArguments& InArgs );

protected:

	// SCompoundWidget overrides

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
private:

	/** The colors used in the gradient. */
	TSlateAttribute<TArray<FLinearColor>, EInvalidateWidgetReason::Paint> GradientColors;

	/** Optional override for desired size */
	TSlateAttribute<TOptional<FVector2D>, EInvalidateWidgetReason::Layout> DesiredSizeOverride;

	/** Whether a checker background is displayed for alpha viewing. */
	bool bHasAlphaBackground;
	
	/** Horizontal or vertical gradient. */
	EOrientation Orientation;
};
