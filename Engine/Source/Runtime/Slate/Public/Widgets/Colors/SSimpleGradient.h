// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * Implements a Slate widget that renders a simple color gradient.
 */
class SSimpleGradient
	: public SCompoundWidget
{
public:

		SLATE_BEGIN_ARGS(SSimpleGradient)
			: _StartColor(FLinearColor(0.0f, 0.0f, 0.0f))
			, _EndColor(FLinearColor(1.0f, 1.0f, 1.0f))
			, _HasAlphaBackground(false)
			, _Orientation(Orient_Vertical)
		{ }

		/** The leftmost gradient color */
		SLATE_ATTRIBUTE(FLinearColor, StartColor)
		
		/** The rightmost gradient color */
		SLATE_ATTRIBUTE(FLinearColor, EndColor)

		/** Whether a checker background is displayed for alpha viewing */
		SLATE_ATTRIBUTE(bool, HasAlphaBackground)

		/** Horizontal or vertical gradient */
		SLATE_ATTRIBUTE(EOrientation, Orientation)

	SLATE_END_ARGS()

public:
	SLATE_API SSimpleGradient();

	/**
	 * Constructs the widget
	 *
	 * @param InArgs The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

protected:

	// SCompoundWidget overrides

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

private:

	/** The leftmost gradient color */
	TSlateAttribute<FLinearColor, EInvalidateWidgetReason::Paint> StartColor;

	/** The rightmost gradient color */
	TSlateAttribute<FLinearColor, EInvalidateWidgetReason::Paint> EndColor;

	/** Whether a checker background is displayed for alpha viewing */
	bool bHasAlphaBackground;
	
	/** Horizontal or vertical gradient */
	EOrientation Orientation;
};
