// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Styling/StyleDefaults.h"
#include "Animation/CurveSequence.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/Docking/TabManager.h"

class FActiveTimerHandle;
class SDockingArea;
class SDockingTabStack;
class SDockingTabWell;
class SImage;
class STextBlock;
class SToolTip;

/** How will this tab be used. */
enum ETabRole : uint8
{
	MajorTab,
	PanelTab,
	NomadTab,
	DocumentTab,
	NumRoles
};

/** The cause of a tab activation */
enum ETabActivationCause : uint8
{
	UserClickedOnTab,
	SetDirectly
};

/** Defines the policy for automatically sizing dock tabs. */
enum class EDockTabAutosizePolicy : uint8
{
	/** No autosize */
	None,
	/** Autosize on whatever axis the layout is using (old ShouldAutosize(true) behavior) */
	LayoutAxis,
	/** Autosize only when the parent splitter is horizontal (width axis) */
	HorizontalOnly,
	/** Autosize only when the parent splitter is vertical (height axis) */
	VerticalOnly
};

class FMenuBuilder;

namespace UE::Slate::Private
{
	class STabPanelDrawer;
}

/**
 * A tab widget that also holds on to some content that should be shown when this tab is selected.
 * Intended to be used in conjunction with SDockingTabStack.
 */
class SDockTab : public SBorder
{
	friend class FTabManager;
	friend class STabSidebar;
	friend UE::Slate::Private::STabPanelDrawer;

public:

	/** Invoked when a tab is closing */
	DECLARE_DELEGATE_OneParam(FOnTabClosedCallback, TSharedRef<SDockTab>);
	
	/** Invoked when a tab is activated */
	DECLARE_DELEGATE_TwoParams(FOnTabActivatedCallback, TSharedRef<SDockTab>, ETabActivationCause);

	/** Invoked when a tab is renamed */
	DECLARE_DELEGATE_OneParam(FOnTabRenamed, TSharedRef<SDockTab>);

	/** Invoked w`en this tab should save some information about its content. */
	DECLARE_DELEGATE(FOnPersistVisualState);

	/** Delegate called before a tab is closed.  Returning false will prevent the tab from closing */
	DECLARE_DELEGATE_RetVal( bool, FCanCloseTab );

	/** Invoked to add entries to the tab context menu */
	DECLARE_DELEGATE_OneParam(FExtendContextMenu, FMenuBuilder&);

	SLATE_BEGIN_ARGS(SDockTab)
		: _Content()
		, _TabWellContentLeft()
		, _TabWellContentRight()
		, _ContentPadding(0.f)
		, _TabRole(ETabRole::PanelTab)
		, _Label()
		, _LabelSuffix()
		, _OnTabClosed()
		, _OnTabActivated()
		, _OnTabRelocated()
		, _OnTabDraggedOverDockArea()
		, _ShouldAutosize(false)
		, _AutosizePolicy(EDockTabAutosizePolicy::None)
		, _OnCanCloseTab()
		, _CanEverClose(true)
		, _OnPersistVisualState()
		, _TabColorScale(FLinearColor::Transparent)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _IconColor()
		, _LabelOverflowPolicy(ETextOverflowPolicy::Ellipsis)
		{}

		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_NAMED_SLOT( FArguments, TabWellContentLeft )
		SLATE_NAMED_SLOT( FArguments, TabWellContentRight )
		SLATE_ATTRIBUTE( FMargin, ContentPadding )
		SLATE_ARGUMENT( ETabRole, TabRole )
		SLATE_ATTRIBUTE( FText, Label )
		SLATE_ATTRIBUTE(FText, LabelSuffix)
		UE_DEPRECATED(5.0, "Tab icons are now being managed by tab spawners and toolkits. In the rare case you need to set an icon manually, use SetTabIcon() instead")
		FArguments& Icon(const FSlateBrush* InIcon)
		{
			return Me();
		}

		SLATE_EVENT( FOnTabClosedCallback, OnTabClosed )
		SLATE_EVENT( FOnTabActivatedCallback, OnTabActivated )
		SLATE_EVENT( FSimpleDelegate, OnTabRelocated )
		SLATE_EVENT( FSimpleDelegate, OnTabDraggedOverDockArea )
		SLATE_ARGUMENT( bool, ShouldAutosize )
		/** Overrides and acts as a replacement for ShouldAutosize to allow more auto-sizing policies */
		SLATE_ARGUMENT( EDockTabAutosizePolicy, AutosizePolicy )
		/** When the close button is pressed, checks whether the tab can be closed in that moment. Example: Show dialog and ask user whether they're sure to close. */
		SLATE_EVENT( FCanCloseTab, OnCanCloseTab )
		/** Whether this tab can ever be closed. Example: certain programs may want to show tabs for the lifetime of the program. */
		SLATE_ARGUMENT( bool, CanEverClose )
		SLATE_EVENT( FOnPersistVisualState, OnPersistVisualState )
		SLATE_EVENT( FExtendContextMenu, OnExtendContextMenu )
		/** Invoked when a tab is closed from a drawer. This does not mean the tab or its contents is destroyed, just hidden. Use OnTabClosed for that */
		SLATE_EVENT( FSimpleDelegate, OnTabDrawerClosed)
		SLATE_ATTRIBUTE( FLinearColor, TabColorScale )
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )
		SLATE_ATTRIBUTE( FLinearColor, IconColor)
		SLATE_ARGUMENT( ETextOverflowPolicy, LabelOverflowPolicy )
	SLATE_END_ARGS()



	/** Construct the widget from the declaration. */
	SLATE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent  ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	SLATE_API virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	// End of SWidget interface

	// SBorder interface
	SLATE_API virtual void SetContent(TSharedRef<SWidget> InContent) override;
	// End of SBorder interface

	/** Content that appears in the TabWell to the left of the tabs */
	SLATE_API void SetLeftContent(TSharedRef<SWidget> InContent);
	/** Content that appears in the TabWell to the right of the tabs */
	SLATE_API void SetRightContent(TSharedRef<SWidget> InContent);
	/** Content that appears on the right side of the title bar in the window this stack is in */
	SLATE_API void SetTitleBarRightContent(TSharedRef<SWidget> InContent);

	/** @return True if this tab is currently focused */
	SLATE_API bool IsActive() const;

	/** @return True if this tab appears active; False otherwise */
	SLATE_API bool IsForeground() const;

	/** @return the Foreground color that this widget sets; unset options if the widget does not set a foreground color */
	SLATE_API virtual FSlateColor GetForegroundColor() const;

	/** Add any entries specific to this tab to the tab context menu */
	SLATE_API void ExtendContextMenu(FMenuBuilder& MenuBuilder);

	/** Is this an MajorTab? A tool panel tab? */
	SLATE_API ETabRole GetTabRole() const;

	/** Similar to GetTabRole() but returns the correct role for UI style and user input purposes */
	SLATE_API ETabRole GetVisualTabRole() const;

	/**
	 * What should the content area look like for this type of tab?
	 * Documents, Apps, and Tool Panels have different backgrounds.
	 */
	SLATE_API const FSlateBrush* GetContentAreaBrush() const;

	/** Depending on the tabs we put into the tab well, we want a different background brush. */
	SLATE_API const FSlateBrush* GetTabWellBrush() const;

	/** @return the content associated with this tab */
	SLATE_API TSharedRef<SWidget> GetContent();
	SLATE_API TSharedRef<SWidget> GetLeftContent();
	SLATE_API TSharedRef<SWidget> GetRightContent();
	SLATE_API TSharedRef<SWidget> GetTitleBarRightContent();

	/** Padding around the content when it is presented by the SDockingTabStack */
	SLATE_API FMargin GetContentPadding() const;

	/** Gets this tab's layout identifier */
	SLATE_API const FTabId GetLayoutIdentifier() const;

	/** Sets the tab's tab well parent, or resets it if nothing is passed in */
	SLATE_API void SetParent(TSharedPtr<SDockingTabWell> Parent = TSharedPtr<SDockingTabWell>());

	/** Set the tab parent when the tab live outside of a tab well (PanelDrawer for example) */
	void SetParentDockingArea(TSharedPtr<SDockingArea> Parent);

	/** Gets the tab's tab well parent, or nothing, if it has none */
	SLATE_API TSharedPtr<SDockingTabWell> GetParent() const;

	/** Gets the dock tab stack this dockable tab resides within, if any */
	SLATE_API TSharedPtr<SDockingTabStack> GetParentDockTabStack() const;

	/** Gets the dock area that this resides in */
	SLATE_API TSharedPtr<class SDockingArea> GetDockArea() const;

	/** Returns true if this docktab is in a sidebar. */
	SLATE_API bool IsInSidebar() const;

	/** Get the window in which this tab's tabmanager has placed it */
	SLATE_API TSharedPtr<SWindow> GetParentWindow() const;

	/** The width that this tab will overlap with side-by-side tabs. */
	SLATE_API float GetOverlapWidth() const;

	/** The label on the tab */
	SLATE_API FText GetTabLabel() const;

	/** The label that appears on the tab. */
	SLATE_API void SetLabel( const TAttribute<FText>& InTabLabel );

	/** Get Label Suffix */
	SLATE_API FText GetTabLabelSuffix() const;

	/** Set Label Suffix.  A second text field at the end of the Label that takes precedence and isn't lost when space is restricted */
	SLATE_API void SetTabLabelSuffix(const TAttribute<FText>& InTabLabelSuffix);

	/** Set the Label overflow policy */
	SLATE_API void SetTabLabelOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy) const;

	/** The tooltip text that appears on the tab. */
	SLATE_API void SetTabToolTipWidget(TSharedPtr<SToolTip> InTabToolTipWidget);

	/** Gets the tab icon */
	SLATE_API const FSlateBrush* GetTabIcon() const;

	/** Sets the tab icon */
	SLATE_API void SetTabIcon( const TAttribute<const FSlateBrush*> InTabIcon );

	/** Should this tab be sized based on its content. */
	SLATE_API bool ShouldAutosize() const;

	/** Set whether this tab should be sized based on its content. */
	SLATE_API void SetShouldAutosize(const bool bNewShouldAutosize);

	/** Gets the policy for how its content is auto-sized */
	SLATE_API EDockTabAutosizePolicy GetAutosizePolicy() const;

	/** Overrides ShouldAutosize() and sets the policy for how the tab should be auto-sized to its content. */
	SLATE_API void SetAutosizePolicy(const EDockTabAutosizePolicy InNewPolicy);

	/** @return true if the tab can be closed right now. Example: Callback could ask user via dialog. */
	SLATE_API bool CanCloseTab() const;

	/** @return true if the tab can be closed right now. Example: Callback could ask user via dialog. */
	SLATE_API bool CanCloseTab(bool bIgnoreLockedTabs) const;

	/** Requests that the tab be closed.  Tabs may prevent closing depending on their state */	
	SLATE_API bool RequestCloseTab();

	/** @return True if this tab is currently locked */
	SLATE_API bool IsTabLocked() const;

	/** A chance for the tab's content to save any internal layout info */
	SLATE_API void PersistVisualState();

	/** 
	 * Pulls this tab out of it's parent tab stack and destroys it
	 * Note: This does not check if its safe to remove the tab.  Use RequestCloseTab to do this safely.
	 */
	SLATE_API void RemoveTabFromParent();

	/** Protected constructor; Widgets may only be constructed via a FArguments (i.e.: SNew(SDockTab) ) */
	SLATE_API SDockTab();

	/** 
	 * Make this tab active in its tabwell 
	 * @param	InActivationMethod	How this tab was activated.
	 */
	SLATE_API void ActivateInParent(ETabActivationCause InActivationCause);

	/** Set the tab manager that is controlling this tab */
	SLATE_API void SetTabManager( const TSharedPtr<FTabManager>& InTabManager );

	/**
	 * Set the custom code to execute for saving visual state in this tab.
	 * e.g. ContentBrowser saves the visible filters.OnExtendContextMenu
	 */
	SLATE_API void SetOnPersistVisualState( const FOnPersistVisualState& Handler );

	/** Set the handler to be invoked when the user requests that this tab be closed. */
	SLATE_API void SetCanCloseTab( const FCanCloseTab& InOnTabClosing );

	/** Set the handler that will be invoked when the tab is closed */
	SLATE_API void SetOnTabClosed( const FOnTabClosedCallback& InDelegate );

	/** Set the handler that will be invoked when the tab is activated */
	SLATE_API void SetOnTabActivated( const FOnTabActivatedCallback& InDelegate );

	/** Set the handler that will be invoked when the tab is relocated to a new tab well */
	SLATE_API void SetOnTabRelocated(const FSimpleDelegate InDelegate);

	/** Set the handler that will be invoked when the tab is dragged over dock area */
	SLATE_API void SetOnTabDraggedOverDockArea(const FSimpleDelegate InDelegate);

	/** Set the handler that will be invoked when the tab is renamed */
	SLATE_API void SetOnTabRenamed(const FOnTabRenamed& InDelegate);

	/** Set the handler that will be invoked when the tab is opened from a drawer */
	SLATE_API void SetOnTabDrawerOpened(const FSimpleDelegate InDelegate);

	/** Set the handler that will be invoked when the tab is closed from a drawer */
	SLATE_API void SetOnTabDrawerClosed(const FSimpleDelegate InDelegate);

	/** Set the handler for extending the tab context menu */
	SLATE_API void SetOnExtendContextMenu( const FExtendContextMenu& Handler );

	/** Get the tab manager currently managing this tab. Note that a user move the tab between Tab Managers, so this return value may change. */
	UE_DEPRECATED(5.0, "The tab manager is not guaranteed to exist, which will cause GetTabManager() to crash. Use GetTabManagerPtr() instead.")
	SLATE_API TSharedRef<FTabManager> GetTabManager() const;

	/** Get the tab manager currently managing this tab. Note that a user move the tab between Tab Managers, so this return value may change. */
	SLATE_API TSharedPtr<FTabManager> GetTabManagerPtr() const;

	/** Draws attention to the tab. */
	SLATE_API void DrawAttention();

	/** Provide a default tab label in case the spawner did not set one. */
	SLATE_API void ProvideDefaultLabel( const FText& InDefaultLabel );

	/** Play an animation showing this tab as opening */
	SLATE_API void PlaySpawnAnim();

	/** Flash the tab, used for drawing attention to it */
	SLATE_API void FlashTab();

	/** Used by the drag/drop operation to signal to this tab what it is dragging over. */
	SLATE_API void SetDraggedOverDockArea( const TSharedPtr<SDockingArea>& Area );

	/** 
	 * Check to see whether this tab has a sibling tab with the given tab ID
	 * 
	 * @param	SiblingTabId				The ID of the tab we want to find
	 * @param	TreatIndexNoneAsWildcard	Note that this variable only takes effect if SiblingTabId has an InstanceId of INDEX_NONE.
	 *										If true, we will consider this a "wildcard" search (matching any tab with the correct TabType, regardless 
	 *										of its InstanceId). If false, we will explicitly look for a tab with an InstanceId of INDEX_NONE
	 */
	SLATE_API bool HasSiblingTab(const FTabId& SiblingTabId, const bool TreatIndexNoneAsWildcard = true) const;

	/** Updates the 'last activated' time to the current time */
	SLATE_API void UpdateActivationTime();

	/** Returns the time this tab was last activated */
	double GetLastActivationTime()
	{
		return LastActivationTime;
	}

	SLATE_API void SetParentDockTabStackTabWellHidden(bool bIsTabWellHidden);

	/** Get if the TabName should be hidden and not take up space for this Tab */
	SLATE_API bool IsTabNameHidden() const;

protected:
	/** Provide a default tab icon. */
	SLATE_API void ProvideDefaultIcon(const FSlateBrush* InDefaultIcon);

	/** @return the style currently applied to the dock tab */
	SLATE_API const FDockTabStyle& GetCurrentStyle() const;

	/** @return the image brush that best represents this tab's in its current state */
	SLATE_API const FSlateBrush* GetImageBrush() const;

	/** @return the padding for the tab widget */
	SLATE_API FMargin GetTabPadding() const;

	/** @return the image brush for the tab's color overlay */
	SLATE_API const FSlateBrush* GetColorOverlayImageBrush() const;

	/** @return The visibility of the active tab indicator */
	SLATE_API EVisibility GetActiveTabIndicatorVisibility() const;

	/** @return Returns a color to scale the background of this tab by */
	SLATE_API FSlateColor GetTabColor() const;

	/** @return Returns the color of this tab's icon */
	SLATE_API FSlateColor GetIconColor() const;

	/** @return the image brush for the tab's flasher overlay */
	SLATE_API const FSlateBrush* GetFlashOverlayImageBrush() const;

	/** @return Returns a color to flash the background of this tab with */
	SLATE_API FSlateColor GetFlashColor() const;

	/** Called when the close button is clicked on the tab. */
	SLATE_API FReply OnCloseButtonClicked();

	/** The close button tooltip showing the appropriate close command shortcut */
	SLATE_API FText GetCloseButtonToolTipText() const;

	/** Specify the TabId that was used to spawn this tab. */
	SLATE_API void SetLayoutIdentifier( const FTabId& TabId );

	/** @return if the close button should be visible. */
	SLATE_API EVisibility HandleIsCloseButtonVisible() const;

	/** @return the size the tab icon should be */
	SLATE_API TOptional<FVector2D> GetTabIconSize() const;

	/** @return the padding for the tab icon border */
	SLATE_API FMargin GetTabIconBorderPadding() const;

private:
	/** Set whether the TabName should be hidden and not take up space for this Tab */
	SLATE_API void SetIsTabNameHidden(bool bInIsTabNameHidden);

	/** Activates the tab in its tab well */
	SLATE_API EActiveTimerReturnType TriggerActivateTab( double InCurrentTime, float InDeltaTime );

	SLATE_API EActiveTimerReturnType OnHandleUpdateStyle(double InCurrentTime, float InDeltaTime);

	SLATE_API void OnParentSet();

	SLATE_API void UpdateTabStyle();

	SLATE_API void OnTabDrawerOpened();
	SLATE_API void OnTabDrawerClosed();

	SLATE_API void NotifyTabRelocated();

	/* Closes tab if permissions filter no longer allows this tab to be open. */
	SLATE_API void CheckTabAllowed();

	/** Do the removal of the tab but doesn't emit tab close events */
	void RemoveTabFromParent_Internal();

	/** The handle to the active tab activation tick */
	TWeakPtr<FActiveTimerHandle> DragDropTimerHandle;
	TWeakPtr<FActiveTimerHandle> UpdateStyleTimerHandle;

	/** If true the TabName will be hidden and not take up space for this tab */
	bool bIsTabNameHidden = false;
protected:

	/** The tab manager that created this tab. */
	TWeakPtr<FTabManager> MyTabManager;

	/** The stuff to show when this tab is selected */
	TSharedRef<SWidget> Content;
	TSharedRef<SWidget> TabWellContentLeft;
	TSharedRef<SWidget> TabWellContentRight;
	TSharedRef<SWidget> TitleBarContentRight;
	
	/** The tab's layout identifier */
	FTabId LayoutIdentifier;

	/** Is this an MajorTab? A tool panel tab? */
	ETabRole TabRole;

	/** Determines whether the close button for the tab is shown. */
	bool bCanEverClose;

	/** The tab's parent tab well. Null if it is a floating tab or hosted by a DrawerPanel. */
	TWeakPtr<SDockingTabWell> ParentPtr;

	/** The tab's parent docking area (Might be null) */
	TWeakPtr<SDockingArea> ParentDockingAreaPtr;

	/** The label on the tab */
	TAttribute<FText> TabLabel;

	/** A second text field at the end of the Label that takes precedence and isn't lost when space is restricted */
	TAttribute<FText> TabLabelSuffix;

	/** The icon on the tab */
	TAttribute<const FSlateBrush*> TabIcon;
	
	/** Callback to call when this tab is destroyed */
	FOnTabClosedCallback OnTabClosed;

	/** Callback to call when this tab is activated */
	FOnTabActivatedCallback OnTabActivated;

	/** Delegate to execute to determine if we can close this tab */
	FCanCloseTab OnCanCloseTab;

	FSimpleDelegate OnTabDrawerClosedEvent;

	FSimpleDelegate OnTabDrawerOpenedEvent;

	FSimpleDelegate OnTabRelocated;

	FSimpleDelegate OnTabDraggedOverDockArea;

	FExtendContextMenu OnExtendContextMenu;

	/**
	 * Invoked during the Save Visual State pass; gives this tab a chance to save misc info about visual state.
	 * e.g. Content Browser might save the current filters, current folder, whether some panel is collapsed, etc.
	 */
	FOnPersistVisualState OnPersistVisualState;

	/** Invoked when the tab is renamed */
	FOnTabRenamed OnTabRenamed;

	/** The styles used to draw the tab in its various states */
	const FDockTabStyle* MajorTabStyle;
	const FDockTabStyle* GenericTabStyle;

	TAttribute<FMargin> ContentAreaPadding;

	/** How should this tab be auto-sized based on its content? */
	EDockTabAutosizePolicy AutosizePolicy;

	/** Color of this tab */
	TAttribute<FLinearColor> TabColorScale;

	/** Color of this tab's icon */
	TAttribute<FLinearColor> IconColor;

	/** @return the scaling of the tab based on the opening/closing animation */
	SLATE_API UE::Slate::FDeprecateVector2DResult GetAnimatedScale() const;

	/** Animation that shows the tab opening up */
	FCurveSequence SpawnAnimCurve;

	/** Animation that causes the tab to flash */
	FCurveSequence FlashTabCurve;

	/** Get the desired color of tab. These change during flashing. */
	SLATE_API float GetFlashValue() const;

	/** The dock area this tab is currently being dragged over. Used in nomad tabs to change style */
	TSharedPtr<SDockingArea> DraggedOverDockingArea;

	/** Widget used to show the label on the tab */
	TSharedPtr<STextBlock> LabelWidget;
	TSharedPtr<STextBlock> LabelSuffix;

	/** Widget used to show the icon on the tab */
	TSharedPtr<SImage> IconWidget;
	
	/** Time this tab was last activated */
	double LastActivationTime;
};
