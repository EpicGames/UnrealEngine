// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Images/SImage.h"

class FPaintArgs;
class FSlateWindowElementList;

/** A widget that displays a spinning image.*/
class SSpinningImage : public SImage
{
public:
	SLATE_BEGIN_ARGS(SSpinningImage)
		: _Image( FCoreStyle::Get().GetDefaultBrush() )
		, _ColorAndOpacity( FLinearColor::White )
		, _OnMouseButtonDown()
		, _Period( 1.0f )
		{}

		/** Image resource */
		SLATE_ATTRIBUTE( const FSlateBrush*, Image )

		/** Color and opacity */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Invoked when the mouse is pressed in the widget. */
		SLATE_EVENT( FPointerEventHandler, OnMouseButtonDown )

		/** The amount of time in seconds for a full rotation */
		SLATE_ARGUMENT( float, Period )

	SLATE_END_ARGS()

	/* Construct this widget */
	SLATE_API void Construct( const FArguments& InArgs );

	//~ Begin SWidget interface
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual bool ComputeVolatility() const override;
	//~ End SWidget interface
	
private:
	/** The sequence to drive the spinning animation */
	FCurveSequence SpinAnimationSequence;
};
