// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/CoreStyle.h"

class FPaintArgs;
class FSlateWindowElementList;
class SHorizontalBox;


template <>
struct TWidgetTypeTraits<class SThrobber>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A throbber widget that uses 5 zooming circles in a row.
 */
class SThrobber
	: public SCompoundWidget
{
	static const int32 DefaultNumPieces = 3;

public:

	enum EAnimation
	{
		Vertical = 0x1 << 0,
		Horizontal = 0x1 << 1,
		Opacity = 0x1 << 2,
		VerticalAndOpacity = Vertical | Opacity,
		All = Vertical | Horizontal | Opacity,
		None = 0x0
	};

	SLATE_BEGIN_ARGS(SThrobber)	
		: _PieceImage( FCoreStyle::Get().GetBrush( "Throbber.Chunk" ) )
		, _NumPieces( DefaultNumPieces )
		, _Animate( SThrobber::All )
		{}

		/** What each segment of the throbber looks like */
		SLATE_ARGUMENT( const FSlateBrush*, PieceImage )
		/** How many pieces there are */
		SLATE_ARGUMENT( int32, NumPieces )
		/** Which aspects of the throbber to animate */
		SLATE_ARGUMENT( EAnimation, Animate )
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs);

	/** Sets what each segment of the throbber looks like. */
	SLATE_API void SetPieceImage(const FSlateBrush* InPieceImage);

	/**
	 * Invalidate the SCircularThrobber with the correct reason.
	 * You should invalidate when you are changing a property of SlateBrush after the SlateBrush was set to the SCircularThrobber.
	 */
	SLATE_API void InvalidatePieceImage();

	/** Sets how many pieces there are */
	SLATE_API void SetNumPieces(int InNumPieces);

	/** Sets which aspects of the throbber to animate */
	SLATE_API void SetAnimate(EAnimation InAnimate);

	//~ Begin SWidget interface
	SLATE_API virtual bool ComputeVolatility() const override;
	//~ End SWidget interface

private:

	FVector2D GetPieceScale(int32 PieceIndex) const;
	FLinearColor GetPieceColor(int32 PieceIndex) const;

	/** Constructs the curves and widgets for the pieces which make up the throbber. */
	void ConstructPieces();

	/** Gets the brush used to draw each piece of the throbber. */
	const FSlateBrush* GetPieceBrush() const;

private:
	
	FCurveSequence AnimCurves;
	TArray<FCurveHandle, TInlineAllocator<DefaultNumPieces> > ThrobberCurve;

	/** The horizontal box which contains the widgets for the throbber pieces. */
	TSharedPtr<SHorizontalBox> HBox;

	/** The image used to draw each piece of the throbber. */
	const FSlateBrush* PieceImage;

	/** The number of pieces to display. */
	int32 NumPieces;

	/** Flags for which aspects of the throbber to animate. */
	EAnimation Animate;
};


template <>
struct TWidgetTypeTraits<class SCircularThrobber>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A throbber widget that orients images in a spinning circle.
 */
class SCircularThrobber
	: public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SCircularThrobber, SLeafWidget, SLATE_API)

public:
	static SLATE_API const float MinimumPeriodValue;

	SLATE_BEGIN_ARGS(SCircularThrobber)	
		: _PieceImage( FCoreStyle::Get().GetBrush( "Throbber.CircleChunk" ) )
		, _NumPieces( 6 )
		, _Period( 0.75f )
		, _Radius( 16.f )
		{}

		/** What each segment of the throbber looks like */
		SLATE_ARGUMENT( const FSlateBrush*, PieceImage )
		/** How many pieces there are */
		SLATE_ARGUMENT( int32, NumPieces )
		/** The amount of time in seconds for a full circle */
		SLATE_ARGUMENT( float, Period )
		/** The radius of the circle */
		SLATE_ARGUMENT( float, Radius )
		/** Throbber color and opacity */
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)

	SLATE_END_ARGS()

	SLATE_API SCircularThrobber();
	SLATE_API virtual ~SCircularThrobber();

	/** Constructs the widget */
	SLATE_API void Construct(const FArguments& InArgs);

	/** Sets what each segment of the throbber looks like */
	SLATE_API void SetPieceImage(const FSlateBrush* InPieceImage);

	/**
	 * Invalidate the SCircularThrobber with the correct reason. 
	 * You should invalidate when you are changing a property of SlateBrush after the SlateBrush was set to the SCircularThrobber.
	 */
	SLATE_API void InvalidatePieceImage();

	/** Sets how many pieces there are */
	SLATE_API void SetNumPieces(int32 InNumPieces);

	/** Sets the amount of time in seconds for a full circle */
	SLATE_API void SetPeriod(float InPeriod);

	/** Sets the radius of the circle */
	SLATE_API void SetRadius(float InRadius);

	//~ Begin SWidget interface
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual bool ComputeVolatility() const override;
	//~ End SWidget interface

private:

	/** Constructs the sequence used to animate the throbber. */
	void ConstructSequence();

private:

	/** The sequence to drive the spinning animation */
	FCurveSequence Sequence;
	FCurveHandle Curve;

	/** What each segment of the throbber looks like */
	const FSlateBrush* PieceImage;

	/** How many pieces there are */
	int32 NumPieces;

	/** The amount of time in seconds for a full circle */
	float Period;

	/** The radius of the circle */
	float Radius;

	/** Color and opacity of the throbber images. */
	TSlateAttribute<FSlateColor> ColorAndOpacity;
	bool bColorAndOpacitySet;
};
