// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"

#include "IVirtualKeyboardEntry.generated.h"

// @todo - hook up keyboard types
enum EKeyboardType
{
	Keyboard_Default,
	Keyboard_Number,
	Keyboard_Web,
	Keyboard_Email,
	Keyboard_Password,
	Keyboard_AlphaNumeric,
};

enum class ETextEntryType : uint8
{
	/** The keyboard entry was canceled, some valid text may still be available. */
	TextEntryCanceled,
	/** The keyboard entry was accepted, full text is available. */
	TextEntryAccepted,
	/** They keyboard is providing a periodic update of the text entered so far. */
	TextEntryUpdated
};

USTRUCT()
struct FVirtualKeyboardOptions
{
public:
	GENERATED_BODY()

	/** Enables autocorrect for this widget, if supported by the platform's virtual keyboard. Autocorrect must also be enabled in Input settings for this to take effect. */
	UPROPERTY(EditAnywhere, Category = Autocorrect)
	bool bEnableAutocorrect;

	// TODO: Add additional VKB features, such as autocapitalization and autocomplete

	FVirtualKeyboardOptions()
		: bEnableAutocorrect(false)
	{
	}
};

DECLARE_DELEGATE(FOnSelectionChangedDelegateVK);

class IVirtualKeyboardEntry
{

public:
	virtual ~IVirtualKeyboardEntry() = default;

	/**
	* Sets the text to that entered by the virtual keyboard
	*
	* @param InNewText  The new text
	* @param TextEntryType What type of text update is being provided
	* @param CommitType If we are sending a TextCommitted event, what commit type is it
	*/
	virtual void SetTextFromVirtualKeyboard(const FText& InNewText, ETextEntryType TextEntryType) = 0;

	virtual void SetSelectionFromVirtualKeyboard(int InSelStart, int SelEnd) = 0;
	
	/**
	* Returns the text.
	*
	* @return  Text
	*/
	virtual FText GetText() const = 0;

	virtual bool GetSelection(int& OutSelStart, int& OutSelEnd) = 0;

	FOnSelectionChangedDelegateVK OnSelectionChanged;

	/**
	* Returns the hint text.
	*
	* @return  HintText
	*/
	virtual FText GetHintText() const = 0;

	/**
	* Returns the virtual keyboard type.
	*
	* @return  VirtualKeyboardType
	*/
	virtual EKeyboardType GetVirtualKeyboardType() const = 0;

	/**
	 * @return	Returns additional virtual keyboard options
	 */
	virtual FVirtualKeyboardOptions GetVirtualKeyboardOptions() const = 0;

	/**
	* Returns whether the entry is multi-line
	*
	* @return Whether the entry is multi-line
	*/
	virtual bool IsMultilineEntry() const = 0;
	
	/**
    * Returns whether the entry is using IntegratedKeyboard
    *
    * @return Whether the entry is using IntegratedKeyboard
    */
	virtual bool IsIntegratedKeyboardEnabled() const { return false; }
};
