// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/Find.h"
#include "Templates/IsIntegral.h"
#include "Templates/ValueOrError.h"
#include "Misc/ExpressionParserTypes.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Misc/Attribute.h"

enum class EUnit : uint8;

DECLARE_MULTICAST_DELEGATE(FOnSettingChanged);

/** Interface to provide specific functionality for dealing with a numeric type. Currently includes string conversion functionality. */
template<typename NumericType>
struct INumericTypeInterface
{
	virtual ~INumericTypeInterface() = default;

	/** Gets the minimum and maximum fractional digits. */
	virtual int32 GetMinFractionalDigits() const = 0;
	virtual int32 GetMaxFractionalDigits() const = 0;
	virtual bool GetIndicateNearlyInteger() const { return false; };

	/** Sets the minimum and maximum fractional digits - A minimum greater than 0 will always have that many trailing zeros */
	virtual void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) = 0;
	virtual void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) = 0;

	/** Sets if we should indicate that a value is being rounded to an integer via '...' (Ex: 0.0 shown, real value 1e-18). */
	virtual void SetIndicateNearlyInteger(const TAttribute<TOptional<bool>>& NewValue) {};

	/** Convert the type to/from a string */
	virtual FString ToString(const NumericType& Value) const = 0;
	virtual TOptional<NumericType> FromString(const FString& InString, const NumericType& ExistingValue) = 0;

	/** Check whether the typed character is valid */
	virtual bool IsCharacterValid(TCHAR InChar) const = 0;

	/** Optional callback to broadcast when a setting in the type interface changes */
	virtual FOnSettingChanged* GetOnSettingChanged() { return nullptr; }
};

/** Default numeric type interface */
template<typename NumericType>
struct TDefaultNumericTypeInterface : INumericTypeInterface<NumericType>
{

	/** The default minimum fractional digits */
	static const int16 DefaultMinFractionalDigits = 1;

	/** The default maximum fractional digits */
	static const int16 DefaultMaxFractionalDigits = 6;

	/** The default indicate nearly integer */
	static const bool DefaultIndicateNearlyInteger = false;

	/** The current minimum fractional digits */
	int16 MinFractionalDigits = DefaultMinFractionalDigits;

	/** The current maximum fractional digits */
	int16 MaxFractionalDigits = DefaultMaxFractionalDigits;

	/** True implies indicate when a value is displayed as rounded via '...' */
	bool bIndicateNearlyInteger = false;

	/** Gets the minimum and maximum fractional digits. */
	virtual int32 GetMinFractionalDigits() const override
	{
		return IntCastChecked<int32>(MinFractionalDigits);
	}
	virtual int32 GetMaxFractionalDigits() const override
	{
		return IntCastChecked<int32>(MaxFractionalDigits);
	}

	/** Get if we should indicate nearly integer values */
	virtual bool GetIndicateNearlyInteger() const override
	{ 
		return bIndicateNearlyInteger;
	};

	/** Sets the minimum and maximum fractional digits - A minimum greater than 0 will always have that many trailing zeros */
	virtual void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override
	{
		MinFractionalDigits = NewValue.Get().IsSet() ? IntCastChecked<int16>(FMath::Max(0, NewValue.Get().GetValue())) :
			DefaultMinFractionalDigits;
	}

	virtual void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override
	{
		MaxFractionalDigits = NewValue.Get().IsSet() ? IntCastChecked<int16>(FMath::Max(0, NewValue.Get().GetValue())) :
			DefaultMaxFractionalDigits;
	}

	virtual void SetIndicateNearlyInteger(const TAttribute<TOptional<bool>>& NewValue) override
	{
		bIndicateNearlyInteger = NewValue.Get().IsSet() ? NewValue.Get().GetValue() :
			DefaultIndicateNearlyInteger;
	}

	/** Convert the type to/from a string */
	virtual FString ToString(const NumericType& Value) const override
	{
		const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumFractionalDigits(TIsIntegral<NumericType>::Value ? 0 : MinFractionalDigits)
			.SetMaximumFractionalDigits(TIsIntegral<NumericType>::Value ? 0 : FMath::Max(MaxFractionalDigits, MinFractionalDigits))
			.SetIndicateNearlyInteger(TIsIntegral<NumericType>::Value ? false : bIndicateNearlyInteger);
		return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
	}

	virtual TOptional<NumericType> FromString(const FString& InString, const NumericType& InExistingValue) override
	{
		// Attempt to parse a number of type NumericType. Need to parse all the characters.
		FNumberParsingOptions ParsingOption = FNumberParsingOptions().SetUseGrouping(false).SetUseClamping(true);

		{
			NumericType PrimaryValue{};
			int32 PrimaryParsedLen = 0;
			bool PrimaryResult =  FastDecimalFormat::StringToNumber(*InString, InString.Len(), ExpressionParser::GetLocalizedNumberFormattingRules(), ParsingOption, PrimaryValue, &PrimaryParsedLen);
			if(PrimaryResult && PrimaryParsedLen == InString.Len())
			{
				return PrimaryValue;
			}
		}

		{
			NumericType FallbackValue{};
			int32 FallbackParsedLen = 0;
			bool FallbackResult = FastDecimalFormat::StringToNumber(*InString, InString.Len(), FastDecimalFormat::GetCultureAgnosticFormattingRules(), ParsingOption, FallbackValue, &FallbackParsedLen);
			if (FallbackResult && FallbackParsedLen == InString.Len())
			{
				return FallbackValue;
			}
		}

		// Attempt to parse it as an expression
		static FBasicMathExpressionEvaluator Parser;

		TValueOrError<double, FExpressionError> Result = Parser.Evaluate(*InString, static_cast<double>(InExistingValue));
		if (Result.IsValid())
		{
			return FMath::Clamp((NumericType)Result.GetValue(), TNumericLimits<NumericType>::Lowest(), TNumericLimits<NumericType>::Max());
		}

		return TOptional<NumericType>();
	}

	/** Check whether the typed character is valid */
	virtual bool IsCharacterValid(TCHAR InChar) const override
	{
		auto IsValidLocalizedCharacter = [InChar]() -> bool
		{
			const FDecimalNumberFormattingRules& NumberFormattingRules = ExpressionParser::GetLocalizedNumberFormattingRules();
			return InChar == NumberFormattingRules.GroupingSeparatorCharacter
				|| InChar == NumberFormattingRules.DecimalSeparatorCharacter
				|| Algo::Find(NumberFormattingRules.DigitCharacters, InChar) != 0;
		};

		static const FString ValidChars = "1234567890()-+=\\/.,*^%%";
		return InChar != 0 && (ValidChars.GetCharArray().Contains(InChar) || IsValidLocalizedCharacter());
	}
};

/** Forward declaration of types defined in UnitConversion.h */
enum class EUnit : uint8;
template<typename> struct FNumericUnit;

/**
 * Numeric interface that specifies how to interact with a number in a specific unit.
 * Include NumericUnitTypeInterface.inl for symbol definitions.
 */
template<typename NumericType>
struct TNumericUnitTypeInterface : TDefaultNumericTypeInterface<NumericType>
{
	/** The underlying units which the numeric type are specified in. */
	const EUnit UnderlyingUnits;

	/** Optional units that this type interface will be fixed on. These are usually auto-calculated by SetupFixedDisplay. */
	TOptional<EUnit> FixedDisplayUnits;

	/** Optional user-specified units that this type interface will be displayed in. If set, FixedDisplayUnits will be ignored. */
	TOptional<EUnit> UserDisplayUnits;

	/** Constructor */
	TNumericUnitTypeInterface(EUnit InUnits);

	/** Convert this type to a string */
	virtual FString ToString(const NumericType& Value) const override;

	/** Attempt to parse a numeral with our units from the specified string. */
	virtual TOptional<NumericType> FromString(const FString& ValueString, const NumericType& InExistingValue) override;

	/** Check whether the specified typed character is valid */
	virtual bool IsCharacterValid(TCHAR InChar) const override;

	/** Set up this interface to use a fixed display unit, calculated based on the specified value.
	 * For example, if underlying units are cm/s, but the provided value is over 100, it will use m/s instead. */
	void SetupFixedDisplay(const NumericType& InValue);
};
