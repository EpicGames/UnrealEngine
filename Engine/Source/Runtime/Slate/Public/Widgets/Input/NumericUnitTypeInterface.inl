// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Internationalization/Text.h"
#include "Templates/ValueOrError.h"
#include "Math/UnitConversion.h"

template<typename NumericType> struct TDefaultNumericTypeInterface;
template<typename NumericType> struct TNumericUnitTypeInterface;

template<typename NumericType>
TNumericUnitTypeInterface<NumericType>::TNumericUnitTypeInterface(EUnit InUnits)
	: UnderlyingUnits(InUnits)
{}

template<typename NumericType>
FString TNumericUnitTypeInterface<NumericType>::ToString(const NumericType& Value) const
{
	if (UnderlyingUnits == EUnit::Unspecified)
	{
		return TDefaultNumericTypeInterface<NumericType>::ToString(Value);
	}

	auto ToUnitString = [this](const FNumericUnit<NumericType>& InNumericUnit) -> FString
	{
		FString String = TDefaultNumericTypeInterface<NumericType>::ToString(InNumericUnit.Value);
		String += TEXT(" ");
		String += FUnitConversion::GetUnitDisplayString(InNumericUnit.Units);
		return String;
	};

	FNumericUnit<NumericType> FinalValue(Value, UnderlyingUnits);

	if (UserDisplayUnits.IsSet())
	{
		auto Converted = FinalValue.ConvertTo(UserDisplayUnits.GetValue());
		if (Converted.IsSet())
		{
			return ToUnitString(Converted.GetValue());
		}
	}

	if (FixedDisplayUnits.IsSet())
	{
		auto Converted = FinalValue.ConvertTo(FixedDisplayUnits.GetValue());
		if (Converted.IsSet())
		{
			return ToUnitString(Converted.GetValue());
		}
	}
	
	return ToUnitString(FinalValue);
}

template<typename NumericType>
TOptional<NumericType> TNumericUnitTypeInterface<NumericType>::FromString(const FString& InString, const NumericType& InExistingValue)
{
	if (UnderlyingUnits == EUnit::Unspecified)
	{
		return TDefaultNumericTypeInterface<NumericType>::FromString(InString, InExistingValue);
	}

	// Get displayed units from the user, if set, otherwise the fixed units, or finally the underlying units
	EUnit DefaultUnits = UserDisplayUnits.IsSet()
		? UserDisplayUnits.GetValue()
		: FixedDisplayUnits.IsSet()
			? FixedDisplayUnits.GetValue()
			: UnderlyingUnits;

	// Always parse in as a double, to allow for input of higher-order units with decimal numerals into integral types (eg, inputting 0.5km as 500m)
	TValueOrError<FNumericUnit<double>, FText> NewValue = FNumericUnit<double>::TryParseExpression( *InString, DefaultUnits, InExistingValue );
	if (NewValue.IsValid())
	{
		// Convert the number into the correct units
		EUnit SourceUnits = NewValue.GetValue().Units;
		if (SourceUnits == EUnit::Unspecified)
		{
			// Use the user overridden input units if set
			if (UserDisplayUnits.IsSet()) 
			{
				SourceUnits = UserDisplayUnits.GetValue();
			}
			// Use the default supplied input units
			else if (FixedDisplayUnits.IsSet())
			{
				SourceUnits = FixedDisplayUnits.GetValue();
			}
		}
		double ConvertedValue = FUnitConversion::Convert(NewValue.GetValue().Value, SourceUnits, UnderlyingUnits);
		return FMath::Clamp((NumericType)ConvertedValue, TNumericLimits<NumericType>::Lowest(), TNumericLimits<NumericType>::Max());
	}

	return TOptional<NumericType>();
}

template<typename NumericType>
bool TNumericUnitTypeInterface<NumericType>::IsCharacterValid(TCHAR InChar) const
{
	return (UnderlyingUnits == EUnit::Unspecified) ? TDefaultNumericTypeInterface<NumericType>::IsCharacterValid(InChar) : InChar != TEXT('\t');
}

template<typename NumericType>
void TNumericUnitTypeInterface<NumericType>::SetupFixedDisplay(const NumericType& InValue)
{
	// We calculate this regardless of whether FixedDisplayUnits is used, so that the moment it is used, it's correct
	EUnit DisplayUnit = FUnitConversion::CalculateDisplayUnit(InValue, UnderlyingUnits);
	if (DisplayUnit != EUnit::Unspecified)
	{
		FixedDisplayUnits = DisplayUnit;
	}
}
