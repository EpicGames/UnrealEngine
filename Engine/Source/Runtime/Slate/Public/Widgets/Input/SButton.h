// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Layout/Margin.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/Layout/SBorder.h"

class FPaintArgs;
class FSlateWindowElementList;
enum class ETextFlowDirection : uint8;
enum class ETextShapingMethod : uint8;

template <>
struct TWidgetTypeTraits<class SButton>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Slate's Buttons are clickable Widgets that can contain arbitrary widgets as its Content().
 */
class SButton : public SBorder
{
	SLATE_DECLARE_WIDGET_API(SButton, SBorder, SLATE_API)

#if WITH_ACCESSIBILITY
	// Allow the accessible button to "click" this button
	friend class FSlateAccessibleButton;
#endif
public:

	SLATE_BEGIN_ARGS( SButton )
		: _Content()
		, _ButtonStyle( &FCoreStyle::Get().GetWidgetStyle< FButtonStyle >( "Button" ) )
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle< FTextBlockStyle >("ButtonText") )
		, _HAlign( HAlign_Fill )
		, _VAlign( VAlign_Fill )
		, _ContentPadding(FMargin(4.0, 2.0))
		, _Text()
		, _OnSlateButtonDragDetected()
		, _OnSlateButtonDragEnter()
		, _OnSlateButtonDragLeave()
		, _OnSlateButtonDragOver()
		, _OnSlateButtonDrop()
		, _ClickMethod( EButtonClickMethod::DownAndUp )
		, _TouchMethod( EButtonTouchMethod::DownAndUp )
		, _PressMethod( EButtonPressMethod::DownAndUp )
		, _DesiredSizeScale( FVector2D(1,1) )
		, _ContentScale( FVector2D(1,1) )
		, _ButtonColorAndOpacity(FLinearColor::White)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _IsFocusable( true )
		, _AllowDragDrop( false )
		{
		}

		/** Slot for this button's content (optional) */
		SLATE_DEFAULT_SLOT( FArguments, Content )

		/** The visual style of the button */
		SLATE_STYLE_ARGUMENT( FButtonStyle, ButtonStyle )

		/** The text style of the button */
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )

		/** Horizontal alignment */
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )

		/** Vertical alignment */
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )
	
		/** Spacing between button's border and the content. */
		SLATE_ATTRIBUTE( FMargin, ContentPadding )

		/** If set, overrides the button style's additional spacing between the button's border and the content when not pressed. */
		SLATE_ATTRIBUTE( FMargin, NormalPaddingOverride )

		/** If set, overrides the button style's additional spacing between the button's border and the content when pressed. */
		SLATE_ATTRIBUTE( FMargin, PressedPaddingOverride )

		/** The text to display in this button, if no custom content is specified */
		SLATE_ATTRIBUTE( FText, Text )
	
		/** Called when the button is clicked */
		SLATE_EVENT( FOnClicked, OnClicked )

		/** Called when the button is pressed */
		SLATE_EVENT( FSimpleDelegate, OnPressed )

		/** Called when the button is released */
		SLATE_EVENT( FSimpleDelegate, OnReleased )

		SLATE_EVENT( FSimpleDelegate, OnHovered )

		SLATE_EVENT( FSimpleDelegate, OnUnhovered )

		SLATE_EVENT( FSimpleDelegate, OnReceivedFocus )

		SLATE_EVENT( FSimpleDelegate, OnLostFocus )

		// Drag and Drop
		SLATE_EVENT(FOnDragDetected, OnSlateButtonDragDetected)

		SLATE_EVENT(FOnDragEnter, OnSlateButtonDragEnter)

		SLATE_EVENT(FOnDragLeave, OnSlateButtonDragLeave)

		SLATE_EVENT(FOnDragOver, OnSlateButtonDragOver)

		SLATE_EVENT(FOnDrop, OnSlateButtonDrop)
		// End Drag and Drop

		/** Sets the rules to use for determining whether the button was clicked.  This is an advanced setting and generally should be left as the default. */
		SLATE_ARGUMENT( EButtonClickMethod::Type, ClickMethod )

		/** How should the button be clicked with touch events? */
		SLATE_ARGUMENT( EButtonTouchMethod::Type, TouchMethod )

		/** How should the button be clicked with keyboard/controller button events? */
		SLATE_ARGUMENT( EButtonPressMethod::Type, PressMethod )

		SLATE_ATTRIBUTE( FVector2D, DesiredSizeScale )

		SLATE_ATTRIBUTE( FVector2D, ContentScale )

		SLATE_ATTRIBUTE( FSlateColor, ButtonColorAndOpacity )

		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )

		/** Sometimes a button should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT( bool, IsFocusable )

		/** True if the button should detect drag on mouse down. */
		SLATE_ARGUMENT(bool, AllowDragDrop)

		/** The sound to play when the button is pressed */
		SLATE_ARGUMENT( TOptional<FSlateSound>, PressedSoundOverride )

		/** The sound to play when the button is clicked */
		SLATE_ARGUMENT( TOptional<FSlateSound>, ClickedSoundOverride )

		/** The sound to play when the button is hovered */
		SLATE_ARGUMENT( TOptional<FSlateSound>, HoveredSoundOverride )

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		
		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

	SLATE_END_ARGS()

		SLATE_API virtual ~SButton();

protected:
		SLATE_API SButton();

public:
	/** @return the Foreground color that this widget sets; unset options if the widget does not set a foreground color */
	virtual FSlateColor GetForegroundColor() const final
	{
		return Super::GetForegroundColor();
	}

	/** @return the Foreground color that this widget sets when this widget or any of its ancestors are disabled; unset options if the widget does not set a foreground color */
	SLATE_API virtual FSlateColor GetDisabledForegroundColor() const final;

	/**
	 * Returns true if this button is currently pressed
	 *
	 * @return	True if pressed, otherwise false
	 * @note IsPressed used to be virtual. Use SetAppearPressed to assign an attribute if you need to override the default behavior.
	 */
	bool IsPressed() const
	{
		return bIsPressed || AppearPressedAttribute.Get();
	}

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/** See ContentPadding attribute */
	SLATE_API void SetContentPadding(TAttribute<FMargin> InContentPadding);

	/** See HoveredSound attribute */
	SLATE_API void SetHoveredSound(TOptional<FSlateSound> InHoveredSound);

	/** See PressedSound attribute */
	SLATE_API void SetPressedSound(TOptional<FSlateSound> InPressedSound);

	/** See ClickedSound attribute */
	SLATE_API void SetClickedSound(TOptional<FSlateSound> InClickedSound);

	/** See OnClicked event */
	SLATE_API void SetOnClicked(FOnClicked InOnClicked);

	/** Set OnHovered event */
	SLATE_API void SetOnHovered(FSimpleDelegate InOnHovered);

	/** Set OnUnhovered event */
	SLATE_API void SetOnUnhovered(FSimpleDelegate InOnUnhovered);

	/** Set OnFocusReceived event */
	SLATE_API void SetOnFocusReceived(FSimpleDelegate InOnFocusReceived);

	/** Set OnFocusLost event */
	SLATE_API void SetOnFocusLost(FSimpleDelegate InOnFocusLost);

	/** See ButtonStyle attribute */
	SLATE_API void SetButtonStyle(const FButtonStyle* ButtonStyle);

	SLATE_API void SetClickMethod(EButtonClickMethod::Type InClickMethod);
	SLATE_API void SetTouchMethod(EButtonTouchMethod::Type InTouchMethod);
	SLATE_API void SetPressMethod(EButtonPressMethod::Type InPressMethod);

	/** Set if this button can be dragged */
	SLATE_API void SetAllowDragDrop(bool bAllowDragDrop);

#if !UE_BUILD_SHIPPING
	SLATE_API void SimulateClick();
#endif // !UE_BUILD_SHIPPING

public:

	//~ SWidget overrides
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override;
	SLATE_API virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override;
	SLATE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	SLATE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	SLATE_API virtual bool IsInteractable() const override;

#if WITH_ACCESSIBILITY
	SLATE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
#endif
protected:
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ SWidget

protected:
	/** Press the button */
	SLATE_API virtual void Press();

	/** Release the button */
	SLATE_API virtual void Release();

	/** Execute the "OnClicked" delegate, and get the reply */
	SLATE_API FReply ExecuteOnClick();

	/** @return combines the user-specified margin and the button's internal margin. */
	SLATE_API FMargin GetCombinedPadding() const;

	/** @return True if the disabled effect should be shown. */
	SLATE_API bool GetShowDisabledEffect() const;

	/** Utility function to translate other input click methods to regular ones. */
	SLATE_API TEnumAsByte<EButtonClickMethod::Type> GetClickMethodFromInputType(const FPointerEvent& MouseEvent) const;

	/** Utility function to determine if the incoming mouse event is for a precise tap or click */
	SLATE_API bool IsPreciseTapOrClick(const FPointerEvent& MouseEvent) const;

	/** Play the pressed sound */
	SLATE_API void PlayPressedSound() const;

	/** Play the clicked sound */
	SLATE_API void PlayClickedSound() const;

	/** Play the hovered sound */
	SLATE_API void PlayHoverSound() const;

	/** Set if this button can be focused */
	void SetIsFocusable(bool bInIsFocusable)
	{
		bIsFocusable = bInIsFocusable;
	}

	SLATE_API void ExecuteHoverStateChanged(bool bPlaySound);

protected:
	/** @return the BorderForegroundColor attribute. */
	TSlateAttributeRef<FSlateColor> GetBorderForegroundColorAttribute() const { return TSlateAttributeRef<FSlateColor>(SharedThis(this), BorderForegroundColorAttribute); }

	/** @return the ContentPadding attribute. */
	TSlateAttributeRef<FMargin> GetContentPaddingAttribute() const { return TSlateAttributeRef<FMargin>(SharedThis(this), ContentPaddingAttribute); }

	/** Set the AppearPressed look. */
	void SetAppearPressed(TAttribute<bool> InValue)
	{
		AppearPressedAttribute.Assign(*this, MoveTemp(InValue));
	}

	/** @return the AppearPressed attribute. */
	TSlateAttributeRef<bool> GetAppearPressedAttribute() const { return TSlateAttributeRef<bool>(SharedThis(this), AppearPressedAttribute); }

private:
	SLATE_API void UpdatePressStateChanged();

	SLATE_API void UpdatePadding();
	SLATE_API void UpdateShowDisabledEffect();
	SLATE_API void UpdateBorderImage();
	SLATE_API void UpdateForegroundColor();
	SLATE_API void UpdateDisabledForegroundColor();

private:
	/** The location in screenspace the button was pressed */
	FVector2D PressedScreenSpacePosition;

	/** Style resource for the button */
	const FButtonStyle* Style;
	/** The delegate to execute when the button is clicked */
	FOnClicked OnClicked;

	/** The delegate to execute when the button is pressed */
	FSimpleDelegate OnPressed;

	/** The delegate to execute when the button is released */
	FSimpleDelegate OnReleased;

	/** The delegate to execute when the button is hovered */
	FSimpleDelegate OnHovered;

	/** The delegate to execute when the button exit the hovered state */
	FSimpleDelegate OnUnhovered;

	/** Delegate fired whenever focus is received */
	FSimpleDelegate OnReceivedFocus;

	/** Delegate fired whenever focus is lost */
	FSimpleDelegate OnLostFocus;

	/** Delegate triggered when a user starts to drag a button */
	FOnDragDetected OnSlateButtonDragDetected;

	/** Delegate triggered when a user's drag enters the bounds of this button*/ 
	FOnDragEnter OnSlateButtonDragEnter;

	/** Delegate triggered when a user's drag leaves the bounds of this button */
	FOnDragLeave OnSlateButtonDragLeave;

	/** Delegate triggered when a user's drag leaves the bounds of this button */
	FOnDragOver OnSlateButtonDragOver;

	/** Delegate triggered when a user's drag is dropped in the bounds of this button */
	FOnDrop OnSlateButtonDrop;

	/** The Sound to play when the button is hovered  */
	FSlateSound HoveredSound;

	/** The Sound to play when the button is pressed */
	FSlateSound PressedSound;

	/** The Sound to play when the button is clicked */
	FSlateSound ClickedSound;

	/** Sets whether a click should be triggered on mouse down, mouse up, or that both a mouse down and up are required. */
	TEnumAsByte<EButtonClickMethod::Type> ClickMethod;

	/** How should the button be clicked with touch events? */
	TEnumAsByte<EButtonTouchMethod::Type> TouchMethod;

	/** How should the button be clicked with keyboard/controller button events? */
	TEnumAsByte<EButtonPressMethod::Type> PressMethod;

	/** True if this button can be dragged */
	bool bAllowDragDrop = false;

	/** Can this button be focused? */
	uint8 bIsFocusable:1;

	/** True if this button is currently in a pressed state */
	uint8 bIsPressed:1;

	/** True if NormalPaddingAttribute is overriding the button style's normal padding */
	uint8 bIsStyleNormalPaddingOverridden:1;

	/** True if PressedPaddingAttribute is overriding the button style's pressed padding */
	uint8 bIsStylePressedPaddingOverridden:1;

private:
	/** Optional foreground color that will be inherited by all of this widget's contents */
	TSlateAttribute<FSlateColor> BorderForegroundColorAttribute;
	/** Padding specified by the user; it will be combind with the button's internal padding. */
	TSlateAttribute<FMargin> ContentPaddingAttribute;
	/** Normal padding override specified by the user, or the button style's normal padding if not being overridden. */
	TSlateAttribute<FMargin> NormalPaddingAttribute;
	/** Pressed padding override specified by the user, or the button style's pressed padding if not being overridden. */
	TSlateAttribute<FMargin> PressedPaddingAttribute;
	/** Optional foreground color that will be inherited by all of this widget's contents */
	TSlateAttribute<bool> AppearPressedAttribute;
};
