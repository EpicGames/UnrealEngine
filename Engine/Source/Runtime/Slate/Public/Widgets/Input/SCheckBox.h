// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"

class SBorder;


/** Delegate that is executed when the check box state changes */
DECLARE_DELEGATE_OneParam( FOnCheckStateChanged, ECheckBoxState );

template<>
struct TWidgetTypeTraits<class SCheckBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Check box Slate control
 */
class SCheckBox : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SCheckBox, SCompoundWidget, SLATE_API)
public:

	SLATE_BEGIN_ARGS( SCheckBox )
		: _Content()
		, _Style( &FCoreStyle::Get().GetWidgetStyle< FCheckBoxStyle >("Checkbox") )
		, _Type()
		, _OnCheckStateChanged()
		, _IsChecked( ECheckBoxState::Unchecked )
		, _HAlign( HAlign_Fill )
		, _CheckBoxImageVAlign( VAlign_Center )
		, _CheckBoxContentUsesAutoWidth(true)
		, _Padding()
		, _ClickMethod( EButtonClickMethod::DownAndUp )
		, _TouchMethod(EButtonTouchMethod::DownAndUp)
		, _PressMethod(EButtonPressMethod::DownAndUp)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _BorderBackgroundColor ()
		, _IsFocusable( true )
		, _UncheckedImage( nullptr )
		, _UncheckedHoveredImage( nullptr )
		, _UncheckedPressedImage( nullptr )
		, _CheckedImage( nullptr )
		, _CheckedHoveredImage( nullptr )
		, _CheckedPressedImage( nullptr )
		, _UndeterminedImage( nullptr )
		, _UndeterminedHoveredImage( nullptr )
		, _UndeterminedPressedImage( nullptr )
		, _BackgroundImage( nullptr )
		, _BackgroundHoveredImage( nullptr )
		, _BackgroundPressedImage( nullptr )
	{
	}

		/** Content to be placed next to the check box, or for a toggle button, the content to be placed inside the button */
		SLATE_DEFAULT_SLOT( FArguments, Content )

		/** The style structure for this checkbox' visual style */
		SLATE_STYLE_ARGUMENT( FCheckBoxStyle, Style )

		/** Type of check box (set by the Style arg but the Style can be overridden with this) */
		SLATE_ARGUMENT( TOptional<ESlateCheckBoxType::Type>, Type )

		/** Called when the checked state has changed */
		SLATE_EVENT( FOnCheckStateChanged, OnCheckStateChanged )

		/** Whether the check box is currently in a checked state */
		SLATE_ATTRIBUTE( ECheckBoxState, IsChecked )

		/** How the content of the toggle button should align within the given space */
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )

		/** How the image of the checkbox should align vertically within the given space */
		SLATE_ARGUMENT( EVerticalAlignment, CheckBoxImageVAlign )

		/** Whether or not the content portion of the checkbox should layout using auto-width. When true the content will always be arranged at its desired size as opposed to resizing to the available space. */
		SLATE_ARGUMENT(bool, CheckBoxContentUsesAutoWidth)

		/** Spacing between the check box image and its content (set by the Style arg but the Style can be overridden with this) */
		SLATE_ATTRIBUTE( FMargin, Padding )

		/** Sets the rules to use for determining whether the button was clicked.  This is an advanced setting and generally should be left as the default. */
		SLATE_ARGUMENT( EButtonClickMethod::Type, ClickMethod )

		/** How should the button be clicked with touch events? */
		SLATE_ARGUMENT(EButtonTouchMethod::Type, TouchMethod)

		/** How should the button be clicked with keyboard/controller button events? */
		SLATE_ARGUMENT(EButtonPressMethod::Type, PressMethod)

		/** Foreground color for the checkbox's content and parts (set by the Style arg but the Style can be overridden with this) */
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )

		/** The color of the background border (set by the Style arg but the Style can be overridden with this) */
		SLATE_ATTRIBUTE( FSlateColor, BorderBackgroundColor )

		SLATE_ARGUMENT( bool, IsFocusable )
		
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )

		/** The sound to play when the check box is checked */
		SLATE_ARGUMENT( TOptional<FSlateSound>, CheckedSoundOverride )

		/** The sound to play when the check box is unchecked */
		SLATE_ARGUMENT( TOptional<FSlateSound>, UncheckedSoundOverride )

		/** The sound to play when the check box is hovered */
		SLATE_ARGUMENT( TOptional<FSlateSound>, HoveredSoundOverride )

		/** The unchecked image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, UncheckedImage)

		/** The unchecked hovered image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, UncheckedHoveredImage)

		/** The unchecked pressed image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, UncheckedPressedImage)

		/** The checked image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, CheckedImage)

		/** The checked hovered image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, CheckedHoveredImage)

		/** The checked pressed image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, CheckedPressedImage)

		/** The undetermined image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, UndeterminedImage)

		/** The undetermined hovered image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, UndeterminedHoveredImage)

		/** The undetermined pressed image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, UndeterminedPressedImage)

		/** The background image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, BackgroundImage)

		/** The background hovered image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, BackgroundHoveredImage)

		/** The background pressed image for the checkbox - overrides the style's */
		SLATE_ARGUMENT(const FSlateBrush*, BackgroundPressedImage)




	SLATE_END_ARGS()

	SLATE_API SCheckBox();
	SLATE_API virtual ~SCheckBox();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	SLATE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual bool IsInteractable() const override;
#if WITH_ACCESSIBILITY
	SLATE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
#endif
	// End of SWidget interface

	/**
	 * Returns true if the checkbox is currently checked
	 *
	 * @return	True if checked, otherwise false
	 */
	bool IsChecked() const
	{
		return ( GetCheckedState() == ECheckBoxState::Checked );
	}

	/** @return The current checked state of the checkbox. */
	SLATE_API ECheckBoxState GetCheckedState() const;

	/**
	 * Returns true if this button is currently pressed
	 *
	 * @return	True if pressed, otherwise false
	 */
	bool IsPressed() const
	{
		return bIsPressed;
	}

	/**
	 * Toggles the checked state for this check box, fire events as needed
	 */
	SLATE_API void ToggleCheckedState();

	/** See the IsChecked attribute */
	SLATE_API void SetIsChecked(TAttribute<ECheckBoxState> InIsChecked);
	
	/** See the Content slot */
	SLATE_API void SetContent(const TSharedRef< SWidget >& InContent);
	
	/** See the Style attribute */
	SLATE_API void SetStyle(const FCheckBoxStyle* InStyle);
	
	/** See the UncheckedImage attribute */
	SLATE_API void SetUncheckedImage(const FSlateBrush* Brush);
	/** See the UncheckedHoveredImage attribute */
	SLATE_API void SetUncheckedHoveredImage(const FSlateBrush* Brush);
	/** See the UncheckedPressedImage attribute */
	SLATE_API void SetUncheckedPressedImage(const FSlateBrush* Brush);
	
	/** See the CheckedImage attribute */
	SLATE_API void SetCheckedImage(const FSlateBrush* Brush);
	/** See the CheckedHoveredImage attribute */
	SLATE_API void SetCheckedHoveredImage(const FSlateBrush* Brush);
	/** See the CheckedPressedImage attribute */
	SLATE_API void SetCheckedPressedImage(const FSlateBrush* Brush);
	
	/** See the UndeterminedImage attribute */
	SLATE_API void SetUndeterminedImage(const FSlateBrush* Brush);
	/** See the UndeterminedHoveredImage attribute */
	SLATE_API void SetUndeterminedHoveredImage(const FSlateBrush* Brush);
	/** See the UndeterminedPressedImage attribute */
	SLATE_API void SetUndeterminedPressedImage(const FSlateBrush* Brush);

	SLATE_API void SetClickMethod(EButtonClickMethod::Type InClickMethod);
	SLATE_API void SetTouchMethod(EButtonTouchMethod::Type InTouchMethod);
	SLATE_API void SetPressMethod(EButtonPressMethod::Type InPressMethod);

protected:

	/** Rebuilds the checkbox based on the current ESlateCheckBoxType */
	SLATE_API void BuildCheckBox(TSharedRef<SWidget> InContent);

	/** Attribute getter for the padding */
	SLATE_API FMargin OnGetPadding() const;
	/** Attribute getter for the border background color */
	SLATE_API FSlateColor OnGetBorderBackgroundColor() const;
	/** Attribute getter for the checkbox type */
	SLATE_API ESlateCheckBoxType::Type OnGetCheckBoxType() const;

	/**
	 * Gets the check image to display for the current state of the check box
	 * @return	The name of the image to display
	 */
	SLATE_API const FSlateBrush* OnGetCheckImage() const;
	
	SLATE_API const FSlateBrush* GetUncheckedImage() const;
	SLATE_API const FSlateBrush* GetUncheckedHoveredImage() const;
	SLATE_API const FSlateBrush* GetUncheckedPressedImage() const;
	
	SLATE_API const FSlateBrush* GetCheckedImage() const;
	SLATE_API const FSlateBrush* GetCheckedHoveredImage() const;
	SLATE_API const FSlateBrush* GetCheckedPressedImage() const;
	
	SLATE_API const FSlateBrush* GetUndeterminedImage() const;
	SLATE_API const FSlateBrush* GetUndeterminedHoveredImage() const;
	SLATE_API const FSlateBrush* GetUndeterminedPressedImage() const;

	/** Attribute getter for the background image */
	SLATE_API const FSlateBrush* OnGetBackgroundImage() const;

	SLATE_API const FSlateBrush* GetBackgroundImage() const;
	SLATE_API const FSlateBrush* GetBackgroundHoveredImage() const;
	SLATE_API const FSlateBrush* GetBackgroundPressedImage() const;

protected:
	SLATE_API void SetPaddingOverride(TAttribute<FMargin> InPaddingOverride);
	TSlateAttributeRef<FMargin> GetPaddingOverrideAttribute() const
	{
		return TSlateAttributeRef<FMargin>{SharedThis(this), PaddingOverrideAttribute};
	}

	SLATE_API void SetForegroundColorOverride(TAttribute<FSlateColor> InForegroundColorOverride);
	TSlateAttributeRef<FSlateColor> GetForegroundColorOverrideAttribute() const
	{
		return TSlateAttributeRef<FSlateColor>{SharedThis(this), ForegroundColorOverrideAttribute};
	}

	SLATE_API void SetBorderBackgroundColorOverride(TAttribute<FSlateColor> InBorderBackgroundColorOverride);
	TSlateAttributeRef<FSlateColor> GetBorderBackgroundColorOverrideAttribute() const
	{
		return TSlateAttributeRef<FSlateColor>{SharedThis(this), BorderBackgroundColorOverrideAttribute};
	}

	/** The Sound to play when the check box is hovered  */
	void SetHoveredSound(TOptional<FSlateSound> InHoveredSound);
	FSlateSound GetHoveredSound() const;

	/** The Sound to play when the check box is checked */
	void SetCheckedSound(TOptional<FSlateSound> InCheckedSound);
	FSlateSound GetCheckedSound() const;

	/** The Sound to play when the check box is unchecked */
	void SetUncheckedSound(TOptional<FSlateSound> InUncheckedSound);
	FSlateSound GetUncheckedSound() const;

private:
	/**
	 * Returns the Foreground Color of this widget.
	 * Used to avoid overriding SCompoundWidget::GetForegroundColor for invalidation to work properly.
	 */
	FSlateColor GetForegroundColorInternal() const;

protected:
	
	const FCheckBoxStyle* Style;

	/** True if this check box is currently in a pressed state */
	bool bIsPressed;

	/** Are we checked */
	TAttribute<ECheckBoxState> IsCheckboxChecked;

	/** Delegate called when the check box changes state */
	FOnCheckStateChanged OnCheckStateChanged;

	/** Image to use when the checkbox is unchecked */
	const FSlateBrush* UncheckedImage;
	/** Image to use when the checkbox is unchecked and hovered*/
	const FSlateBrush* UncheckedHoveredImage;
	/** Image to use when the checkbox is unchecked and pressed*/
	const FSlateBrush* UncheckedPressedImage;
	/** Image to use when the checkbox is checked*/
	const FSlateBrush* CheckedImage;
	/** Image to use when the checkbox is checked and hovered*/
	const FSlateBrush* CheckedHoveredImage;
	/** Image to use when the checkbox is checked and pressed*/
	const FSlateBrush* CheckedPressedImage;
	/** Image to use when the checkbox is in an ambiguous state*/
	const FSlateBrush* UndeterminedImage;
	/** Image to use when the checkbox is in an ambiguous state and hovered*/
	const FSlateBrush* UndeterminedHoveredImage;
	/** Image to use when the checkbox is in an ambiguous state and pressed*/
	const FSlateBrush* UndeterminedPressedImage;
	/** Image to use for the checkbox background */
	const FSlateBrush* BackgroundImage;
	/** Image to use for the checkbox background when hovered*/
	const FSlateBrush* BackgroundHoveredImage;
	/** Image to use for the checkbox background when pressed*/
	const FSlateBrush* BackgroundPressedImage;

#if WITH_EDITORONLY_DATA
	/** Overrides padding in the widget style, if set */
	UE_DEPRECATED(5.8, "Use SetPaddingOverride / OnGetPadding / GetPaddingOverrideAttribute")
	TSlateDeprecatedTAttribute<FMargin> PaddingOverride;
	/** Overrides foreground color in the widget style, if set */
	UE_DEPRECATED(5.8, "Use SetForegroundColorOverride / GetForegroundColor / GetForegroundColorOverrideAttribute")
	TSlateDeprecatedTAttribute<FSlateColor> ForegroundColorOverride;
	/** Overrides border background color in the widget style, if set */
	UE_DEPRECATED(5.8, "Use SetBorderBackgroundColorOverride / OnGetBorderBackgroundColor / GetBorderBackgroundColorOverrideAttribute")
	TSlateDeprecatedTAttribute<FSlateColor> BorderBackgroundColorOverride;
#endif
	/** Overrides checkbox type in the widget style, if set */
	TOptional<ESlateCheckBoxType::Type> CheckBoxTypeOverride;

	/** How the content of the toggle button should align within the given space */
	EHorizontalAlignment HorizontalAlignment;

	/** How the image of the checkbox should align vertically within the given space */
	EVerticalAlignment CheckBoxImageVAlign;

	/** Whether or not the checkbox content is arranged using auto-width when in checkbox mode. */
	bool bCheckBoxContentUsesAutoWidth;

	/** Sets whether a click should be triggered on mouse down, mouse up, or that both a mouse down and up are required. */
	EButtonClickMethod::Type ClickMethod;

	/** How should the button be clicked with touch events? */
	TEnumAsByte<EButtonTouchMethod::Type> TouchMethod;

	/** How should the button be clicked with keyboard/controller button events? */
	TEnumAsByte<EButtonPressMethod::Type> PressMethod;

	/** When true, this checkbox will be keyboard focusable. Defaults to true. */
	bool bIsFocusable;

	/** Delegate to execute to get the menu content of this button */
	FOnGetContent OnGetMenuContent;

	/** Play the checked sound */
	SLATE_API void PlayCheckedSound() const;

	/** Play the unchecked sound */
	SLATE_API void PlayUncheckedSound() const;

	/** Play the hovered sound */
	SLATE_API void PlayHoverSound() const;

	/** Utility function to translate other input click methods to regular ones. */
	SLATE_API TEnumAsByte<EButtonClickMethod::Type> GetClickMethodFromInputType(const FPointerEvent& MouseEvent) const;

	/** Utility function to determine if the incoming mouse event is for a precise tap or click */
	SLATE_API bool IsPreciseTapOrClick(const FPointerEvent& MouseEvent) const;

#if WITH_EDITORONLY_DATA
	/** The Sound to play when the check box is hovered  */
	UE_DEPRECATED(5.8, "Use GetHoveredSound / SetHoveredSound")
	FSlateSound HoveredSound;

	/** The Sound to play when the check box is checked */
	UE_DEPRECATED(5.8, "Use GetCheckedSound / SetCheckedSound")
	FSlateSound CheckedSound;

	/** The Sound to play when the check box is unchecked */
	UE_DEPRECATED(5.8, "Use GetUncheckedSound / SetUncheckedSound")
	FSlateSound UncheckedSound;
#endif

protected:
	/** When in toggle button mode, this will hold the pointer to the toggle button's border */
	TSharedPtr<SBorder> ContentContainer;

private:
	/** Overrides padding in the widget style, if set */
	TSlateAttribute<FMargin> PaddingOverrideAttribute;
	bool bIsPaddingAttrSet = false;
	
	/** Overrides foreground color in the widget style, if set */
	TSlateAttribute<FSlateColor> ForegroundColorOverrideAttribute;
	/** Overrides border background color in the widget style, if set */
	TSlateAttribute<FSlateColor> BorderBackgroundColorOverrideAttribute;
	bool bIsBorderBackgroundColorOverrideAttrSet = false;

	/** The Sound to play when the check box is hovered  */
	TOptional<FSlateSound> HoveredSoundOverride;

	/** The Sound to play when the check box is checked */
	TOptional<FSlateSound> CheckedSoundOverride;

	/** The Sound to play when the check box is unchecked */
	TOptional<FSlateSound> UncheckedSoundOverride;
};
