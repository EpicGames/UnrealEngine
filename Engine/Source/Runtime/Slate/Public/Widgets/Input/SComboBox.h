// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#if WITH_ACCESSIBILITY
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

DECLARE_DELEGATE( FOnComboBoxOpening )

namespace SlateComboRow
{
	extern SLATE_API bool bMouseButtonHandlingV2;
}

template<typename OptionType>
class SComboRow : public STableRow< OptionType >
{
public:

	SLATE_BEGIN_ARGS( SComboRow )
		: _Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
		, _Content()
		, _Padding(FMargin(0))
		{}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs this widget.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable )
	{
		STableRow< OptionType >::Construct(
			typename STableRow<OptionType>::FArguments()
			.Style(InArgs._Style)
			.Padding(InArgs._Padding)
			.bAllowPreselectedItemActivation(SlateComboRow::bMouseButtonHandlingV2)
			.Content()
			[
				InArgs._Content.Widget
			]
			, InOwnerTable
		);
	}

	// handle case where user clicks on an existing selected item
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		if (!STableRow<OptionType>::bAllowPreselectedItemActivation)
		{
			if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
			{
				TSharedPtr< ITypedTableView<OptionType> > OwnerWidget = this->OwnerTablePtr.Pin();

				const TObjectPtrWrapTypeOf<OptionType>* MyItem = OwnerWidget->Private_ItemFromWidget( this );
				const bool bIsSelected = OwnerWidget->Private_IsItemSelected( *MyItem );

				if (bIsSelected)
				{
					// Reselect content to ensure selection is taken
					OwnerWidget->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
					return FReply::Handled();
				}
			}
		}
		return STableRow<OptionType>::OnMouseButtonDown(MyGeometry, MouseEvent);
	}
};


/**
 * A combo box that shows arbitrary content.
 */
template< typename OptionType >
class SComboBox : public SComboButton
{
public:

	typedef TListTypeTraits< OptionType > ListTypeTraits;
	typedef typename TListTypeTraits< OptionType >::NullableType NullableOptionType;

	/** Type of list used for showing menu options. */
	typedef SListView< OptionType > SComboListType;
	/** Delegate type used to generate widgets that represent Options */
	typedef typename TSlateDelegates< OptionType >::FOnGenerateWidget FOnGenerateWidget;
	typedef typename TSlateDelegates< NullableOptionType >::FOnSelectionChanged FOnSelectionChanged;

	SLATE_BEGIN_ARGS( SComboBox )
		: _Content()
		, _ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle< FComboBoxStyle >("ComboBox"))
		, _ButtonStyle(nullptr)
		, _ItemStyle(&FAppStyle::Get().GetWidgetStyle< FTableRowStyle >("ComboBox.Row"))
		, _ScrollBarStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
		, _ContentPadding(_ComboBoxStyle->ContentPadding)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _OnSelectionChanged()
		, _OnGenerateWidget()
		, _InitiallySelectedItem(ListTypeTraits::MakeNullPtr())
		, _Method()
		, _MaxListHeight(450.0f)
		, _HasDownArrow( true )
		, _EnableGamepadNavigationMode(false)
		, _IsFocusable( true )
		, _ReselectionTriggersOnSelectionChanged(false)
		{}
		
		/** Slot for this button's content (optional) */
		SLATE_DEFAULT_SLOT( FArguments, Content )

		SLATE_STYLE_ARGUMENT( FComboBoxStyle, ComboBoxStyle )

		/** The visual style of the button part of the combo box (overrides ComboBoxStyle) */
		SLATE_STYLE_ARGUMENT( FButtonStyle, ButtonStyle )

		SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemStyle)
		
		SLATE_STYLE_ARGUMENT( FScrollBarStyle, ScrollBarStyle )

		SLATE_ATTRIBUTE( FMargin, ContentPadding )
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )

		SLATE_ITEMS_SOURCE_ARGUMENT( OptionType, OptionsSource )
		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
		SLATE_EVENT( FOnGenerateWidget, OnGenerateWidget )

		/** Called when combo box is opened, before list is actually created */
		SLATE_EVENT( FOnComboBoxOpening, OnComboBoxOpening )

		/** The custom scrollbar to use in the ListView */
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, CustomScrollbar)

		/** The option that should be selected when the combo box is first created */
		SLATE_ARGUMENT( NullableOptionType, InitiallySelectedItem )

		SLATE_ARGUMENT( TOptional<EPopupMethod>, Method )

		/** The max height of the combo box menu */
		SLATE_ARGUMENT(float, MaxListHeight)

		/** The sound to play when the button is pressed (overrides ComboBoxStyle) */
		SLATE_ARGUMENT( TOptional<FSlateSound>, PressedSoundOverride )

		/** The sound to play when the selection changes (overrides ComboBoxStyle) */
		SLATE_ARGUMENT( TOptional<FSlateSound>, SelectionChangeSoundOverride )

		/**
		 * When false, the down arrow is not generated and it is up to the API consumer
		 * to make their own visual hint that this is a drop down.
		 */
		SLATE_ARGUMENT( bool, HasDownArrow )

		/**
		 *  When false, directional keys will change the selection. When true, ComboBox
		 *	must be activated and will only capture arrow input while activated.
		*/
		SLATE_ARGUMENT(bool, EnableGamepadNavigationMode)

		/** When true, allows the combo box to receive keyboard focus */
		SLATE_ARGUMENT( bool, IsFocusable )

		/** True if this combo's menu should be collapsed when our parent receives focus, false (default) otherwise */
		SLATE_ARGUMENT(bool, CollapseMenuOnParentFocus)

		/** Whether OnSelectionChanged event gets triggered when selecting the item that's already selected */
		SLATE_ARGUMENT(bool, ReselectionTriggersOnSelectionChanged)

	SLATE_END_ARGS()

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the combo box
	 */
	void Construct( const FArguments& InArgs )
	{
		check(InArgs._ComboBoxStyle);

		ItemStyle = InArgs._ItemStyle;
		ComboBoxStyle = InArgs._ComboBoxStyle;
		MenuRowPadding = ComboBoxStyle->MenuRowPadding;

		// Work out which values we should use based on whether we were given an override, or should use the style's version
		const FComboButtonStyle& OurComboButtonStyle = ComboBoxStyle->ComboButtonStyle;
		const FButtonStyle* const OurButtonStyle = InArgs._ButtonStyle ? InArgs._ButtonStyle : &OurComboButtonStyle.ButtonStyle;
		PressedSound = InArgs._PressedSoundOverride.Get(ComboBoxStyle->PressedSlateSound);
		SelectionChangeSound = InArgs._SelectionChangeSoundOverride.Get(ComboBoxStyle->SelectionChangeSlateSound);

		this->OnComboBoxOpening = InArgs._OnComboBoxOpening;
		this->OnSelectionChanged = InArgs._OnSelectionChanged;
		this->OnGenerateWidget = InArgs._OnGenerateWidget;
		this->EnableGamepadNavigationMode = InArgs._EnableGamepadNavigationMode;
		this->bControllerInputCaptured = false;
		this->bReselectionTriggersOnSelectionChanged = InArgs._ReselectionTriggersOnSelectionChanged;

		CustomScrollbar = InArgs._CustomScrollbar;

		ComboBoxMenuContent =
			SNew(SBox)
			.MaxDesiredHeight(InArgs._MaxListHeight)
			[
				SAssignNew(this->ComboListView, SComboListType)
				.ListItemsSource(InArgs.GetOptionsSource())
				.OnGenerateRow(this, &SComboBox< OptionType >::GenerateMenuItemRow)
				.OnSelectionChanged(this, &SComboBox< OptionType >::OnSelectionChanged_Internal)
				.OnKeyDownHandler(this, &SComboBox< OptionType >::OnKeyDownHandler)
				.SelectionMode(ESelectionMode::Single)
				.ScrollBarStyle(InArgs._ScrollBarStyle)
				.ExternalScrollbar(InArgs._CustomScrollbar)
				
			];
		
		// Repel tooltips from appearing on the list item we are hovering...
		ComboListView->EnableToolTipForceField(true);

		// Set up content
		TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
		if (InArgs._Content.Widget == SNullWidget::NullWidget)
		{
			 SAssignNew(ButtonContent, STextBlock)
			.Text(NSLOCTEXT("SComboBox", "ContentWarning", "No Content Provided"))
			.ColorAndOpacity( FLinearColor::Red);
		}


		SComboButton::Construct( SComboButton::FArguments()
			.ComboButtonStyle(&OurComboButtonStyle)
			.ButtonStyle(OurButtonStyle)
			.Method( InArgs._Method )
			.ButtonContent()
			[
				ButtonContent.ToSharedRef()
			]
			.MenuContent()
			[
				ComboBoxMenuContent.ToSharedRef()
			]
			.HasDownArrow( InArgs._HasDownArrow )
			.ContentPadding( InArgs._ContentPadding )
			.ForegroundColor( InArgs._ForegroundColor )
			.OnMenuOpenChanged(this, &SComboBox< OptionType >::OnMenuOpenChanged)
			.IsFocusable(InArgs._IsFocusable)
			.CollapseMenuOnParentFocus(InArgs._CollapseMenuOnParentFocus)
		);
		SetMenuContentWidgetToFocus(ComboListView);

		bShowMenuBackground = false;

		// Need to establish the selected item at point of construction so its available for querying
		// NB: If you need a selection to fire use SetItemSelection rather than setting an IntiallySelectedItem
		SelectedItem = InArgs._InitiallySelectedItem;
		if( TListTypeTraits<OptionType>::IsPtrValid( SelectedItem ) )
		{
			OptionType ValidatedItem = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(SelectedItem);
			ComboListView->Private_SetItemSelection(ValidatedItem, true);
			ComboListView->RequestScrollIntoView(ValidatedItem, 0);
		}

		ComboListView->SetBackgroundBrush(FStyleDefaults::GetNoBrush());
	}

		SComboBox()
		{
#if WITH_ACCESSIBILITY
			AccessibleBehavior = EAccessibleBehavior::Auto;
			bCanChildrenBeAccessible = true;
#endif
		}

#if WITH_ACCESSIBILITY
		protected:
		friend class FSlateAccessibleComboBox;
		/**
		* An accessible implementation of SComboBox to expose to platform accessibility APIs.
		* We inherit from IAccessibleProperty as Windows will use the interface to read out 
		* the value associated with the combo box. Convenient place to return the value of the currently selected option. 
		* For subclasses of SComboBox, inherit and override the necessary functions
		*/
		class FSlateAccessibleComboBox
			: public FSlateAccessibleWidget
			, public IAccessibleProperty
		{
		public:
			FSlateAccessibleComboBox(TWeakPtr<SWidget> InWidget)
				: FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::ComboBox)
			{}

			// IAccessibleWidget
			virtual IAccessibleProperty* AsProperty() override 
			{ 
				return this; 
			}
			// ~

			// IAccessibleProperty
			virtual FString GetValue() const override
			{
				if (Widget.IsValid())
				{
					TSharedPtr<SComboBox<OptionType>> ComboBox = StaticCastSharedPtr<SComboBox<OptionType>>(Widget.Pin());
					if (TListTypeTraits<OptionType>::IsPtrValid(ComboBox->SelectedItem))
					{
						OptionType SelectedOption = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(ComboBox->SelectedItem);
						TSharedPtr<ITableRow> SelectedTableRow = ComboBox->ComboListView->WidgetFromItem(SelectedOption);
						if (SelectedTableRow.IsValid())
						{
							TSharedRef<SWidget> TableRowWidget = SelectedTableRow->AsWidget();
							return TableRowWidget->GetAccessibleText().ToString();
						}
					}
				}
				return FText::GetEmpty().ToString();
			}

			virtual FVariant GetValueAsVariant() const override
			{
				return FVariant(GetValue());
			}
			// ~
		};

		public:
		virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override
		{
			return MakeShareable<FSlateAccessibleWidget>(new SComboBox<OptionType>::FSlateAccessibleComboBox(SharedThis(this)));
		}

		virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType) const
		{
			// current behaviour will red out the  templated type of the combo box which is verbose and unhelpful 
			// This coupled with UIA type will announce Combo Box twice, but it's the best we can do for now if there's no label
			//@TODOAccessibility: Give a better name
			static FString Name("Combo Box");
			return FText::FromString(Name);
		}
#endif

	void ClearSelection( )
	{
		ComboListView->ClearSelection();
	}

	void SetSelectedItem(NullableOptionType InSelectedItem)
	{
		if (TListTypeTraits<OptionType>::IsPtrValid(InSelectedItem))
		{
			OptionType InSelected = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(InSelectedItem);
			ComboListView->SetSelection(InSelected);
		}
		else
		{
			ComboListView->ClearSelection();
		}
	}

	void SetEnableGamepadNavigationMode(bool InEnableGamepadNavigationMode)
	{
		this->EnableGamepadNavigationMode = InEnableGamepadNavigationMode;
	}

	void SetMaxHeight(float InMaxHeight)
	{
		ComboBoxMenuContent->SetMaxDesiredHeight(InMaxHeight);
	}

	void SetStyle(const FComboBoxStyle* InStyle) 
	{ 
		if (ComboBoxStyle != InStyle)
		{
			ComboBoxStyle = InStyle;
			InvalidateStyle();
		}
	}

	void InvalidateStyle() 
	{ 
		Invalidate(EInvalidateWidgetReason::Layout); 
	}

	void SetItemStyle(const FTableRowStyle* InItemStyle) 
	{ 
		if (ItemStyle != InItemStyle)
		{
			ItemStyle = InItemStyle;
			InvalidateItemStyle();
		}
	}

	void InvalidateItemStyle() 
	{ 
		Invalidate(EInvalidateWidgetReason::Layout); 
	}

	/** @return the item currently selected by the combo box. */
	NullableOptionType GetSelectedItem()
	{
		return SelectedItem;
	}

	/** Sets new item source */
	void SetItemsSource(const TArray<OptionType>* InListItemsSource)
	{
		ComboListView->SetItemsSource(InListItemsSource);
	}

	/** Sets new item source */
	void SetItemsSource(TSharedRef<::UE::Slate::Containers::TObservableArray<OptionType>> InListItemsSource)
	{
		ComboListView->SetItemsSource(InListItemsSource);
	}

	/** Clears current item source */
	void ClearItemsSource()
	{
		ComboListView->ClearItemsSource();
	}

	/** 
	 * Requests a list refresh after updating options 
	 * Call SetSelectedItem to update the selected item if required
	 * @see SetSelectedItem
	 */
	void RefreshOptions()
	{
		ComboListView->RequestListRefresh();
	}

protected:
	/** Handle key presses that SListView ignores */
	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if (IsInteractable())
		{
			const EUINavigationAction NavAction = FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent);
			const EUINavigation NavDirection = FSlateApplication::Get().GetNavigationDirectionFromKey(InKeyEvent);
			if (EnableGamepadNavigationMode)
			{
				// The controller's bottom face button must be pressed once to begin manipulating the combobox's value.
				// Navigation away from the widget is prevented until the button has been pressed again or focus is lost.
				if (NavAction == EUINavigationAction::Accept)
				{
					if (bControllerInputCaptured == false)
					{
						// Begin capturing controller input and open the ListView
						bControllerInputCaptured = true;
						PlayPressedSound();
						OnComboBoxOpening.ExecuteIfBound();
						return SComboButton::OnButtonClicked();
					}
					else
					{
						// Set selection to the selected item on the list and close
						bControllerInputCaptured = false;

						// Re-select first selected item, just in case it was selected by navigation previously
						TArray<OptionType> SelectedItems = ComboListView->GetSelectedItems();
						if (SelectedItems.Num() > 0)
						{
							OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::Direct);
						}

						// Set focus back to ComboBox
						FReply Reply = FReply::Handled();
						Reply.SetUserFocus(this->AsShared(), EFocusCause::SetDirectly);
						return Reply;
					}

				}
				else if (NavAction == EUINavigationAction::Back || InKeyEvent.GetKey() == EKeys::BackSpace)
				{
					const bool bWasInputCaptured = bControllerInputCaptured;

					OnMenuOpenChanged(false);
					if (bWasInputCaptured)
					{
						return FReply::Handled();
					}
				}
				else
				{
					if (bControllerInputCaptured)
					{
						return FReply::Handled();
					}
				}
			}
			else
			{
				if (NavDirection == EUINavigation::Up)
				{
					NullableOptionType NullableSelected = GetSelectedItem();
					if (TListTypeTraits<OptionType>::IsPtrValid(NullableSelected))
					{
						OptionType ActuallySelected = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(NullableSelected);
						const TArrayView<const OptionType> OptionsSource = ComboListView->GetItems();
						const int32 SelectionIndex = OptionsSource.Find(ActuallySelected);
						if (SelectionIndex >= 1)
						{
							// Select an item on the prev row
							SetSelectedItem(OptionsSource[SelectionIndex - 1]);
						}
					}

					return FReply::Handled();
				}
				else if (NavDirection == EUINavigation::Down)
				{
					NullableOptionType NullableSelected = GetSelectedItem();
					if (TListTypeTraits<OptionType>::IsPtrValid(NullableSelected))
					{
						OptionType ActuallySelected = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(NullableSelected);
						const TArrayView<const OptionType> OptionsSource = ComboListView->GetItems();
						const int32 SelectionIndex = OptionsSource.Find(ActuallySelected);
						if (SelectionIndex < OptionsSource.Num() - 1)
						{
							// Select an item on the next row
							SetSelectedItem(OptionsSource[SelectionIndex + 1]);
						}
					}
					return FReply::Handled();
				}

				return SComboButton::OnKeyDown(MyGeometry, InKeyEvent);
			}
		}
		return SWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return bIsFocusable;
	}

	virtual bool IsInteractable() const
	{
		return IsEnabled();
	}

private:

	/** Generate a row for the InItem in the combo box's list (passed in as OwnerTable). Do this by calling the user-specified OnGenerateWidget */
	TSharedRef<ITableRow> GenerateMenuItemRow( OptionType InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		if (OnGenerateWidget.IsBound())
		{
			return SNew(SComboRow<OptionType>, OwnerTable)
				.Style(ItemStyle)
				.Padding(MenuRowPadding)
				[
					OnGenerateWidget.Execute(InItem)
				];
		}
		else
		{
			return SNew(SComboRow<OptionType>, OwnerTable)
				[
					SNew(STextBlock).Text(NSLOCTEXT("SlateCore", "ComboBoxMissingOnGenerateWidgetMethod", "Please provide a .OnGenerateWidget() handler."))
				];

		}
	}

	//** Called if the menu is closed
	void OnMenuOpenChanged(bool bOpen)
	{
		if (bOpen == false)
		{
			bControllerInputCaptured = false;

			if (TListTypeTraits<OptionType>::IsPtrValid(SelectedItem))
			{
				// Ensure the ListView selection is set back to the last committed selection
				OptionType ActuallySelected = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(SelectedItem);

				ComboListView->SetSelection(ActuallySelected, ESelectInfo::OnNavigation);
				ComboListView->RequestScrollIntoView(ActuallySelected, 0);
			}

			// Set focus back to ComboBox for users focusing the ListView that just closed
			FSlateApplication::Get().ForEachUser([this](FSlateUser& User) 
			{
				TSharedRef<SWidget> ThisRef = this->AsShared();
				if (User.IsWidgetInFocusPath(this->ComboListView))
				{
					User.SetFocus(ThisRef);
				}
			});

		}
	}

	/** Invoked when the selection in the list changes */
	void OnSelectionChanged_Internal( NullableOptionType ProposedSelection, ESelectInfo::Type SelectInfo )
	{
		// Ensure that the proposed selection is different
		if(SelectInfo != ESelectInfo::OnNavigation)
		{
			// Ensure that the proposed selection is different from selected
			if (bReselectionTriggersOnSelectionChanged || ProposedSelection != SelectedItem)
			{
				PlaySelectionChangeSound();
				SelectedItem = ProposedSelection;
				OnSelectionChanged.ExecuteIfBound( ProposedSelection, SelectInfo );
			}
			// close combo even if user reselected item
			this->SetIsOpen( false );
		}
	}

	/** Handle clicking on the content menu */
	virtual FReply OnButtonClicked() override
	{
		// if user clicked to close the combo menu
		if (this->IsOpen())
		{
			// Re-select first selected item, just in case it was selected by navigation previously
			TArray<OptionType> SelectedItems = ComboListView->GetSelectedItems();
			if (SelectedItems.Num() > 0)
			{
				OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::Direct);
			}
		}
		else
		{
			PlayPressedSound();
			OnComboBoxOpening.ExecuteIfBound();
		}

		return SComboButton::OnButtonClicked();
	}

	FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Enter)
		{
			// Select the first selected item on hitting enter
			TArray<OptionType> SelectedItems = ComboListView->GetSelectedItems();
			if (SelectedItems.Num() > 0)
			{
				OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::OnKeyPress);
				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}


	/** Play the pressed sound */
	void PlayPressedSound() const
	{
		FSlateApplication::Get().PlaySound( PressedSound );
	}

	/** Play the selection changed sound */
	void PlaySelectionChangeSound() const
	{
		FSlateApplication::Get().PlaySound( SelectionChangeSound );
	}

	/** The Sound to play when the button is pressed */
	FSlateSound PressedSound;

	/** The Sound to play when the selection is changed */
	FSlateSound SelectionChangeSound;

	/** The item style to use. */
	const FTableRowStyle* ItemStyle;

	/** The combo box style to use. */
	const FComboBoxStyle* ComboBoxStyle;
	
	/** The padding around each menu row */
	FMargin MenuRowPadding;

private:
	/** Delegate that is invoked when the selected item in the combo box changes */
	FOnSelectionChanged OnSelectionChanged;
	/** The item currently selected in the combo box */
	NullableOptionType SelectedItem;
	/** The ListView that we pop up; visualized the available options. */
	TSharedPtr< SComboListType > ComboListView;
	/** The Scrollbar used in the ListView. */
	TSharedPtr< SScrollBar > CustomScrollbar;
	/** Delegate to invoke before the combo box is opening. */
	FOnComboBoxOpening OnComboBoxOpening;
	/** Delegate to invoke when we need to visualize an option as a widget. */
	FOnGenerateWidget OnGenerateWidget;
	// Use activate button to toggle ListView when enabled
	bool EnableGamepadNavigationMode;
	// Holds a flag indicating whether a controller/keyboard is manipulating the combobox's value. 
	// When true, navigation away from the widget is prevented until a new value has been accepted or canceled. 
	bool bControllerInputCaptured;
	// Whether OnSelectionChanged event gets triggered when selecting the item that's already selected
	bool bReselectionTriggersOnSelectionChanged;

	TSharedPtr<SBox> ComboBoxMenuContent;
};

