// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "SEditableComboBox"


/**
 * Delegate type for getting the editable combo box text.
 */
DECLARE_DELEGATE_RetVal(FString, FOnGetEditableComboBoxText)


/**
 * Implements an editable combo box.
 */
template<typename OptionType>
class SEditableComboBox
	: public SCompoundWidget
{
public:

	typedef typename TSlateDelegates<OptionType>::FOnGenerateWidget FOnGenerateWidget;
	typedef typename TSlateDelegates<OptionType>::FOnSelectionChanged FOnSelectionChanged;

	SLATE_BEGIN_ARGS(SEditableComboBox)
		: _ButtonStyle( &FCoreStyle::Get().GetWidgetStyle< FButtonStyle >( "Button" ) )
		, _Content()
		, _ContentPadding(FMargin(4.0, 2.0))
		, _InitiallySelectedItem(nullptr)
		, _MaxListHeight(450.0f)
		, _IsRenameVisible(EVisibility::Visible)
		, _OnGetEditableText()
		, _OnRemoveClicked()
		, _OnSelectionChanged()
		, _OnSelectionRenamed()
		, _OptionsSource()
	 { }

		SLATE_ATTRIBUTE(FText, AddButtonToolTip)

		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		SLATE_NAMED_SLOT(FArguments, Content)

		SLATE_ATTRIBUTE(FMargin, ContentPadding)

		SLATE_ARGUMENT(OptionType, InitiallySelectedItem)

		SLATE_ARGUMENT(float, MaxListHeight)

		SLATE_ARGUMENT(EVisibility, IsRenameVisible)

		SLATE_EVENT(FOnClicked, OnAddClicked)

		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)

		SLATE_EVENT(FOnGetEditableComboBoxText, OnGetEditableText)

		SLATE_EVENT(FOnClicked, OnRemoveClicked)

		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

		SLATE_EVENT(FOnTextCommitted, OnSelectionRenamed)

		SLATE_ARGUMENT(const TArray<OptionType>*, OptionsSource)

		SLATE_ATTRIBUTE(FText, RemoveButtonToolTip)

		SLATE_ATTRIBUTE(FText, RenameButtonToolTip)

	SLATE_END_ARGS()


public:

	/** Clears the combo box selection. */
	void ClearSelection( )
	{
		ComboBox->ClearSelection();
	}

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 */
	void Construct( const FArguments& InArgs )
	{
		OnSelectionRenamed = InArgs._OnSelectionRenamed;
		OnGetEditableText = InArgs._OnGetEditableText;

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
						[
							SNew(SHorizontalBox)
								.Visibility(this, &SEditableComboBox::HandleNormalModeBoxVisibility)

							+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								[
									// combo box
									SAssignNew(ComboBox, SComboBox<OptionType>)
										.ButtonStyle(InArgs._ButtonStyle)
										.ContentPadding(InArgs._ContentPadding)
										.InitiallySelectedItem(InArgs._InitiallySelectedItem)
										.MaxListHeight(InArgs._MaxListHeight)
										.OptionsSource(InArgs._OptionsSource)
										.OnGenerateWidget(InArgs._OnGenerateWidget)
										.OnSelectionChanged(InArgs._OnSelectionChanged)
										.Content()
										[
											InArgs._Content.Widget
										]
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(1.0)
								[
									// rename button
									SNew(SButton)
										.ContentPadding(2.f)
										.ForegroundColor(FSlateColor::UseForeground())
										.IsEnabled(this, &SEditableComboBox::HandleRemoveRenameButtonIsEnabled)
										.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
										.OnClicked(this, &SEditableComboBox::HandleRenameButtonClicked)
										.ToolTipText(InArgs._RenameButtonToolTip)
										.VAlign(VAlign_Center)
										.Visibility(InArgs._IsRenameVisible)
										.Content()
										[
											SNew(SImage)
												.Image(FCoreStyle::Get().GetBrush("EditableComboBox.Rename"))
												.ColorAndOpacity(FSlateColor::UseForeground())
										]
								]
						]

					+ SOverlay::Slot()
						[
							SNew(SHorizontalBox)
								.Visibility(this, &SEditableComboBox::HandleEditModeBoxVisibility)

							+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								.Padding(0.0, 0.0, 0.0, 3.0)
								[
									// text box
									SAssignNew(TextBox, SEditableTextBox)
										.OnTextCommitted(this, &SEditableComboBox::HandleTextBoxTextCommitted)
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(1.0)
								[
									// accept button
									SNew(SButton)
										.ContentPadding(2.f)
										.ForegroundColor(FSlateColor::UseForeground())
										.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
										.OnClicked(this, &SEditableComboBox::HandleAcceptButtonClicked)
										.ToolTipText(LOCTEXT("AcceptButtonTooltip", "Accept"))
										.VAlign(VAlign_Center)
										.Content()
										[
											SNew(SImage)
												.Image(FCoreStyle::Get().GetBrush("EditableComboBox.Accept"))
												.ColorAndOpacity(FSlateColor::UseForeground())
										]
								]
						]
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.0)
				[
					// add button
					SNew(SButton)
						.ContentPadding(2.f)
						.ForegroundColor(FSlateColor::UseForeground())
						.IsEnabled(this, &SEditableComboBox::HandleAddButtonIsEnabled)
						.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
						.OnClicked(InArgs._OnAddClicked)
						.ToolTipText(InArgs._AddButtonToolTip)
						.VAlign(VAlign_Center)
						.Content()
						[
							SNew(SImage)
								.Image(FCoreStyle::Get().GetBrush("EditableComboBox.Add"))
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.0)
				[
					// remove button
					SNew(SButton)
						.ContentPadding(2.f)
						.ForegroundColor(FSlateColor::UseForeground())
						.IsEnabled(this, &SEditableComboBox::HandleRemoveRenameButtonIsEnabled)
						.ButtonStyle( FCoreStyle::Get(), "NoBorder" )
						.OnClicked(InArgs._OnRemoveClicked)
						.ToolTipText(InArgs._RemoveButtonToolTip)
						.VAlign(VAlign_Center)
						.Content()
						[
							SNew(SImage)
								.Image(FCoreStyle::Get().GetBrush("EditableComboBox.Delete"))
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]
		];
	}

	/**
	 * Gets the item that is currently selected in the combo box.
	 *
	 * @return The selected item, or nullptr if no item is selected.
	 */
	OptionType GetSelectedItem( ) 
	{
		return ComboBox->GetSelectedItem();
	}

	/** 
	 * Requests a list refresh after updating options.
	 *
	 * Call SetSelectedItem to update the selected item if required
	 *
	 * @see SetSelectedItem
	 */
	void RefreshOptions( )
	{
		EditedItem.Reset();

		ComboBox->RefreshOptions();
	}

	/**
	 * Sets the item that is selected in the combo box.
	 *
	 * @param InSelectedItem The item to select.
	 */
	void SetSelectedItem( OptionType InSelectedItem )
	{
		EditedItem.Reset();

		ComboBox->SetSelectedItem(InSelectedItem);
	}

private:

	// Callback for getting the enabled state of the 'Add' button.
	bool HandleAddButtonIsEnabled( ) const
	{
		return !EditedItem.IsValid();
	}

	// Callback for clicking the 'Accept' button.
	FReply HandleAcceptButtonClicked( )
	{
		return FReply::Handled().SetUserFocus(ComboBox->AsShared(), EFocusCause::Mouse);
	}

	// Callback for getting the visibility of the edit mode box.
	EVisibility HandleEditModeBoxVisibility( ) const
	{
		return EditedItem.IsValid() ? EVisibility::Visible : EVisibility::Hidden;
	}

	// Callback for getting the visibility of the normal mode box.
	EVisibility HandleNormalModeBoxVisibility( ) const
	{
		return EditedItem.IsValid() ? EVisibility::Hidden : EVisibility::Visible;
	}

	// Callback for getting the enabled state of the 'Remove' and 'Rename' buttons.
	bool HandleRemoveRenameButtonIsEnabled( ) const
	{
		return (!EditedItem.IsValid() && ComboBox->GetSelectedItem().IsValid());
	}

	// Callback for clicking the 'Rename' button.
	FReply HandleRenameButtonClicked( )
	{
		if (OnGetEditableText.IsBound())
		{
			EditedItem = ComboBox->GetSelectedItem();

			TextBox->SetText( FText::FromString( OnGetEditableText.Execute() ));
		}

		return FReply::Handled().SetUserFocus(TextBox->AsShared(), EFocusCause::Mouse);
	}

	// Callback for committing the text in the text box.
	void HandleTextBoxTextCommitted( const FText& CommittedText, ETextCommit::Type CommitType )
	{
		if (EditedItem.IsValid())
		{
			if (CommitType != ETextCommit::OnCleared)
			{
				OnSelectionRenamed.ExecuteIfBound(TextBox->GetText(), CommitType);
			}

			EditedItem.Reset();
		}
	}

private:

	// Holds the combo box.
	TSharedPtr<SComboBox<OptionType> > ComboBox;

	// Holds the currently edited item.
	OptionType EditedItem;

	// Holds the text box.
	TSharedPtr<SEditableTextBox> TextBox;

private:

	// Holds a delegate to be invoked before the editable text box is populated and shown.
	FOnGetEditableComboBoxText OnGetEditableText;

	// Holds a delegate to be invoked after the text changes have been committed.
	FOnTextCommitted OnSelectionRenamed;
};


#undef LOCTEXT_NAMESPACE
