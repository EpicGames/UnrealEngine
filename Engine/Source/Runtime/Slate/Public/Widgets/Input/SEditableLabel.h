// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"

class SEditableText;
class STextBlock;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

class SLATE_API UE_DEPRECATED(4.21, "SEditableLabel is deprecated, please use SInlineEditableTextBlock instead") SEditableLabel
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SEditableLabel)
		: _CanEdit(true)
		, _EditableTextStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextStyle>("NormalEditableText"))
		, _Font( FCoreStyle::Get().GetFontStyle(TEXT("NormalFont")))
		, _HighlightColor()
		, _HighlightShape()
		, _HighlightText()
		, _MinDesiredWidth(0.0f)
		, _ShadowColorAndOpacity()
		, _ShadowOffset()
		, _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		, _Text()
		, _ColorAndOpacity()
	{ }

		/** Whether the label can be edited. */
		SLATE_ATTRIBUTE(bool, CanEdit)

		/** The style of the text block, which dictates the font, color */
		SLATE_STYLE_ARGUMENT(FEditableTextStyle, EditableTextStyle)

		/** Font used to display text in label. */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** The color used to highlight the specified text */
		SLATE_ATTRIBUTE(FLinearColor, HighlightColor)

		/** The brush used to highlight the specified text*/
		SLATE_ATTRIBUTE(const FSlateBrush*, HighlightShape)

		/** Highlight this text in the text block */
		SLATE_ATTRIBUTE(FText, HighlightText)

		/** Minimum width that a spin box should be */
		SLATE_ATTRIBUTE( float, MinDesiredWidth )

		/** Shadow color and opacity */
		SLATE_ATTRIBUTE(FLinearColor, ShadowColorAndOpacity)

		/** Drop shadow offset in pixels */
		SLATE_ATTRIBUTE(FVector2D, ShadowOffset)

		/** Pointer to a style of the text block, which dictates the font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)

		/** The text displayed in this label. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Text color and opacity */
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

	SLATE_END_ARGS()

	/** Default constructor. */
	SEditableLabel() { }

public:

	/** Constructs the widget. */
	void Construct(const FArguments& InArgs);

	/** Make the label switch to keyboard-based input mode. */
	void EnterTextMode();
	
	/** Make the label switch to mouse-based input mode. */
	void ExitTextMode();

public:

	// SWidget interface

	virtual bool HasKeyboardFocus() const override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual bool SupportsKeyboardFocus() const override;

private:

	void HandleEditableTextTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	FReply HandleTextBlockDoubleClicked();

private:

	TAttribute<bool> CanEditAttribute;
	FOnTextChanged OnTextChanged;
	TAttribute<FText> TextAttribute;

	TSharedPtr<STextBlock> TextBlock;
	TSharedPtr<SEditableText> EditableText;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS