// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Framework/SlateDelegates.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Widgets/Text/ISlateEditableTextWidget.h"
#include "Widgets/Text/SlateEditableTextLayout.h"

class FActiveTimerHandle;
class FArrangedChildren;
class FChildren;
class FPaintArgs;
class FPlainTextLayoutMarshaller;
class FSlateWindowElementList;
class IBreakIterator;
struct FTextLocation;
enum class ETextFlowDirection : uint8;
enum class ETextShapingMethod : uint8;

DECLARE_DELEGATE_TwoParams(FOnCursorMovedWithSelection, const FTextLocation&, const FTextSelection&);

template <>
struct TWidgetTypeTraits<class SEditableText>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Editable text widget
 */
class SEditableText : public SWidget, public ISlateEditableTextWidget
{
	SLATE_DECLARE_WIDGET_API(SEditableText, SWidget, SLATE_API)
public:
	SLATE_BEGIN_ARGS( SEditableText )
		: _Text()
		, _HintText()
		, _SearchText()
		, _Style(&FCoreStyle::Get().GetWidgetStyle< FEditableTextStyle >("NormalEditableText"))
		, _Font()
		, _ColorAndOpacity()
		, _HintTextOpacity()
		, _BackgroundImageSelected()
		, _BackgroundImageComposing()
		, _CaretImage()
		, _IsReadOnly( false )
		, _IsPassword( false )
		, _IsCaretMovedWhenGainFocus( true )
		, _SelectAllTextWhenFocused( false )
		, _SelectWordOnMouseDoubleClick(true)
		, _RevertTextOnEscape( false )
		, _ClearKeyboardFocusOnCommit(true)
		, _Justification(ETextJustify::Left)
		, _AllowContextMenu(true)
		, _MinDesiredWidth(0.0f)
		, _SelectAllTextOnCommit( false )
		, _VirtualKeyboardType(EKeyboardType::Keyboard_Default)
		, _VirtualKeyboardOptions(FVirtualKeyboardOptions())
		, _VirtualKeyboardTrigger(EVirtualKeyboardTrigger::OnFocusByPointer)
		, _VirtualKeyboardDismissAction(EVirtualKeyboardDismissAction::TextChangeOnDismiss)
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _OverflowPolicy()
		, _FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
		{
			_Clipping = EWidgetClipping::ClipToBounds;
		}

		/** Sets the text content for this editable text widget */
		SLATE_ATTRIBUTE( FText, Text )

		/** The text that appears when there is nothing typed into the search box */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Text to search for (a new search is triggered whenever this text changes) */
		SLATE_ATTRIBUTE( FText, SearchText )

		/** The style of the text block, which dictates the font, color */
		SLATE_STYLE_ARGUMENT( FEditableTextStyle, Style )

		/** Sets the font used to draw the text (overrides Style) */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Text color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Hint text opacity in the editable box */
		SLATE_ATTRIBUTE(float, HintTextOpacity)

		/** Background image for the selected text (overrides Style) */
		SLATE_ATTRIBUTE( const FSlateBrush*, BackgroundImageSelected )

		/** Background image for the composing text (overrides Style) */
		SLATE_ATTRIBUTE( const FSlateBrush*, BackgroundImageComposing )

		/** Image brush used for the caret (overrides Style) */
		SLATE_ATTRIBUTE( const FSlateBrush*, CaretImage )

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE( bool, IsReadOnly )

		/** Sets whether this text box is for storing a password */
		SLATE_ATTRIBUTE( bool, IsPassword )

		/** Workaround as we loose focus when the auto completion closes. */
		SLATE_ATTRIBUTE( bool, IsCaretMovedWhenGainFocus )

		/** Whether to select all text when the user clicks to give focus on the widget */
		SLATE_ATTRIBUTE( bool, SelectAllTextWhenFocused )

		/** Whether to select word on mouse double click on the widget */
		SLATE_ATTRIBUTE(bool, SelectWordOnMouseDoubleClick)

		/** Whether to allow the user to back out of changes when they press the escape key */
		SLATE_ATTRIBUTE( bool, RevertTextOnEscape )

		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, ClearKeyboardFocusOnCommit )

		/** How should the value be justified in the editable text field. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)

		/** Whether the context menu can be opened  */
		SLATE_ATTRIBUTE(bool, AllowContextMenu)
	
		/** Whether the IntegratedKeyboard is enabled  */
		SLATE_ATTRIBUTE(bool, EnableIntegratedKeyboard)

		/** Delegate to call before a context menu is opened. User returns the menu content or null to the disable context menu */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

		/**
		 * This is NOT for validating input!
		 * 
		 * Called whenever a character is typed.
		 * Not called for copy, paste, or any other text changes!
		 */
		SLATE_EVENT( FOnIsTypedCharValid, OnIsTypedCharValid )

		/** Callback when the text starts to be edited */
		SLATE_EVENT( FOnBeginTextEdit, OnBeginTextEdit )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )
	
		/** Called whenever the user's selection or cursor position of the text in the text field has changed. */
		SLATE_EVENT( FOnCursorMovedWithSelection, OnCursorMovedWithSelection )

		/** Minimum width that a text block should be */
		SLATE_ATTRIBUTE( float, MinDesiredWidth )

		/** Whether to select all text when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, SelectAllTextOnCommit )

		/** Callback delegate to have first chance handling of the OnKeyChar event */
		SLATE_EVENT(FOnKeyChar, OnKeyCharHandler)

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

		/** Menu extender for the right-click context menu */
		SLATE_EVENT( FMenuExtensionDelegate, ContextMenuExtender )

		/** The type of virtual keyboard to use on mobile devices */
		SLATE_ATTRIBUTE( EKeyboardType, VirtualKeyboardType)

		/** Additional options used by the virtual keyboard summoned by this widget */
		SLATE_ARGUMENT( FVirtualKeyboardOptions, VirtualKeyboardOptions )

		/** The type of event that will trigger the display of the virtual keyboard */
		SLATE_ATTRIBUTE( EVirtualKeyboardTrigger, VirtualKeyboardTrigger )

		/** The message action to take when the virtual keyboard is dismissed by the user */
		SLATE_ATTRIBUTE( EVirtualKeyboardDismissAction, VirtualKeyboardDismissAction )

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT(TOptional<ETextShapingMethod>, TextShapingMethod)

		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT(TOptional<ETextFlowDirection>, TextFlowDirection)
		
		/** Determines what happens to text that is clipped and doesnt fit within the allotted area for this widget */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

		/** The paint policy to use for text if the font faces are still async loading. */
		SLATE_ARGUMENT(EFontFacesLoadingPaintPolicy, FontFacesLoadingPaintPolicy)

		/** Called when all font faces for the text have finished loading asynchronously */
		SLATE_EVENT(FSimpleDelegate, OnAllFontFacesFinishLoading)
	SLATE_END_ARGS()

	/** Constructor */
	SLATE_API SEditableText();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Sets the text currently being edited 
	 *
	 * @param  InNewText  The new text
	 */
	SLATE_API void SetText(TAttribute<FText> InNewText);
	
	/**
	 * Returns the text string
	 *
	 * @return  Text string
	 */
	SLATE_API FText GetText() const;

	/**
	 * Sets the text currently being edited
	 * Note: Doesn't override the bound Text attribute, nor does it call OnTextChanged
	 *
	 * @param InNewText	The new editable text
	 *
	 * @return true if the text was updated, false if the text was already up-to-date
	 */
	SLATE_API bool SetEditableText(const FText& InNewText);

	/** See the HintText attribute */
	SLATE_API void SetHintText(TAttribute<FText> InHintText);
	
	/** Get the text that appears when there is no text in the text box */
	SLATE_API FText GetHintText() const;
	
	/** Sets the hint text opacity */
	SLATE_API void SetHintTextOpacity(TAttribute<float> InHintTextOpacity);
	
	/** Get the hint text opacity */
	SLATE_API float GetHintTextOpacity() const;

	/** Set the text that is currently being searched for (if any) */
	SLATE_API void SetSearchText(TAttribute<FText> InSearchText);

	/** Get the text that is currently being searched for (if any) */
	SLATE_API FText GetSearchText() const;

	/** See the IsReadOnly attribute */
	SLATE_API void SetIsReadOnly(TAttribute<bool> InIsReadOnly);
	
	/** See the IsPassword attribute */
	SLATE_API void SetIsPassword(TAttribute<bool> InIsPassword);
	
	/** See the ColorAndOpacity attribute */
	SLATE_API void SetColorAndOpacity(TAttribute<FSlateColor> Color);

	/** See the BackgroundImageSelected attribute */
	SLATE_API void SetBackgroundImageSelected(TAttribute<const FSlateBrush*> InBackgroundImageSelected);

	/** See the AllowContextMenu attribute */
	SLATE_API void SetAllowContextMenu(TAttribute<bool> InAllowContextMenu);

	/** See the EnableIntegratedKeyboard attribute */
	SLATE_API void SetEnableIntegratedKeyboard(TAttribute<bool> InEnableIntegratedKeyboard);

	/** Set the VirtualKeyboardDismissAction attribute */
	SLATE_API void SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction);

	/**
	 * Sets the font used to draw the text
	 *
	 * @param  InNewFont	The new font to use
	 */
	SLATE_API void SetFont(TAttribute<FSlateFontInfo> InNewFont);

	/** Gets the font used to draw the text. */
	SLATE_API FSlateFontInfo GetFont() const;

	/**
	 * Sets the text style used to draw the text
	 *
	 * @param  InNewTextStyle	The new text style to use
	 */
	SLATE_API void SetTextStyle( const FEditableTextStyle& InNewTextStyle );

	/** @See TextStyle */
	/**
	 * Sets the text block style used to draw the text
	 *
	 * @param  InTextStyle	The new text block style to use
	 */
	SLATE_API void SetTextBlockStyle(const FTextBlockStyle* InTextStyle);

	/**
	 * Sets the minimum width that a text block should be.
	 *
	 * @param  InMinDesiredWidth	The minimum width
	 */
	SLATE_API void SetMinDesiredWidth(TAttribute<float> InMinDesiredWidth);

	/**
	 * Workaround as we loose focus when the auto completion closes.
	 *
	 * @param  InIsCaretMovedWhenGainFocus	Workaround
	 */
	SLATE_API void SetIsCaretMovedWhenGainFocus(TAttribute<bool> InIsCaretMovedWhenGainFocus);

	/**
	 * Sets whether to select all text when the user clicks to give focus on the widget
	 *
	 * @param  InSelectAllTextWhenFocused	Select all text when the user clicks?
	 */
	SLATE_API void SetSelectAllTextWhenFocused(TAttribute<bool> InSelectAllTextWhenFocused);

	/**
	 * Sets whether to allow the user to back out of changes when they press the escape key
	 *
	 * @param  InRevertTextOnEscape			Allow the user to back out of changes?
	 */
	SLATE_API void SetRevertTextOnEscape(TAttribute<bool> InRevertTextOnEscape);

	/**
	 * Sets whether to clear keyboard focus when pressing enter to commit changes
	 *
	 * @param  InClearKeyboardFocusOnCommit		Clear keyboard focus when pressing enter?
	 */
	SLATE_API void SetClearKeyboardFocusOnCommit(TAttribute<bool> InClearKeyboardFocusOnCommit);

	/**
	 * Sets whether to select all text when pressing enter to commit changes
	 *
	 * @param  InSelectAllTextOnCommit		Select all text when pressing enter?
	 */
	SLATE_API void SetSelectAllTextOnCommit(TAttribute<bool> InSelectAllTextOnCommit);

	/**
	 * Sets whether to select word on the mouse double click
	 *
	 * @param  InSelectWordOnMouseDoubleClick		Select word on the mouse double click
	 */
	SLATE_API void SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick);

	/** See Justification attribute */
	SLATE_API void SetJustification(TAttribute<ETextJustify::Type> InJustification);

	/**
	 * Sets the OnKeyCharHandler to provide first chance handling of the OnKeyChar event
	 *
	 * @param InOnKeyCharHandler			Delegate to call during OnKeyChar event
	 */
	void SetOnKeyCharHandler(FOnKeyChar InOnKeyCharHandler)
	{
		OnKeyCharHandler = MoveTemp(InOnKeyCharHandler);
	}

	/**
	 * Sets the OnKeyDownHandler to provide first chance handling of the OnKeyDown event
	 *
	 * @param InOnKeyDownHandler			Delegate to call during OnKeyDown event
	 */
	void SetOnKeyDownHandler(FOnKeyDown InOnKeyDownHandler)
	{
		OnKeyDownHandler = MoveTemp(InOnKeyDownHandler);
	}

	/** See TextShapingMethod attribute */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** See TextFlowDirection attribute */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy);

	/** Sets the paint policy to use for text if the font faces are still async loading. */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
	/** Query to see if any text is selected within the document */
	SLATE_API bool AnyTextSelected() const;

	/** Select all the text in the document */
	SLATE_API void SelectAllText();

	/** Clear the active text selection */
	SLATE_API void ClearSelection();

	/** Get the currently selected text */
	SLATE_API FText GetSelectedText() const;

	/** Move the cursor to the given location in the document (will also scroll to this point) */
	SLATE_API void GoTo(const FTextLocation& NewLocation);

	/** Move the cursor specified location */
	SLATE_API void GoTo(const ETextLocation NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const FTextLocation& NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const ETextLocation NewLocation);

	/** Begin a new text search (this is called automatically when the bound search text changes) */
	SLATE_API void BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase, const bool InReverse = false);

	/** Advance the current search to the next match (does nothing if not currently searching) */
	SLATE_API void AdvanceSearch(const bool InReverse = false);

	/** Register and activate the IME context for this text layout */
	SLATE_API void EnableTextInputMethodContext();

	/** Get the current selection */
	SLATE_API FTextSelection GetSelection() const;

	/** Select a block of text */
	SLATE_API void SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation);

	/* Function to spawn virtual keyboard on demand */
	SLATE_API void ToggleVirtualKeyboard(bool bShow, const int32 UserIndex);

protected:
	//~ Begin SWidget Interface
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual void CacheDesiredSize(float LayoutScaleMultiplier) override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	SLATE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnKeyChar( const FGeometry& MyGeometry,  const FCharacterEvent& InCharacterEvent ) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	SLATE_API virtual const FSlateBrush* GetFocusBrush() const override;
	SLATE_API virtual bool IsInteractable() const override;
	SLATE_API virtual bool ComputeVolatility() const override;
#if WITH_ACCESSIBILITY
	SLATE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
	SLATE_API virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType = EAccessibleType::Main) const override;
#endif
	//~ End SWidget Interface

protected:
	/** Synchronize the text style currently set (including from overrides) and update the text layout if required */
	SLATE_API void SynchronizeTextStyle();

public:
	//~ Begin ISlateEditableTextWidget Interface
	SLATE_API virtual bool IsWidgetConstructed() const override;
	SLATE_API virtual bool IsTextReadOnly() const override;
	SLATE_API virtual bool IsTextPassword() const override;
	SLATE_API virtual bool IsMultiLineTextEdit() const override;
	SLATE_API virtual bool IsIntegratedKeyboardEnabled() const override;
	//~ End ISlateEditableTextWidget Interface

protected:
	//~ Begin ISlateEditableTextWidget Interface
	SLATE_API virtual bool ShouldJumpCursorToEndWhenFocused() const override;
	SLATE_API virtual bool ShouldSelectAllTextWhenFocused() const override;
	SLATE_API virtual bool ShouldClearTextSelectionOnFocusLoss() const override;
	SLATE_API virtual bool ShouldRevertTextOnEscape() const override;
	SLATE_API virtual bool ShouldClearKeyboardFocusOnCommit() const override;
	SLATE_API virtual bool ShouldSelectAllTextOnCommit() const override;
	SLATE_API virtual bool ShouldSelectWordOnMouseDoubleClick() const override;
	SLATE_API virtual bool CanInsertCarriageReturn() const override;
	SLATE_API virtual bool CanTypeCharacter(const TCHAR InChar) const override;
	SLATE_API virtual void EnsureActiveTick() override;
	SLATE_API virtual EKeyboardType GetVirtualKeyboardType() const override;
	SLATE_API virtual FVirtualKeyboardOptions GetVirtualKeyboardOptions() const override;
	SLATE_API virtual EVirtualKeyboardTrigger GetVirtualKeyboardTrigger() const override;
	SLATE_API virtual EVirtualKeyboardDismissAction GetVirtualKeyboardDismissAction() const override;
	SLATE_API virtual TSharedRef<SWidget> GetSlateWidget() override;
	SLATE_API virtual TSharedPtr<SWidget> GetSlateWidgetPtr() override;
	SLATE_API virtual TSharedPtr<SWidget> BuildContextMenuContent() const override;
	SLATE_API virtual void OnBeginTextEdit(const FText& InText) override;
	SLATE_API virtual void OnTextChanged(const FText& InText) override;
	SLATE_API virtual void OnTextCommitted(const FText& InText, const ETextCommit::Type InTextAction) override;
	SLATE_API virtual void OnCursorMoved(const FTextLocation& InLocation) override;
	SLATE_API virtual float UpdateAndClampHorizontalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride) override;
	SLATE_API virtual float UpdateAndClampVerticalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride) override;
	//~ End ISlateEditableTextWidget Interface

protected:
	SLATE_API ETextJustify::Type GetJustification() const;

	TSlateAttributeRef<FSlateFontInfo> GetFontAttribute() const
	{
		return TSlateAttributeRef<FSlateFontInfo>{SharedThis(this), FontAttribute};
	}

	SLATE_API FSlateColor GetColorAndOpacity() const;
	TSlateAttributeRef<FSlateColor> GetColorAndOpacityAttribute() const
	{
		return TSlateAttributeRef<FSlateColor>{SharedThis(this), ColorAndOpacityAttribute};
	}

	SLATE_API const FSlateBrush* GetBackgroundImageSelected() const;
	TSlateAttributeRef<const FSlateBrush*> GetBackgroundImageSelectedAttribute() const
	{
		return TSlateAttributeRef<const FSlateBrush*>{SharedThis(this), BackgroundImageSelectedAttribute};
	}

	TSlateAttributeRef<bool> GetIsPasswordAttribute() const
	{
		return TSlateAttributeRef<bool>{SharedThis(this), bIsPasswordAttribute};
	}

	SLATE_API float GetMinDesiredWidth() const;
	TSlateAttributeRef<float> GetMinDesiredWidthAttribute() const
	{
		return TSlateAttributeRef<float>{SharedThis(this), MinDesiredWidthAttribute};
	}

protected:
	/** Text marshaller used by the editable text layout */
	TSharedPtr<FPlainTextLayoutMarshaller> PlainTextMarshaller;

#if WITH_EDITORONLY_DATA
	/** The text layout that deals with the editable text */
	UE_DEPRECATED(5.8, "Use SlateEditableTextLayout, this variable will never be set.")
	TUniquePtr<FSlateEditableTextLayout> EditableTextLayout;
#endif
	/** The text layout that deals with the editable text */
	FSlateEditableTextLayout SlateEditableTextLayout;

#if WITH_EDITORONLY_DATA
	/** The font used to draw the text */
	UE_DEPRECATED(5.8, "Use SetFont / GetFont / GetFontAttribute")
	TSlateDeprecatedTAttribute<FSlateFontInfo> Font;

	/** Text color and opacity */
	UE_DEPRECATED(5.8, "Use SetColorAndOpacity / GetColorAndOpacity / GetColorAndOpacityAttribute")
	TSlateDeprecatedTAttribute<FSlateColor> ColorAndOpacity;

	/** Background image for the selected text */
	UE_DEPRECATED(5.8, "Use SetBackgroundImageSelected / GetBackgroundImageSelected / GetBackgroundImageSelectedAttribute")
	TSlateDeprecatedTAttribute<const FSlateBrush*> BackgroundImageSelected;

	/** Sets whether this text box can actually be modified interactively by the user */
	UE_DEPRECATED(5.8, "Use SetIsReadOnly / IsTextReadOnly")
	TSlateDeprecatedTAttribute<bool> bIsReadOnly;

	/** Sets whether this text box is for storing a password */
	UE_DEPRECATED(5.8, "Use SetIsPassword / IsTextPassword / GetIsPasswordAttribute")
	TSlateDeprecatedTAttribute<bool> bIsPassword;
#endif

	/** Workaround as we loose focus when the auto completion closes. */
	TAttribute<bool> bIsCaretMovedWhenGainFocus;
	
	/** Whether to select all text when the user clicks to give focus on the widget */
	TAttribute<bool> bSelectAllTextWhenFocused;

	/** Whether to allow the user to back out of changes when they press the escape key */
	TAttribute<bool> bRevertTextOnEscape;

	/** Whether to clear keyboard focus when pressing enter to commit changes */
	TAttribute<bool> bClearKeyboardFocusOnCommit;

	/** Whether to select all text when pressing enter to commit changes */
	TAttribute<bool> bSelectAllTextOnCommit;

	/** Whether to select word on mouse double click */
	TAttribute<bool> bSelectWordOnMouseDoubleClick;

	/** Whether to disable the context menu */
	TAttribute<bool> bAllowContextMenu;

	/** Whether to enable integrated keyboard */
	TAttribute<bool> bEnableIntegratedKeyboard;

	/** Delegate to call before a context menu is opened */
	FOnContextMenuOpening OnContextMenuOpening;

	/** Called when a character is typed and we want to know if the text field supports typing this character. */
	FOnIsTypedCharValid OnIsTypedCharValid;

	/** Called when the text starts to be edited */
	FOnBeginTextEdit OnBeginTextEditCallback;

	/** Called whenever the text is changed programmatically or interactively by the user */
	FOnTextChanged OnTextChangedCallback;

	/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
	FOnTextCommitted OnTextCommittedCallback;
	
	/** Called whenever the user's selection and/or cursor position of the text has changed. */
	FOnCursorMovedWithSelection OnCursorMovedWithSelectionCallback;

#if WITH_EDITORONLY_DATA
	/** Prevents the editable text from being smaller than desired in certain cases (e.g. when it is empty) */
	UE_DEPRECATED(5.8, "Use SetMinDesiredWidth / GetMinDesiredWidth / GetMinDesiredWidthAttribute")
	TSlateDeprecatedTAttribute<float> MinDesiredWidth;
#endif

	/** Menu extender for right-click context menu */
	TSharedPtr<FExtender> MenuExtender;

	/** The timer that is actively driving this widget to Tick() even when Slate is idle */
	TWeakPtr<FActiveTimerHandle> ActiveTickTimer;

	/** The iterator to use to detect word boundaries */
	mutable TSharedPtr<IBreakIterator> WordBreakIterator;

	/** Callback delegate to have first chance handling of the OnKeyChar event */
	FOnKeyChar OnKeyCharHandler;

	/** Callback delegate to have first chance handling of the OnKeyDown event */
	FOnKeyDown OnKeyDownHandler;

	/** The type of virtual keyboard to use for editing this text on mobile */
	TAttribute<EKeyboardType> VirtualKeyboardType;

	/** The type of event that will trigger the display of the virtual keyboard */
	TAttribute<EVirtualKeyboardTrigger> VirtualKeyboardTrigger;

	/** The message action to take when the virtual keyboard is dismissed by the user */
	TAttribute<EVirtualKeyboardDismissAction> VirtualKeyboardDismissAction;

	/** Additional options used by the virtual keyboard summoned by this widget */
	FVirtualKeyboardOptions VirtualKeyboardOptions;

private:
	/** The font used to draw the text */
	TSlateAttribute<FSlateFontInfo> FontAttribute;
	bool bIsFontAttrSet = false;

	/** Text color and opacity */
	TSlateAttribute<FSlateColor> ColorAndOpacityAttribute;
	bool bIsColorAndOpacityAttrSet = false;

	/** Background image for the selected text */
	TSlateAttribute<const FSlateBrush*> BackgroundImageSelectedAttribute;
	bool bIsBackgroundImageSelectedAttrSet = false;

	/** Sets whether this text box is for storing a password */
	TSlateAttribute<bool> bIsPasswordAttribute;

	/** Prevents the editable text from being smaller than desired in certain cases (e.g. when it is empty) */
	TSlateAttribute<float> MinDesiredWidthAttribute;
};
