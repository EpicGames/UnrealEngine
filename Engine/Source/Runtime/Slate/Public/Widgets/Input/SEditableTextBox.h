// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"

class IErrorReportingWidget;
class SBox;
class SHorizontalBox;
enum class ETextFlowDirection : uint8;
enum class ETextShapingMethod : uint8;

template <>
struct TWidgetTypeTraits<class SEditableTextBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Editable text box widget
 */
class SEditableTextBox : public SBorder
{

public:

	SLATE_BEGIN_ARGS( SEditableTextBox )
		: _Style(&FCoreStyle::Get().GetWidgetStyle< FEditableTextBoxStyle >("NormalEditableTextBox"))
		, _Text()
		, _HintText()
		, _SearchText()
		, _Font()
		, _ForegroundColor()
		, _ReadOnlyForegroundColor()
		, _FocusedForegroundColor()
		, _IsReadOnly( false )
		, _IsPassword( false )
		, _IsCaretMovedWhenGainFocus ( true )
		, _SelectAllTextWhenFocused( false )
		, _RevertTextOnEscape( false )
		, _ClearKeyboardFocusOnCommit( true )
		, _Justification(ETextJustify::Left)
		, _MaximumLength(-1)
		, _AllowContextMenu(true)
		, _MinDesiredWidth( 0.0f )
		, _SelectAllTextOnCommit( false )
		, _SelectWordOnMouseDoubleClick(true)
		, _BackgroundColor()
		, _Padding()
		, _ErrorReporting()
		, _VirtualKeyboardOptions(FVirtualKeyboardOptions())
		, _VirtualKeyboardTrigger(EVirtualKeyboardTrigger::OnFocusByPointer)
		, _VirtualKeyboardDismissAction(EVirtualKeyboardDismissAction::TextChangeOnDismiss)
		, _OverflowPolicy()
		, _FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
		{
		}

		/** The styling of the textbox */
		SLATE_STYLE_ARGUMENT( FEditableTextBoxStyle, Style )

		/** Sets the text content for this editable text box widget */
		SLATE_ATTRIBUTE( FText, Text )

		/** Hint text that appears when there is no text in the text box */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Text to search for (a new search is triggered whenever this text changes) */
		SLATE_ATTRIBUTE( FText, SearchText )

		/** Font color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Text color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )
		
		/** Text color and opacity when read-only (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, ReadOnlyForegroundColor )

		/** Text color and opacity when this box has keyboard focus (overrides Style) */
		SLATE_ATTRIBUTE(FSlateColor, FocusedForegroundColor)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE( bool, IsReadOnly )

		/** Sets whether this text box is for storing a password */
		SLATE_ATTRIBUTE( bool, IsPassword )

		/** Workaround as we loose focus when the auto completion closes. */
		SLATE_ATTRIBUTE( bool, IsCaretMovedWhenGainFocus )

		/** Whether to select all text when the user clicks to give focus on the widget */
		SLATE_ATTRIBUTE( bool, SelectAllTextWhenFocused )

		/** Whether to allow the user to back out of changes when they press the escape key */
		SLATE_ATTRIBUTE( bool, RevertTextOnEscape )

		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, ClearKeyboardFocusOnCommit )

		/** How should the value be justified in the editable text field. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)

		/** Maximum length of the text. Input will refuse text longer than that. */
		SLATE_ATTRIBUTE(int32, MaximumLength)

		/** Whether the context menu can be opened */
		SLATE_ATTRIBUTE(bool, AllowContextMenu)

		/** Delegate to call before a context menu is opened. User returns the menu content or null to the disable context menu */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

		/** Menu extender for the right-click context menu */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtender)

		/** Callback when the text starts to be edited */
		SLATE_EVENT( FOnBeginTextEdit, OnBeginTextEdit )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )
	
		/** Called whenever the user's selection or cursor position of the text in the text field has changed. */
		SLATE_EVENT( FOnCursorMovedWithSelection, OnCursorMovedWithSelection )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnVerifyTextChanged, OnVerifyTextChanged )

		/** Minimum width that a text block should be */
		SLATE_ATTRIBUTE( float, MinDesiredWidth )

		/** Whether to select all text when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, SelectAllTextOnCommit )

		/** Whether to select word on mouse double click on the widget */
		SLATE_ATTRIBUTE(bool, SelectWordOnMouseDoubleClick)

		/** Callback delegate to have first chance handling of the OnKeyChar event */
		SLATE_EVENT(FOnKeyChar, OnKeyCharHandler)

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

		/** The color of the background/border around the editable text (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, BackgroundColor )

		/** Padding between the box/border and the text widget inside (overrides Style) */
		SLATE_ATTRIBUTE( FMargin, Padding )

		/** Provide a alternative mechanism for error reporting. */
		SLATE_ARGUMENT( TSharedPtr<class IErrorReportingWidget>, ErrorReporting )

		/** The type of virtual keyboard to use on mobile devices */
		SLATE_ATTRIBUTE(EKeyboardType, VirtualKeyboardType)

		/** Additional options for the virtual keyboard summoned by this widget */
		SLATE_ARGUMENT(FVirtualKeyboardOptions, VirtualKeyboardOptions)

		/** The type of event that will trigger the display of the virtual keyboard */
		SLATE_ATTRIBUTE(EVirtualKeyboardTrigger, VirtualKeyboardTrigger)

		/** The message action to take when the virtual keyboard is dismissed by the user */
		SLATE_ATTRIBUTE(EVirtualKeyboardDismissAction, VirtualKeyboardDismissAction)

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT(TOptional<ETextShapingMethod>, TextShapingMethod)

		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT(TOptional<ETextFlowDirection>, TextFlowDirection)

		/** Determines what happens to text that is clipped and doesnt fit within the allotted area for this text box */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

		/** The paint policy to use for text if the font faces are still async loading. */
		SLATE_ARGUMENT(EFontFacesLoadingPaintPolicy, FontFacesLoadingPaintPolicy)

		/** Called when all font faces for the text have finished loading asynchronously */
		SLATE_EVENT(FSimpleDelegate, OnAllFontFacesFinishLoading)
	SLATE_END_ARGS()

	SLATE_API SEditableTextBox();
	SLATE_API virtual ~SEditableTextBox();
	
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	/**
	 * Returns the text string
	 *
	 * @return  Text string
	 */
	FText GetText() const
	{
		return EditableText->GetText();
	}

	/** See attribute Style */
	SLATE_API void SetStyle(const FEditableTextBoxStyle* InStyle);

	/**
	 * Sets the text block style currently used
	 *
	 * @param  InTextStyle  The new text string
	 */
	SLATE_API void SetTextBlockStyle(const FTextBlockStyle* InTextStyle);

	/**
	 * Sets the text string currently being edited 
	 *
	 * @param  InNewText  The new text string
	 */
	SLATE_API void SetText(TAttribute<FText> InNewText);
	
	/**
	 * Returns the Hint text string
	 *
	 * @return  Hint Text string
	 */
	FText GetHintText() const
	{
		return EditableText->GetHintText();
	}

	/** See the HintText attribute */
	SLATE_API void SetHintText(TAttribute<FText> InHintText);
	
	/** Set the text that is currently being searched for (if any) */
	SLATE_API void SetSearchText(TAttribute<FText> InSearchText);

	/** Get the text that is currently being searched for (if any) */
	SLATE_API FText GetSearchText() const;

	/** See the IsReadOnly attribute */
	SLATE_API void SetIsReadOnly(TAttribute<bool> InIsReadOnly);

	bool IsReadOnly() const { return EditableText->IsTextReadOnly(); }
	
	/** See the IsPassword attribute */
	SLATE_API void SetIsPassword(TAttribute<bool> InIsPassword);

	bool IsPassword() const { return EditableText->IsTextPassword(); }

	/** See the AllowContextMenu attribute */
	SLATE_API void SetAllowContextMenu(TAttribute<bool> InAllowContextMenu);

	/** Set the VirtualKeyboardDismissAction attribute */
	SLATE_API void SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction);
	
	/**
	 * Sets the font used to draw the text
	 *
	 * @param  InFont	The new font to use
	 */
	SLATE_API void SetFont(TAttribute<FSlateFontInfo> InFont);

	/**
	 * Sets the text color and opacity (overrides Style)
	 *
	 * @param  InForegroundColor 	The text color and opacity
	 */
	SLATE_API void SetTextBoxForegroundColor(TAttribute<FSlateColor> InForegroundColor);

	/**
	 * Sets the color of the background/border around the editable text (overrides Style) 
	 *
	 * @param  InBackgroundColor 	The background/border color
	 */
	SLATE_API void SetTextBoxBackgroundColor(TAttribute<FSlateColor> InBackgroundColor);

	/**
	 * Sets the text color and opacity when read-only (overrides Style) 
	 *
	 * @param  InReadOnlyForegroundColor 	The read-only text color and opacity
	 */
	SLATE_API void SetReadOnlyForegroundColor(TAttribute<FSlateColor> InReadOnlyForegroundColor);

	/**
	 * Sets the text color and opacity when this box has keyboard focus(overrides Style)
	 *
	 * @param  InFocusedForegroundColor 	The focused color and opacity
	 */
	SLATE_API void SetFocusedForegroundColor(TAttribute<FSlateColor> InFocusedForegroundColor);

	/**
	 * Sets the maximum text length that will be accepted by the widget
	 *
	 * @param  InMaximumLength		Length to check for. Only values superior to 0 will be taken into account, otherwise no length check will occur.
	 */
	SLATE_API void SetMaximumLength(TAttribute<int32> InMaximumLength);

	/**
	 * Sets the minimum width that a text box should be.
	 *
	 * @param  InMinimumDesiredWidth	The minimum width
	 */
	SLATE_API void SetMinimumDesiredWidth(TAttribute<float> InMinimumDesiredWidth);

	/**
	 * Workaround as we loose focus when the auto completion closes.
	 *
	 * @param  InIsCaretMovedWhenGainFocus	Workaround
	 */
	SLATE_API void SetIsCaretMovedWhenGainFocus(TAttribute<bool> InIsCaretMovedWhenGainFocus);

	/**
	 * Sets whether to select all text when the user clicks to give focus on the widget
	 *
	 * @param  InSelectAllTextWhenFocused	Select all text when the user clicks?
	 */
	SLATE_API void SetSelectAllTextWhenFocused(TAttribute<bool> InSelectAllTextWhenFocused);

	/**
	 * Sets whether to allow the user to back out of changes when they press the escape key
	 *
	 * @param  InRevertTextOnEscape			Allow the user to back out of changes?
	 */
	SLATE_API void SetRevertTextOnEscape(TAttribute<bool> InRevertTextOnEscape);

	/**
	 * Sets whether to clear keyboard focus when pressing enter to commit changes
	 *
	 * @param  InClearKeyboardFocusOnCommit		Clear keyboard focus when pressing enter?
	 */
	SLATE_API void SetClearKeyboardFocusOnCommit(TAttribute<bool> InClearKeyboardFocusOnCommit);

	/**
	 * Sets whether to select all text when pressing enter to commit changes
	 *
	 * @param  InSelectAllTextOnCommit		Select all text when pressing enter?
	 */
	SLATE_API void SetSelectAllTextOnCommit(TAttribute<bool> InSelectAllTextOnCommit);

	/**
	 * Sets whether to select select word on mouse double click
	 *
	 * @param  InSelectWordOnMouseDoubleClick		Select select word on mouse double click?
	 */
	SLATE_API void SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick);

	/** See Justification attribute */
	SLATE_API void SetJustification(TAttribute<ETextJustify::Type> InJustification);

	/**
	 * If InError is a non-empty string the TextBox will the ErrorReporting provided during construction
	 * If no error reporting was provided, the TextBox will create a default error reporter.
	 */
	SLATE_API void SetError(const FText& InError);
	SLATE_API void SetError(const FString& InError);

	/**
	 * Sets the OnKeyCharHandler to provide first chance handling of the SEditableText's OnKeyChar event
	 *
	 * @param InOnKeyCharHandler			Delegate to call during OnKeyChar event
	 */
	SLATE_API void SetOnKeyCharHandler(FOnKeyChar InOnKeyCharHandler);

	/**
	 * Sets the OnKeyDownHandler to provide first chance handling of the SEditableText's OnKeyDown event
	 *
	 * @param InOnKeyDownHandler			Delegate to call during OnKeyDown event
	 */
	SLATE_API void SetOnKeyDownHandler(FOnKeyDown InOnKeyDownHandler);

	/** See TextShapingMethod attribute */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** See TextFlowDirection attribute */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy);

	/** Sets the paint policy to use for text if the font faces are still async loading. */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);

	/** Query to see if any text is selected within the document */
	SLATE_API bool AnyTextSelected() const;

	/** Select all the text in the document */
	SLATE_API void SelectAllText();

	/** Clear the active text selection */
	SLATE_API void ClearSelection();

	/** Get the currently selected text */
	SLATE_API FText GetSelectedText() const;

	/** Move the cursor to the given location in the document */
	SLATE_API void GoTo(const FTextLocation& NewLocation);

	/** Move the cursor to the specified location */
	void GoTo(const ETextLocation NewLocation)
	{
		EditableText->GoTo(NewLocation);
	}

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const FTextLocation& NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	void ScrollTo(const ETextLocation NewLocation)
	{
		EditableText->ScrollTo(NewLocation);
	}

	/** Begin a new text search (this is called automatically when the bound search text changes) */
	SLATE_API void BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase, const bool InReverse = false);

	/** Advance the current search to the next match (does nothing if not currently searching) */
	SLATE_API void AdvanceSearch(const bool InReverse = false);

	/** Register and activate the IME context for the text layout of this textbox */
	SLATE_API void EnableTextInputMethodContext();

	SLATE_API bool HasError() const;

	// SWidget overrides
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual bool HasKeyboardFocus() const override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

protected:
#if WITH_ACCESSIBILITY
	SLATE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
	SLATE_API virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType = EAccessibleType::Main) const override;
#endif
	/** Callback for the editable text's OnTextChanged event */
	SLATE_API void OnEditableTextChanged(const FText& InText);

	/** Callback when the editable text is committed. */
	SLATE_API void OnEditableTextCommitted(const FText& InText, ETextCommit::Type InCommitType);
	
	/** Callback when the user's cursor position and/or selection has changed. */
	SLATE_API void OnEditableTextCursorMovedWithSelection(const FTextLocation& InLocation, const FTextSelection& TextSelection);

	const FEditableTextBoxStyle* Style;

	/** Box widget that adds padding around the editable text */
	TSharedPtr< SBox > PaddingBox;

	/** Editable text widget */
	TSharedPtr< SEditableText > EditableText;

	/** Padding (overrides style) */
	TAttribute<FMargin> PaddingOverride;

	/** Font (overrides style) */
	TAttribute<FSlateFontInfo> FontOverride;

	/** Foreground color (overrides style) */
	TAttribute<FSlateColor> ForegroundColorOverride;

	/** Background color (overrides style) */
	TAttribute<FSlateColor> BackgroundColorOverride;

	/** Read-only foreground color (overrides style) */
	TAttribute<FSlateColor> ReadOnlyForegroundColorOverride;

	/** Focused foreground color (overrides style) */
	TAttribute<FSlateColor> FocusedForegroundColorOverride;

	/** Maximum text length that will be accepted by the widget */
	TAttribute<int32> MaximumLength;

	/** Allows for inserting additional widgets that extend the functionality of the text box */
	TSharedPtr<SHorizontalBox> Box;

	/** SomeWidget reporting */
	TSharedPtr<class IErrorReportingWidget> ErrorReporting;

	/** Called when the text is changed interactively */
	FOnTextChanged OnTextChanged;

	/** Called when the user commits their change to the editable text control */
	FOnTextCommitted OnTextCommitted;	

	/** Callback to verify text when changed. Will return an error message to denote problems. */
	FOnVerifyTextChanged OnVerifyTextChanged;
	
	/** Callback when the user's selection and/or cursor position of the text has changed. */
	FOnCursorMovedWithSelection OnCursorMovedWithSelection;

private:

	FMargin DeterminePadding() const;
	FSlateFontInfo DetermineFont() const;
	FSlateColor DetermineBackgroundColor() const;
	FSlateColor DetermineForegroundColor() const;

	/** Styling: border image to draw when not hovered or focused */
	const FSlateBrush* BorderImageNormal;
	/** Styling: border image to draw when hovered */
	const FSlateBrush* BorderImageHovered;
	/** Styling: border image to draw when focused */
	const FSlateBrush* BorderImageFocused;
	/** Styling: border image to draw when read only */
	const FSlateBrush* BorderImageReadOnly;

	/** @return Border image for the text box based on the hovered and focused state */
	const FSlateBrush* DetermineBorderImage() const;

};
