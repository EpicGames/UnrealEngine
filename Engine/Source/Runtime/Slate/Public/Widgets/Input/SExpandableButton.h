// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Layout/SBorder.h"

/**
 * A button that can either be collapsed or expanded, containing different content in each state.
 */
class SExpandableButton
	: public SBorder
{
	SLATE_DECLARE_WIDGET_API(SExpandableButton, SBorder, SLATE_API)

public:

	SLATE_BEGIN_ARGS( SExpandableButton )
		: _IsExpanded( true )
		{}

		/** The text to display in this button in it's collapsed state (if nothing is specified for CollapsedButtonContent) */
		SLATE_ATTRIBUTE( FText, CollapsedText )

		/** The text to display in this button in it's expanded state (if nothing is specified for ExpandedButtonContent) */
		SLATE_ATTRIBUTE( FText, ExpandedText )

		/** Slot for this button's collapsed content (optional) */
		SLATE_NAMED_SLOT( FArguments, CollapsedButtonContent )

		/** Slot for this button's expanded content (optional) */
		SLATE_NAMED_SLOT( FArguments, ExpandedButtonContent )

		/** Slot for this button's expanded body */
		SLATE_NAMED_SLOT( FArguments, ExpandedChildContent )

		/** Called when the expansion button is clicked */
		SLATE_EVENT( FOnClicked, OnExpansionClicked )

		/** Called when the close button is clicked */
		SLATE_EVENT( FOnClicked, OnCloseClicked )

		/** Current expansion state */
		SLATE_ATTRIBUTE( bool, IsExpanded )

	SLATE_END_ARGS()

	SLATE_API SExpandableButton();
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);

protected:

	/** Callbacks to determine visibility of parts that should be shown when the button state is collapsed or expanded */
	SLATE_API EVisibility GetCollapsedVisibility() const;
	SLATE_API EVisibility GetExpandedVisibility() const;

	SLATE_API void UpdateVisibility();

private:

	/** The attribute of the current expansion state */
	TSlateAttribute<bool> IsExpanded;

	TSharedPtr<SWidget> ExpandedChildContent;
	TSharedPtr<SWidget> ToggleButtonClosed;
	TSharedPtr<SWidget> ToggleButtonExpanded;
	TSharedPtr<SWidget> CloseExpansionButton;
};
