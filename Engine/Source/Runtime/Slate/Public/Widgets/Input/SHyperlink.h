// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

enum class ETextFlowDirection : uint8;
enum class ETextShapingMethod : uint8;

template<>
struct TWidgetTypeTraits<class SHyperlink>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A hyperlink widget is what you would expect from a browser hyperlink.
 * When a hyperlink is clicked in invokes an OnNavigate() delegate.
 */
class SHyperlink
	: public SButton
{
public:

	SLATE_BEGIN_ARGS(SHyperlink)
		: _Text()
		, _Style(&FCoreStyle::Get().GetWidgetStyle< FHyperlinkStyle >("Hyperlink"))
		, _TextStyle(nullptr)
		, _UnderlineStyle(nullptr)
		, _Padding()
		, _OnNavigate()
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _HighlightColor()
		, _HighlightShape()
		, _HighlightText()
		, _OverflowPolicy()
		{}

		SLATE_ATTRIBUTE( FText, Text )
		SLATE_STYLE_ARGUMENT( FHyperlinkStyle, Style )
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )
		SLATE_STYLE_ARGUMENT( FButtonStyle, UnderlineStyle )
		SLATE_ATTRIBUTE( FMargin, Padding )
		SLATE_EVENT( FSimpleDelegate, OnNavigate )
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )
		SLATE_ATTRIBUTE( FLinearColor, HighlightColor )
		SLATE_ATTRIBUTE( const FSlateBrush*, HighlightShape )
		SLATE_ATTRIBUTE( FText, HighlightText )
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)
	SLATE_END_ARGS()

	/**
	 * Construct the hyperlink widgets from a declaration
	 * 
	 * @param InArgs    Widget declaration from which to construct the hyperlink.
	 */
	void Construct( const FArguments& InArgs )
	{
		this->OnNavigate = InArgs._OnNavigate;

		check (InArgs._Style);
		const FButtonStyle* UnderlineStyle = InArgs._UnderlineStyle != nullptr ? InArgs._UnderlineStyle : &InArgs._Style->UnderlineStyle;
		const FTextBlockStyle* TextStyle = InArgs._TextStyle != nullptr ? InArgs._TextStyle : &InArgs._Style->TextStyle;
		TAttribute<FMargin> Padding = InArgs._Padding.IsSet() ? InArgs._Padding : InArgs._Style->Padding;

		SButton::Construct(
			SButton::FArguments()
			.ContentPadding(Padding)
			.ButtonStyle(UnderlineStyle)
			.OnClicked(this, &SHyperlink::Hyperlink_OnClicked)
			.ForegroundColor(FSlateColor::UseForeground())
			.TextShapingMethod(InArgs._TextShapingMethod)
			.TextFlowDirection(InArgs._TextFlowDirection)
			[
				SNew(STextBlock)
				.TextStyle(TextStyle)
				.Text(InArgs._Text)
				.HighlightColor(InArgs._HighlightColor)
				.HighlightShape(InArgs._HighlightShape)
				.HighlightText(InArgs._HighlightText)
				.OverflowPolicy(InArgs._OverflowPolicy)
			]
		);
	}

public:

	// SWidget overrides

	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override
	{
		return FCursorReply::Cursor( EMouseCursor::Hand );
	}
#if WITH_ACCESSIBILITY
	virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override
	{
		return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleHyperlink(SharedThis(this)));
	}
#endif

protected:

	/** Invoke the OnNavigate method */
	FReply Hyperlink_OnClicked()
	{
		OnNavigate.ExecuteIfBound();

		return FReply::Handled();
	}

	/** The delegate to invoke when someone clicks the hyperlink */
	FSimpleDelegate OnNavigate;
};
