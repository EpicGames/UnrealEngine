// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Layout/Visibility.h"

class SButton;
class STextBlock;

/**
 * A widget for selecting keys or input chords.
 */
class SInputKeySelector : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnKeySelected, const FInputChord& )
	DECLARE_DELEGATE( FOnIsSelectingKeyChanged )

	SLATE_BEGIN_ARGS( SInputKeySelector )
		: _SelectedKey( FInputChord(EKeys::Invalid) )
		, _ButtonStyle( &FCoreStyle::Get().GetWidgetStyle<FButtonStyle>( "Button" ) )
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle< FTextBlockStyle >("NormalText") )
		, _KeySelectionText( NSLOCTEXT("InputKeySelector", "DefaultKeySelectionText", "...") )
		, _NoKeySpecifiedText(NSLOCTEXT("InputKeySelector", "DefaultEmptyText", "Empty"))
		, _AllowModifierKeys( true )
		, _AllowGamepadKeys( false )
		, _EscapeCancelsSelection( true )
		, _IsFocusable(true)
		{}

		/** The currently selected key */
		SLATE_ATTRIBUTE( FInputChord, SelectedKey )

		/** The font used to display the currently selected key. */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** The margin around the selected key text. */
		SLATE_ATTRIBUTE( FMargin, Margin )

		/** The style of the button used to enable key selection. */
		SLATE_STYLE_ARGUMENT( FButtonStyle, ButtonStyle )

		/** The text style of the button text */
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle)

		/** The text to display while selecting a new key. */
		SLATE_ARGUMENT( FText, KeySelectionText )

		/** The text to display while no key text is available or not selecting a key. */
		SLATE_ARGUMENT(FText, NoKeySpecifiedText)

		/** When true modifier keys are captured in the selected key chord, otherwise they are ignored. */
		SLATE_ARGUMENT( bool, AllowModifierKeys )

		/** When true gamepad keys are captured in the selected key chord, otherwise they are ignored. */
		SLATE_ARGUMENT( bool, AllowGamepadKeys )

		/** When true, pressing escape will cancel the key selection, when false, pressing escape will select the escape key. */
		SLATE_ARGUMENT( bool, EscapeCancelsSelection )

		/** When EscapeCancelsSelection is true, escape on specific keys that are unbind able by the user. */
		SLATE_ARGUMENT( TArray<FKey>, EscapeKeys )

		/** Occurs whenever a new key is selected. */
		SLATE_EVENT( FOnKeySelected, OnKeySelected )

		/** Occurs whenever key selection mode starts and stops. */
		SLATE_EVENT( FOnIsSelectingKeyChanged, OnIsSelectingKeyChanged )

		/** Sometimes a button should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, IsFocusable)
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

public:

	/** Gets the currently selected key chord. */
	SLATE_API FInputChord GetSelectedKey() const;

	/** Sets the currently selected key chord. */
	SLATE_API void SetSelectedKey( TAttribute<FInputChord> InSelectedKey );

	/** Sets the margin around the text used to display the currently selected key */
	SLATE_API void SetMargin( TAttribute<FMargin> InMargin );

	/** Sets the style of the button which is used enter key selection mode. */
	SLATE_API void SetButtonStyle( const FButtonStyle* ButtonStyle );

	/** Sets the style of the text on the button which is used enter key selection mode. */
	SLATE_API void SetTextStyle(const FTextBlockStyle* InTextStyle);

	/** Sets the text which is displayed when selecting a key. */
	void SetKeySelectionText( FText InKeySelectionText ) { KeySelectionText = MoveTemp(InKeySelectionText); }

	/** Sets the text to display when no key text is available or not selecting a key. */
	void SetNoKeySpecifiedText(FText InNoKeySpecifiedText) { NoKeySpecifiedText = MoveTemp(InNoKeySpecifiedText); }

	/** When true modifier keys are captured in the selected key chord, otherwise they are ignored. */
	void SetAllowModifierKeys(const bool bInAllowModifierKeys) { bAllowModifierKeys = bInAllowModifierKeys; }

	/** When true gamepad keys are captured in the selected key chord, otherwise they are ignored. */
	void SetAllowGamepadKeys(const bool bInAllowGamepadKeys) { bAllowGamepadKeys = bInAllowGamepadKeys; }

	/** Sets the escape keys to check against. */
	void SetEscapeKeys(TArray<FKey> InEscapeKeys) { EscapeKeys = MoveTemp(InEscapeKeys); }

	/** Returns true whenever key selection mode is active, otherwise returns false. */
	bool GetIsSelectingKey() const { return bIsSelectingKey; }

	/** Sets the visibility of the text block. */
	SLATE_API void SetTextBlockVisibility(EVisibility InVisibility);

public:

	SLATE_API virtual FReply OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	SLATE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

private:

	/** Gets the display text for the currently selected key. */
	SLATE_API FText GetSelectedKeyText() const;

	/**  Gets the margin around the text used to display the currently selected key. */
	SLATE_API FMargin GetMargin() const;

	/** Handles the OnClicked event from the button which enables key selection mode. */
	SLATE_API FReply OnClicked();

	/** Sets the currently selected key and invokes the associated events. */
	SLATE_API void SelectKey( FKey Key, bool bShiftDown, bool bControllDown, bool bAltDown, bool bCommandDown );

	/** Sets bIsSelectingKey and invokes the associated events. */
	SLATE_API void SetIsSelectingKey(bool bInIsSelectingKey);

	/** Returns true, if the key has been specified as an escape key, else false. */
	SLATE_API bool IsEscapeKey(const FKey& InKey) const;

private:

	/** True when key selection mode is active. */
	bool bIsSelectingKey;

	/** The currently selected key chord. */
	TAttribute<FInputChord> SelectedKey;

	/** The margin around the text used to display the currently selected key. */
	TAttribute<FMargin> Margin;

	/** The text to display when selecting keys. */
	FText KeySelectionText;

	/**  The text to display while no key text is available or not selecting a key. */
	FText NoKeySpecifiedText;

	/** When true modifier keys are recorded on the selected key chord, otherwise they are ignored. */
	bool bAllowModifierKeys;

	/** When true gamepad keys are recorded on the selected key chord, otherwise they are ignored. */
	bool bAllowGamepadKeys;

	/** When true, pressing escape will cancel the key selection, when false, pressing escape will select the escape key. */ 
	bool bEscapeCancelsSelection;

	/** When EscapeCancelsSelection is true, escape on specific keys that are unbind able by the user. */
	TArray<FKey> EscapeKeys;

	/** Delegate which is run any time a new key is selected. */
	FOnKeySelected OnKeySelected;

	/** Delegate which is run when key selection mode starts and stops. */
	FOnIsSelectingKeyChanged OnIsSelectingKeyChanged;

	/** The button which starts the key selection mode. */
	TSharedPtr<SButton> Button;

	/** The text which is rendered on the button. */
	TSharedPtr<STextBlock> TextBlock;

	/** Can this button be focused? */
	bool bIsFocusable;
};
