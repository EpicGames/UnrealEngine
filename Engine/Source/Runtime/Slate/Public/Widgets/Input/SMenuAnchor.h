// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/PopupMethodReply.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/IMenu.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class SWindow;

/** Notification when popup is opened/closed. */
DECLARE_DELEGATE_OneParam(FOnIsOpenChanged, bool)



template <>
struct TWidgetTypeTraits<class SMenuAnchor>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A PopupAnchor summons a Popup relative to its content.
 * Summoning a popup relative to the cursor is accomplished via the application.
 */
class SMenuAnchor : public SPanel, public IMenuHost
{
	SLATE_DECLARE_WIDGET_API(SMenuAnchor, SPanel, SLATE_API)
public:
	SLATE_BEGIN_ARGS( SMenuAnchor )
		: _Content()
		, _Padding(FMargin(0.f))
		, _MenuContent( SNew(STextBlock) .Text( NSLOCTEXT("SMenuAnchor", "NoMenuContent", "No Menu Content Assigned; use .MenuContent") ) )
		, _OnGetMenuContent()
		, _Placement( MenuPlacement_BelowAnchor )
		, _FitInWindow( true )
		, _Method()
		, _ShouldDeferPaintingAfterWindowContent(true)
		, _UseApplicationMenuStack(true)
		, _ShowMenuBackground(true)
		, _IsCollapsedByParent(false)
		, _ApplyWidgetStyleToMenu(true)
		{}
		
		SLATE_DEFAULT_SLOT( FArguments, Content )

		SLATE_ARGUMENT( FMargin, Padding )
		
		SLATE_ARGUMENT( TSharedPtr<SWidget>, MenuContent )

		SLATE_EVENT( FOnGetContent, OnGetMenuContent )
		
		SLATE_EVENT( FOnIsOpenChanged, OnMenuOpenChanged )

		SLATE_ATTRIBUTE( EMenuPlacement, Placement )

		SLATE_ARGUMENT(bool, FitInWindow)

		SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)

		SLATE_ARGUMENT(bool, ShouldDeferPaintingAfterWindowContent)

		SLATE_ARGUMENT(bool, UseApplicationMenuStack)

		SLATE_ARGUMENT(bool, ShowMenuBackground)
		
		/** True if this menu anchor should be collapsed when its parent receives focus, false (default) otherwise */
		SLATE_ARGUMENT(bool, IsCollapsedByParent)

		/** True to apply the InWidgetStyle of the menu anchor when painting the popup, false to always paint the popup with full opacity and no tint. */
		SLATE_ARGUMENT(bool, ApplyWidgetStyleToMenu)

	SLATE_END_ARGS()

	SLATE_API virtual ~SMenuAnchor();
	
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );
	
	SLATE_API SMenuAnchor();

	/** See Content Slot attribute */
	SLATE_API void SetContent(TSharedRef<SWidget> InContent);

	/** See MenuContent attribute */
	SLATE_API virtual void SetMenuContent(TSharedRef<SWidget> InMenuContent);
	
	/**
	 * Open or close the popup
	 *
	 * @param InIsOpen    If true, open the popup. Otherwise close it.
	 * @param bFocusMenu  Should we focus the popup as soon as it opens?
	 */
	SLATE_API virtual void SetIsOpen( bool InIsOpen, const bool bFocusMenu = true, const int32 FocusUserIndex = 0 );
	
	/** @return true if the popup is open; false otherwise. */
	SLATE_API bool IsOpen() const;

	/** @return true if we should open the menu due to a click. Sometimes we should not, if
	   the same MouseDownEvent that just closed the menu is about to re-open it because it happens to land on the button.*/
	SLATE_API bool ShouldOpenDueToClick() const;

	/** @return The current menu position */
	SLATE_API FVector2D GetMenuPosition() const;

	SLATE_API void SetMenuPlacement(TAttribute<EMenuPlacement> InMenuPlacement);

	SLATE_API void SetFitInWindow(bool bFit);

	/** @return Whether this menu has open submenus */
	SLATE_API bool HasOpenSubMenus() const;

	// IMenuHost interface
	SLATE_API virtual TSharedPtr<SWindow> GetMenuWindow() const override;
	SLATE_API virtual void OnMenuDismissed() override;
	SLATE_API virtual bool UsingApplicationMenuStack() const override;
	// End of IMenuHost interface

	static SLATE_API void DismissAllApplicationMenus();

protected:
	// SWidget interface
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual bool ComputeVolatility() const override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	// End of SWidget interface

	/** @return true if the popup is currently open and reusing an existing window */
	SLATE_API bool IsOpenAndReusingWindow() const;

	/** @return true if the popup is currently open and we created a dedicated window for it */
	SLATE_API bool IsOpenViaCreatedWindow() const;

	/** Handler/callback called by menus created by this anchor, when they are dismissed */
	SLATE_API void OnMenuClosed(TSharedRef<IMenu> InMenu);

	/** Reset the popup state to prepopup. */
	SLATE_API void ResetPopupMenuContent();

	/** Computes the placement geometry for menus displayed in a separately created window */
	SLATE_API FGeometry ComputeNewWindowMenuPlacement(const FGeometry& AllottedGeometry, const FVector2D& PopupDesiredSize, EMenuPlacement PlacementMode) const;

	/**
	 * Checks if any parent of the current SMenuAnchor is hidden (in which case this returns EVisibility::Collapsed).
	 * If all the ancestors are visible, this returns the value of the OriginalVisibilityAttribute.
	 */
	SLATE_API EVisibility GetMenuContentVisibility(TAttribute<EVisibility> OriginalVisibilityAttribute) const;

	static SLATE_API TArray<TWeakPtr<IMenu>> OpenApplicationMenus;

	TSlateAttributeRef<EMenuPlacement> GetPlacementAttribute() const
	{
		return TSlateAttributeRef<EMenuPlacement>{SharedThis(this), PlacementAttribute};
	}

protected:
	struct FPopupPlacement
	{
		SLATE_API FPopupPlacement(const FGeometry& PlacementGeometry, const FVector2D& PopupDesiredSize, EMenuPlacement PlacementMode);
		
		FVector2D LocalPopupSize;
		FVector2D LocalPopupOffset;
		FSlateRect AnchorLocalSpace;
		EOrientation Orientation;
	};

	/**
	 * A pointer to the window presenting this popup.
	 * Can be the window created to hold a menu or the window containing this anchor if the menu is drawn as a child of the this anchor.
	 * Pointer is null when a popup is not visible.
	 */
	TWeakPtr<SWindow> PopupWindowPtr;

	/**
	 * An interface pointer to the menu object presenting this popup.
	 * Pointer is null when a popup is not visible.
	 */
	TWeakPtr<IMenu> PopupMenuPtr;

	/**
	 * An interface pointer to the menu object presenting this popup.
	 * This one is for menus owned by this widget and not by the application's menu stack
	 */
	TSharedPtr<IMenu> OwnedMenuPtr;

	/** Static menu content to use when the delegate used when OnGetMenuContent is not defined. */
	TSharedPtr<SWidget> MenuContent;

	/** MenuContent plus any extra wrapping widgets needed by the menu infrastructure. */
	TSharedPtr<SWidget> WrappedContent;
	
	/** Callback invoked when the popup is being summoned */
	FOnGetContent OnGetMenuContent;

	/** Callback invoked when the popup is being opened/closed */
	FOnIsOpenChanged OnMenuOpenChanged;

#if WITH_EDITORONLY_DATA
	/** How should the popup be placed relative to the anchor. */
	UE_DEPRECATED(5.8, "Use SetMenuPlacement / GetPlacementAttribute")
	TSlateDeprecatedTAttribute<EMenuPlacement> Placement;
#endif

	/** Should the menu anchor fit inside the window? */
	bool bFitInWindow;

	/** Was the menu just dismissed this tick? */
	bool bDismissedThisTick;

	/** Whether this menu should be collapsed when its parent gets focus */
	bool bIsCollapsedByParent;

	/** Should we summon a new window for this popup or  */
	TOptional<EPopupMethod> Method;

	/** Method currently being used to show the popup. No value if popup is closed. */
	FPopupMethodReply MethodInUse;

	/** Should the menu content painting be deferred? If not, the menu content will layer over the menu anchor, rather than above all window contents. */
	bool bShouldDeferPaintingAfterWindowContent;

	/** Should the menu by created by the application stack code making it behave like and have the lifetime of a normal menu? */
	bool bUseApplicationMenuStack;

	/** Should we paint the popup using the received InWidgetStyle? */
	bool bApplyWidgetStyleToMenu;

	/**
	 * @todo Slate : Unify geometry so that this is not necessary.
	 * Stores the local offset of the popup as we have to compute it in OnTick;
	 * Cannot compute in OnArrangeChildren() because it can be
	 * called in different spaces (window or desktop.)
	 */
	FVector2D LocalPopupPosition;

	/**
	 * Screen-space version of LocalPopupPosition, also cached in Tick.
	 * Used to satisfy calls to GetMenuPosition() for menus that are reusing the current window.
	 */
	FVector2D ScreenPopupPosition;

	/** The currently arranged children in the menu anchor.  Changes as the opened/closed state of the widget changes. */
	TPanelChildren<FBasicLayoutWidgetSlot> Children;

private:
	/** How should the popup be placed relative to the anchor. */
	TSlateAttribute<EMenuPlacement> PlacementAttribute;
};
