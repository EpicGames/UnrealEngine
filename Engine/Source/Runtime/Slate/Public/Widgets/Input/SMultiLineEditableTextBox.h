// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "SlateGlobals.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Text/SMultiLineEditableText.h"

class IErrorReportingWidget;
class ITextLayoutMarshaller;
class SBox;
class SHorizontalBox;
enum class ETextShapingMethod : uint8;

#if WITH_FANCY_TEXT

template <>
struct TWidgetTypeTraits<class SMultiLineEditableTextBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Editable text box widget
 */
class SMultiLineEditableTextBox : public SBorder
{

public:

	SLATE_BEGIN_ARGS( SMultiLineEditableTextBox )
		: _Style(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		, _Marshaller()
		, _Text()
		, _HintText()
		, _SearchText()
		, _Font()
		, _ForegroundColor()
		, _ReadOnlyForegroundColor()
		, _FocusedForegroundColor()
		, _Justification(ETextJustify::Left)
		, _MaximumLength(-1)
		, _LineHeightPercentage(1.0f)
		, _IsReadOnly( false )
		, _AllowMultiLine( true )
		, _IsCaretMovedWhenGainFocus ( true )
		, _SelectAllTextWhenFocused( false )
		, _ClearTextSelectionOnFocusLoss( true )
		, _RevertTextOnEscape( false )
		, _ClearKeyboardFocusOnCommit( true )
		, _AllowContextMenu(true)
		, _AlwaysShowScrollbars( false )
		, _HScrollBar()
		, _VScrollBar()
		, _WrapTextAt(0.0f)
		, _AutoWrapText(false)
		, _WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
		, _SelectAllTextOnCommit( false )
		, _SelectWordOnMouseDoubleClick( true )
		, _BackgroundColor()		
		, _Padding()
		, _Margin()
		, _ErrorReporting()
		, _ModiferKeyForNewLine(EModifierKey::None)
		, _VirtualKeyboardOptions(FVirtualKeyboardOptions())
		, _VirtualKeyboardTrigger(EVirtualKeyboardTrigger::OnFocusByPointer)
		, _VirtualKeyboardDismissAction(EVirtualKeyboardDismissAction::TextChangeOnDismiss)
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _OverflowPolicy()
		, _FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
		{}

		/** The styling of the textbox */
		SLATE_STYLE_ARGUMENT(FEditableTextBoxStyle, Style)

		/** Pointer to a style of the text block, which dictates the font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT_DEPRECATED(FTextBlockStyle, TextStyle, 5.2, "TextStyle is deprecated and will be ignored. Please use the TextStyle embedded in FEditableTextBoxStyle Style.")

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr< ITextLayoutMarshaller >, Marshaller)

		/** Sets the text content for this editable text box widget */
		SLATE_ATTRIBUTE( FText, Text )

		/** Hint text that appears when there is no text in the text box */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Text to search for (a new search is triggered whenever this text changes) */
		SLATE_ATTRIBUTE( FText, SearchText )

		/** Font color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Text color and opacity (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )
		
		/** Text color and opacity when read-only (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, ReadOnlyForegroundColor )

		/** Text color and opacity when this box has keyboard focus (overrides Style) */
		SLATE_ATTRIBUTE(FSlateColor, FocusedForegroundColor)

		/** How the text should be aligned with the margin. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)

		/** Maximum length of the text. Input will refuse text longer than that. */
		SLATE_ATTRIBUTE(int32, MaximumLength)

		/** The amount to scale each lines height by. */
		SLATE_ATTRIBUTE(float, LineHeightPercentage)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE( bool, IsReadOnly )

		/** Whether to allow multi-line text */
		SLATE_ATTRIBUTE(bool, AllowMultiLine)

		/** Workaround as we loose focus when the auto completion closes. */
		SLATE_ATTRIBUTE( bool, IsCaretMovedWhenGainFocus )

		/** Whether to select all text when the user clicks to give focus on the widget */
		SLATE_ATTRIBUTE( bool, SelectAllTextWhenFocused )

		/** Whether to clear text selection when focus is lost */
		SLATE_ATTRIBUTE( bool, ClearTextSelectionOnFocusLoss )

		/** Whether to allow the user to back out of changes when they press the escape key */
		SLATE_ATTRIBUTE( bool, RevertTextOnEscape )

		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, ClearKeyboardFocusOnCommit )

		/** Whether the context menu can be opened  */
		SLATE_ATTRIBUTE(bool, AllowContextMenu)

		/** Should we always show the scrollbars (only affects internally created scroll bars) */
		SLATE_ARGUMENT(bool, AlwaysShowScrollbars)

		/** The horizontal scroll bar widget, or null to create one internally */
		SLATE_ARGUMENT( TSharedPtr< SScrollBar >, HScrollBar )

		/** The vertical scroll bar widget, or null to create one internally */
		SLATE_ARGUMENT( TSharedPtr< SScrollBar >, VScrollBar )

		/** Padding around the horizontal scrollbar (overrides Style) */
		SLATE_ATTRIBUTE( FMargin, HScrollBarPadding )

		/** Padding around the vertical scrollbar (overrides Style) */
		SLATE_ATTRIBUTE( FMargin, VScrollBarPadding )

		/** Delegate to call before a context menu is opened. User returns the menu content or null to the disable context menu */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

		/**
		 * This is NOT for validating input!
		 * 
		 * Called whenever a character is typed.
		 * Not called for copy, paste, or any other text changes!
		 */
		SLATE_EVENT( FOnIsTypedCharValid, OnIsTypedCharValid )

		/** Callback when the text starts to be edited */
		SLATE_EVENT( FOnBeginTextEdit, OnBeginTextEdit )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnVerifyTextChanged, OnVerifyTextChanged )

		/** Called whenever the horizontal scrollbar is moved by the user */
		SLATE_EVENT( FOnUserScrolled, OnHScrollBarUserScrolled )

		/** Called whenever the vertical scrollbar is moved by the user */
		SLATE_EVENT( FOnUserScrolled, OnVScrollBarUserScrolled )

		/** Called when the cursor is moved within the text area */
		SLATE_EVENT( SMultiLineEditableText::FOnCursorMoved, OnCursorMoved )

		/** Callback delegate to have first chance handling of the OnKeyChar event */
		SLATE_EVENT(FOnKeyChar, OnKeyCharHandler)

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

		/** Menu extender for the right-click context menu */
		SLATE_EVENT( FMenuExtensionDelegate, ContextMenuExtender )

		/** Delegate used to create text layouts for this widget. If none is provided then FSlateTextLayout will be used. */
		SLATE_EVENT( FCreateSlateTextLayout, CreateSlateTextLayout )

		/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
		SLATE_ATTRIBUTE( float, WrapTextAt )

		/** Whether to wrap text automatically based on the widget's computed horizontal space.  IMPORTANT: Using automatic wrapping can result
			in visual artifacts, as the the wrapped size will computed be at least one frame late!  Consider using WrapTextAt instead.  The initial 
			desired size will not be clamped.  This works best in cases where the text block's size is not affecting other widget's layout. */
		SLATE_ATTRIBUTE( bool, AutoWrapText )

		/** The wrapping policy to use */
		SLATE_ATTRIBUTE( ETextWrappingPolicy, WrappingPolicy )

		/** Whether to select all text when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, SelectAllTextOnCommit )

		/** Whether to select word on mouse double click on the widget */
		SLATE_ATTRIBUTE(bool, SelectWordOnMouseDoubleClick)

		/** The color of the background/border around the editable text (overrides Style) */
		SLATE_ATTRIBUTE( FSlateColor, BackgroundColor )

		/** Padding between the box/border and the text widget inside (overrides Style) */
		SLATE_ATTRIBUTE( FMargin, Padding )

		/** The amount of blank space left around the edges of text area. 
			This is different to Padding because this area is still considered part of the text area, and as such, can still be interacted with */
		SLATE_ATTRIBUTE( FMargin, Margin )

		/** Provide a alternative mechanism for error reporting. */
		SLATE_ARGUMENT( TSharedPtr<class IErrorReportingWidget>, ErrorReporting )

		/** The optional modifier key necessary to create a newline when typing into the editor. */
		SLATE_ARGUMENT( EModifierKey::Type, ModiferKeyForNewLine)

		/** Additional options used by the virtual keyboard summoned by this widget */
		SLATE_ARGUMENT( FVirtualKeyboardOptions, VirtualKeyboardOptions  )

		/** The type of event that will trigger the display of the virtual keyboard */
		SLATE_ATTRIBUTE( EVirtualKeyboardTrigger, VirtualKeyboardTrigger )

		/** The message action to take when the virtual keyboard is dismissed by the user */
		SLATE_ATTRIBUTE( EVirtualKeyboardDismissAction, VirtualKeyboardDismissAction )

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		
		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

		/** Determines what happens to text that is clipped and doesn't fit within the allotted area for this widget */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

		/** The paint policy to use for text if the font faces are still async loading. */
		SLATE_ARGUMENT(EFontFacesLoadingPaintPolicy, FontFacesLoadingPaintPolicy)

		/** Called when all font faces for the text have finished loading asynchronously */
		SLATE_EVENT(FSimpleDelegate, OnAllFontFacesFinishLoading)
	SLATE_END_ARGS()
	
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);
	
	/**
	 * Returns the text string
	 *
	 * @return  Text string
	 */
	FText GetText() const
	{
		return EditableText->GetText();
	}

	/**
	 * Returns the plain text string without richtext formatting
	 *
	 * @return  Text string
	 */
	FText GetPlainText() const
	{
		return EditableText->GetPlainText();
	}

	/**
	 * Return the text line where the current cursor location is at.
	 *
	 * @param  OutTextLine	Text string
	 */	
	SLATE_API void GetCurrentTextLine(FString& OutTextLine) const;

	/** See attribute Style */
	SLATE_API void SetStyle(const FEditableTextBoxStyle* InStyle);

	/** See attribute TextStyle */
	SLATE_API void SetTextStyle(const FTextBlockStyle* InTextStyle);	

	/**
	 * Sets the text string currently being edited 
	 *
	 * @param  InNewText  The new text string
	 */
	SLATE_API void SetText(TAttribute<FText> InNewText);

	/**
	 * Returns the hint text string
	 *
	 * @return  Hint text string
	 */
	FText GetHintText() const
	{
		return EditableText->GetHintText();
	}

	/**
	 * Sets the text that appears when there is no text in the text box
	 *
	 * @param  InHintText The hint text string
	 */
	SLATE_API void SetHintText(TAttribute<FText> InHintText);

	/** Set the text that is currently being searched for (if any) */
	SLATE_API void SetSearchText(TAttribute<FText> InSearchText);

	/** Get the text that is currently being searched for (if any) */
	SLATE_API FText GetSearchText() const;

	/**
	 * Sets the text color and opacity (overrides Style)
	 *
	 * @param  InForegroundColor 	The text color and opacity
	 */
	SLATE_API void SetTextBoxForegroundColor(TAttribute<FSlateColor> InForegroundColor);

	/**
	 * Sets the color of the background/border around the editable text (overrides Style) 
	 *
	 * @param  InBackgroundColor 	The background/border color
	 */
	SLATE_API void SetTextBoxBackgroundColor(TAttribute<FSlateColor> InBackgroundColor);

	/**
	 * Sets the text color and opacity when read-only (overrides Style) 
	 *
	 * @param  InReadOnlyForegroundColor 	The read-only text color and opacity
	 */
	SLATE_API void SetReadOnlyForegroundColor(TAttribute<FSlateColor> InReadOnlyForegroundColor);

	/**
	 * Sets the maximum text length that will be accepted by the widget
	 *
	 * @param  InMaximumLength		Length to check for. Only values superior to 0 will be taken into account, otherwise no length check will occur.
	 */
	SLATE_API void SetMaximumLength(TAttribute<int32> InMaximumLength);

	/**
	 * Sets whether to select word on the mouse double click
	 *
	 * @param  InSelectWordOnMouseDoubleClick		Select word on the mouse double click
	 */
	SLATE_API void SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick);

	/** See TextShapingMethod attribute */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** See TextFlowDirection attribute */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** See WrapTextAt attribute */
	SLATE_API void SetWrapTextAt(TAttribute<float> InWrapTextAt);

	/** See AutoWrapText attribute */
	SLATE_API void SetAutoWrapText(TAttribute<bool> InAutoWrapText);

	/** Set WrappingPolicy attribute */
	SLATE_API void SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy);

	/** See LineHeightPercentage attribute */
	SLATE_API void SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage);

	/** See ApplyLineHeightToBottomLine attribute */
	SLATE_API void SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine);

	/** See Margin attribute */
	SLATE_API void SetMargin(TAttribute<FMargin> InMargin);

	/** See Justification attribute */
	SLATE_API void SetJustification(TAttribute<ETextJustify::Type> InJustification);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy);

	/** See the AllowContextMenu attribute */
	SLATE_API void SetAllowContextMenu(TAttribute<bool> InAllowContextMenu);

	/** Set the VirtualKeyboardDismissAction attribute */
	SLATE_API void SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction);
	
	/** Set the ReadOnly attribute */
	SLATE_API void SetIsReadOnly(TAttribute<bool> InIsReadOnly);

	/** Set the FontFacesLoadingPaintPolicy argument */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);

	/**
	 * If InError is a non-empty string the TextBox will the ErrorReporting provided during construction
	 * If no error reporting was provided, the TextBox will create a default error reporter.
	 */
	SLATE_API void SetError(const FText& InError);
	SLATE_API void SetError(const FString& InError);
	SLATE_API bool HasError() const;

	// SWidget overrides
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual bool HasKeyboardFocus() const override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;


	/** Query to see if any text is selected within the document */
	SLATE_API bool AnyTextSelected() const;

	/** Select all the text in the document */
	SLATE_API void SelectAllText();

	/** Clear the active text selection */
	SLATE_API void ClearSelection();

	/** Get the currently selected text */
	SLATE_API FText GetSelectedText() const;

	/** Insert the given text at the current cursor position, correctly taking into account new line characters */
	SLATE_API void InsertTextAtCursor(const FText& InText);
	SLATE_API void InsertTextAtCursor(const FString& InString);

	/** Insert the given run at the current cursor position */
	SLATE_API void InsertRunAtCursor(TSharedRef<IRun> InRun);

	/** Move the cursor to the given location in the document */
	SLATE_API void GoTo(const FTextLocation& NewLocation);

	/** Move the cursor to the specified location */
	void GoTo(const ETextLocation NewLocation)
	{
		EditableText->GoTo(NewLocation);
	}

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const FTextLocation& NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	void ScrollTo(const ETextLocation NewLocation)
	{
		EditableText->ScrollTo(NewLocation);
	}

	/** Apply the given style to the currently selected text (or insert a new run at the current cursor position if no text is selected) */
	SLATE_API void ApplyToSelection(const FRunInfo& InRunInfo, const FTextBlockStyle& InStyle);

	/** Begin a new text search (this is called automatically when the bound search text changes) */
	SLATE_API void BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase, const bool InReverse = false);

	/** Advance the current search to the next match (does nothing if not currently searching) */
	SLATE_API void AdvanceSearch(const bool InReverse = false);

	/** Get the run currently under the cursor, or null if there is no run currently under the cursor */
	SLATE_API TSharedPtr<const IRun> GetRunUnderCursor() const;

	/** Get the runs currently that are current selected, some of which may be only partially selected */
	SLATE_API TArray<TSharedRef<const IRun>> GetSelectedRuns() const;

	/** Get the interaction position of the cursor (where to insert, delete, etc, text from/to) */
	SLATE_API FTextLocation GetCursorLocation() const;

	/** Get the horizontal scroll bar widget */
	SLATE_API TSharedPtr<const SScrollBar> GetHScrollBar() const;

	/** Get the vertical scroll bar widget */
	SLATE_API TSharedPtr<const SScrollBar> GetVScrollBar() const;

	/** Refresh this text box immediately, rather than wait for the usual caching mechanisms to take affect on the text Tick */
	SLATE_API void Refresh();

	/**
	 * Sets the OnKeyCharHandler to provide first chance handling of the SMultiLineEditableText's OnKeyChar event
	 *
	 * @param InOnKeyCharHandler			Delegate to call during OnKeyChar event
	 */
	SLATE_API void SetOnKeyCharHandler(FOnKeyChar InOnKeyCharHandler);

	/**
	 * Sets the OnKeyDownHandler to provide first chance handling of the SMultiLineEditableText's OnKeyDown event
	 *
	 * @param InOnKeyDownHandler			Delegate to call during OnKeyDown event
	 */
	SLATE_API void SetOnKeyDownHandler(FOnKeyDown InOnKeyDownHandler);


	/**
	 * 
	 */
	SLATE_API void ForceScroll(int32 UserIndex, float ScrollAxisMagnitude);

protected:
	/** Callback for the editable text's OnTextChanged event */
	SLATE_API void OnEditableTextChanged(const FText& InText);

	/** Callback when the editable text is committed. */
	SLATE_API void OnEditableTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

protected:

	/** Editable text widget */
	TSharedPtr<SMultiLineEditableText> EditableText;

	/** Padding (overrides style) */
	TAttribute<FMargin> PaddingOverride;

	/** Horiz scrollbar padding (overrides style) */
	TAttribute<FMargin> HScrollBarPaddingOverride;

	/** Vert scrollbar padding (overrides style) */
	TAttribute<FMargin> VScrollBarPaddingOverride;

	/** Font (overrides style) */
	TAttribute<FSlateFontInfo> FontOverride;

	/** Foreground color (overrides style) */
	TAttribute<FSlateColor> ForegroundColorOverride;

	/** Background color (overrides style) */
	TAttribute<FSlateColor> BackgroundColorOverride;

	/** Read-only foreground color (overrides style) */
	TAttribute<FSlateColor> ReadOnlyForegroundColorOverride;

	/** Focused foreground color (overrides style) */
	TAttribute<FSlateColor> FocusedForegroundColorOverride;

	/** Maximum text length that will be accepted by the widget */
	TAttribute<int32> MaximumLength;

	/** Whether to disable the context menu */
	TAttribute<bool> AllowContextMenu;

	/** Whether to select work on mouse double click */
	TAttribute<bool> bSelectWordOnMouseDoubleClick;

	/** Allows for inserting additional widgets that extend the functionality of the text box */
	TSharedPtr<SHorizontalBox> Box;

	/** Whether we have an externally supplied horizontal scrollbar or one created internally */
	bool bHasExternalHScrollBar;

	/** Horiz scrollbar */
	TSharedPtr<SScrollBar> HScrollBar;

	/** Box around the horiz scrollbar used for adding padding */
	TSharedPtr<SBox> HScrollBarPaddingBox;

	/** Whether we have an externally supplied vertical scrollbar or one created internally */
	bool bHasExternalVScrollBar;

	/** Vert scrollbar */
	TSharedPtr<SScrollBar> VScrollBar;

	/** Box around the vert scrollbar used for adding padding */
	TSharedPtr<SBox> VScrollBarPaddingBox;

	/** SomeWidget reporting */
	TSharedPtr<class IErrorReportingWidget> ErrorReporting;

	/** Called when the text is changed interactively */
	FOnTextChanged OnTextChanged;

	/** Called when the user commits their change to the editable text control */
	FOnTextCommitted OnTextCommitted;

	/** Callback to verify text when changed. Will return an error message to denote problems. */
	FOnVerifyTextChanged OnVerifyTextChanged;

	const FEditableTextBoxStyle* Style;

private:

	FMargin DeterminePadding() const;
	FMargin DetermineHScrollBarPadding() const;
	FMargin DetermineVScrollBarPadding() const;
	FSlateFontInfo DetermineFont() const;
	FSlateColor DetermineBackgroundColor() const;
	FSlateColor DetermineForegroundColor() const;

	/** Styling: border image to draw when not hovered or focused */
	const FSlateBrush* BorderImageNormal;
	/** Styling: border image to draw when hovered */
	const FSlateBrush* BorderImageHovered;
	/** Styling: border image to draw when focused */
	const FSlateBrush* BorderImageFocused;
	/** Styling: border image to draw when read only */
	const FSlateBrush* BorderImageReadOnly;

	/** @return Border image for the text box based on the hovered and focused state */
	const FSlateBrush* DetermineBorderImage() const;

};


#endif //WITH_FANCY_TEXT
