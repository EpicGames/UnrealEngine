// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/Events.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"

/** A widget which allows the user to enter a digit or choose a number from a drop down menu. */
template<typename NumericType>
class SNumericDropDown : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnValueChanged, NumericType );

public:
	/** Represents a named numeric value for display in the drop down menu. */
	class FNamedValue
	{
	public:
		/** Creates a new FNamedValue 
		  * @InValue The numeric value to be assigned
		  * @InName The display text for the value in the UI 
		  * @InDescription The description of the value used in tooltips or wherever a longer description is needed. */
		FNamedValue( NumericType InValue, FText InName, FText InDescription )
		{
			Value = InValue;
			Name = InName;
			Description = InDescription;
		}

		NumericType GetValue() const
		{
			return Value;
		}

		FText GetName() const
		{
			return Name;
		}

		FText GetDescription() const
		{
			return Description;
		}

	private:
		NumericType Value;
		FText Name;
		FText Description;
	};

public:
	SLATE_BEGIN_ARGS( SNumericDropDown<NumericType> )
		: _DropDownValues()
		, _LabelText()
		, _Orientation( EOrientation::Orient_Horizontal )
		, _MinDesiredValueWidth( 40 )
		, _bShowNamedValue(false)
		{}

		/** The values which are used to populate the drop down menu. */
		SLATE_ARGUMENT( TArray<FNamedValue>, DropDownValues )
		/** The text which is displayed in the label next to the control. */
		SLATE_ATTRIBUTE( FText, LabelText )
		/** Controls the label placement for the control.  Vertical will place the label above, horizontal will place the label to the left. */
		SLATE_ATTRIBUTE( EOrientation, Orientation )
		/** Controls the minimum width for the text box portion of the control. */
		SLATE_ATTRIBUTE( float, MinDesiredValueWidth )
		/** Toggle to show the drop down text value if the value matches the numeric value. */
		SLATE_ATTRIBUTE( bool, bShowNamedValue )
		/** The value displayed by the control. */
		SLATE_ATTRIBUTE( NumericType, Value )
		/** The callback for when the value changes. */
		SLATE_EVENT( FOnValueChanged, OnValueChanged )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		DropDownValues = InArgs._DropDownValues;
		LabelText = InArgs._LabelText;
		Orientation = InArgs._Orientation;
		bShowNamedValue = InArgs._bShowNamedValue;
		Value = InArgs._Value;
		OnValueChanged = InArgs._OnValueChanged;

		ChildSlot
		[
			SNew( SVerticalBox )
			// Vertical Label
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 0, 0, 0, 3 ) )
			[
				SNew( STextBlock )
				.Text(LabelText)
				.Visibility(this, &SNumericDropDown<NumericType>::GetLabelVisibility, EOrientation::Orient_Vertical)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SHorizontalBox )
				// Horizontal Icon Label
				+ SHorizontalBox::Slot()
				.VAlign( EVerticalAlignment::VAlign_Center )
				.AutoWidth()
				.Padding( FMargin( 0, 0, 3, 0 ) )
				[
					
					SNew( STextBlock )
					.Text(LabelText)
					.Visibility( this, &SNumericDropDown<NumericType>::GetLabelVisibility, EOrientation::Orient_Horizontal )
				]
				+ SHorizontalBox::Slot()
				[
					SNew( SComboButton )
					.ContentPadding( 1 )
					.OnGetMenuContent( this, &SNumericDropDown<NumericType>::BuildMenu )
					.ButtonContent()
					[
						SNew( SEditableTextBox )
						.MinDesiredWidth( InArgs._MinDesiredValueWidth )
						.RevertTextOnEscape(true)
						.SelectAllTextWhenFocused(true)
						.Text( this, &SNumericDropDown<NumericType>::GetValueText )
						.OnTextCommitted( this, &SNumericDropDown<NumericType>::ValueTextComitted )
					]
				]
			]
		];
	}

private:
	EVisibility GetLabelVisibility( EOrientation LabelOrientation ) const
	{
		return Orientation.Get() == LabelOrientation
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

	FText GetValueText() const
	{
		if (bShowNamedValue.Get())
		{
			for ( FNamedValue DropDownValue : DropDownValues )
			{
				if (FMath::IsNearlyEqual(DropDownValue.GetValue(), Value.Get()))
				{
					return DropDownValue.GetName();
				}
			}
		}
		return FText::AsNumber( Value.Get() );
	}

	void ValueTextComitted( const FText& InNewText, ETextCommit::Type InTextCommit )
	{
		if ( InNewText.IsNumeric() )
		{
			NumericType NewValue;
			TTypeFromString<NumericType>::FromString( NewValue, *InNewText.ToString() );
			OnValueChanged.ExecuteIfBound(NewValue);
		}
	}

	TSharedRef<SWidget> BuildMenu()
	{
		FMenuBuilder MenuBuilder( true, NULL );

		for ( FNamedValue DropDownValue : DropDownValues )
		{
			FUIAction MenuAction(FExecuteAction::CreateSP(this, &SNumericDropDown::SetValue, DropDownValue.GetValue()));
			MenuBuilder.AddMenuEntry(DropDownValue.GetName(), DropDownValue.GetDescription(), FSlateIcon(), MenuAction);
		}

		return MenuBuilder.MakeWidget();
	}

	void SetValue( NumericType InValue )
	{
		FSlateApplication::Get().ClearKeyboardFocus( EFocusCause::Cleared );
		OnValueChanged.ExecuteIfBound(InValue);
	}

private:
	TArray<FNamedValue> DropDownValues;
	TAttribute<FText> LabelText;
	TAttribute<EOrientation> Orientation;
	TAttribute<bool> bShowNamedValue;
	TAttribute<NumericType> Value;
	FOnValueChanged OnValueChanged;
};
