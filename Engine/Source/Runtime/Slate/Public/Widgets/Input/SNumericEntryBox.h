// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Visibility.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Containers/UnrealString.h"


template <typename NumericType>
class SNumericEntryBox;

template <typename NumericType>
struct TWidgetTypeTraits<SNumericEntryBox<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Implementation for a box that only accepts a numeric value or that can display an undetermined value via a string
 * Supports an optional spin box for manipulating a value by dragging with the mouse
 * Supports an optional label inset in the text box
 */ 
template<typename NumericType>
class SNumericEntryBox : public SCompoundWidget
{
public: 

	static const FLinearColor RedLabelBackgroundColor;
	static const FLinearColor GreenLabelBackgroundColor;
	static const FLinearColor BlueLabelBackgroundColor;
	static const FLinearColor LilacLabelBackgroundColor;
	static const FText DefaultUndeterminedString;

	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnValueChanged, NumericType /*NewValue*/);

	/** Notification for numeric value committed */
	DECLARE_DELEGATE_TwoParams(FOnValueCommitted, NumericType /*NewValue*/, ETextCommit::Type /*CommitType*/);

	/** Notification for change of undetermined values */
	DECLARE_DELEGATE_OneParam(FOnUndeterminedValueChanged, FText /*NewValue*/ );

	/** Notification for committing undetermined values */
	DECLARE_DELEGATE_TwoParams(FOnUndeterminedValueCommitted, FText /*NewValue*/, ETextCommit::Type /*CommitType*/);

	/** Notification when the max/min spinner values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_DELEGATE_FourParams(FOnDynamicSliderMinMaxValueChanged, NumericType, TWeakPtr<SWidget>, bool, bool);

	enum class ELabelLocation
	{
		// Outside the bounds of the editable area of this box. Usually preferred for text based labels
		Outside,
		// Inside the bounds of the editable area of this box. Usually preferred for non-text based labels
		// when a spin box is used the label will appear on top of the spin box in this case
		Inside
	};
public:

	SLATE_BEGIN_ARGS( SNumericEntryBox<NumericType> )
		: _EditableTextBoxStyle( &FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox") )
		, _SpinBoxStyle(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("NumericEntrySpinBox") )
		, _Label()
		, _LabelVAlign( VAlign_Fill )
		, _Justification(ETextJustify::Left)
		, _LabelLocation(ELabelLocation::Outside)
		, _LabelPadding(FMargin(3.f,0.f) )
		, _BorderForegroundColor(FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("NumericEntrySpinBox").ForegroundColor)
		, _BorderBackgroundColor(FLinearColor::White)
		, _UndeterminedString( SNumericEntryBox<NumericType>::DefaultUndeterminedString )
		, _AllowSpin(false)
		, _ShiftMultiplier(10.f)
		, _CtrlMultiplier(0.1f)
		, _SupportDynamicSliderMaxValue(false)
		, _SupportDynamicSliderMinValue(false)
		, _Delta(NumericType(0))
		, _MinFractionalDigits(DefaultMinFractionalDigits)
		, _MaxFractionalDigits(DefaultMaxFractionalDigits)
		, _MinValue(TNumericLimits<NumericType>::Lowest())
		, _MaxValue(TNumericLimits<NumericType>::Max())
		, _MinSliderValue(NumericType(0))
		, _MaxSliderValue(NumericType(100))
		, _SliderExponent(1.f)
		, _AllowWheel(true)
		, _PreventThrottling(false)
		, _BroadcastValueChangesPerKey(false)
		, _MinDesiredValueWidth(0.f)
		, _DisplayToggle(false)
		, _ToggleChecked(ECheckBoxState::Checked)
		, _TogglePadding(FMargin(1.f,0.f,1.f,0.f) )
		, _ToolTipTextFormat(TOptional<FTextFormat>())
	{}

		/** Style to use for the editable text box within this widget */
		SLATE_STYLE_ARGUMENT( FEditableTextBoxStyle, EditableTextBoxStyle )

		/** Style to use for the spin box within this widget */
		SLATE_STYLE_ARGUMENT( FSpinBoxStyle, SpinBoxStyle )

		/** Slot for this button's content (optional) */
		SLATE_NAMED_SLOT( FArguments, Label )
		/** Vertical alignment of the label content */
		SLATE_ARGUMENT( EVerticalAlignment, LabelVAlign )
		/** How should the value be justified in the editable text field. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)

		SLATE_ARGUMENT(ELabelLocation, LabelLocation)
		/** Padding around the label content */
		SLATE_ARGUMENT( FMargin, LabelPadding )
		/** Border Foreground Color */
		SLATE_ARGUMENT( FSlateColor, BorderForegroundColor )
		/** Border Background Color */
		SLATE_ARGUMENT( FSlateColor, BorderBackgroundColor )
		/** The value that should be displayed.  This value is optional in the case where a value cannot be determined */
		SLATE_ATTRIBUTE( TOptional<NumericType>, Value )
		/** The string to display if the value cannot be determined */
		SLATE_ARGUMENT( FText, UndeterminedString )
		/** Font color and opacity */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
		/** Whether or not the user should be able to change the value by dragging with the mouse cursor */
		SLATE_ARGUMENT( bool, AllowSpin )
		SLATE_ATTRIBUTE_DEPRECATED( int32, ShiftMouseMovePixelPerDelta, 5.4, "Shift Mouse Move Pixel Per Delta is deprecated, please use ShiftMultiplier" )
		/** Multiplier to use when shift is held down */
		SLATE_ATTRIBUTE( float, ShiftMultiplier )
		/** Multiplier to use when ctrl is held down */
		SLATE_ATTRIBUTE( float, CtrlMultiplier )
		/** If we're an unbounded spinbox, what value do we divide mouse movement by before multiplying by Delta. Requires Delta to be set. */
		SLATE_ATTRIBUTE( int32, LinearDeltaSensitivity)
		/** Tell us if we want to support dynamically changing of the max value using ctrl  (only use if there is a spinbox allow) */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMaxValue)
		/** Tell us if we want to support dynamically changing of the min value using ctrl  (only use if there is a spinbox allow) */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMinValue)
		/** Called right after the spinner max value is changed (only relevant if SupportDynamicSliderMaxValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMaxValueChanged)
		/** Called right after the spinner min value is changed (only relevant if SupportDynamicSliderMinValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMinValueChanged)
		/** Delta to increment the value as the slider moves.  If not specified will determine automatically */
		SLATE_ATTRIBUTE( NumericType, Delta )
		/** The minimum fractional digits the spin box displays, defaults to 1 */
		SLATE_ATTRIBUTE(TOptional< int32 >, MinFractionalDigits)
		/** The maximum fractional digits the spin box displays, defaults to 6 */
		SLATE_ATTRIBUTE(TOptional< int32 >, MaxFractionalDigits)
		/** The minimum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE( TOptional< NumericType >, MinValue )
		/** The maximum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE( TOptional< NumericType >, MaxValue )
		/** The minimum value that can be specified by using the slider */
		SLATE_ATTRIBUTE( TOptional< NumericType >, MinSliderValue )
		/** The maximum value that can be specified by using the slider */
		SLATE_ATTRIBUTE( TOptional< NumericType >, MaxSliderValue )
		/** Use exponential scale for the slider */
		SLATE_ATTRIBUTE( float, SliderExponent )
		/** When using exponential scale specify a neutral value where we want the maximum precision (by default it is the smallest slider value)*/
		SLATE_ATTRIBUTE(NumericType, SliderExponentNeutralValue )
		/** Whether this spin box should have mouse wheel feature enabled, defaults to true */
		SLATE_ARGUMENT( bool, AllowWheel )
		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)
		/** True to broadcast every time we type */
		SLATE_ARGUMENT( bool, BroadcastValueChangesPerKey)
		/** Step to increment or decrement the value by when scrolling the mouse wheel. If not specified will determine automatically */
		SLATE_ATTRIBUTE( TOptional< NumericType >, WheelStep )
		/** The minimum desired width for the value portion of the control. */
		SLATE_ATTRIBUTE( float, MinDesiredValueWidth )
		/** The text margin to use if overridden. */
		SLATE_ATTRIBUTE( FMargin, OverrideTextMargin )
		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnValueChanged, OnValueChanged )
		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT( FOnValueCommitted, OnValueCommitted )
		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnUndeterminedValueChanged, OnUndeterminedValueChanged )
		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT( FOnUndeterminedValueCommitted, OnUndeterminedValueCommitted )
		/** Called right before the slider begins to move */
		SLATE_EVENT( FSimpleDelegate, OnBeginSliderMovement )
		/** Called right after the slider handle is released by the user */
		SLATE_EVENT( FOnValueChanged, OnEndSliderMovement )		
		/** Menu extender for right-click context menu */
		SLATE_EVENT( FMenuExtensionDelegate, ContextMenuExtender )
		/** Provide custom type conversion functionality to this spin box */
		SLATE_ARGUMENT( TSharedPtr< INumericTypeInterface<NumericType> >, TypeInterface )
		/** Whether or not to include a toggle checkbox to the left of the widget */
		SLATE_ARGUMENT( bool, DisplayToggle )
		/** The value of the toggle checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleChecked )
		/** Called whenever the toggle changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleChanged )
		/** Padding around the toggle checkbox */
		SLATE_ARGUMENT( FMargin, TogglePadding )
		/** An optional format pattern that is used to format the ToolTipText. If set, this is used with a single argument ({0} = the display value). */
		SLATE_ATTRIBUTE(TOptional<FTextFormat>, ToolTipTextFormat)

	SLATE_END_ARGS()
	SNumericEntryBox()
	{
	}

	void Construct( const FArguments& InArgs )
	{
		check(InArgs._EditableTextBoxStyle);

		OnValueChanged = InArgs._OnValueChanged;
		OnValueCommitted = InArgs._OnValueCommitted;
		OnUndeterminedValueChanged = InArgs._OnUndeterminedValueChanged;
		OnUndeterminedValueCommitted = InArgs._OnUndeterminedValueCommitted;
		ValueAttribute = InArgs._Value;
		UndeterminedString = InArgs._UndeterminedString;
		MinDesiredValueWidth = InArgs._MinDesiredValueWidth;
		BorderImageNormal = &InArgs._EditableTextBoxStyle->BackgroundImageNormal;
		BorderImageHovered = &InArgs._EditableTextBoxStyle->BackgroundImageHovered;
		BorderImageFocused = &InArgs._EditableTextBoxStyle->BackgroundImageFocused;
		Interface = InArgs._TypeInterface.IsValid() ? InArgs._TypeInterface : MakeShareable( new TDefaultNumericTypeInterface<NumericType> );

		if (InArgs._TypeInterface.IsValid() && Interface->GetOnSettingChanged())
		{
			Interface->GetOnSettingChanged()->AddSP(this, &SNumericEntryBox::ResetCachedValueString);
		}

		MinFractionalDigits = (InArgs._MinFractionalDigits.Get().IsSet()) ? InArgs._MinFractionalDigits : DefaultMinFractionalDigits;
		MaxFractionalDigits = (InArgs._MaxFractionalDigits.Get().IsSet()) ? InArgs._MaxFractionalDigits : DefaultMaxFractionalDigits;
		SetMinFractionalDigits(MinFractionalDigits);
		SetMaxFractionalDigits(MaxFractionalDigits);

		ToolTipTextFormat = InArgs._ToolTipTextFormat;

		CachedExternalValue = ValueAttribute.Get();
		if (CachedExternalValue.IsSet())
		{
			CachedValueString = Interface->ToString(CachedExternalValue.GetValue());
		}
		bCachedValueStringDirty = false;

		const bool bDisplayToggle = InArgs._DisplayToggle;
		if(bDisplayToggle)
		{
			SAssignNew(ToggleCheckBox, SCheckBox)
			.Padding(FMargin(0.f, 0.f, 2.f, 0.f))
			.IsChecked(InArgs._ToggleChecked)
			.OnCheckStateChanged(this, &SNumericEntryBox::HandleToggleCheckBoxChanged, InArgs._OnToggleChanged);
		}

		const bool bAllowSpin = InArgs._AllowSpin;
		TSharedPtr<SWidget> FinalWidget;

		if( bAllowSpin )
		{
			SAssignNew(SpinBox, SSpinBox<NumericType>)
				.Style(InArgs._SpinBoxStyle)
				.Font(InArgs._Font.IsSet() ? InArgs._Font : InArgs._EditableTextBoxStyle->TextStyle.Font)
				.Value(this, &SNumericEntryBox<NumericType>::OnGetValueForSpinBox)
				.Delta(InArgs._Delta)
				.ShiftMultiplier(InArgs._ShiftMultiplier)
				.CtrlMultiplier(InArgs._CtrlMultiplier)
				.LinearDeltaSensitivity(InArgs._LinearDeltaSensitivity)
				.SupportDynamicSliderMaxValue(InArgs._SupportDynamicSliderMaxValue)
				.SupportDynamicSliderMinValue(InArgs._SupportDynamicSliderMinValue)
				.OnDynamicSliderMaxValueChanged(InArgs._OnDynamicSliderMaxValueChanged)
				.OnDynamicSliderMinValueChanged(InArgs._OnDynamicSliderMinValueChanged)
				.OnValueChanged(OnValueChanged)
				.OnValueCommitted(OnValueCommitted)
				.MinFractionalDigits(MinFractionalDigits)
				.MaxFractionalDigits(MaxFractionalDigits)
				.MinSliderValue(InArgs._MinSliderValue)
				.MaxSliderValue(InArgs._MaxSliderValue)
				.MaxValue(InArgs._MaxValue)
				.MinValue(InArgs._MinValue)
				.ContextMenuExtender(InArgs._ContextMenuExtender)
				.SliderExponent(InArgs._SliderExponent)
				.SliderExponentNeutralValue(InArgs._SliderExponentNeutralValue)
				.EnableWheel(InArgs._AllowWheel)
				.PreventThrottling(InArgs._PreventThrottling)
				.BroadcastValueChangesPerKey(InArgs._BroadcastValueChangesPerKey)
				.WheelStep(InArgs._WheelStep)
				.OnBeginSliderMovement(InArgs._OnBeginSliderMovement)
				.OnEndSliderMovement(InArgs._OnEndSliderMovement)
				.MinDesiredWidth(InArgs._MinDesiredValueWidth)
				.TypeInterface(Interface)
				.ToolTipText(this, &SNumericEntryBox<NumericType>::GetToolTipText);
		}

		// Always create an editable text box.  In the case of an undetermined value being passed in, we cant use the spinbox.
		SAssignNew(EditableText, SEditableText)
			.Text(this, &SNumericEntryBox<NumericType>::OnGetValueForTextBox)
			.ToolTipText(this, &SNumericEntryBox<NumericType>::GetToolTipText)
			.ColorAndOpacity(InArgs._EditableTextBoxStyle->ForegroundColor)
			.Visibility(bAllowSpin ? EVisibility::Collapsed : EVisibility::Visible)
			.Font(InArgs._Font.IsSet() ? InArgs._Font : InArgs._EditableTextBoxStyle->TextStyle.Font)
			.SelectAllTextWhenFocused(true)
			.ClearKeyboardFocusOnCommit(false)
			.OnTextChanged(this, &SNumericEntryBox<NumericType>::OnTextChanged)
			.OnTextCommitted(this, &SNumericEntryBox<NumericType>::OnTextCommitted)
			.SelectAllTextOnCommit(true)
			.ContextMenuExtender(InArgs._ContextMenuExtender)
			.Justification(InArgs._Justification)
			.MinDesiredWidth(InArgs._MinDesiredValueWidth);

		TSharedRef<SOverlay> Overlay = SNew(SOverlay);

		// Add the spin box if we have one
		if( bAllowSpin )
		{
			Overlay->AddSlot()
				.HAlign(HAlign_Fill) 
				.VAlign(VAlign_Center) 
				[
					SpinBox.ToSharedRef()
				];
		}


		TAttribute<FMargin> TextMargin = InArgs._OverrideTextMargin.IsSet() ? InArgs._OverrideTextMargin : InArgs._EditableTextBoxStyle->Padding;

		Overlay->AddSlot()
			.HAlign(HAlign_Fill) 
			.VAlign(VAlign_Center)
			.Padding(TextMargin)
			[
				EditableText.ToSharedRef()
			];

		TSharedPtr<SWidget> MainContents;

		const bool bHasLabel = InArgs._Label.Widget != SNullWidget::NullWidget;

		bool bHasInsideLabel = false;
		if (bHasLabel && InArgs._LabelLocation == ELabelLocation::Inside)
		{
			bHasInsideLabel = true;

			Overlay->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(InArgs._LabelVAlign)
				.Padding(InArgs._LabelPadding)
				[
					InArgs._Label.Widget
				];
		}


		if (bAllowSpin && !bHasInsideLabel)
		{
			MainContents = Overlay;
		}
		else 
		{
			MainContents =
				SNew(SBorder)
				.BorderImage(this, &SNumericEntryBox<NumericType>::GetBorderImage)
				.BorderBackgroundColor(InArgs._BorderBackgroundColor)
				.ForegroundColor(InArgs._BorderForegroundColor)
				.Padding(0.f)
				[
					Overlay
				];
		}

		if(!bHasLabel || bHasInsideLabel)
		{
			if(bDisplayToggle)
			{
				ChildSlot
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Fill) 
					.VAlign(VAlign_Center) 
					.Padding(InArgs._TogglePadding)
					[
						ToggleCheckBox.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					[
						MainContents.ToSharedRef()
					]
				];
			}
			else
			{
				ChildSlot
				[
					MainContents.ToSharedRef()
				];
			}
		}
		else
		{
			TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

			HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(InArgs._LabelVAlign)
			.Padding(InArgs._LabelPadding)
			[
				InArgs._Label.Widget
			];

			if(bDisplayToggle)
			{
				HorizontalBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(InArgs._TogglePadding)
				[
					ToggleCheckBox.ToSharedRef()
				];
			}
			
			HorizontalBox->AddSlot()
			[
				MainContents.ToSharedRef()
			];

			ChildSlot
			[
				HorizontalBox
			];
		}

		if (bDisplayToggle)
		{
			HandleToggleCheckBoxChanged(InArgs._ToggleChecked.Get(), FOnCheckStateChanged());
		}
	}

	static TSharedRef<SWidget> BuildLabel(TAttribute<FText> LabelText, const FSlateColor& ForegroundColor, const FSlateColor& BackgroundColor)
	{
		return
			SNew(SBorder)
			.Visibility(EVisibility::HitTestInvisible)
			.BorderImage(FCoreStyle::Get().GetBrush("NumericEntrySpinBox.Decorator"))
			.BorderBackgroundColor(BackgroundColor)
			.ForegroundColor(ForegroundColor)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(1.f, 0.f, 6.f, 0.f))
			[
				SNew(STextBlock)
				.Text(LabelText)
			];
	}


	static TSharedRef<SWidget> BuildNarrowColorLabel(FLinearColor LabelColor)
	{
		return
			SNew(SBorder)
			.Visibility(EVisibility::HitTestInvisible)
			.BorderImage(FAppStyle::Get().GetBrush("NumericEntrySpinBox.NarrowDecorator"))
			.BorderBackgroundColor(LabelColor)
			.HAlign(HAlign_Left)
			.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f));
	}


	/** Return the internally created SpinBox if bAllowSpin is true */
	TSharedPtr<SWidget> GetSpinBox() const { return SpinBox; }

	/** See the MinFractionalDigits attribute */
	int32 GetMinFractionalDigits() const { return Interface->GetMinFractionalDigits(); }
	void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& InMinFractionalDigits)
	{
		Interface->SetMinFractionalDigits((InMinFractionalDigits.Get().IsSet()) ? InMinFractionalDigits.Get() : MinFractionalDigits);
		bCachedValueStringDirty = true;
	}

	/** See the MaxFractionalDigits attribute */
	int32 GetMaxFractionalDigits() const { return Interface->GetMaxFractionalDigits(); }
	void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& InMaxFractionalDigits)
	{
		Interface->SetMaxFractionalDigits((InMaxFractionalDigits.Get().IsSet()) ? InMaxFractionalDigits.Get() : MaxFractionalDigits);
		bCachedValueStringDirty = true;
	}

private:

	//~ SWidget Interface

	virtual bool SupportsKeyboardFocus() const override
	{
		return StaticCastSharedPtr<SWidget>(EditableText)->SupportsKeyboardFocus();
	}

	virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override
	{
		FReply Reply = FReply::Handled();

		// The widget to forward focus to changes depending on whether we have a SpinBox or not.
		TSharedPtr<SWidget> FocusWidget;
		if (SpinBox.IsValid() && SpinBox->GetVisibility() == EVisibility::Visible) 
		{
			FocusWidget = SpinBox;
		}
		else
		{
			FocusWidget = EditableText;
		}

		if ( InFocusEvent.GetCause() != EFocusCause::Cleared )
		{
			// Forward keyboard focus to our chosen widget
			Reply.SetUserFocus(FocusWidget.ToSharedRef(), InFocusEvent.GetCause());
		}

		return Reply;
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		FKey Key = InKeyEvent.GetKey();

		if( Key == EKeys::Escape && EditableText->HasKeyboardFocus() )
		{
			return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Cleared);
		}

		return FReply::Unhandled();
	}

private:

	/**
	 * @return the Label that should be displayed                   
	 */
	FString GetLabel() const
	{
		// Should always be set if this is being called
		return LabelAttribute.Get().GetValue();
	}

	/**
	 * Called to get the value for the spin box                   
	 */
	NumericType OnGetValueForSpinBox() const
	{
		const auto& Value = ValueAttribute.Get();

		// Get the value or 0 if its not set
		if( Value.IsSet() == true )
		{
			return Value.GetValue();
		}

		return 0;
	}

	void SetCachedString(const NumericType CurrentValue)
	{
		if (!CachedExternalValue.IsSet() || CachedExternalValue.GetValue() != CurrentValue || bCachedValueStringDirty)
		{
			CachedExternalValue = CurrentValue;
			CachedValueString = Interface->ToString(CurrentValue);
			bCachedValueStringDirty = false;
		}
	}

	FString GetCachedString(const NumericType CurrentValue) const
	{
		const bool bUseCachedString = CachedExternalValue.IsSet() && CurrentValue == CachedExternalValue.GetValue() && !bCachedValueStringDirty;
		return bUseCachedString ? CachedValueString : Interface->ToString(CurrentValue);  
	}

	FText GetToolTipText() const
	{
		const TOptional<NumericType>& Value = ValueAttribute.Get();

#if WITH_EDITOR
		if constexpr (std::is_floating_point_v<NumericType>)
		{
			if (Value.IsSet() && CachedValueString.Contains(TEXT("...")))
			{
				NumericType NumericValue = Value.GetValue();
				FString ValueScientificStr = FString::Printf(TEXT("%e"), NumericValue);

				// Dev Note: If you are seeing this message when not expected it can mean the following:
				// 1. Simple content / value change is needed assuming the ~1e-16 off value is not desired.
				// 2. If the above doesn't fix the issue, then there probably is an editor customization / some data writer that needs fixing
				// 3. If the value is a true denorm, smaller than 1.175e-38 / 2.225e-307 for floats / doubles respectively. There is a larger issue.
				//    In this case, it is possible that you may be reading from garbage / random memory not intended for floating point types.

				return FText::Format(NSLOCTEXT("SNumericEntryBox", "NearZeroWarning_ToolTip", "Exact value: {0}. Set value or reimport after updating source content to change."), FText::FromString(ValueScientificStr));
			}
		}
#endif // WITH_EDITOR

		if (Value.IsSet() == true)
		{
			NumericType CurrentValue = Value.GetValue();

			FText CurrentValueText = FText::FromString(GetCachedString(CurrentValue));

			if (ToolTipTextFormat.Get().IsSet())
			{
				return FText::Format(ToolTipTextFormat.Get().GetValue(), CurrentValueText);
			}

			return CurrentValueText;
		}

		return FText::GetEmpty();
	}

	/**
	 * Called to get the value for the text box as FText                 
	 */
	FText OnGetValueForTextBox() const
	{
		if( EditableText->GetVisibility() == EVisibility::Visible )
		{
			const TOptional<NumericType>& Value = ValueAttribute.Get();
			if (Value.IsSet() == true)
			{
				return FText::FromString(GetCachedString(Value.GetValue()));
			}
			else
			{
				return UndeterminedString;
			}
		}

		// The box isnt visible, just return an empty Text
		return  FText::GetEmpty();
	}


	/**
	 * Called when the text changes in the text box                   
	 */
	void OnTextChanged( const FText& NewValue )
	{
		const auto& Value = ValueAttribute.Get();

		if (Value.IsSet() || !OnUndeterminedValueChanged.IsBound())
		{
			SendChangesFromText( NewValue, false, ETextCommit::Default );
		}
		else
		{
			OnUndeterminedValueChanged.Execute(NewValue);
		}
	}

	/**
	 * Called when the text is committed from the text box                   
	 */
	void OnTextCommitted( const FText& NewValue, ETextCommit::Type CommitInfo )
	{
		const auto& Value = ValueAttribute.Get();

		if (Value.IsSet() || !OnUndeterminedValueCommitted.IsBound())
		{
			SendChangesFromText( NewValue, true, CommitInfo );
		}
		else
		{
			OnUndeterminedValueCommitted.Execute(NewValue, CommitInfo);
		}
	}

	/**
	 * Called to get the border image of the box                   
	 */
	const FSlateBrush* GetBorderImage() const
	{
		TSharedPtr<const SWidget> EditingWidget;

		if (SpinBox.IsValid() && SpinBox->GetVisibility() == EVisibility::Visible) 
		{
			EditingWidget = SpinBox;
		}
		else
		{
			EditingWidget = EditableText;
		}

		if ( EditingWidget->HasKeyboardFocus() )
		{
			return BorderImageFocused;
		}

		if ( EditingWidget->IsHovered() )
		{
			return BorderImageHovered;
		}

		return BorderImageNormal;
	}

	/**
	 * Calls the value commit or changed delegate set for this box when the value is set from a string
	 *
	 * @param NewValue	The new value as a string
	 * @param bCommit	Whether or not to call the commit or changed delegate
	 */
	void SendChangesFromText( const FText& NewValue, bool bCommit, ETextCommit::Type CommitInfo )
	{
		if (NewValue.IsEmpty())
		{
			return;
		}

		TOptional<NumericType> ExistingValue = ValueAttribute.Get();
		TOptional<NumericType> NumericValue = Interface->FromString(NewValue.ToString(), ExistingValue.Get(0));

		if (NumericValue.IsSet())
		{
			SetCachedString(NumericValue.GetValue());

			if (bCommit)
			{
				OnValueCommitted.ExecuteIfBound(NumericValue.GetValue(), CommitInfo);
			}
			else
			{
				OnValueChanged.ExecuteIfBound(NumericValue.GetValue());
			}
		}
	}

	/**
	 * Caches the value and performs widget visibility maintenance
	 */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		// Update the cached value, if needed.
#if WITH_EDITOR
		if constexpr (std::is_floating_point_v<NumericType>)
		{
			if (!GetPersistentState().IsProjectContent() && !Interface->GetIndicateNearlyInteger())
			{
				Interface->SetIndicateNearlyInteger(true);
				bCachedValueStringDirty = true;

				if (SpinBox.IsValid())
				{
					SpinBox->ResetCachedValueString();
				}
			}
		}
#endif // WITH_EDITOR

		const TOptional<NumericType>& Value = ValueAttribute.Get();
		if (Value.IsSet() == true)
		{
			SetCachedString(Value.GetValue());
		}
		
		// Visibility toggle only matters if the spinbox is used
		if (SpinBox.IsValid())
		{
			if (Value.IsSet() == true)
			{
				if (SpinBox->GetVisibility() != EVisibility::Visible)
				{
					// Set the visibility of the spinbox to visible if we have a valid value
					SpinBox->SetVisibility( EVisibility::Visible );
					// The text box should be invisible
					EditableText->SetVisibility( EVisibility::Collapsed );
				}
			}
			else
			{
				// The value isn't set so the spinbox should be hidden and the text box shown
				SpinBox->SetVisibility(EVisibility::Collapsed);
				EditableText->SetVisibility(EVisibility::Visible);
			}
		}
	}

	bool IsToggleEnabled() const
	{
		if(!IsEnabled())
		{
			return false;
		}
		return ToggleCheckBox->IsChecked();
	}
	
	void HandleToggleCheckBoxChanged(ECheckBoxState InCheckState, FOnCheckStateChanged OnToggleChanged) const
	{
		if(SpinBox.IsValid())
		{
			SpinBox->SetEnabled(InCheckState == ECheckBoxState::Checked);
		}
		if(EditableText.IsValid())
		{
			EditableText->SetEnabled(InCheckState == ECheckBoxState::Checked);
		}
		if(OnToggleChanged.IsBound())
		{
			OnToggleChanged.Execute(InCheckState);
		}
	}

	void ResetCachedValueString() 
	{ 
		if (ValueAttribute.Get().IsSet())
		{
			bCachedValueStringDirty = true;
		}
	}

private:

	/** The default minimum fractional digits */
	static const int32 DefaultMinFractionalDigits;

	/** The default maximum fractional digits */
	static const int32 DefaultMaxFractionalDigits;

	/** Attribute for getting the label */
	TAttribute< TOptional<FString > > LabelAttribute;
	/** Attribute for getting the value.  If the value is not set we display the undetermined string */
	TAttribute< TOptional<NumericType> > ValueAttribute;
	/** Toggle checkbox */
	TSharedPtr<SCheckBox> ToggleCheckBox;
	/** Spinbox widget */
	TSharedPtr<SSpinBox<NumericType>> SpinBox;
	/** Editable widget */
	TSharedPtr<SEditableText> EditableText;
	/** Delegate to call when the value changes */
	FOnValueChanged OnValueChanged;
	/** Delegate to call when the value is committed */
	FOnValueCommitted OnValueCommitted;
	/** Delegate to call when an undetermined value changes */
	FOnUndeterminedValueChanged OnUndeterminedValueChanged;
	/** Delegate to call when an undetermined is committed */
	FOnUndeterminedValueCommitted OnUndeterminedValueCommitted;
	/** The undetermined string to display when needed */
	FText UndeterminedString;
	/** Styling: border image to draw when not hovered or focused */
	const FSlateBrush* BorderImageNormal;
	/** Styling: border image to draw when hovered */
	const FSlateBrush* BorderImageHovered;
	/** Styling: border image to draw when focused */
	const FSlateBrush* BorderImageFocused;
	/** Prevents the value portion of the control from being smaller than desired in certain cases. */
	TAttribute<float> MinDesiredValueWidth;
	/** Type interface that defines how we should deal with the templated numeric type. Always valid after construction. */
	TSharedPtr< INumericTypeInterface<NumericType> > Interface;
	/** Cached value of entry box, updated on set & per tick */
	TOptional<NumericType> CachedExternalValue;
	/** Used to prevent per-frame re-conversion of the cached numeric value to a string. */
	FString CachedValueString;
	/** Whetever the interfaced setting changed and the CachedValueString needs to be recomputed. */
	bool bCachedValueStringDirty;
	TAttribute< TOptional<int32> > MinFractionalDigits;
	TAttribute< TOptional<int32> > MaxFractionalDigits;
	TAttribute< TOptional<FTextFormat> > ToolTipTextFormat;
};


template <typename NumericType>
const FLinearColor SNumericEntryBox<NumericType>::RedLabelBackgroundColor(0.594f,0.0197f,0.0f);

template <typename NumericType>
const FLinearColor SNumericEntryBox<NumericType>::GreenLabelBackgroundColor(0.1349f,0.3959f,0.0f);

template <typename NumericType>
const FLinearColor SNumericEntryBox<NumericType>::BlueLabelBackgroundColor(0.0251f,0.207f,0.85f);

template <typename NumericType>
const FLinearColor SNumericEntryBox<NumericType>::LilacLabelBackgroundColor(0.8f,0.121f,0.8f);

template <typename NumericType>
const FText SNumericEntryBox<NumericType>::DefaultUndeterminedString = FText::FromString(TEXT("---"));

template<typename NumericType>
const int32 SNumericEntryBox<NumericType>::DefaultMinFractionalDigits = 1;

template<typename NumericType>
const int32 SNumericEntryBox<NumericType>::DefaultMaxFractionalDigits = 6;

