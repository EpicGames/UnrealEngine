// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SHyperlink.h"

class FWidgetViewModel;
enum class ETextShapingMethod : uint8;

#if WITH_FANCY_TEXT

class SRichTextHyperlink : public SHyperlink
{
public:

	SLATE_BEGIN_ARGS(SRichTextHyperlink)
		: _Text()
		, _Style(&FCoreStyle::Get().GetWidgetStyle< FHyperlinkStyle >("Hyperlink"))
		, _OnNavigate()
		, _TextShapingMethod()
		, _TextFlowDirection()
	{}
		SLATE_ATTRIBUTE( FText, Text )
		SLATE_STYLE_ARGUMENT( FHyperlinkStyle, Style )
		SLATE_EVENT( FSimpleDelegate, OnNavigate )
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )
	SLATE_END_ARGS()

public:

	void Construct( const FArguments& InArgs, const TSharedRef< FSlateHyperlinkRun::FWidgetViewModel >& InViewModel )
	{
		ViewModel = InViewModel;

		SHyperlink::Construct(
			SHyperlink::FArguments()
			.Text( InArgs._Text )
			.Style( InArgs._Style )
			.Padding( FMargin(0))
			.OnNavigate( InArgs._OnNavigate )
			.TextShapingMethod( InArgs._TextShapingMethod )
			.TextFlowDirection( InArgs._TextFlowDirection )
		);
	}

	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		SHyperlink::OnMouseEnter( MyGeometry, MouseEvent );
		ViewModel->SetIsHovered( IsHovered() );
	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		SHyperlink::OnMouseLeave( MouseEvent );
		ViewModel->SetIsHovered( IsHovered() );
	}

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		FReply Reply = SHyperlink::OnMouseButtonDown( MyGeometry, MouseEvent );
		ViewModel->SetIsPressed( IsPressed() );

		return Reply;
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		FReply Reply = SHyperlink::OnMouseButtonUp( MyGeometry, MouseEvent );
		ViewModel->SetIsPressed( IsPressed() );

		return Reply;
	}

private:

	TSharedPtr< FSlateHyperlinkRun::FWidgetViewModel > ViewModel;
};


#endif //WITH_FANCY_TEXT
