// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Misc/AxisDisplayInfo.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SNumericEntryBox.h"

class FArrangedChildren;

/**
 * FRotator Slate control
 */
template<typename NumericType>
class SNumericRotatorInputBox : public SCompoundWidget
{
	using ThisClass = SNumericRotatorInputBox<NumericType>;
	static constexpr int32 ComponentNum = 3;
public:
	/** Notification for float value change */
	DECLARE_DELEGATE_OneParam(FOnNumericValueChanged, NumericType);

	/** Notification for float value committed */
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);
	
	SLATE_BEGIN_ARGS( SNumericRotatorInputBox<NumericType> )
		: _RollDisplayName(AxisDisplayInfo::GetRotationAxisNameShort(EAxisList::X))
		, _PitchDisplayName(AxisDisplayInfo::GetRotationAxisNameShort(EAxisList::Y))
		, _YawDisplayName(AxisDisplayInfo::GetRotationAxisNameShort(EAxisList::Z))
		, _bColorAxisLabels(false)
		, _Swizzle(FIntVector4(0, 1, 2, 3))
		, _Font(FAppStyle::Get().GetFontStyle("NormalFont"))
		, _AllowSpin(true)
		, _SpinDelta(1.f)
		, _LinearDeltaSensitivity(1)
		, _MinSliderValue(TOptional<NumericType>())
		, _MaxSliderValue(TOptional<NumericType>())
		, _DisplayToggle( false )
		, _TogglePitchChecked( ECheckBoxState::Checked )
		, _ToggleYawChecked( ECheckBoxState::Checked )
		, _ToggleRollChecked( ECheckBoxState::Checked )
		, _TogglePadding( FMargin( 1.f,0.f,1.f,0.f ) )
		, _PreventThrottling(false)
		{}

		/** Roll component of the rotator */
		SLATE_ATTRIBUTE( TOptional<NumericType>, Roll )

		/** Pitch component of the rotator */
		SLATE_ATTRIBUTE( TOptional<NumericType>, Pitch )

		/** Yaw component of the rotator */
		SLATE_ATTRIBUTE( TOptional<NumericType>, Yaw )

		SLATE_ATTRIBUTE( FText, RollDisplayName )
		
		SLATE_ATTRIBUTE( FText, PitchDisplayName )
		
		SLATE_ATTRIBUTE( FText, YawDisplayName )

		/** Should the axis labels be colored */
		SLATE_ARGUMENT( bool, bColorAxisLabels )

		/** Swizzles the display of 0: Roll, 1: Pitch, 2: Yaw
		 * 4th component unused
		 */
		SLATE_ARGUMENT( FIntVector4, Swizzle)

		/** Font to use for the text in this box */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Whether or not values can be spun or if they should be typed in */
		SLATE_ARGUMENT( bool, AllowSpin )

		/** The delta amount to apply, per pixel, when the spinner is dragged. */
		SLATE_ATTRIBUTE(NumericType, SpinDelta)

		/** If we're an unbounded spinbox, what value do we divide mouse movement by before multiplying by Delta. Requires Delta to be set. */
		SLATE_ATTRIBUTE(int32, LinearDeltaSensitivity)

		/** The minimum value that can be specified by using the slider */
		SLATE_ATTRIBUTE( TOptional<NumericType>, MinSliderValue )

		/** The maximum value that can be specified by using the slider */
		SLATE_ATTRIBUTE( TOptional<NumericType>, MaxSliderValue )

		/** Called when the pitch value is changed */
		SLATE_EVENT( FOnNumericValueChanged, OnPitchChanged )

		/** Called when the yaw value is changed */
		SLATE_EVENT( FOnNumericValueChanged, OnYawChanged )

		/** Called when the roll value is changed */
		SLATE_EVENT( FOnNumericValueChanged, OnRollChanged )

		/** Called when the pitch value is committed */
		SLATE_EVENT( FOnNumericValueCommitted, OnPitchCommitted )

		/** Called when the yaw value is committed */
		SLATE_EVENT( FOnNumericValueCommitted, OnYawCommitted )

		/** Called when the roll value is committed */
		SLATE_EVENT( FOnNumericValueCommitted, OnRollCommitted )

		/** Called when the slider begins to move on any axis */
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)

		/** Called when the slider for any axis is released */
		SLATE_EVENT(FOnNumericValueChanged, OnEndSliderMovement)
	
		/** Called when the slider begins to move on pitch */
		SLATE_EVENT( FSimpleDelegate, OnPitchBeginSliderMovement )

		/** Called when the slider begins to move on yaw */
		SLATE_EVENT( FSimpleDelegate, OnYawBeginSliderMovement )

		/** Called when the slider begins to move on roll */
		SLATE_EVENT( FSimpleDelegate, OnRollBeginSliderMovement )

		/** Called when the slider for pitch is released */
		SLATE_EVENT( FOnNumericValueChanged, OnPitchEndSliderMovement )

		/** Called when the slider for yaw is released */
		SLATE_EVENT( FOnNumericValueChanged, OnYawEndSliderMovement )

		/** Called when the slider for roll is released */
		SLATE_EVENT( FOnNumericValueChanged, OnRollEndSliderMovement )

		/** Provide custom type functionality for the rotator */
		SLATE_ARGUMENT( TSharedPtr< INumericTypeInterface<NumericType> >, TypeInterface )

		/** Whether or not to include a toggle checkbox to the left of the widget */
		SLATE_ARGUMENT( bool, DisplayToggle )
			
		/** The value of the toggle Pitch checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, TogglePitchChecked )

		/** The value of the toggle Yaw checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleYawChecked )

		/** The value of the toggle Roll checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleRollChecked )

		/** Called whenever the toggle Pitch changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnTogglePitchChanged )

		/** Called whenever the toggle Yaw changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleYawChanged )

		/** Called whenever the toggle Roll changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleRollChanged )

		/** Padding around the toggle checkbox */
		SLATE_ARGUMENT( FMargin, TogglePadding )
		
		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)

	SLATE_END_ARGS()

private:

	TAttribute<TOptional<FTextFormat>> BuildToolTipTextFormatString(FStringView DisplayNameString)
	{
		TStringBuilder<32> ToolTipTextFormatString;
		ToolTipTextFormatString.Append(DisplayNameString);
		ToolTipTextFormatString.Append(TEXT(": {0}"));
		TAttribute<TOptional<FTextFormat>> Attribute(FText::FromString(ToolTipTextFormatString.ToString()));
		return Attribute;
	}

	TSharedRef<SWidget> ConstructRoll(const FArguments& InArgs)
	{
		TSharedRef<SWidget> Widget =
			SNew(SNumericEntryBox<NumericType>)
			.AllowSpin(InArgs._AllowSpin)
			.Delta(InArgs._SpinDelta)
			.LinearDeltaSensitivity(InArgs._LinearDeltaSensitivity)
			.MinValue(InArgs._MinSliderValue)
			.MaxValue(InArgs._MaxSliderValue)
			.MinSliderValue(InArgs._MinSliderValue)
			.MaxSliderValue(InArgs._MaxSliderValue)
			.LabelPadding(FMargin(3.0f))
			.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
			.Label()
			[
				InArgs._bColorAxisLabels ? SNumericEntryBox<NumericType>::BuildNarrowColorLabel(AxisDisplayInfo::GetAxisColor(EAxisList::X)) : SNullWidget::NullWidget
			]
			.Font( InArgs._Font )
			.Value( InArgs._Roll )
			.OnValueChanged( InArgs._OnRollChanged )
			.OnValueCommitted( InArgs._OnRollCommitted )
			.OnBeginSliderMovement( CreatePerComponentSliderMovementEvent(InArgs._OnBeginSliderMovement, InArgs._OnRollBeginSliderMovement) )
			.OnEndSliderMovement( CreatePerComponentSliderMovementEvent< FOnNumericValueChanged, NumericType >(InArgs._OnEndSliderMovement, InArgs._OnRollEndSliderMovement) )
			.UndeterminedString( NSLOCTEXT("SRotatorInputBox", "MultipleValues", "Multiple Values") )
			.ToolTipTextFormat(BuildToolTipTextFormatString(InArgs._RollDisplayName.Get().ToString()))
			.TypeInterface(InArgs._TypeInterface)
			.MinFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMinFractionalDigits() : TOptional<int32>())
			.MaxFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMaxFractionalDigits() : TOptional<int32>())
			.DisplayToggle(InArgs._DisplayToggle)
			.ToggleChecked(InArgs._ToggleRollChecked)
			.OnToggleChanged(InArgs._OnToggleRollChanged)
			.TogglePadding(InArgs._TogglePadding)
			.PreventThrottling(InArgs._PreventThrottling);
		return Widget;
	};
	
	TSharedRef<SWidget> ConstructPitch(const FArguments& InArgs)
	{
		TSharedRef<SWidget> Widget =
			SNew(SNumericEntryBox<NumericType>)
			.AllowSpin(InArgs._AllowSpin)
			.Delta(InArgs._SpinDelta)
			.LinearDeltaSensitivity(InArgs._LinearDeltaSensitivity)
			.MinValue(InArgs._MinSliderValue)
			.MaxValue(InArgs._MaxSliderValue)
			.MinSliderValue(InArgs._MinSliderValue)
			.MaxSliderValue(InArgs._MaxSliderValue)
			.LabelPadding(FMargin(3.0f))
			.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
			.Label()
			[
				InArgs._bColorAxisLabels ? SNumericEntryBox<NumericType>::BuildNarrowColorLabel(AxisDisplayInfo::GetAxisColor(EAxisList::Y)) : SNullWidget::NullWidget
			]
			.Font( InArgs._Font )
			.Value( InArgs._Pitch )
			.OnValueChanged( InArgs._OnPitchChanged )
			.OnValueCommitted( InArgs._OnPitchCommitted )
			.OnBeginSliderMovement( CreatePerComponentSliderMovementEvent( InArgs._OnBeginSliderMovement, InArgs._OnPitchBeginSliderMovement ) )
			.OnEndSliderMovement( CreatePerComponentSliderMovementEvent< FOnNumericValueChanged, NumericType >( InArgs._OnEndSliderMovement, InArgs._OnPitchEndSliderMovement ) )
			.UndeterminedString( NSLOCTEXT("SRotatorInputBox", "MultipleValues", "Multiple Values") )
			.ToolTipTextFormat(BuildToolTipTextFormatString(InArgs._PitchDisplayName.Get().ToString()))
			.TypeInterface(InArgs._TypeInterface)
			.MinFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMinFractionalDigits() : TOptional<int32>())
			.MaxFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMaxFractionalDigits() : TOptional<int32>())
			.DisplayToggle(InArgs._DisplayToggle)
			.ToggleChecked(InArgs._TogglePitchChecked)
			.OnToggleChanged(InArgs._OnTogglePitchChanged)
			.TogglePadding(InArgs._TogglePadding)
			.PreventThrottling(InArgs._PreventThrottling);
		return Widget;
	};

	TSharedRef<SWidget> ConstructYaw(const FArguments& InArgs)
	{
		TSharedRef<SWidget> Widget =
			SNew(SNumericEntryBox<NumericType>)
			.AllowSpin(InArgs._AllowSpin)
			.Delta(InArgs._SpinDelta)
			.LinearDeltaSensitivity(InArgs._LinearDeltaSensitivity)
			.MinValue(InArgs._MinSliderValue)
			.MaxValue(InArgs._MaxSliderValue)
			.MinSliderValue(InArgs._MinSliderValue)
			.MaxSliderValue(InArgs._MaxSliderValue)
			.LabelPadding(FMargin(3.0f))
			.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
			.Label()
			[
				InArgs._bColorAxisLabels ? SNumericEntryBox<NumericType>::BuildNarrowColorLabel(AxisDisplayInfo::GetAxisColor(EAxisList::Z)) : SNullWidget::NullWidget
			]
			.Font( InArgs._Font )
			.Value( InArgs._Yaw )
			.OnValueChanged( InArgs._OnYawChanged )
			.OnValueCommitted( InArgs._OnYawCommitted )
			.OnBeginSliderMovement( CreatePerComponentSliderMovementEvent(InArgs._OnBeginSliderMovement, InArgs._OnYawBeginSliderMovement) )
			.OnEndSliderMovement( CreatePerComponentSliderMovementEvent< FOnNumericValueChanged, NumericType >(InArgs._OnEndSliderMovement, InArgs._OnYawEndSliderMovement) )
			.UndeterminedString( NSLOCTEXT("SRotatorInputBox", "MultipleValues", "Multiple Values") )
			.ToolTipTextFormat(BuildToolTipTextFormatString(InArgs._YawDisplayName.Get().ToString()))
			.TypeInterface(InArgs._TypeInterface)
			.MinFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMinFractionalDigits() : TOptional<int32>())
			.MaxFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMaxFractionalDigits() : TOptional<int32>())
			.DisplayToggle(InArgs._DisplayToggle)
			.ToggleChecked(InArgs._ToggleYawChecked)
			.OnToggleChanged(InArgs._OnToggleYawChanged)
			.TogglePadding(InArgs._TogglePadding)
			.PreventThrottling(InArgs._PreventThrottling);
		return Widget;
	};
	
public:

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs )
	{
		using WidgetConstructorFn = TSharedRef<SWidget> (ThisClass::*)(const FArguments&);
		constexpr TStaticArray<WidgetConstructorFn, ComponentNum> Constructors =
		{
			&ThisClass::ConstructRoll,
			&ThisClass::ConstructPitch,
			&ThisClass::ConstructYaw
		};

		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		for (int32 SwizzleIndex = 0; SwizzleIndex < ComponentNum; SwizzleIndex++)
		{
			TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
			const auto& Swizzle = InArgs._Swizzle;
			int32 SwizzleValue = Swizzle[SwizzleIndex];
			if (ensureMsgf(SwizzleValue < 3 && SwizzleValue >= 0, TEXT("Invalid Swizzle Value")))
			{
				Widget = (this->*Constructors[SwizzleValue])(InArgs);
			}

			TSharedRef<SWidget> WidgetRef = Widget.ToSharedRef();
			
			HorizontalBox->AddSlot()
			[
				WidgetRef
			];
		}

		ChildSlot
		[
			HorizontalBox
		];
	}

	/**
	 * Creates a lambda to react to a begin/end slider movement event
	 */
	template<typename EventType, typename... ArgsType>
	EventType CreatePerComponentSliderMovementEvent(
		const EventType OnSliderMovement,
		const EventType OnComponentSliderMovement)
	{
		if(OnSliderMovement.IsBound())
		{
			return EventType::CreateLambda(
				[OnSliderMovement, OnComponentSliderMovement](ArgsType... Args)
				{
					OnSliderMovement.ExecuteIfBound(Args...);
					OnComponentSliderMovement.ExecuteIfBound(Args...);
				});
		}
		return OnComponentSliderMovement;
	}
};

/**
 * For backward compatibility
 */
using SRotatorInputBox = SNumericRotatorInputBox<float>;
