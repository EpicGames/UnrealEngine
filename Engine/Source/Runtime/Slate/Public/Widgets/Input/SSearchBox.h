// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "Slate"

class FActiveTimerHandle;

/** A text box that is used for searching. Meant to be as easy to use as possible with as few options as possible. */
class SSearchBox : public SEditableTextBox
{

public:
	/** Which direction to go when searching */
	enum SearchDirection
	{
		Previous,
		Next,
	};

	/** Data for external search results to be shown in the search box. */
	struct FSearchResultData
	{
		/** The total number of search results. */
		int32 NumSearchResults;
		/** The index of the currently focused search result. */
		int32 CurrentSearchResultIndex;
	};

	DECLARE_DELEGATE_OneParam(FOnSearch, SSearchBox::SearchDirection);

	SLATE_BEGIN_ARGS(SSearchBox)
		: _Style( &FCoreStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox") )
		, _HintText( LOCTEXT("SearchHint", "Search") )
		, _InitialText()
		, _SearchResultData()
		, _IsSearching(false)
		, _OnTextChanged()
		, _OnTextCommitted()
		, _OnVerifyTextChanged()
		, _OnSearch()
		, _SelectAllTextWhenFocused( true )
		, _DelayChangeNotificationsWhileTyping( true )
		, _DelayChangeNotificationsWhileTypingSeconds(0.25f)
	{ }

		/** Style used to draw this search box */
		SLATE_STYLE_ARGUMENT( FSearchBoxStyle, Style )

		/** The text displayed in the SearchBox when no text has been entered */
		SLATE_ATTRIBUTE( FText, HintText )

		/** The text displayed in the SearchBox when it's created */
		SLATE_ATTRIBUTE( FText, InitialText )

		/** Optional search result data to be shown in the search bar. */
		SLATE_ATTRIBUTE( TOptional<FSearchResultData>, SearchResultData )

		/** Whether or not an external search is currently running. */
		SLATE_ATTRIBUTE( bool, IsSearching )

		/** Invoked whenever the text changes */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Invoked whenever the text is committed (e.g. user presses enter) */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Callback when the text starts to be edited */
		SLATE_EVENT( FOnBeginTextEdit, OnBeginTextEdit )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnVerifyTextChanged, OnVerifyTextChanged )
	
		/** This will add a next and previous button to your search box */
		SLATE_EVENT( FOnSearch, OnSearch )

		/** Whether to select all text when the user clicks to give focus on the widget */
		SLATE_ATTRIBUTE( bool, SelectAllTextWhenFocused )
		
		/** Minimum width that a text block should be */
		SLATE_ATTRIBUTE( float, MinDesiredWidth )

		/** Whether the SearchBox should delay notifying listeners of text changed events until the user is done typing */
		SLATE_ATTRIBUTE( bool, DelayChangeNotificationsWhileTyping )

		/** If we're delaying change notifications how many seconds should we wait? */
		SLATE_ATTRIBUTE( float, DelayChangeNotificationsWhileTypingSeconds )

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

		/** Optional callback to allow for special functionality to be invoked when the search glass button is clicked */
		SLATE_EVENT(FOnClicked, OnSearchGlassClicked)
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	/** Get the current selection */
	SLATE_API FTextSelection GetSelection() const;

	/** Select a block of text */
	SLATE_API void SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation);

private:

	/** Callback for changes in the editable text box. */
	void HandleTextChanged(const FText& NewText);

	/** Callback for committing changes in the editable text box. */
	void HandleTextCommitted(const FText& NewText, ETextCommit::Type CommitType);

private:

	/** Fires the text changed delegate on a delay */
	EActiveTimerReturnType TriggerOnTextChanged( double InCurrentTime, float InDeltaTime, FText NewText );

	/** Gets the display text for the current search results. */
	FText GetSearchResultText() const;

	/** @return should we show the search result navigation buttons. */
	EVisibility GetSearchResultNavigationButtonVisibility() const;

	/** @return should we show the X to clear search? */
	EVisibility GetXVisibility() const;

	/** @return should we show the search results text. */
	EVisibility GetSearchResultDataVisibility() const;

	/** @return should we show the is searching throbber. */
	EVisibility GetIsSearchingThrobberVisibility() const;

	/** @return should we show the search glass icon? */
	EVisibility GetSearchGlassVisibility() const;

	FReply OnClickedSearch(SSearchBox::SearchDirection Direction);

	/** Invoked when user clicks the X*/
	FReply OnClearSearch();

	/** Invoked to get the font to use for the editable text box */
	FSlateFontInfo GetWidgetFont() const;

private:
	/** Handle to the active trigger text changed timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Delegate that is invoked when the user does next or previous */
	FOnSearch OnSearchDelegate;

	/** Delegate that is invoked when the text changes */
	FOnTextChanged OnTextChangedDelegate;

	/** Delegate that is invoked when the text is committed */
	FOnTextCommitted OnTextCommittedDelegate;

	/** Whether the SearchBox should delay notifying listeners of text changed events until the user is done typing */
	TAttribute< bool > DelayChangeNotificationsWhileTyping;

	/** If we're delaying change notifications how many seconds should we wait? */
	TAttribute< float > DelayChangeNotificationsWhileTypingSeconds;

	/** Fonts that specify how to render search text when inactive, and active */
	FSlateFontInfo ActiveFont, InactiveFont;

	/** Optional search result data to be shown in the search bar. */
	TAttribute<TOptional<FSearchResultData>> SearchResultData;

	/** Whether or not an external search is currently running. */
	TAttribute<bool> bIsSearching;
};


#undef LOCTEXT_NAMESPACE
