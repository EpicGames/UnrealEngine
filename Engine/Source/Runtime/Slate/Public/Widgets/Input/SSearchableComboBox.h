// Copyright Epic Games, Inc. All Rights Reservekd.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboButton.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Widgets/Input/SEditableTextBox.h"
#endif
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"

class SEditableTextBox;

template <>
struct TWidgetTypeTraits<class SSearchableComboBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A searchable text combo box
 */
class SSearchableComboBox : public SComboButton
{
public:
	/** Type of list used for showing menu options. */
	typedef SListView< TSharedPtr<FString> > SComboListType;
	/** Delegate type used to generate widgets that represent Options */
	typedef typename TSlateDelegates< TSharedPtr<FString> >::FOnGenerateWidget FOnGenerateWidget;
	typedef typename TSlateDelegates< TSharedPtr<FString> >::FOnSelectionChanged FOnSelectionChanged;

	SLATE_BEGIN_ARGS(SSearchableComboBox)
		: _Content()
		, _ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox"))
		, _ButtonStyle(nullptr)
		, _ItemStyle(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
		, _ContentPadding(_ComboBoxStyle->ContentPadding)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _OptionsSource()
		, _OnSelectionChanged()
		, _bAlwaysSelectItem(false)
		, _OnGenerateWidget()
		, _InitiallySelectedItem(nullptr)
		, _Method()
		, _MaxListHeight(450.0f)
		, _HasDownArrow(true)
		, _SearchVisibility()
	{}

	/** Slot for this button's content (optional) */
	SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboBoxStyle)

		/** The visual style of the button part of the combo box (overrides ComboBoxStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemStyle)

		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)

		SLATE_ARGUMENT(const TArray< TSharedPtr<FString> >*, OptionsSource)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_ARGUMENT(bool, bAlwaysSelectItem)
		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)

		/** Called when combo box is opened, before list is actually created */
		SLATE_EVENT(FOnComboBoxOpening, OnComboBoxOpening)

		/** The custom scrollbar to use in the ListView */
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, CustomScrollbar)

		/** The option that should be selected when the combo box is first created */
		SLATE_ARGUMENT(TSharedPtr<FString>, InitiallySelectedItem)

		SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)

		/** The max height of the combo box menu */
		SLATE_ARGUMENT(float, MaxListHeight)

		/**
		 * When false, the down arrow is not generated and it is up to the API consumer
		 * to make their own visual hint that this is a drop down.
		 */
		SLATE_ARGUMENT(bool, HasDownArrow)

		/** Allow setting the visibility of the search box dynamically */
		SLATE_ATTRIBUTE(EVisibility, SearchVisibility)

	SLATE_END_ARGS()

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the combo box
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API void ClearSelection();

	/**
	 * Sets the selected item.  By default, registers as a navigation request, which *does not*
	 * set the cached selected item, only updating the visually selected item.
	 * 
	 * @param InSelectedItem Item to select
	 * @param InSelectInfo (optional) How the selected item is being committed (default: OnNavigation)
	 */
	SLATE_API void SetSelectedItem(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type InSelectInfo = ESelectInfo::OnNavigation);

	/** @return the item currently selected by the combo box. */
	SLATE_API TSharedPtr<FString> GetSelectedItem();

	/**
	 * Requests a list refresh after updating options
	 * Call SetSelectedItem to update the selected item if required
	 * @see SetSelectedItem
	 */
	SLATE_API void RefreshOptions();

protected:
	/** Set ths source data for this combo box */
	SLATE_API void SetOptionsSource(const TArray< TSharedPtr<FString> >* InOptionsSource);

private:

	/** Generate a row for the InItem in the combo box's list (passed in as OwnerTable). Do this by calling the user-specified OnGenerateWidget */
	TSharedRef<ITableRow> GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called if the menu is closed */
	void OnMenuOpenChanged(bool bOpen);

	/** Invoked when the selection in the list changes */
	void OnSelectionChanged_Internal(TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo);

	/** Invoked when the search text changes */
	void OnSearchTextChanged(const FText& ChangedText);

	/** Sets the current selection to the first valid match when user presses enter in the filter box */
	void OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Handle clicking on the content menu */
	SLATE_API virtual FReply OnButtonClicked() override;

	FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** The item style to use. */
	const FTableRowStyle* ItemStyle;

	/** The padding around each menu row */
	FMargin MenuRowPadding;
private:
	/** Delegate that is invoked when the selected item in the combo box changes */
	FOnSelectionChanged OnSelectionChanged;
	/** The item currently selected in the combo box */
	TSharedPtr<FString> SelectedItem;
	/** The search field used for the combox box's contents */
	TSharedPtr<SEditableTextBox> SearchField;
	/** The ListView that we pop up; visualized the available options. */
	TSharedPtr< SComboListType > ComboListView;
	/** The Scrollbar used in the ListView. */
	TSharedPtr< SScrollBar > CustomScrollbar;
	/** Delegate to invoke before the combo box is opening. */
	FOnComboBoxOpening OnComboBoxOpening;
	/** Delegate to invoke when we need to visualize an option as a widget. */
	FOnGenerateWidget OnGenerateWidget;

	/** Updated whenever search text is changed */
	FText SearchText;

	/** Source data for this combo box */
	const TArray< TSharedPtr<FString> >* OptionsSource;

	/** Filtered list that is actually displayed */
	TArray< TSharedPtr<FString> > FilteredOptionsSource;

	/** If true OnSelectionChanged delegate is executed on every input. */
	bool bAlwaysSelectItem : 1;
};


