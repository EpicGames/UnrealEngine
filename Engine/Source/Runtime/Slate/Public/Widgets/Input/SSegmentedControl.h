// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Styling/SegmentedControlStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/Application/SlateApplication.h"

/**
 * A Slate Segmented Control is functionally similar to a group of Radio Buttons.
 * Slots require a templated value to return when the segment is selected by the user.
 * Users can specify text, icon or provide custom content to each Segment.
 *
 * Note: It is currently not possible to add segments after initialization
 *  (i.e. there is no AddSlot).
 */

template< typename OptionType >
class SSegmentedControl : public SCompoundWidget
{

public:

	/** Stores the per-child info for this panel type */
	struct FSlot : public TSlotBase<FSlot>, public TAlignmentWidgetSlotMixin<FSlot>
	{
		FSlot(const OptionType& InValue)
			: TSlotBase<FSlot>()
			, TAlignmentWidgetSlotMixin<FSlot>(HAlign_Center, VAlign_Fill)
			, _Text()
			, _Tooltip()
			, _Icon(nullptr)
			, _Visibility(EVisibility::Visible)
			, _Value(InValue)
			{ }

		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TSlotBase<FSlot>, TAlignmentWidgetSlotMixin<FSlot>)
			SLATE_ATTRIBUTE(FText, Text)
			SLATE_ATTRIBUTE(FText, ToolTip)
			SLATE_ATTRIBUTE(TSharedPtr<IToolTip>, ToolTipWidget)
			SLATE_ATTRIBUTE(const FSlateBrush*, Icon)
			SLATE_ATTRIBUTE(EVisibility, Visibility)
			SLATE_ARGUMENT(TOptional<OptionType>, Value)
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			TAlignmentWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
			if (InArgs._Text.IsSet())
			{
				_Text = MoveTemp(InArgs._Text);
			}
			if (InArgs._ToolTip.IsSet())
			{
				_Tooltip = MoveTemp(InArgs._ToolTip);
			}
			if (InArgs._ToolTipWidget.IsSet())
			{
				_ToolTipWidget = MoveTemp(InArgs._ToolTipWidget);
			}
			if (InArgs._Icon.IsSet())
			{
				_Icon = MoveTemp(InArgs._Icon);
			}
			if (InArgs._Visibility.IsSet())
			{
				_Visibility = MoveTemp(InArgs._Visibility);
			}
			if (InArgs._Value.IsSet())
			{
				_Value = MoveTemp(InArgs._Value.GetValue());
			}
		}

		void SetText(TAttribute<FText> InText)
		{
			_Text = MoveTemp(InText);
		}

		FText GetText() const
		{
			return _Text.Get();
		}

		void SetIcon(TAttribute<const FSlateBrush*> InBrush)
		{
			_Icon = MoveTemp(InBrush);
		}

		const FSlateBrush* GetIcon() const
		{
			return _Icon.Get();
		}

		void SetToolTip(TAttribute<FText> InTooltip)
		{
			_Tooltip = MoveTemp(InTooltip);
		}

		FText GetToolTip() const
		{
			return _Tooltip.Get();
		}

		void SetVisibility(EVisibility InVisibility)
		{
			_Visibility = MoveTemp(InVisibility);
		}

		EVisibility GetVisibility() const
		{
			return _Visibility.Get();
		}

	friend SSegmentedControl<OptionType>;

	private:
		TAttribute<FText> _Text;
		TAttribute<FText> _Tooltip;
		TAttribute<TSharedPtr<IToolTip>> _ToolTipWidget;
		TAttribute<const FSlateBrush*> _Icon;
		TAttribute<EVisibility> _Visibility;

		OptionType _Value;
		TWeakPtr<SCheckBox> _CheckBox;
	};

	static typename FSlot::FSlotArguments Slot(const OptionType& InValue)
	{
		return typename FSlot::FSlotArguments(MakeUnique<FSlot>(InValue));
	}

	DECLARE_DELEGATE_OneParam( FOnValueChanged, OptionType );
	DECLARE_DELEGATE_OneParam( FOnValuesChanged, TArray<OptionType> );
	DECLARE_DELEGATE_TwoParams( FOnValueChecked, OptionType, ECheckBoxState );

	SLATE_BEGIN_ARGS( SSegmentedControl<OptionType> )
		: _Style(&FAppStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("SegmentedControl"))
		, _TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallButtonText"))
		, _SupportsMultiSelection(false)
		, _SupportsEmptySelection(false)
		, _MaxSegmentsPerLine(0)
	{}
		/** Slot type supported by this panel */
		SLATE_SLOT_ARGUMENT(FSlot, Slots)

		/** Styling for this control */
		SLATE_STYLE_ARGUMENT(FSegmentedControlStyle, Style)
	
		/** Styling for the text in each slot. If a custom widget is supplied for a slot this argument is not used */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)

		/**
		 * If enabled the widget will support multi selection.
		 * For single selection the widget relies on the Value attribute,
		 * for multi selection the widget relies on the MultiValue attribute.
		 */
		SLATE_ARGUMENT(bool, SupportsMultiSelection)

		/**
		 * If enabled the widget will support an empty selection.
		 * This is only enabled if SupportsMultiSelection is also enabled.
		 */
		SLATE_ARGUMENT(bool, SupportsEmptySelection)

		/** The current control value. */
		SLATE_ATTRIBUTE(OptionType, Value)

		/** The current (multiple) control values (if SupportsMultiSelection is enabled) */ 
		SLATE_ATTRIBUTE(TArray<OptionType>, Values)
	
		/** Padding to apply to each slot */
		SLATE_ATTRIBUTE(FMargin, UniformPadding)

		/** Called when the (primary) value is changed */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)

		/** Called when the any value is changed */
		SLATE_EVENT(FOnValuesChanged, OnValuesChanged)

		/** Called when the value is changed (useful for multi selection) */
		SLATE_EVENT(FOnValueChecked, OnValueChecked)

		/** Optional maximum number of segments per line before the control wraps vertically to the next line. If this value is <= 0 no wrapping happens */
		SLATE_ARGUMENT(int32, MaxSegmentsPerLine)
	SLATE_END_ARGS()

	SSegmentedControl()
		: Children(this)
		, CurrentValues(*this)
	{}

	void Construct( const FArguments& InArgs )
	{
		check(InArgs._Style);

		Style = InArgs._Style;
		TextStyle = InArgs._TextStyle;

		SupportsMultiSelection = InArgs._SupportsMultiSelection;
		SupportsEmptySelection = InArgs._SupportsEmptySelection;
		CurrentValuesIsBound = false; // will be set by SetValue or SetValues

		if(InArgs._Value.IsBound() || InArgs._Value.IsSet())
		{
			SetValue(InArgs._Value, false);
		}
		else if(InArgs._Values.IsBound() || InArgs._Values.IsSet())
		{
			SetValues(InArgs._Values, false);
		}

		OnValueChanged = InArgs._OnValueChanged;
		OnValuesChanged = InArgs._OnValuesChanged;
		OnValueChecked = InArgs._OnValueChecked;

		UniformPadding = InArgs._UniformPadding;

		MaxSegmentsPerLine = InArgs._MaxSegmentsPerLine;
		Children.AddSlots(MoveTemp(const_cast<TArray<typename FSlot::FSlotArguments>&>(InArgs._Slots)));
		RebuildChildren();
	}

	void RebuildChildren()
	{
		// The right padding will be applied later at the end so we dont accumulate left+right padding between all buttons
		FMargin SlotPadding = Style->UniformPadding;
		SlotPadding.Right = 0.0f;

		TSharedPtr<SUniformGridPanel> UniformBox = SNew(SUniformGridPanel).SlotPadding(SlotPadding);

		const int32 NumSlots = Children.Num();
		for ( int32 SlotIndex = 0; SlotIndex < NumSlots; ++SlotIndex )
		{
			TSharedRef<SWidget> Child = Children[SlotIndex].GetWidget();
			FSlot* ChildSlotPtr = &Children[SlotIndex];
			const OptionType ChildValue = ChildSlotPtr->_Value;

			TAttribute<FVector2D> SpacerLambda = FVector::ZeroVector;
			if (ChildSlotPtr->_Icon.IsBound() || ChildSlotPtr->_Text.IsBound())
			{
				SpacerLambda = MakeAttributeLambda([ChildSlotPtr]() { return (ChildSlotPtr->_Icon.Get() != nullptr && !ChildSlotPtr->_Text.Get().IsEmpty()) ? FVector2D(8.0f, 1.0f) : FVector2D::ZeroVector; });
			}
			else
			{
				SpacerLambda = (ChildSlotPtr->_Icon.Get() != nullptr && !ChildSlotPtr->_Text.Get().IsEmpty()) ? FVector2D(8.0f, 1.0f) : FVector2D::ZeroVector;
			}

			if (Child == SNullWidget::NullWidget)
			{
				Child = SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(ChildSlotPtr->_Icon)
				]	

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpacer)
					.Size(SpacerLambda)
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(TextStyle)
					.Text(ChildSlotPtr->_Text) 
				];
			}

			const FCheckBoxStyle* CheckBoxStyle = &Style->ControlStyle;
			if (SlotIndex == 0)
			{
				CheckBoxStyle = &Style->FirstControlStyle;
			}
			else if (SlotIndex == NumSlots - 1)
			{
				CheckBoxStyle = &Style->LastControlStyle;
			}

			const int32 ColumnIndex = MaxSegmentsPerLine > 0 ? SlotIndex % MaxSegmentsPerLine : SlotIndex;
			const int32 RowIndex = MaxSegmentsPerLine > 0 ? SlotIndex / MaxSegmentsPerLine : 0;

			// Note HAlignment is applied at the check box level because if it were applied here it would make the slots look physically disconnected from each other 
			UniformBox->AddSlot(ColumnIndex, RowIndex)
			.VAlign(ChildSlotPtr->GetVerticalAlignment())
			[
				SAssignNew(ChildSlotPtr->_CheckBox, SCheckBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				.HAlign(ChildSlotPtr->GetHorizontalAlignment())
				.ToolTipText(ChildSlotPtr->_Tooltip)
				.ToolTip(ChildSlotPtr->_ToolTipWidget)
				.Style(CheckBoxStyle)
				.IsChecked(GetCheckBoxStateAttribute(ChildValue))
				.OnCheckStateChanged(this, &SSegmentedControl::CommitValue, ChildValue)
				.Padding(UniformPadding)
				.Visibility(ChildSlotPtr->_Visibility)
				[
					Child
				]
			];
		}

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(&Style->BackgroundBrush)
			.Padding(FMargin(0,0,Style->UniformPadding.Right,0))
			[
				UniformBox.ToSharedRef()
			]
		];

		UpdateCheckboxValuesIfNeeded();
	}

	// Slot Management
	using FScopedWidgetSlotArguments = typename TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	FScopedWidgetSlotArguments AddSlot(const OptionType& InValue, bool bRebuildChildren = true)
	{
		if (bRebuildChildren)
		{
			TWeakPtr<SSegmentedControl> AsWeak = SharedThis(this);
			return FScopedWidgetSlotArguments { MakeUnique<FSlot>(InValue), this->Children, INDEX_NONE, [AsWeak](const FSlot*, int32)
				{
					if (TSharedPtr<SSegmentedControl> SharedThis = AsWeak.Pin())
					{
						SharedThis->RebuildChildren();
					}
				}};
		}
		else
		{
			return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(InValue), this->Children, INDEX_NONE };
		}
	}

	int32 NumSlots() const
	{
		return Children.Num();
	}

	OptionType GetValue() const
	{
		const TArray<OptionType> Values = GetValues();
		if(Values.IsEmpty())
		{
			return OptionType();
		}
		return Values[0];
	}

	TArray<OptionType> GetValues() const
	{
		return CurrentValues.Get();
	}

	bool HasValue(OptionType InValue)
	{
		const TArray<OptionType> Values = GetValues();
		return Values.Contains(InValue);
	}

	/** See the Value attribute */
	void SetValue(TAttribute<OptionType> InValue, bool bUpdateChildren = true)
	{
		if(InValue.IsBound())
		{
			SetValues(TAttribute<TArray<OptionType>>::CreateLambda([InValue]() -> TArray<OptionType>
			{
				TArray<OptionType> Values;
				Values.Add(InValue.Get());
				return Values;
			}), bUpdateChildren); 
		}
		else if(InValue.IsSet())
		{
			TArray<OptionType> Values = {InValue.Get()};
			SetValues(TAttribute<TArray<OptionType>>(Values), bUpdateChildren); 
		}
		else
		{
			SetValues(TAttribute<TArray<OptionType>>(), bUpdateChildren);
		}
	}

	/** See the Values attribute */
	void SetValues(TAttribute<TArray<OptionType>> InValues, bool bUpdateChildren = true) 
	{
		CurrentValuesIsBound = InValues.IsBound();

		if(CurrentValuesIsBound)
		{
			CurrentValues.Assign(*this, InValues);
		}
		else if(InValues.IsSet())
		{
			CurrentValues.Set(*this, InValues.Get());
		}
		else
		{
			CurrentValues.Unbind(*this);
		};

		if(bUpdateChildren)
		{
			UpdateCheckboxValuesIfNeeded();
		}
	}

	static  TSharedPtr<SSegmentedControl<OptionType>> Create(
		const TArray<OptionType>& InKeys,
		const TArray<FText>& InLabels,
		const TArray<FText>& InTooltips,
		const TAttribute<TArray<OptionType>>& InValues,
		bool bSupportsMultiSelection = true,
		FOnValuesChanged OnValuesChanged = FOnValuesChanged())
	{
		TSharedPtr<SSegmentedControl<OptionType>> Widget;

		SAssignNew(Widget, SSegmentedControl<OptionType>)
		.SupportsMultiSelection(bSupportsMultiSelection)
		.Values(InValues)
		.OnValuesChanged(OnValuesChanged);

		ensure(InKeys.Num() == InLabels.Num());
		ensure(InKeys.Num() == InTooltips.Num());

		for(int32 Index = 0; Index < InKeys.Num(); Index++)
		{
			Widget->AddSlot(InKeys[Index], false)
			.Text(InLabels[Index])
			.ToolTip(InTooltips[Index]);
		}

		Widget->RebuildChildren();

		return Widget;
	}

private:
	
	TAttribute<ECheckBoxState> GetCheckBoxStateAttribute(OptionType InValue) const
	{
		auto Lambda = [this, InValue]()
		{
			const TArray<OptionType> Values = GetValues();
			return Values.Contains(InValue) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		};

		if (CurrentValuesIsBound)
		{
			return MakeAttributeLambda(Lambda);
		}

		return Lambda();
	}

	void UpdateCheckboxValuesIfNeeded()
	{
		if (!CurrentValuesIsBound)
		{
			const TArray<OptionType> Values = GetValues();
			
			for (int32 Index = 0; Index < Children.Num(); ++Index)
			{
				const FSlot& Slot = Children[Index];
				if (const TSharedPtr<SCheckBox> CheckBox = Slot._CheckBox.Pin())
				{
					CheckBox->SetIsChecked(Values.Contains(Slot._Value) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
				}
			}
		}
	}

	void CommitValue(const ECheckBoxState InCheckState, OptionType InValue)
	{
		const TArray<OptionType> PreviousValues = CurrentValues.Get();
		TArray<OptionType> Values = PreviousValues;

		// don't allow to deselect the last checkbox
		if(InCheckState != ECheckBoxState::Checked && Values.Num() == 1)
		{
			if(!SupportsEmptySelection)
			{
				UpdateCheckboxValuesIfNeeded();
				return;
			}
		}

		bool bModifierIsDown = false;
		if(SupportsMultiSelection)
		{
			bModifierIsDown =
				FSlateApplication::Get().GetModifierKeys().IsShiftDown() ||
				FSlateApplication::Get().GetModifierKeys().IsControlDown();
		}

		// if the attribute is not bound update our internal state
		if(bModifierIsDown)
		{
			if (InCheckState == ECheckBoxState::Checked)
			{
				Values.AddUnique(InValue);
			}
			else
			{
				Values.Remove(InValue);
			}
		}
		else
		{
			if((InCheckState == ECheckBoxState::Checked) || Values.Contains(InValue))
			{
				Values.Reset();
				Values.Add(InValue);
			}
		}

		if (!CurrentValuesIsBound)
		{
			CurrentValues.Set(*this, Values);

			UpdateCheckboxValuesIfNeeded();
		}

		if(OnValueChecked.IsBound())
		{
			if(!bModifierIsDown && InCheckState == ECheckBoxState::Checked)
			{
				for(const OptionType PreviousValue : PreviousValues)
				{
					if(!Values.Contains(PreviousValue))
					{
						OnValueChecked.Execute(PreviousValue, ECheckBoxState::Unchecked);
					}
				}
			}

			OnValueChecked.Execute(InValue, InCheckState);
		}

		if (InCheckState == ECheckBoxState::Checked)
		{
			OnValueChanged.ExecuteIfBound(InValue);
		}

		OnValuesChanged.ExecuteIfBound(Values);
	}

private:
	TPanelChildren<FSlot> Children;

	FOnValueChanged OnValueChanged;
	FOnValuesChanged OnValuesChanged;
	FOnValueChecked OnValueChecked;

	TSlateAttribute<TArray<OptionType>, EInvalidateWidgetReason::Paint> CurrentValues;

	TAttribute<FMargin> UniformPadding;

	const FSegmentedControlStyle* Style;

	const FTextBlockStyle* TextStyle;

	int32 MaxSegmentsPerLine = 0;

	bool CurrentValuesIsBound = false;
	bool SupportsMultiSelection = false;
	bool SupportsEmptySelection = false;
};
