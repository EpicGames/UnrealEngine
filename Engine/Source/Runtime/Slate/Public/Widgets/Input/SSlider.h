// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * A Slate slider control is a linear scale and draggable handle.
 */
class SSlider : public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SSlider, SLeafWidget, SLATE_API)

public:

	SLATE_BEGIN_ARGS(SSlider)
		: _IndentHandle(true)
		, _MouseUsesStep(false)
		, _RequiresControllerLock(true)
		, _Locked(false)
		, _Orientation(EOrientation::Orient_Horizontal)
		, _SliderBarColor(FLinearColor::White)
		, _SliderHandleColor(FLinearColor::White)
		, _Style(&FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider"))
		, _StepSize(0.01f)
		, _Value(1.f)
		, _MinValue(0.0f)
		, _MaxValue(1.0f)
		, _IsFocusable(true)
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		, _PreventThrottling(false)
		{
		}

		/** Whether the slidable area should be indented to fit the handle. */
		SLATE_ATTRIBUTE( bool, IndentHandle )

		/** Sets new value if mouse position is greater/less than half the step size. */
		SLATE_ARGUMENT( bool, MouseUsesStep )

		/** Sets whether we have to lock input to change the slider value. */
		SLATE_ARGUMENT( bool, RequiresControllerLock )

		/** Whether the handle is interactive or fixed. */
		SLATE_ATTRIBUTE( bool, Locked )

		/** The slider's orientation. */
		SLATE_ARGUMENT( EOrientation, Orientation)

		/** The color to draw the slider bar in. */
		SLATE_ATTRIBUTE( FSlateColor, SliderBarColor )

		/** The color to draw the slider handle in. */
		SLATE_ATTRIBUTE( FSlateColor, SliderHandleColor )

		/** The style used to draw the slider. */
		SLATE_STYLE_ARGUMENT( FSliderStyle, Style )

		/** The input mode while using the controller. */
		SLATE_ATTRIBUTE(float, StepSize)

		/** A value that drives where the slider handle appears. Value is normalized between 0 and 1. */
		SLATE_ATTRIBUTE( float, Value )

		/** The minimum value that can be specified by using the slider. */
		SLATE_ARGUMENT(float, MinValue)
		/** The maximum value that can be specified by using the slider. */
		SLATE_ARGUMENT(float, MaxValue)

		/** Sometimes a slider should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, IsFocusable)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when the Controller is pressed and capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureBegin)

		/** Invoked when the controller capture is released.  */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureEnd)

		/** Called when the value is changed by the slider. */
		SLATE_EVENT( FOnFloatValueChanged, OnValueChanged )

		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)

	SLATE_END_ARGS()

	SLATE_API SSlider();
	SLATE_API virtual ~SSlider();

	/**
	 * Construct the widget.
	 * 
	 * @param InDeclaration A declaration from which to construct the widget.
	 */
	SLATE_API void Construct( const SSlider::FArguments& InDeclaration );

	/** Set the widget style. */
	SLATE_API void SetStyle(const FSliderStyle* InStyle);

	/** Get the MinValue attribute */
	float GetMinValue() const { return MinValue; }

	/** Get the MaxValue attribute */
	float GetMaxValue() const { return MaxValue; }

	/** Get the Value attribute */
	SLATE_API float GetValue() const;

	/** Get the Value attribute scaled from 0 to 1 */
	SLATE_API float GetNormalizedValue() const;

	/** Set the Value attribute */
	SLATE_API void SetValue(TAttribute<float> InValueAttribute);

	/** Set the MinValue and MaxValue attributes. If the new MinValue is more than the new MaxValue, MaxValue will be changed to equal MinValue. */
	SLATE_API void SetMinAndMaxValues(float InMinValue, float InMaxValue);
	
	/** Set the IndentHandle attribute */
	SLATE_API void SetIndentHandle(TAttribute<bool> InIndentHandle);
	
	/** Set the Locked attribute */
	SLATE_API void SetLocked(TAttribute<bool> InLocked);

	/** Set the Orientation attribute */
	SLATE_API void SetOrientation(EOrientation InOrientation);
	
	/** Set the SliderBarColor attribute */
	SLATE_API void SetSliderBarColor(TAttribute<FSlateColor> InSliderBarColor);
	
	/** Set the SliderHandleColor attribute */
	SLATE_API void SetSliderHandleColor(TAttribute<FSlateColor> InSliderHandleColor);

	/** Get the StepSize attribute */
	SLATE_API float GetStepSize() const;

	/** Set the StepSize attribute */
	SLATE_API void SetStepSize(TAttribute<float> InStepSize);

	/** Set the MouseUsesStep attribute */
	SLATE_API void SetMouseUsesStep(bool MouseUsesStep);

	/** Set the RequiresControllerLock attribute */
	SLATE_API void SetRequiresControllerLock(bool RequiresControllerLock);

public:

	// SWidget overrides

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	SLATE_API virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	SLATE_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	SLATE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	SLATE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	SLATE_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;

	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual bool IsInteractable() const override;
#if WITH_ACCESSIBILITY
	SLATE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
#endif

	/** @return Is the handle locked or not? Defaults to false */
	SLATE_API bool IsLocked() const;

protected:

	/**
	 * Commits the specified slider value.
	 *
	 * @param NewValue The value to commit.
	 */
	SLATE_API virtual void CommitValue(float NewValue);

	/**
	 * Calculates the new value based on the given absolute coordinates.
	 *
	 * @param MyGeometry The slider's geometry.
	 * @param AbsolutePosition The absolute position of the slider.
	 * @return The new value.
	 */
	SLATE_API float PositionToValue( const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& AbsolutePosition );

	SLATE_API const FSlateBrush* GetBarImage() const;
	SLATE_API const FSlateBrush* GetThumbImage() const;

protected:
	/** @return an attribute reference of IndentHandle */
	TSlateAttributeRef<float> GetValueAttribute() const
	{
		return TSlateAttributeRef<float>(SharedThis(this), ValueSlateAttribute);
	}

	/** @return an attribute reference of IndentHandle */
	TSlateAttributeRef<bool> GetIndentHandleAttribute() const
	{
		return TSlateAttributeRef<bool>(SharedThis(this), IndentHandleSlateAttribute);
	}

	/** @return an attribute reference of Locked */
	TSlateAttributeRef<bool> GetLockedAttribute() const
	{
		return TSlateAttributeRef<bool>(SharedThis(this), LockedSlateAttribute);
	}

	/** @return an attribute reference of SliderBarColor */
	TSlateAttributeRef<FSlateColor> GetSliderBarColorAttribute() const
	{
		return TSlateAttributeRef<FSlateColor>(SharedThis(this), SliderBarColorSlateAttribute);
	}

	/** @return an attribute reference of SliderHandleColor */
	TSlateAttributeRef<FSlateColor> GetSliderHandleColorAttribute() const
	{
		return TSlateAttributeRef<FSlateColor>(SharedThis(this), SliderHandleColorSlateAttribute);
	}

	// Holds the style passed to the widget upon construction.
	const FSliderStyle* Style;

	// Holds the slider's orientation.
	EOrientation Orientation;

	// Holds the initial cursor in case a custom cursor has been specified, so we can restore it after dragging the slider
	EMouseCursor::Type CachedCursor;

	/** The location in screenspace the slider was pressed by a touch */
	UE::Slate::FDeprecateVector2DResult PressedScreenSpaceTouchDownPosition;

	/** Holds the amount to adjust the value by when using a controller or keyboard */
	TAttribute<float> StepSize;

	float MinValue;
	float MaxValue;

	// Holds a flag indicating whether a controller/keyboard is manipulating the slider's value. 
	// When true, navigation away from the widget is prevented until a new value has been accepted or canceled. 
	bool bControllerInputCaptured;

	/** Sets new value if mouse position is greater/less than half the step size. */
	bool bMouseUsesStep;

	/** Sets whether we have to lock input to change the slider value. */
	bool bRequiresControllerLock;

	/** When true, this slider will be keyboard focusable. Defaults to false. */
	bool bIsFocusable;

	/*
	 * Holds whether or not to prevent throttling during mouse capture
	 * When true, the viewport will be updated with every single change to the value during dragging
	 */
	bool bPreventThrottling = false;

private:

	// Resets controller input state. Fires delegates.
	SLATE_API void ResetControllerState();

	// Holds the slider's current value.
	TSlateAttribute<float> ValueSlateAttribute;

	// Holds a flag indicating whether the slideable area should be indented to fit the handle.
	TSlateAttribute<bool> IndentHandleSlateAttribute;

	// Holds a flag indicating whether the slider is locked.
	TSlateAttribute<bool> LockedSlateAttribute;

	// Holds the color of the slider bar.
	TSlateAttribute<FSlateColor> SliderBarColorSlateAttribute;

	// Holds the color of the slider handle.
	TSlateAttribute<FSlateColor> SliderHandleColorSlateAttribute;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

	// Holds a delegate that is executed when capture begins for controller or keyboard.
	FSimpleDelegate OnControllerCaptureBegin;

	// Holds a delegate that is executed when capture ends for controller or keyboard.
	FSimpleDelegate OnControllerCaptureEnd;

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;
};
