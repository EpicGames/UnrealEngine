// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Input/CursorReply.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/IsIntegral.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

#include <limits>

namespace SpinBoxPrivate
{
	SLATE_API extern bool bUseSpinBoxMouseMoveOptimization;
}

/*
 * This function compute a slider position by simulating two log on both side of the neutral value
 * Example a slider going from 0.0 to 2.0 with a neutral value of 1.0, the user will have a lot of precision around the neutral value
 * on both side.
 |
 ||                              |
 | -_                          _-
 |   --__                  __--
 |       ----__________----
 ----------------------------------
  0              1               2

  The function return a float representing the slider fraction used to position the slider handle
  FractionFilled: this is the value slider position with no exponent
  StartFractionFilled: this is the neutral value slider position with no exponent
  SliderExponent: this is the slider exponent
*/
SLATE_API float SpinBoxComputeExponentSliderFraction(float FractionFilled, float StartFractionFilled, float SliderExponent);

template <typename NumericType>
class SSpinBox;

template <typename NumericType>
struct TWidgetTypeTraits<SSpinBox<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A Slate SpinBox resembles traditional spin boxes in that it is a widget that provides
 * keyboard-based and mouse-based manipulation of a numeric value.
 * Mouse-based manipulation: drag anywhere on the spinbox to change the value.
 * Keyboard-based manipulation: click on the spinbox to enter text mode.
 */
template<typename NumericType>
class SSpinBox
	: public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SSpinBox, SCompoundWidget, SLATE_API)
	static inline SLATE_IMPLEMENT_WIDGET(SSpinBox)
public:

	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnValueChanged, NumericType);

	/** Notification for numeric value committed */
	DECLARE_DELEGATE_TwoParams(FOnValueCommitted, NumericType, ETextCommit::Type);

	/** Notification when the max/min spinner values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_DELEGATE_FourParams(FOnDynamicSliderMinMaxValueChanged, NumericType, TWeakPtr<SWidget>, bool, bool);
	
	/** Optional customization of the display value based on the current value. */
	DECLARE_DELEGATE_RetVal_OneParam(TOptional<FText>, FOnGetDisplayValue, NumericType);

	SLATE_BEGIN_ARGS(SSpinBox<NumericType>)
		: _Style(&FCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("SpinBox"))
		, _Value(0)
		, _MinFractionalDigits(DefaultMinFractionalDigits)
		, _MaxFractionalDigits(DefaultMaxFractionalDigits)
		, _AlwaysUsesDeltaSnap(false)
		, _EnableSlider(true)
		, _Delta(0)
		, _ShiftMultiplier(10.f)
		, _CtrlMultiplier(0.1f)
		, _ShiftAltMultiplier(100.f)
		, _CtrlAltMultiplier(0.01f)
		, _SupportDynamicSliderMaxValue(false)
		, _SupportDynamicSliderMinValue(false)
		, _SliderExponent(1.f)
		, _EnableWheel(true)
		, _BroadcastValueChangesPerKey(false)
		, _Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalFont")))
		, _ContentPadding(FMargin(2.0f, 1.0f))
		, _OnValueChanged()
		, _OnValueCommitted()
		, _ClearKeyboardFocusOnCommit(false)
		, _SelectAllTextOnCommit(true)
		, _MinDesiredWidth(0.0f)
		, _Justification(ETextJustify::Left)
		, _KeyboardType(Keyboard_Default)
		, _VirtualKeyboardDismissAction(EVirtualKeyboardDismissAction::TextChangeOnDismiss)
		, _PreventThrottling(true)
		, _RevertTextOnEscape(true)
	{}

	/** The style used to draw this spinbox */
	SLATE_STYLE_ARGUMENT(FSpinBoxStyle, Style)

		/** The value to display */
		SLATE_ATTRIBUTE(NumericType, Value)
		/** The minimum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE(TOptional< NumericType >, MinValue)
		/** The maximum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE(TOptional< NumericType >, MaxValue)
		/** The minimum value that can be specified by using the slider, defaults to MinValue */
		SLATE_ATTRIBUTE(TOptional< NumericType >, MinSliderValue)
		/** The maximum value that can be specified by using the slider, defaults to MaxValue */
		SLATE_ATTRIBUTE(TOptional< NumericType >, MaxSliderValue)
		/** The minimum fractional digits the spin box displays, defaults to 1 */
		SLATE_ATTRIBUTE(TOptional< int32 >, MinFractionalDigits)
		/** The maximum fractional digits the spin box displays, defaults to 6 */
		SLATE_ATTRIBUTE(TOptional< int32 >, MaxFractionalDigits)
		/** Whether typed values should use delta snapping, defaults to false */
		SLATE_ATTRIBUTE(bool, AlwaysUsesDeltaSnap)
		/** Whether this spin box should have slider feature enabled, defaults to true */
		SLATE_ATTRIBUTE(bool, EnableSlider)
		/** Delta to increment the value as the slider moves.  If not specified will determine automatically */
		SLATE_ATTRIBUTE(NumericType, Delta)
		/** How many pixel the mouse must move to change the value of the delta step */
		SLATE_ATTRIBUTE_DEPRECATED(int32, ShiftMouseMovePixelPerDelta, 5.4, "Shift Mouse Move Pixel Per Delta is deprecated and incrementing by a fixed delta per pixel is no longer supported. Please use ShiftMultiplier and CtrlMultiplier which will multiply the step per mouse move")
		/** Multiplier to use when shift is held down */
		SLATE_ATTRIBUTE(float, ShiftMultiplier)
		/** Multiplier to use when ctrl is held down */
		SLATE_ATTRIBUTE(float, CtrlMultiplier)
		/** Multiplier to use when shift+alt is held down */
		SLATE_ATTRIBUTE(float, ShiftAltMultiplier)
		/** Multiplier to use when ctrl+alt is held down */
		SLATE_ATTRIBUTE(float, CtrlAltMultiplier)
		/** If we're an unbounded spinbox, what value do we divide mouse movement by before multiplying by Delta. Requires Delta to be set. */
		SLATE_ATTRIBUTE(int32, LinearDeltaSensitivity)
		/** Tell us if we want to support dynamically changing of the max value using alt */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMaxValue)
		/** Tell us if we want to support dynamically changing of the min value using alt */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMinValue)
		/** Called right after the max slider value is changed (only relevant if SupportDynamicSliderMaxValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMaxValueChanged)
		/** Called right after the min slider value is changed (only relevant if SupportDynamicSliderMinValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMinValueChanged)
		/** Use exponential scale for the slider */
		SLATE_ATTRIBUTE(float, SliderExponent)
		/** When use exponential scale for the slider which is the neutral value */
		SLATE_ATTRIBUTE(NumericType, SliderExponentNeutralValue)
		/** Whether this spin box should have mouse wheel feature enabled, defaults to true */
		SLATE_ARGUMENT(bool, EnableWheel)
		/** True to broadcast every time we type */
		SLATE_ARGUMENT(bool, BroadcastValueChangesPerKey)
		/** Step to increment or decrement the value by when scrolling the mouse wheel. If not specified will determine automatically */
		SLATE_ATTRIBUTE(TOptional< NumericType >, WheelStep)
		/** Font used to display text in the slider */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		/** Padding to add around this widget and its internal widgets */
		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		/** Called when the value is changed by slider or typing */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)
		/** Called when the value is committed (by pressing enter) */
		SLATE_EVENT(FOnValueCommitted, OnValueCommitted)
		/** Called right before the slider begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)
		/** Called right after the slider handle is released by the user */
		SLATE_EVENT(FOnValueChanged, OnEndSliderMovement)
		/** Called to allow customization of what text is displayed when not typing. An empty value falls back to the default behavior. */
		SLATE_EVENT(FOnGetDisplayValue, OnGetDisplayValue)
		/** Called to allow customization of what text is edited when entering text mode. An empty value falls back to the default behavior. */
		SLATE_EVENT(FOnGetDisplayValue, OnEditEditableText)
		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE(bool, ClearKeyboardFocusOnCommit)
		/** Whether to select all text when pressing enter to commit changes */
		SLATE_ATTRIBUTE(bool, SelectAllTextOnCommit)
		/** Minimum width that a spin box should be */
		SLATE_ATTRIBUTE(float, MinDesiredWidth)
		/** How should the value be justified in the spinbox. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)
		/** What keyboard to display. */
		SLATE_ATTRIBUTE(EKeyboardType, KeyboardType)
		/** What keyboard to display. */
		SLATE_ATTRIBUTE(EVirtualKeyboardDismissAction, VirtualKeyboardDismissAction)
		/** Provide custom type conversion functionality to this spin box */
		SLATE_ATTRIBUTE(TSharedPtr< INumericTypeInterface<NumericType> >, TypeInterface)
		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)
		/** If the text should be reverted when pressing the escape key **/
		SLATE_ARGUMENT(bool, RevertTextOnEscape)
		/** Menu extender for the right-click context menu */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtender)

	SLATE_END_ARGS()

	SLATE_API SSpinBox();

	SLATE_API virtual ~SSpinBox();

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	SLATE_API void Tick(const FGeometry& AlottedGeometry, const double InCurrentTime, const float InDeltaTime);

	SLATE_API const bool CommitWithMultiplier(const FPointerEvent& MouseEvent);

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	SLATE_API void ApplySliderMaxValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfHigher);

	SLATE_API void ApplySliderMinValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfLower);

	/**
	 * The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * Called when the mouse wheel is spun. This event is bubbled.
	 *
	 * @param  MouseEvent  Mouse event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	SLATE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	SLATE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	SLATE_API virtual bool SupportsKeyboardFocus() const override;


	SLATE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

	SLATE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	SLATE_API virtual bool HasKeyboardFocus() const override;

	/** Return the Value attribute */
	SLATE_API TAttribute<NumericType> GetValueAttribute() const;

	/** See the Value attribute */
	SLATE_API NumericType GetValue() const;

	SLATE_API void SetValue(TAttribute<NumericType> InValueAttribute, const bool bShouldCommit = false);

	/** See the MinValue attribute */
	SLATE_API NumericType GetMinValue() const;

	SLATE_API void SetMinValue(TAttribute<TOptional<NumericType>> InMinValue);

	/** See the MaxValue attribute */
	SLATE_API NumericType GetMaxValue() const;

	SLATE_API void SetMaxValue(TAttribute<TOptional<NumericType>> InMaxValue);

	/** See the MinSliderValue attribute */
	SLATE_API bool IsMinSliderValueBound() const;

	SLATE_API NumericType GetMinSliderValue() const;

	SLATE_API void SetMinSliderValue(TAttribute<TOptional<NumericType>> InMinSliderValue);

	/** See the MaxSliderValue attribute */
	SLATE_API bool IsMaxSliderValueBound() const;

	SLATE_API NumericType GetMaxSliderValue() const;

	SLATE_API void SetMaxSliderValue(TAttribute<TOptional<NumericType>> InMaxSliderValue);

	/** See the MinFractionalDigits attribute */
	SLATE_API int32 GetMinFractionalDigits() const;

	SLATE_API void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& InMinFractionalDigits);

	/** See the MaxFractionalDigits attribute */
	SLATE_API int32 GetMaxFractionalDigits() const;

	SLATE_API void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& InMaxFractionalDigits);

	/** See the AlwaysUsesDeltaSnap attribute */
	SLATE_API bool GetAlwaysUsesDeltaSnap() const;
	SLATE_API void SetAlwaysUsesDeltaSnap(bool bNewValue);

	/** See the EnableSlider attribute */
	SLATE_API bool GetEnableSlider() const;
	SLATE_API void SetEnableSlider(bool bNewValue);

	/** See the Delta attribute */
	SLATE_API NumericType GetDelta() const;
	SLATE_API void SetDelta(NumericType InDelta);

	/** See the SliderExponent attribute */
	SLATE_API float GetSliderExponent() const;
	SLATE_API void SetSliderExponent(TAttribute<float> InSliderExponent);

	/** See the MinDesiredWidth attribute */
	SLATE_API float GetMinDesiredWidth() const;
	SLATE_API void SetMinDesiredWidth(TAttribute<float> InMinDesiredWidth);

	SLATE_API const FSpinBoxStyle* GetWidgetStyle() const;
	SLATE_API void SetWidgetStyle(const FSpinBoxStyle* InStyle);
	SLATE_API void InvalidateStyle();

	SLATE_API void SetTextBlockFont(FSlateFontInfo InFont);
	SLATE_API void SetTextJustification(ETextJustify::Type InJustification);
	SLATE_API void SetTextClearKeyboardFocusOnCommit(bool bNewValue);
	SLATE_API void SetTextRevertTextOnEscape(bool bNewValue);
	SLATE_API void SetTextSelectAllTextOnCommit(bool bNewValue);

	/** Reset the cached string. Typically used when the value is the same but the display format changed (through the callback). */
	SLATE_API void ResetCachedValueString();

protected:
	/** Make the spinbox switch to keyboard-based input mode. */
	SLATE_API void EnterTextMode();

	/** Make the spinbox switch to mouse-based input mode. */
	SLATE_API void ExitTextMode();

	/** @return the value being observed by the spinbox as a string */
	SLATE_API FString GetValueAsString() const;

	/** @return the value being observed by the spinbox as FText - todo: spinbox FText support (reimplement me) */
	SLATE_API FText GetValueAsText() const;
	
	/** @return the value to be displayed when not manually editing text */
	SLATE_API FText GetDisplayValue() const;

	/**
	 * Invoked when the text in the text field changes
	 *
	 * @param NewText		The value of the text in the text field
	 */
	SLATE_API void TextField_OnTextChanged(const FText& NewText);

	/**
	 * Invoked when the text field commits its text.
	 *
	 * @param NewText		The value of text coming from the editable text field.
	 * @param CommitInfo	Information about the source of the commit
	 */
	SLATE_API void TextField_OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);


	/** How user changed the value in the spinbox */
	enum ECommitMethod
	{
		CommittedViaSpin,
		CommittedViaTypeIn,
		CommittedViaArrowKey,
		CommittedViaCode,
		CommittedViaSpinMultiplier
	};

	/**
	 * Call this method when the user's interaction has changed the value
	 *
	 * @param NewValue               Value resulting from the user's interaction
	 * @param CommitMethod           Did the user type in the value or spin to it.
	 * @param OriginalCommitInfo	 If the user typed in the value, information about the source of the commit
	 * @param bShouldCommit          Should the value be committed (despite the CommitMethod parameter)
	 */
	SLATE_API void CommitValue(NumericType NewValue, double NewSpinValue, ECommitMethod CommitMethod, ETextCommit::Type OriginalCommitInfo, const bool bShouldCommit = false);

	SLATE_API void NotifyValueCommitted(NumericType CurrentValue) const;

	/** @return true when we are in keyboard-based input mode; false otherwise */
	SLATE_API bool IsInTextMode() const;

	/** Calculates range fraction. Possible to use on full numeric range  */
	SLATE_API static float Fraction(double InValue, double InMinValue, double InMaxValue);

private:
	/** Returns the specific step multiplier based on the modifier keys pressed */
	float GetInputEventMultiplier(const FInputEvent& InEvent) const;

	// New value to be Committed on Tick for MouseMove events
	// This exists to insulate high-frequency mouse move events which can fire many times during input processing
	// from the side-effects of committing the spinbox value
	struct FPendingCommitValue
	{
		double NewValue;
		ECommitMethod CommitMethod;
	};

	/** The default minimum fractional digits */
	static constexpr int32 DefaultMinFractionalDigits = 1;

	/** The default maximum fractional digits */
	static constexpr int32 DefaultMaxFractionalDigits = 6;

	TSlateAttribute<NumericType> ValueAttribute;
	FOnValueChanged OnValueChanged;
	FOnValueCommitted OnValueCommitted;
	FSimpleDelegate OnBeginSliderMovement;
	FOnValueChanged OnEndSliderMovement;
	TSharedPtr<STextBlock> TextBlock;
	TSharedPtr<SEditableText> EditableText;

	/** Interface that defines conversion functionality for the templated type */
	TAttribute<TSharedPtr< INumericTypeInterface<NumericType> >> InterfaceAttr;

	const FSpinBoxStyle* Style;

	const FSlateBrush* BackgroundHoveredBrush;
	const FSlateBrush* BackgroundActiveBrush;
	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* ActiveFillBrush;
	const FSlateBrush* HoveredFillBrush;
	const FSlateBrush* InactiveFillBrush;

	float DistanceDragged;
	TSlateAttribute<NumericType> Delta;

	// Step multipliers
	TAttribute<float> ShiftMultiplier;
	TAttribute<float> CtrlMultiplier;
	TAttribute<float> ShiftAltMultiplier;
	TAttribute<float> CtrlAltMultiplier;

	TAttribute<int32> LinearDeltaSensitivity;
	TSlateAttribute<float> SliderExponent;
	TSlateAttribute<NumericType> SliderExponentNeutralValue;
	TSlateAttribute< TOptional<NumericType> > MinValue;
	TSlateAttribute< TOptional<NumericType> > MaxValue;
	TSlateAttribute< TOptional<NumericType> > MinSliderValue;
	TSlateAttribute< TOptional<NumericType> > MaxSliderValue;
	TAttribute< TOptional<int32> > MinFractionalDigits;
	TAttribute< TOptional<int32> > MaxFractionalDigits;
	TAttribute<bool> AlwaysUsesDeltaSnap;
	TAttribute<bool> EnableSlider;
	TAttribute<bool> SupportDynamicSliderMaxValue;
	TAttribute<bool> SupportDynamicSliderMinValue;
	TAttribute< TOptional<NumericType> > WheelStep;
	FOnDynamicSliderMinMaxValueChanged OnDynamicSliderMaxValueChanged;
	FOnDynamicSliderMinMaxValueChanged OnDynamicSliderMinValueChanged;
	FOnGetDisplayValue OnGetDisplayValue;
	FOnGetDisplayValue OnEditEditableText;

	/** Prevents the spinbox from being smaller than desired in certain cases (e.g. when it is empty) */
	TAttribute<float> MinDesiredWidth;
	float GetTextMinDesiredWidth() const;

	/** Check whether a typed character is valid */
	bool IsCharacterValid(TCHAR InChar) const;

	/**
	 * Rounds the submitted value to the correct value if it's an integer.
	 * For int64, not all values can be represented by a double. We can only round until we reach that limit.
	 * This function should only be used when we drag the value. We accept that we can't drag huge numbers.
	 */
	NumericType RoundIfIntegerValue(double ValueToRound) const;

	void CancelMouseCapture();

	/** Tracks which cursor is currently dragging the slider (e.g., the mouse cursor or a specific finger) */
	int32 PointerDraggingSliderIndex;

	/** Cached mouse position to restore after scrolling. */
	FIntPoint CachedMousePosition;

	/**
	 * This value represents what the spinbox believes the value to be, regardless of delta and the user binding to an int.
	 * The spinbox will always count using floats between values, this is important to keep it flowing smoothly and feeling right,
	 * and most importantly not conflicting with the user truncating the value to an int.
	 */
	double InternalValue;

	/** The state of InternalValue before a drag operation was started */
	NumericType PreDragValue;

	/**
	 * This is the cached value the user believes it to be (usually different due to truncation to an int). Used for identifying
	 * external forces on the spinbox and syncing the internal value to them. Synced when a value is committed to the spinbox.
	 */
	NumericType CachedExternalValue;

	/** Used to prevent per-frame re-conversion of the cached numeric value to a string. */
	FString CachedValueString;

	/** Whether the user is dragging the slider */
	TSlateAttribute<bool> bDragging;

	/** True when the SliderExponentNeutralValue attribute is set */
	bool bIsSliderExponentNeutralValueSet;

	/** True when no range is specified, spinner can be spun indefinitely */
	bool bUnlimitedSpinRange;
	void UpdateIsSpinRangeUnlimited();

	/** Whetever the interfaced setting changed and the CachedValueString needs to be recomputed. */
	mutable bool bCachedValueStringDirty;

	/** Re-entrant guard for the text changed handler */
	bool bIsTextChanging;

	/*
	 * Holds whether or not to prevent throttling during mouse capture
	 * When true, the viewport will be updated with every single change to the value during dragging
	 */
	bool bPreventThrottling;

	/** Does this spin box have the mouse wheel feature enabled? */
	bool bEnableWheel = true;

	/** True to broadcast every time we type. */
	bool bBroadcastValueChangesPerKey = false;
	
	TOptional<FPendingCommitValue> PendingCommitValue;

	/*
	* Gets the default amount to change the slider when delta is not applicable. 
	* Control takes priority over shift
	*/
	double GetDefaultStepSize(const FInputEvent& InputEvent);

	/** Gets the default amount to change the slider when delta is not applicable. **/
	const double StepSize = 1.f;

	/** Step size to use when range is below SmallStepSizeMax. **/
	const double SmallStepSize = .1f; 

	/** Largest numerical value to use the SmallStepSize instead of StepSize. **/
	const double SmallStepSizeMax = 10.f;
};