// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SMenuOwner.h"

class FActiveTimerHandle;

/**
 * Wrapper for any widget that is used in a table view that wants to handle sub-menus with the same functionality as a normal multibox menu
 */
class SSubMenuHandler : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSubMenuHandler )
		: _Placement( EMenuPlacement::MenuPlacement_MenuRight )
	{}
		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_ARGUMENT( TSharedPtr<SMenuAnchor>, MenuAnchor )
		SLATE_ARGUMENT( TSharedPtr<SWidget>, MenuContent )
		SLATE_ATTRIBUTE( EMenuPlacement, Placement )
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs, TWeakPtr<SMenuOwner> InMenuOwner);

public:
	// SWidget interface
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	// End of SWidget interface
	
	/**
	 * Returns whether or the sub-menu entry should appear hovered.  If the sub-menu is open we will always show the menu as hovered to indicate which sub-menu is open
	 * In the case that the user is interacting in this menu we do not show the menu as hovered because we need to show what the user is actually selecting
	 */
	SLATE_API bool ShouldSubMenuAppearHovered() const;

	/**
	 * Requests that the sub-menu associated with this widget be toggled on or off.
	 * It does not immediately toggle the menu.  After a set amount of time is passed the menu will toggle
	 *
	 * @param bOpenMenu					true to open the menu, false to close the menu if one is currently open
	 * @param bClobber					true if we want to open a menu when another menu is already open
	 * @param const bool bImmediate		Default false, when true the menu will open immediately
	 */
	SLATE_API void RequestSubMenuToggle( bool bOpenMenu, const bool bClobber, const bool bImmediate = false);
	
	/**
	 * Cancels any open requests to toggle a sub-menu       
	 */
	SLATE_API void CancelPendingSubMenu();

	/** Returns TRUE if there is a sub-menu available for this item */
	bool HasSubMenu() const
	{
		return MenuAnchor.IsValid();
	}

	/** Returns TRUE if the sub-menu is open */
	SLATE_API bool IsSubMenuOpen() const;

protected:
	/** One-off delayed active timer to update the open/closed state of the sub menu. */
	SLATE_API EActiveTimerReturnType UpdateSubMenuState(double InCurrentTime, float InDeltaTime, bool bWantsOpen);

	/** The handle to the active timer to update the sub-menu state */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** For pull-down or sub-menu entries, this stores a weak reference to the menu anchor widget that we'll use to summon the menu */
	TWeakPtr< SMenuAnchor > MenuAnchor;

	/** Weak reference back to the MenuOwner widget that owns us */
	TWeakPtr< class SMenuOwner > MenuOwnerWidget;
};
