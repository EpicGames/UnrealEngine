// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class SMenuAnchor;
class SVerticalBox;

template< typename ObjectType > class TAttribute;

/**
 * Delegate type for showing the input history.
 *
 * The first parameter will contain the history of input strings to show.
 */
DECLARE_DELEGATE_OneParam(FOnShowingHistory, OUT TArray<FString>&)

/**
 * Delegate type for showing a list of suggestions.
 *
 * The first parameter is the current input text.
 * The second parameter will contain the list of suggestions to show.
 */
DECLARE_DELEGATE_TwoParams(FOnShowingSuggestions, const FString&, OUT TArray<FString>&)


/**
 * Implements an editable text box that can show auto-complete histories and suggestions lists.
 */
class SSuggestionTextBox
	: public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SSuggestionTextBox)
		: _BackgroundColor(FLinearColor::White)
		, _BackgroundImage(FCoreStyle::Get().GetBrush("SuggestionTextBox.Background"))
		, _ClearKeyboardFocusOnCommit(true)
		, _CommitOnSuggestionSelected(false)
		, _ErrorReporting()
		, _Font( FCoreStyle::Get().GetFontStyle( TEXT( "NormalFont" ) ) )
		, _ForegroundColor( FCoreStyle::Get().GetSlateColor( "InvertedForeground" ) )
		, _HintText()
		, _IsCaretMovedWhenGainFocus (true)
		, _MinDesiredWidth(0.0f)
		, _RevertTextOnEscape(false)
		, _SelectAllTextOnCommit(false)
		, _SelectAllTextWhenFocused(false)
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox") )
		, _SuggestionTextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SuggestionTextBox.Text") )
		, _SuggestionListMaxHeight(250.0)
		, _Text()
	{ }

		/** The color of the background/border around the editable text. */
		SLATE_ATTRIBUTE(FSlateColor, BackgroundColor)

		/** The image of the background/border around the editable text. */
		SLATE_ARGUMENT(const FSlateBrush*, BackgroundImage)

		/** Whether to clear keyboard focus when pressing enter to commit changes. */
		SLATE_ATTRIBUTE(bool, ClearKeyboardFocusOnCommit)

		/** Whether to commit text that was selected from the suggestions list. */
		SLATE_ATTRIBUTE(bool, CommitOnSuggestionSelected)

		/** Provides an alternative mechanism for error reporting. */
		SLATE_ARGUMENT(TSharedPtr<class IErrorReportingWidget>, ErrorReporting)

		/** Font color and opacity. */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** Text color and opacity. */
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)

		/** Hint text that appears when there is no text in the text box. */
		SLATE_ATTRIBUTE(FText, HintText)

		/** Workaround as we loose focus when the auto completion closes. */
		SLATE_ATTRIBUTE(bool, IsCaretMovedWhenGainFocus)

		/** Minimum width that a text block should be. */
		SLATE_ATTRIBUTE(float, MinDesiredWidth)

		/** Called before the history list is shown. */
		SLATE_EVENT(FOnShowingHistory, OnShowingHistory)

		/** Called before the suggestion list is shown. */
		SLATE_EVENT(FOnShowingSuggestions, OnShowingSuggestions)

		/** Called whenever the text is changed programmatically or interactively by the user. */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

		/** Called when the text has been committed. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** Whether to allow the user to back out of changes when they press the escape key. */
		SLATE_ATTRIBUTE(bool, RevertTextOnEscape)

		/** Whether to select all text when pressing enter to commit changes. */
		SLATE_ATTRIBUTE(bool, SelectAllTextOnCommit)

		/** Whether to select all text when the user clicks to give focus on the widget. */
		SLATE_ATTRIBUTE(bool, SelectAllTextWhenFocused)

		/** The styling of the text box. */
		SLATE_STYLE_ARGUMENT(FEditableTextBoxStyle, TextStyle)

		/** The styling of the suggestion text. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, SuggestionTextStyle)

		/** The maximum height of the suggestion list. */
		SLATE_ATTRIBUTE(float, SuggestionListMaxHeight)

		/** Sets the text content for this editable text box widget. */
		SLATE_ATTRIBUTE(FText, Text)

	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SLATE_API SSuggestionTextBox( );

public:

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param InArgs Declaration used by the SNew() macro to construct this widget.
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Returns the text string.
	 *
	 * @return The text string.
	 */
	FText GetText( ) const
	{
		return TextBox->GetText();
	}

	/**
	 * Sets an optional error string.
	 *
	 * If InError is a non-empty string the TextBox will the ErrorReporting provided during construction.
	 * If no error reporting was provided, the TextBox will create a default error reporter.
	 *
	 * @param InError The error string to set. 
	 */
	void SetError( const FString& InError )
	{
		TextBox->SetError(InError);
	}

	/**
	 * Sets the text string currently being edited 
	 *
	 * @param InNewText The new text string.
	 */
	SLATE_API void SetText( const TAttribute< FText >& InNewText );

protected:
	
	/** Clears the list of suggestions and hides the suggestions list. */
	SLATE_API void ClearSuggestions( );

	/** Sets the keyboard focus to the text box. */
	SLATE_API void FocusTextBox( );

	/**
	 * Gets the string value of the currently selected suggestion.
	 *
	 * @return Suggestion string, or empty if no suggestion is selected.
	 */
	SLATE_API FString GetSelectedSuggestionString( ) const;

	/** Highlights the selected suggestion in the suggestion list. */
	SLATE_API void MarkActiveSuggestion( );

	/**
	 * Sets the list of suggestions.
	 *
	 * @param SuggestionStrings The suggestion strings.
	 * @param InHistoryMode Whether the suggestions represent the input history.
	 */
	SLATE_API void SetSuggestions( TArray<FString>& SuggestionStrings, bool InHistoryMode );

protected:

	// SWidget overrides

	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent ) override;
	SLATE_API virtual bool SupportsKeyboardFocus( ) const override;

private:

	// Callback for generating a row widget for the suggestion list view.
	SLATE_API TSharedRef<ITableRow> HandleSuggestionListViewGenerateRow( TSharedPtr<FString> Message, const TSharedRef<STableViewBase>& OwnerTable );

	// Callback for changing the selected suggestion.
	SLATE_API void HandleSuggestionListViewSelectionChanged( TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo );

	// Callback for getting the highlight string in a suggestion list row widget.
	SLATE_API FText HandleSuggestionListWidgetHighlightText( ) const;

	// Callback for changing the text box input.
	SLATE_API void HandleTextBoxTextChanged( const FText& InText );

	// Callback committing the text box input.
	SLATE_API void HandleTextBoxTextCommitted( const FText& InText, ETextCommit::Type CommitInfo );

	// Called when global focus changes so we can detect when to close the SuggestionList.
	SLATE_API void OnGlobalFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget);

private:

	// Holds a flag to prevent recursive calls in UI callbacks.
	bool IgnoreUIUpdate; 

	// Holds the box for history and auto-complete suggestions.
	TSharedPtr<SMenuAnchor> MenuAnchor;

	// Holds the current list of suggestions.
	TArray<TSharedPtr<FString> > Suggestions;

	// Holds the list view that displays the suggestions.
	TSharedPtr<SListView<TSharedPtr<FString> > > SuggestionListView;

	// Holds the index of the selected suggestion (-1 for no selection).
	int32 SelectedSuggestion;

	// Holds the editable text box.
	TSharedPtr<SEditableTextBox> TextBox;

	// The styling of the suggestion text.
	const FTextBlockStyle* SuggestionTextStyle;

	// The VerticalBox containing the SuggestionList.
	TSharedPtr<SVerticalBox> VerticalBox;

	// Trigger commit text event as soon as suggestion was selected.
	bool CommitsOnSuggestionSelected;

private:

	// Holds a delegate that is executed before the input history is shown.
	FOnShowingHistory OnShowingHistory;

	// Holds a delegate that is executed before the suggestions list is shown.
	FOnShowingSuggestions OnShowingSuggestions;

	// Holds a delegate that is executed when the text has changed.
	FOnTextChanged OnTextChanged;

	// Holds a delegate that is executed when the text has been committed.
	FOnTextCommitted OnTextCommitted;
};
