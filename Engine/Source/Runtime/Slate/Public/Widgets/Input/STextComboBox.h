// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Input/SComboBox.h"

/**
 * A combo box that shows text content.
 */
class STextComboBox : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal_OneParam( FString, FGetTextComboLabel, TSharedPtr<FString> );
	typedef TSlateDelegates< TSharedPtr<FString> >::FOnSelectionChanged FOnTextSelectionChanged;

	SLATE_BEGIN_ARGS( STextComboBox )
		: _ComboBoxStyle(&FCoreStyle::Get().GetWidgetStyle< FComboBoxStyle >("ComboBox"))
		, _ButtonStyle(nullptr)
		, _ColorAndOpacity( FSlateColor::UseForeground() )
		, _ContentPadding(_ComboBoxStyle->ContentPadding)
		, _OnGetTextLabelForItem()
		{}

		SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboBoxStyle)

		/** The visual style of the button part of the combo box (overrides ComboBoxStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** Selection of strings to pick from */
		SLATE_ITEMS_SOURCE_ARGUMENT(TSharedPtr<FString>, OptionsSource)

		/** Text color and opacity */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Sets the font used to draw the text */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** Visual padding of the button content for the combobox */
		SLATE_ATTRIBUTE( FMargin, ContentPadding )

		/** Called when the text is chosen. */
		SLATE_EVENT( FOnTextSelectionChanged, OnSelectionChanged)

		/** Called when the combo box is opened */
		SLATE_EVENT( FOnComboBoxOpening, OnComboBoxOpening )

		/** Called when combo box needs to establish selected item */
		SLATE_ARGUMENT( TSharedPtr<FString>, InitiallySelectedItem )

		/** [Optional] Called to get the label for the currently selected item */
		SLATE_EVENT( FGetTextComboLabel, OnGetTextLabelForItem ) 
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	/** Called to create a widget for each string */
	SLATE_API TSharedRef<SWidget> MakeItemWidget( TSharedPtr<FString> StringItem );

	SLATE_API void SetSelectedItem (TSharedPtr<FString> NewSelection);

	/** Returns the currently selected text string */
	TSharedPtr<FString> GetSelectedItem()
	{
		return SelectedItem;
	}

	/** Sets new item source */
	void SetItemsSource(const TArray<TSharedPtr<FString>>* InListItemsSource)
	{
		StringCombo->SetItemsSource(InListItemsSource);
	}

	/** Sets new item source */
	void SetItemsSource(TSharedRef<::UE::Slate::Containers::TObservableArray<TSharedPtr<FString>>> InListItemsSource)
	{
		StringCombo->SetItemsSource(InListItemsSource);
	}

	/** Clears current item source */
	void ClearItemsSource()
	{
		StringCombo->ClearItemsSource();
	}

	/** Update the style of the internal ComboBox */
	void SetStyle(const FComboBoxStyle* InComboStyle)
	{
		StringCombo->SetStyle(InComboStyle);
	}

	/** @return true if the popup is open; false otherwise. */
	bool IsOpen() const
	{
		return StringCombo->IsOpen();
	}

	/** Request to reload the text options in the combobox from the OptionsSource attribute */
	SLATE_API void RefreshOptions();

	/** Clears the selected item in the text combo */
	SLATE_API void ClearSelection();

private:
	TSharedPtr<FString> OnGetSelection() const {return SelectedItem;}

	/** Called when selection changes in the combo pop-up */
	SLATE_API void OnSelectionChanged (TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Helper method to get the text for a given item in the combo box */
	SLATE_API FText GetSelectedTextLabel() const;

	SLATE_API FText GetItemTextLabel(TSharedPtr<FString> StringItem) const;

private:

	/** Called to get the text label for an item */
	FGetTextComboLabel GetTextLabelForItem;

	/** The text item selected */
	TSharedPtr<FString> SelectedItem;

	/** Array of shared pointers to strings so combo widget can work on them */
	TArray< TSharedPtr<FString> >		Strings;

	/** The combo box */
	TSharedPtr< SComboBox< TSharedPtr<FString> > >	StringCombo;

	/** Forwarding Delegate */
	FOnTextSelectionChanged SelectionChanged;

	/** Sets the font used to draw the text */
	TAttribute< FSlateFontInfo > Font;
};
