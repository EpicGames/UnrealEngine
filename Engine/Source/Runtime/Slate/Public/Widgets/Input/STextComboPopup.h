// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class SButton;

DECLARE_DELEGATE_OneParam( FOnTextChosen, const FString& );

/** Simple box containing a combobox of strings to pick from, usually used within a MenuStack */
class STextComboPopup : public SCompoundWidget
{
public:
	/** Array of shared pointers to strings so combo widget can work on them */
	TArray< TSharedPtr<FString> >		Strings;

	/** Delegate to call when OK button is pressed */
	FOnTextChosen						OnTextChosen;

	/** The combo box */
	TSharedPtr< SComboBox< TSharedPtr<FString> > >	StringCombo;
	/** The OK button */
	TSharedPtr< SButton > OkButton;

	SLATE_BEGIN_ARGS( STextComboPopup ) : _AutoFocus(true) {}

		/** Label, placed before combo box */
		SLATE_ARGUMENT( FText, Label )

		/** Selection of strings to pick from */
		SLATE_ARGUMENT( TArray<FString>, TextOptions )

		/** Called when the text is chosen. */
		SLATE_EVENT( FOnTextChosen, OnTextChosen )

		/** When set, this widget will automatically attempt to set focus to itself when it is created, or when its owner window is activated */
		SLATE_ARGUMENT( bool, AutoFocus )

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	/** SWidget interface */
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Called to create a widget for each string */
	SLATE_API TSharedRef<SWidget> MakeItemWidget( TSharedPtr<FString> StringItem );
	/** Called when 'OK' button is pressed */
	SLATE_API FReply OnOK();

	/** focuses the control on the OK button */
	SLATE_API void FocusDefaultWidget();

private:
	TSharedPtr<FString> OnGetSelection() const {return SelectedItem;}

	/** Called when selection changes in the combo popup */
	void OnSelectionChanged (TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Returns the item to select in the combo popup */
	FText GetSelectedItem () const;
	
	/** Called when AutoFocussing to automatically set focus to this widget */
	EActiveTimerReturnType TickAutoFocus(double InCurrentTime, float InDeltaTime);

private:
	/** The text item selected */
	TSharedPtr<FString> SelectedItem;
};
