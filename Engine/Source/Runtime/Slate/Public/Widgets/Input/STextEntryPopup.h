// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Notifications/SErrorText.h"

class SVerticalBox;

/** Simple text entry popup, usually used within a MenuStack */
class STextEntryPopup : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( STextEntryPopup )
		: _SelectAllTextWhenFocused( false )
		, _MaxWidth( 0.0f )
		, _AutoFocus(true)
		{}

		/** Label, placed before text entry box */
		SLATE_ARGUMENT( FText, Label )

		/** Test to place into text entry box before anything is typed */
		SLATE_ARGUMENT( FText, DefaultText )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Called when the text is committed. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Whether to select all text when the user clicks to give focus on the widget */
		SLATE_ARGUMENT( bool, SelectAllTextWhenFocused )

		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, ClearKeyboardFocusOnCommit )

		/** Hint text that appears when there is no text in the text box */
 		SLATE_ATTRIBUTE( FText, HintText )

		/** The maximum width for text entry */
		SLATE_ATTRIBUTE( float, MaxWidth )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnVerifyTextChanged, OnVerifyTextChanged )

		/** Provide a alternative mechanism for error reporting. */
		SLATE_ARGUMENT( TSharedPtr<class IErrorReportingWidget>, ErrorReporting )

		/** When set, this widget will automatically attempt to set focus to itself when it is created, or when its owner window is activated */
		SLATE_ARGUMENT( bool, AutoFocus )

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	/** @return the widget that we want to be focused when the popup is shown  */
	SLATE_API void FocusDefaultWidget();

	/**
	 * If InError is a non-empty string the TextBox will use the ErrorReporting widget provided during construction
	 * If no error reporting was provided, the TextBox will create a default error reporter.
	 * @param InError An error string used to give extra information about an error
	 */
	SLATE_API void SetError( const FText& InError );
	SLATE_API void SetError( const FString& InError );

protected:
	/** Allows for inserting additional widgets that extend the functionality of the Popup */
	TSharedPtr<SVerticalBox> Box;

	/** SomeWidget used for error reporting */
	TSharedPtr<class IErrorReportingWidget> ErrorReporting;

private:
	/** Called when AutoFocussing to automatically set focus to this widget */
	EActiveTimerReturnType TickAutoFocus(double InCurrentTime, float InDeltaTime);

	/** Widget that we want to be focused when the popup is shown  */
	TSharedPtr<SWidget> WidgetWithDefaultFocus;
};
