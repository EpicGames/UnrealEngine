// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Misc/AxisDisplayInfo.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"

#include <type_traits>

class FArrangedChildren;
class SHorizontalBox;

/**
 * Vector Slate control
 */
template<typename NumericType, typename VectorType = UE::Math::TVector<NumericType>, int32 NumberOfComponents = 3>
class SNumericVectorInputBox : public SCompoundWidget
{
public:
	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnNumericValueChanged, NumericType);

	/** Notification for numeric value committed */
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);

	/** Notification for vector value change */
	DECLARE_DELEGATE_OneParam(FOnVectorValueChanged, VectorType);

	/** Notification for vector value committed */
	DECLARE_DELEGATE_TwoParams(FOnVectorValueCommitted, VectorType, ETextCommit::Type);

	/** Delegate to constrain the vector during a change */
	DECLARE_DELEGATE_ThreeParams(FOnConstrainVector, int32 /* Component */, VectorType /* old */ , VectorType& /* new */);

	struct FArguments;

private:
	using ThisClass = SNumericVectorInputBox<NumericType, VectorType, NumberOfComponents>;

	struct FVectorXArgumentsEmpty {};
	template<typename ArgumentType>
	struct FVectorXArguments : FVectorXArgumentsEmpty
	{
		using WidgetArgsType = ArgumentType;

		FORCENOINLINE FVectorXArguments()
			: _ToggleXChecked(ECheckBoxState::Checked)
			, _XDisplayName(NSLOCTEXT("SVectorInputBox", "X_DisplayName", "X"))
		{
			if constexpr (NumberOfComponents == 3)
			{
				if (AxisDisplayInfo::UseForwardRightUpDisplayNames())
				{
					_XDisplayName.Set(AxisDisplayInfo::GetAxisDisplayName(EAxisList::X));
				}
			}
			_XColor.Set(AxisDisplayInfo::GetAxisColor(EAxisList::X));
		}

		/** X Component of the vector */
		SLATE_ATTRIBUTE(TOptional<NumericType>, X)

		/** Called when the x value of the vector is changed */
		SLATE_EVENT(FOnNumericValueChanged, OnXChanged)

		/** Called when the x value of the vector is committed */
		SLATE_EVENT(FOnNumericValueCommitted, OnXCommitted)

		/** The value of the toggle X checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleXChecked )

		/** Called whenever the toggle X changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleXChanged )

		/** Menu extender delegate for the X value */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtenderX)

		/** Called when the x value of the vector slider began movement */
		SLATE_EVENT(FSimpleDelegate, OnXBeginSliderMovement)

		/** Called when the x value of the vector slider ended movement */
		SLATE_EVENT(FOnNumericValueChanged, OnXEndSliderMovement)

		/** The DisplayName of the X component of the vector */
		SLATE_ATTRIBUTE(FText, XDisplayName)

		SLATE_ATTRIBUTE(FLinearColor, XColor)
	};

	struct FVectorYArgumentsEmpty {};
	template<typename ArgumentType>
	struct FVectorYArguments : FVectorYArgumentsEmpty
	{
		using WidgetArgsType = ArgumentType;

		FORCENOINLINE FVectorYArguments()
			: _ToggleYChecked(ECheckBoxState::Checked)
			, _YDisplayName(NSLOCTEXT("SVectorInputBox", "Y_DisplayName", "Y"))
		{
			if constexpr (NumberOfComponents == 3)
			{
				if (AxisDisplayInfo::UseForwardRightUpDisplayNames())
				{
					_YDisplayName.Set(AxisDisplayInfo::GetAxisDisplayName(EAxisList::Y));
				}
			}
			_YColor.Set(AxisDisplayInfo::GetAxisColor(EAxisList::Y));
		}

		/** Y Component of the vector */
		SLATE_ATTRIBUTE(TOptional<NumericType>, Y)

		/** Called when the Y value of the vector is changed */
		SLATE_EVENT(FOnNumericValueChanged, OnYChanged)

		/** Called when the Y value of the vector is committed */
		SLATE_EVENT(FOnNumericValueCommitted, OnYCommitted)

		/** The value of the toggle Y checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleYChecked )

		/** Called whenever the toggle Y changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleYChanged )

		/** Menu extender delegate for the Y value */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtenderY)

		/** Called when the y value of the vector slider began movement */
		SLATE_EVENT(FSimpleDelegate, OnYBeginSliderMovement)

		/** Called when the y value of the vector slider ended movement */
		SLATE_EVENT(FOnNumericValueChanged, OnYEndSliderMovement)

		/** The DisplayName of the Y component of the vector */
		SLATE_ATTRIBUTE(FText, YDisplayName)

		SLATE_ATTRIBUTE(FLinearColor, YColor)
	};

	struct FVectorZArgumentsEmpty {};
	template<typename ArgumentType>
	struct FVectorZArguments : FVectorZArgumentsEmpty
	{
		using WidgetArgsType = ArgumentType;

		FORCENOINLINE FVectorZArguments()
			: _ToggleZChecked(ECheckBoxState::Checked)
			, _ZDisplayName(NSLOCTEXT("SVectorInputBox", "Z_DisplayName", "Z"))
		{
			if constexpr (NumberOfComponents == 3)
			{
				if (AxisDisplayInfo::UseForwardRightUpDisplayNames())
				{
					_ZDisplayName.Set(AxisDisplayInfo::GetAxisDisplayName(EAxisList::Z));
				}
			}
			_ZColor.Set(AxisDisplayInfo::GetAxisColor(EAxisList::Z));
		}

		/** Z Component of the vector */
		SLATE_ATTRIBUTE(TOptional<NumericType>, Z)

		/** Called when the Z value of the vector is changed */
		SLATE_EVENT(FOnNumericValueChanged, OnZChanged)

		/** Called when the Z value of the vector is committed */
		SLATE_EVENT(FOnNumericValueCommitted, OnZCommitted)

		/** The value of the toggle Z checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleZChecked )

		/** Called whenever the toggle Z changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleZChanged )

		/** Menu extender delegate for the Z value */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtenderZ)

		/** Called when the z value of the vector slider began movement */
		SLATE_EVENT(FSimpleDelegate, OnZBeginSliderMovement)

		/** Called when the z value of the vector slider ended movement */
		SLATE_EVENT(FOnNumericValueChanged, OnZEndSliderMovement)

		/** The DisplayName of the Z component of the vector */
		SLATE_ATTRIBUTE(FText, ZDisplayName)

		SLATE_ATTRIBUTE(FLinearColor, ZColor)
	};

	struct FVectorWArgumentsEmpty {};
	template<typename ArgumentType>
	struct FVectorWArguments : FVectorWArgumentsEmpty
	{
		using WidgetArgsType = ArgumentType;

		FORCENOINLINE FVectorWArguments()
			: _ToggleWChecked(ECheckBoxState::Checked)
			, _WDisplayName(NSLOCTEXT("SVectorInputBox", "W_DisplayName", "W"))
		{
			if constexpr (NumberOfComponents >= 4)
			{
				_WColor.Set(SNumericEntryBox<NumericType>::LilacLabelBackgroundColor);
			}
		}

		/** W Component of the vector */
		SLATE_ATTRIBUTE(TOptional<NumericType>, W)

		/** Called when the W value of the vector is changed */
		SLATE_EVENT(FOnNumericValueChanged, OnWChanged)

		/** Called when the W value of the vector is committed */
		SLATE_EVENT(FOnNumericValueCommitted, OnWCommitted)

		/** The value of the toggle W checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleWChecked )

		/** Called whenever the toggle W changes state */
		SLATE_EVENT( FOnCheckStateChanged, OnToggleWChanged )

		/** Menu extender delegate for the W value */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtenderW)

		/** Called when the w value of the vector slider began movement */
		SLATE_EVENT(FSimpleDelegate, OnWBeginSliderMovement)

		/** Called when the w value of the vector slider ended movement */
		SLATE_EVENT(FOnNumericValueChanged, OnWEndSliderMovement)

		/** The DisplayName of the W component of the vector */
		SLATE_ATTRIBUTE(FText, WDisplayName)

		SLATE_ATTRIBUTE(FLinearColor, WColor)
	};

public:
	//SLATE_BEGIN_ARGS(SNumericVectorInputBox<NumericType>)
	struct FArguments : public TSlateBaseNamedArgs<ThisClass>
		, std::conditional<NumberOfComponents >= 1, FVectorXArguments<FArguments>, FVectorXArgumentsEmpty>::type
		, std::conditional<NumberOfComponents >= 2, FVectorYArguments<FArguments>, FVectorYArgumentsEmpty>::type
		, std::conditional<NumberOfComponents >= 3, FVectorZArguments<FArguments>, FVectorZArgumentsEmpty>::type
		, std::conditional<NumberOfComponents >= 4, FVectorWArguments<FArguments>, FVectorWArgumentsEmpty>::type
	{
		typedef FArguments WidgetArgsType;
		FORCENOINLINE FArguments()
			: _EditableTextBoxStyle( &FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox") )
			, _SpinBoxStyle(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("NumericEntrySpinBox") )
			, _Font(FAppStyle::Get().GetFontStyle("NormalFont"))
			, _AllowSpin(false)
			, _SpinDelta(1)
			, _LinearDeltaSensitivity(1)
			, _bColorAxisLabels(false)
			, _Swizzle(FIntVector4(0, 1, 2, 3))
			, _DisplayToggle(false)
			, _TogglePadding(FMargin(1.f,0.f,1.f,0.f) )
			, _PreventThrottling(false)
		{}

		/** Optional Value of the vector */
		SLATE_ATTRIBUTE(TOptional<VectorType>, Vector)

		/** Optional minimum value of the vector */
		SLATE_ATTRIBUTE(TOptional<VectorType>, MinVector)

		/** Optional maximum value of the vector */
		SLATE_ATTRIBUTE(TOptional<VectorType>, MaxVector)

		/** Optional minimum (slider) value of the vector */
		SLATE_ATTRIBUTE(TOptional<VectorType>, MinSliderVector)

		/** Optional maximum (slider) value of the vector */
		SLATE_ATTRIBUTE(TOptional<VectorType>, MaxSliderVector)

		/** Called when the vector is changed */
		SLATE_EVENT(FOnVectorValueChanged, OnVectorChanged)

		/** Called when the vector is committed */
		SLATE_EVENT(FOnVectorValueCommitted, OnVectorCommitted)

		/** Style to use for the editable text box within this widget */
		SLATE_STYLE_ARGUMENT( FEditableTextBoxStyle, EditableTextBoxStyle )

		/** Style to use for the spin box within this widget */
		SLATE_STYLE_ARGUMENT( FSpinBoxStyle, SpinBoxStyle )
		
		/** Font to use for the text in this box */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Whether or not values can be spun or if they should be typed in */
		SLATE_ARGUMENT( bool, AllowSpin )

		/** The delta amount to apply, per pixel, when the spinner is dragged. */
		SLATE_ATTRIBUTE( NumericType, SpinDelta )

		/** If we're an unbounded spinbox, what value do we divide mouse movement by before multiplying by Delta. Requires Delta to be set. */
		SLATE_ATTRIBUTE( int32, LinearDeltaSensitivity )

		/** Should the axis labels be colored */
		SLATE_ARGUMENT( bool, bColorAxisLabels )
		
		/** Controls the display swizzle of the vector */
		SLATE_ARGUMENT(FIntVector4, Swizzle)

		/** Called right before the slider begins to move for any of the vector components */
		SLATE_EVENT( FSimpleDelegate, OnBeginSliderMovement )

		/** Called right after the slider handle is released by the user for any of the vector components */
		SLATE_EVENT( FOnNumericValueChanged, OnEndSliderMovement )

		/** Provide custom type functionality for the vector */
		SLATE_ARGUMENT( TSharedPtr< INumericTypeInterface<NumericType> >, TypeInterface )

		/** Whether or not to include a toggle checkbox to the left of the widget */
		SLATE_ARGUMENT( bool, DisplayToggle )
			
		/** Padding around the toggle checkbox */
		SLATE_ARGUMENT( FMargin, TogglePadding )

		/** Delegate to constrain the vector */
		SLATE_ARGUMENT( FOnConstrainVector, ConstrainVector )

		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)
	}; // SLATE_END_ARGS()
	

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs)
	{
		bUseVectorGetter = true;

		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		ChildSlot
		[
			HorizontalBox
		];
		
		VectorAttribute = InArgs._Vector;
		OnVectorValueChanged = InArgs._OnVectorChanged;
		OnVectorValueCommitted = InArgs._OnVectorCommitted;
			
		if(!VectorAttribute.IsBound() && !VectorAttribute.IsSet())
		{
			bUseVectorGetter = false;

			VectorAttribute = TAttribute<TOptional<VectorType>>::CreateLambda([InArgs]() -> TOptional<VectorType>
			{
				if constexpr (NumberOfComponents == 2)
				{
					TOptional<NumericType> X = InArgs._X.Get();
					TOptional<NumericType> Y = InArgs._Y.Get();
					if(X.IsSet() && Y.IsSet())
					{
						return VectorType(X.GetValue(), Y.GetValue());
					}
				}
				if constexpr (NumberOfComponents == 3)
				{
					TOptional<NumericType> X = InArgs._X.Get();
					TOptional<NumericType> Y = InArgs._Y.Get();
					TOptional<NumericType> Z = InArgs._Z.Get();
					if(X.IsSet() && Y.IsSet() && Z.IsSet())
					{
						return VectorType(X.GetValue(), Y.GetValue(), Z.GetValue());
					}
				}
				if constexpr (NumberOfComponents == 4)
				{
					TOptional<NumericType> X = InArgs._X.Get();
					TOptional<NumericType> Y = InArgs._Y.Get();
					TOptional<NumericType> Z = InArgs._Z.Get();
					TOptional<NumericType> W = InArgs._W.Get();
					if(X.IsSet() && Y.IsSet() && Z.IsSet() && W.IsSet())
					{
						return VectorType(X.GetValue(), Y.GetValue(), Z.GetValue(), W.GetValue());
					}
				}

				return TOptional<VectorType>();
			});
		}

		if(!OnVectorValueChanged.IsBound())
		{
			OnVectorValueChanged = FOnVectorValueChanged::CreateLambda([InArgs](VectorType Vector)
			{
				if constexpr (NumberOfComponents >= 1)
				{
					InArgs._OnXChanged.ExecuteIfBound(Vector.X);
				}
				if constexpr (NumberOfComponents >= 2)
				{
					InArgs._OnYChanged.ExecuteIfBound(Vector.Y);
				}
				if constexpr (NumberOfComponents >= 3)
				{
					InArgs._OnZChanged.ExecuteIfBound(Vector.Z);
				}
				if constexpr (NumberOfComponents >= 4)
				{
					InArgs._OnWChanged.ExecuteIfBound(Vector.W);
				}
			});
		}
		if(!OnVectorValueCommitted.IsBound())
		{
			OnVectorValueCommitted = FOnVectorValueCommitted::CreateLambda([InArgs](VectorType Vector, ETextCommit::Type CommitType)
			{
				if constexpr (NumberOfComponents >= 1)
				{
					InArgs._OnXCommitted.ExecuteIfBound(Vector.X, CommitType);
				}
				if constexpr (NumberOfComponents >= 2)
				{
					InArgs._OnYCommitted.ExecuteIfBound(Vector.Y, CommitType);
				}
				if constexpr (NumberOfComponents >= 3)
				{
					InArgs._OnZCommitted.ExecuteIfBound(Vector.Z, CommitType);
				}
				if constexpr (NumberOfComponents >= 4)
				{
					InArgs._OnWCommitted.ExecuteIfBound(Vector.W, CommitType);
				}
			});
		}

		using ComponentConstructorFn = void (ThisClass::*)(const FArguments&, TSharedRef<SHorizontalBox>);
		TStaticArray<ComponentConstructorFn, NumberOfComponents> ComponentConstructors;
		
		if constexpr (NumberOfComponents >= 1)
		{
			ComponentConstructors[0] = ComponentConstructorFn(&ThisClass::ConstructX);
		}
		if constexpr (NumberOfComponents >= 2)
		{
			ComponentConstructors[1] = ComponentConstructorFn(&ThisClass::ConstructY);
		}
		if constexpr (NumberOfComponents >= 3)
		{
			ComponentConstructors[2] = ComponentConstructorFn(&ThisClass::ConstructZ);
		}
		if constexpr (NumberOfComponents >= 4)
		{
			ComponentConstructors[3] = ComponentConstructorFn(&ThisClass::ConstructW);
		}

		// Call the widget construction functions in swizzle order
		for (int32 ComponentIndex = 0; ComponentIndex < NumberOfComponents; ++ComponentIndex)
		{
			int32 ConstructorIndex = InArgs._Swizzle[ComponentIndex];
			if (ensureMsgf(ConstructorIndex < NumberOfComponents, TEXT("Invalid swizzle index (%d >= %d)"), ConstructorIndex, NumberOfComponents))
			{
				(this->*ComponentConstructors[ConstructorIndex])(InArgs, HorizontalBox);
			}
		}
	}

private:
	/**
	 * Construct the widget component
	 */
	void ConstructComponent(int32 ComponentIndex,
		const FArguments& InArgs,
		const TAttribute<FLinearColor>& LabelColor,
		const TAttribute<FText>& ComponentDisplayName,
		TSharedRef<SHorizontalBox>& HorizontalBox,
		const TAttribute<TOptional<NumericType>>& Component,
		const FOnNumericValueChanged& OnComponentChanged,
		const FOnNumericValueCommitted& OnComponentCommitted,
		const TAttribute<ECheckBoxState> ToggleChecked,
		const FOnCheckStateChanged& OnToggleChanged,
		const FMenuExtensionDelegate& OnContextMenuExtenderComponent,
		const FSimpleDelegate& OnComponentBeginSliderMovement,
		const FOnNumericValueChanged& OnComponentEndSliderMovement)
	{
		TSharedRef<SWidget> LabelWidget = SNullWidget::NullWidget;
		if (InArgs._bColorAxisLabels)
		{
			LabelWidget = SNumericEntryBox<NumericType>::BuildNarrowColorLabel(LabelColor.Get());
		}

		TStringBuilder<32> ToolTipTextFormatString;
		if (ComponentDisplayName.IsSet())
		{			
			ToolTipTextFormatString.Append(FText::Format(NSLOCTEXT("SVectorInputBox", "ToolTipTextFormatDisplayName", "{0}: "), ComponentDisplayName.Get()).ToString());
		}
		ToolTipTextFormatString.Append(TEXT("{0}"));

		TAttribute<TOptional<NumericType>> Value = CreatePerComponentGetter(ComponentIndex, Component, VectorAttribute);

		// any other getter below can use the vector
		TGuardValue<bool> UseVectorGetterGuard(bUseVectorGetter, true);

		HorizontalBox->AddSlot()
		[
			SNew(SNumericEntryBox<NumericType>)
			.AllowSpin(InArgs._AllowSpin)
			.EditableTextBoxStyle(InArgs._EditableTextBoxStyle)
			.SpinBoxStyle(InArgs._SpinBoxStyle)
			.Font(InArgs._Font)
			.Value(Value)
			.OnValueChanged(CreatePerComponentChanged(ComponentIndex, OnComponentChanged, InArgs._ConstrainVector))
			.OnValueCommitted(CreatePerComponentCommitted(ComponentIndex, OnComponentCommitted, InArgs._ConstrainVector))
			.ToolTipTextFormat(TAttribute<TOptional<FTextFormat>>(FText::FromString(ToolTipTextFormatString.ToString())))
			.UndeterminedString(NSLOCTEXT("SVectorInputBox", "MultipleValues", "Multiple Values"))
			.ContextMenuExtender(OnContextMenuExtenderComponent)
			.TypeInterface(InArgs._TypeInterface)
			.MinFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMinFractionalDigits() : TOptional<int32>())
			.MaxFractionalDigits(InArgs._TypeInterface ? InArgs._TypeInterface->GetMaxFractionalDigits() : TOptional<int32>())
			.MinValue(CreatePerComponentGetter(ComponentIndex, TOptional<NumericType>(), InArgs._MinVector))
			.MaxValue(CreatePerComponentGetter(ComponentIndex, TOptional<NumericType>(), InArgs._MaxVector))
			.MinSliderValue(CreatePerComponentGetter(ComponentIndex, TOptional<NumericType>(), InArgs._MinSliderVector))
			.MaxSliderValue(CreatePerComponentGetter(ComponentIndex, TOptional<NumericType>(), InArgs._MaxSliderVector))
			.LinearDeltaSensitivity(InArgs._LinearDeltaSensitivity)
			.Delta(InArgs._SpinDelta)
			.OnBeginSliderMovement(CreatePerComponentSliderMovementEvent(InArgs._OnBeginSliderMovement, OnComponentBeginSliderMovement))
			.OnEndSliderMovement(CreatePerComponentSliderMovementEvent<FOnNumericValueChanged, NumericType>(InArgs._OnEndSliderMovement, OnComponentEndSliderMovement))
			.LabelPadding(FMargin(3.f))
			.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
			.Label()
			[
				LabelWidget
			]
			.DisplayToggle(InArgs._DisplayToggle)
			.TogglePadding(InArgs._TogglePadding)
			.ToggleChecked(ToggleChecked)
			.OnToggleChanged(OnToggleChanged)
			.PreventThrottling(InArgs._PreventThrottling)
		];
	}

	/**
	 * Construct widgets for the X Value
	 */
	void ConstructX(const FArguments& InArgs, TSharedRef<SHorizontalBox> HorizontalBox)
	{
		ConstructComponent(0,
			InArgs,
			InArgs._XColor,
			InArgs._XDisplayName,
			HorizontalBox,
			InArgs._X,
			InArgs._OnXChanged,
			InArgs._OnXCommitted,
			InArgs._ToggleXChecked,
			InArgs._OnToggleXChanged,
			InArgs._ContextMenuExtenderX,
			InArgs._OnXBeginSliderMovement,
			InArgs._OnXEndSliderMovement
		);
	}

	/**
	 * Construct widgets for the Y Value
	 */
	void ConstructY(const FArguments& InArgs, TSharedRef<SHorizontalBox> HorizontalBox)
	{
		ConstructComponent(1,
			InArgs,
			InArgs._YColor,
			InArgs._YDisplayName,
			HorizontalBox,
			InArgs._Y,
			InArgs._OnYChanged,
			InArgs._OnYCommitted,
			InArgs._ToggleYChecked,
			InArgs._OnToggleYChanged,
			InArgs._ContextMenuExtenderY,
			InArgs._OnYBeginSliderMovement,
			InArgs._OnYEndSliderMovement
		);
	}

	/**
	 * Construct widgets for the Z Value
	 */
	void ConstructZ(const FArguments& InArgs, TSharedRef<SHorizontalBox> HorizontalBox)
	{
		ConstructComponent(2,
			InArgs,
			InArgs._ZColor,
			InArgs._ZDisplayName,
			HorizontalBox,
			InArgs._Z,
			InArgs._OnZChanged,
			InArgs._OnZCommitted,
			InArgs._ToggleZChecked,
			InArgs._OnToggleZChanged,
			InArgs._ContextMenuExtenderZ,
			InArgs._OnZBeginSliderMovement,
			InArgs._OnZEndSliderMovement
		);
	}

	/**
	 * Construct widgets for the W Value
	 */
	void ConstructW(const FArguments& InArgs, TSharedRef<SHorizontalBox> HorizontalBox)
	{
		ConstructComponent(3,
			InArgs,
			InArgs._WColor,
			InArgs._WDisplayName,
			HorizontalBox,
			InArgs._W,
			InArgs._OnWChanged,
			InArgs._OnWCommitted,
			InArgs._ToggleWChecked,
			InArgs._OnToggleWChanged,
			InArgs._ContextMenuExtenderW,
			InArgs._OnWBeginSliderMovement,
			InArgs._OnWEndSliderMovement
		);
	}

	/*
	 * Creates a lambda to retrieve a component off a vector
	 */
	TAttribute<TOptional<NumericType>> CreatePerComponentGetter(
		int32 ComponentIndex,
		const TAttribute<TOptional<NumericType>>& Component,
		const TAttribute<TOptional<VectorType>>& Vector)
	{
		if(bUseVectorGetter && (Vector.IsBound() || Vector.IsSet()))
		{
			return TAttribute<TOptional<NumericType>>::CreateLambda(
				[ComponentIndex, Component, Vector]() -> TOptional<NumericType>
				{
					const TOptional<VectorType> OptionalVectorValue = Vector.Get();
					if(OptionalVectorValue.IsSet())
					{
						return OptionalVectorValue.GetValue()[ComponentIndex];
					}
					return Component.Get();
				});
		}
		return Component;
	}		

	/*
	 * Creates a lambda to react to a change event
	 */
	FOnNumericValueChanged CreatePerComponentChanged(
		int32 ComponentIndex,
		const FOnNumericValueChanged OnComponentChanged,
		const FOnConstrainVector ConstrainVector)
	{
		if(ConstrainVector.IsBound() && OnVectorValueChanged.IsBound())
		{
			return FOnNumericValueChanged::CreateLambda(
				[ComponentIndex, OnComponentChanged, this, ConstrainVector](NumericType InValue)
				{
					const TOptional<VectorType> OptionalVectorValue = VectorAttribute.Get();
					if(OptionalVectorValue.IsSet())
					{
						VectorType VectorValue = OptionalVectorValue.GetValue();
						VectorValue[ComponentIndex] = InValue;

						ConstrainVector.ExecuteIfBound(ComponentIndex, OptionalVectorValue.GetValue(), VectorValue);
						OnVectorValueChanged.Execute(VectorValue);
					}
					else
					{
						OnComponentChanged.ExecuteIfBound(InValue);
					}
				});
		}
		return OnComponentChanged;
	}		

	/*
	 * Creates a lambda to react to a commit event
	 */
	FOnNumericValueCommitted CreatePerComponentCommitted(
		int32 ComponentIndex,
		const FOnNumericValueCommitted OnComponentCommitted,
		const FOnConstrainVector ConstrainVector)
	{
		if(ConstrainVector.IsBound() && OnVectorValueCommitted.IsBound())
		{
			return FOnNumericValueCommitted::CreateLambda(
				[ComponentIndex, OnComponentCommitted, this, ConstrainVector](NumericType InValue, ETextCommit::Type CommitType)
				{
					const TOptional<VectorType> OptionalVectorValue = VectorAttribute.Get();
					if(OptionalVectorValue.IsSet())
					{
						VectorType VectorValue = OptionalVectorValue.GetValue();
						VectorValue[ComponentIndex] = InValue;

						if(ConstrainVector.IsBound())
						{
							ConstrainVector.Execute(ComponentIndex, OptionalVectorValue.GetValue(), VectorValue);
						}

						OnVectorValueCommitted.ExecuteIfBound(VectorValue, CommitType);
					}
					else
					{
						OnComponentCommitted.ExecuteIfBound(InValue, CommitType);
					}
				});
		}
		return OnComponentCommitted;
	}

	/**
	 * Creates a lambda to react to a begin/end slider movement event
	 */
	template<typename EventType, typename... ArgsType>
	EventType CreatePerComponentSliderMovementEvent(
		const EventType OnSliderMovement,
		const EventType OnComponentSliderMovement)
	{
		if(OnSliderMovement.IsBound())
		{
			return EventType::CreateLambda(
				[OnSliderMovement, OnComponentSliderMovement](ArgsType... Args)
				{
					OnSliderMovement.ExecuteIfBound(Args...);
					OnComponentSliderMovement.ExecuteIfBound(Args...);
				});
		}
		return OnComponentSliderMovement;
	}

	bool bUseVectorGetter = true;
	TAttribute<TOptional<VectorType>> VectorAttribute;
	FOnVectorValueChanged OnVectorValueChanged;
	FOnVectorValueCommitted OnVectorValueCommitted;
};

/**
 * For backward compatibility
 */
using SVectorInputBox = SNumericVectorInputBox<float, UE::Math::TVector<float>, 3>;

