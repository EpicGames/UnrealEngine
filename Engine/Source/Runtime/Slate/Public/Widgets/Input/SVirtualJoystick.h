// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class ISlateBrushSource;

/**
 * A virtual joystsick
 */
class SVirtualJoystick : public SLeafWidget
{

public:
	/** The settings of each zone we render */ 
	struct FControlInfo
	{
		// Set by the game

		/** If true, this joystick will be treated as a simple press-release button */
		bool bTreatAsButton = false;

		/** The brush to use to draw the background for joysticks, or unclicked for buttons */
		TSharedPtr< ISlateBrushSource > Image1;

		/** The brush to use to draw the thumb for joysticks, or clicked for buttons */
		TSharedPtr< ISlateBrushSource > Image2;

		/** The actual center of the control */
		FVector2D Center = FVector2D::ZeroVector;

		/** The size of a joystick that can be re-centered within InteractionSize area */
		FVector2D VisualSize = FVector2D::ZeroVector;

		/** The size of the thumb that can be re-centered within InteractionSize area */
		FVector2D ThumbSize = FVector2D::ZeroVector;

		/** The size of a the interactable area around Center */
		FVector2D InteractionSize = FVector2D::ZeroVector;

		/** The scale for control input */
		FVector2D InputScale = FVector2D(1.f, 1.f);

		/** The input to send from this control (for sticks, this is the horizontal/X input) */ 
		FKey MainInputKey;

		/** The secondary input (for sticks, this is the vertical/Y input, unused for buttons) */ 
		FKey AltInputKey;
	};

	/** The settings and current state of each zone we render */ 
	struct FControlData
	{
		/** Control settings */
		FControlInfo Info;
		
		/**
		 * Reset the control to a centered/inactive state
		 */
		SLATE_API void Reset();

		// Current state

		/** The position of the thumb, in relation to the VisualCenter */
		FVector2D ThumbPosition = FVector2D::ZeroVector;

		/** For recentered joysticks, this is the re-center location */
		FVector2D VisualCenter = FVector2D::ZeroVector;

		/** The corrected actual center of the control */
		FVector2D CorrectedCenter = FVector2D::ZeroVector;

		/** The corrected size of a joystick that can be re-centered within InteractionSize area */
		FVector2D CorrectedVisualSize = FVector2D::ZeroVector;

		/** The corrected size of the thumb that can be re-centered within InteractionSize area */
		FVector2D CorrectedThumbSize = FVector2D::ZeroVector;

		/** The corrected size of a the interactable area around Center */
		FVector2D CorrectedInteractionSize = FVector2D::ZeroVector;

		/** The corrected scale for control input */
		FVector2D CorrectedInputScale = FVector2D::ZeroVector;

		/** Which pointer index is interacting with this control right now, or -1 if not interacting */
		int32 CapturedPointerIndex = -1;

		/** Time to activate joystick **/
		float ElapsedTime = 0.0f;

		/** Visual center to be updated */
		FVector2D NextCenter = FVector2D::ZeroVector;

		/** Whether or not to send one last "release" event next tick */
		bool bSendOneMoreEvent = false;

		/** Whether or not we need position the control against the geometry */
		bool bHasBeenPositioned = false;

		/** Whether or not to update center position */
		bool bNeedUpdatedCenter = false;
	};

	SLATE_BEGIN_ARGS(SVirtualJoystick)
		{}
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs);

	/**
	 * Static function to return if external users should create/activate/etc a touch interface
	 * Note that this function is also used internally, so even if this returns false but an SVirtualJoystick
	 * is created, it won't actually show any controls
	 */
	static SLATE_API bool ShouldDisplayTouchInterface();

	/**
	 * Shows or hides the controls (for instance during cinematics)
	 */
	SLATE_API void SetJoystickVisibility(const bool bVisible, const bool bFade);

	SLATE_API void AddControl(const FControlInfo& Control);
	SLATE_API void ClearControls();
	SLATE_API void SetControls(const TArray<FControlInfo>& InControls);

	/**
	 * Sets parameters that control all controls
	 */
	SLATE_API void SetGlobalParameters(float InActiveOpacity, float InInactiveOpacity, float InTimeUntilDeactive, float InTimeUntilReset, float InActivationDelay, bool InbPreventReCenter, float InStartupDelay);

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	SLATE_API virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& Event) override;
	SLATE_API virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& Event) override;
	SLATE_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& Event) override;

	SLATE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	SLATE_API virtual bool SupportsKeyboardFocus() const override;

protected:
	/** Callback for handling display metrics changes. */
	SLATE_API virtual void HandleDisplayMetricsChanged(const FDisplayMetrics& NewDisplayMetric);

	static SLATE_API void AlignBoxIntoScreen(FVector2D& Position, const FVector2D& Size, const FVector2D& ScreenSize);
	SLATE_API FVector2D ComputeThumbPosition(int32 ControlIndex, const FVector2D& LocalCoord, float* OutDistanceToTouchSqr = nullptr, float* OutDistanceToEdgeSqr = nullptr);

	/**
	 * Process a touch event (on movement and possibly on initial touch)
	 *
	 * @return true if the touch was successful
	 */
	SLATE_API virtual bool HandleTouch(int32 ControlIndex, const FVector2D& LocalCoord, const FVector2D& ScreenSize);

	/** 
	 * Return the target opacity to lerp to given the current state
	 */
	inline float GetBaseOpacity();

	/**
	 * TArray specialization for controls. In the game only few joysticks presented
	 * so we can predict their count and store in memory in more efficient way
	 */ 
	template <typename T>
	using TControlArray = TArray<T, TInlineAllocator<2>>;
	
	/** List of controls set by the UTouchInterface */
	TControlArray<FControlData> Controls;

	/** Global settings from the UTouchInterface */
	float ActiveOpacity = 1.0f;
	float InactiveOpacity = 0.1f;
	float TimeUntilDeactive = 0.5f;
	float TimeUntilReset = 2.0f;
	float ActivationDelay = 0.0f;
	float StartupDelay = 0.0f;

	enum EVirtualJoystickState
	{
		State_Active,
		State_CountingDownToInactive,
		State_CountingDownToReset,
		State_Inactive,
		State_WaitForStart,
		State_CountingDownToStart,
	};

	/** The current state of all controls */
	EVirtualJoystickState State = State_Inactive;

	/** True if the joystick should be visible */
	uint32 bVisible:1;

	/** If true, this zone will have it's "center" set when you touch it, otherwise the center will be set to the center of the zone */
	uint32 bCenterOnEvent:1;

	/** If true, ignore re-centering */
	uint32 bPreventReCenter:1;

	/** Target opacity */
	float CurrentOpacity = InactiveOpacity;

	/* Countdown until next state change */
	float Countdown;

	/** Last used scaling value for  */
	float PreviousScalingFactor = 0.0f;
};
