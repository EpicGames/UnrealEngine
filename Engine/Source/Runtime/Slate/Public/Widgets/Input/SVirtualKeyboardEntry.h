// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "Framework/Layout/SlateScrollHelper.h"

class FPaintArgs;
class FSlateWindowElementList;

class SVirtualKeyboardEntry : public SLeafWidget, public IVirtualKeyboardEntry
{

public:

	SLATE_BEGIN_ARGS( SVirtualKeyboardEntry )
		: _Text()
		, _HintText()
		, _Font( FCoreStyle::Get().GetFontStyle("NormalFont") )
		, _ColorAndOpacity( FSlateColor::UseForeground() )
		, _IsReadOnly( false )
		, _ClearKeyboardFocusOnCommit( true )
		, _MinDesiredWidth( 0.0f )
		, _KeyboardType ( EKeyboardType::Keyboard_Default )
		, _VirtualKeyboardOptions ( FVirtualKeyboardOptions() )
		, _EnableIntegratedKeyboard ( false )
		{}

		/** Sets the text content for this editable text widget */
		SLATE_ATTRIBUTE( FText, Text )

		/** The text that appears when there is nothing typed into the search box */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Sets the font used to draw the text */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Text color and opacity */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE( bool, IsReadOnly )

		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE( bool, ClearKeyboardFocusOnCommit )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Minimum width that a text block should be */
		SLATE_ATTRIBUTE( float, MinDesiredWidth )

		/** Sets the text content for this editable text widget */
		SLATE_ATTRIBUTE( EKeyboardType, KeyboardType )

		/** Sets additional arguments to be used by the virtual keyboard summoned by this widget */
		SLATE_ARGUMENT( FVirtualKeyboardOptions, VirtualKeyboardOptions )

		/** Whether the IntegratedKeyboard is enabled  */
		SLATE_ARGUMENT( bool, EnableIntegratedKeyboard )

		SLATE_END_ARGS()


	/** Constructor */
	SLATE_API SVirtualKeyboardEntry();
	
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/** Restores the text to the original state */
	SLATE_API void RestoreOriginalText();

	/** @return whether the current text varies from the original */
	SLATE_API bool HasTextChangedFromOriginal() const;

	/**
	* Sets the text currently being edited
	*
	* @param  InNewText  The new text
	*/
	SLATE_API void SetText(const TAttribute< FText >& InNewText);

	/**
	 * Sets the font used to draw the text
	 *
	 * @param  InNewFont	The new font to use
	 */
	SLATE_API void SetFont( const TAttribute< FSlateFontInfo >& InNewFont );

	/** @return Whether the user can edit the text. */
	SLATE_API bool GetIsReadOnly() const;

public:
	//~ Begin IVirtualKeyboardEntry Interface
	SLATE_API virtual void SetTextFromVirtualKeyboard(const FText& InNewText, ETextEntryType TextEntryType) override;
	SLATE_API virtual void SetSelectionFromVirtualKeyboard(int InSelStart, int InSelEnd) override;

	virtual bool GetSelection(int& OutSelStart, int& OutSelEnd) override
	{
		return false;
	}

	virtual FText GetText() const override
	{
		check(IsInGameThread());

		return Text.Get();
	}

	virtual FText GetHintText() const override
	{
		check(IsInGameThread());

		return HintText.Get();
	}

	virtual EKeyboardType GetVirtualKeyboardType() const override
	{
		check(IsInGameThread());

		return KeyboardType.Get();
	}

	virtual FVirtualKeyboardOptions GetVirtualKeyboardOptions() const override
	{
		check(IsInGameThread());

		return VirtualKeyboardOptions;
	}

	virtual bool IsMultilineEntry() const override
	{
		check(IsInGameThread());

		return false;
	}

	virtual bool IsIntegratedKeyboardEnabled() const override
	{
		check(IsInGameThread());

		return bEnableIntegratedKeyboard.Get();
	}
	//~ End IVirtualKeyboardEntry Interface

protected:

	//~ Begin SWidget Interface
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	//~ End SWidget Interface

private:

	/** @return The string that needs to be rendered */
	SLATE_API FString GetStringToRender() const;

private:

	/** The text content for this editable text widget */
	TAttribute< FText > Text;

	TAttribute< FText > HintText;

	/** The font used to draw the text */
	TAttribute< FSlateFontInfo > Font;

	/** Text color and opacity */
	TAttribute<FSlateColor> ColorAndOpacity;

	/** Sets whether this text box can actually be modified interactively by the user */
	TAttribute< bool > IsReadOnly;

	/** Whether to clear keyboard focus when pressing enter to commit changes */
	TAttribute< bool > ClearKeyboardFocusOnCommit;

	/** Called whenever the text is changed programmatically or interactively by the user */
	FOnTextChanged OnTextChanged;

	/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
	FOnTextCommitted OnTextCommitted;

	/** Text string currently being edited */
	FText EditedText;

	/** Original text prior to user edits.  This is used to restore the text if the user uses the escape key. */
	FText OriginalText;

	/** Current scrolled position */
	FScrollHelper ScrollHelper;

	/** True if the last mouse down caused us to receive keyboard focus */
	bool bWasFocusedByLastMouseDown;

	/** True if we're currently (potentially) changing the text string */
	bool bIsChangingText;

	/** Prevents the editabletext from being smaller than desired in certain cases (e.g. when it is empty) */
	TAttribute<float> MinDesiredWidth;

	TAttribute<EKeyboardType> KeyboardType;

	/** Whether to enable integrated keyboard */
	TAttribute<bool> bEnableIntegratedKeyboard;

	FVirtualKeyboardOptions VirtualKeyboardOptions;

	bool bNeedsUpdate;

};
