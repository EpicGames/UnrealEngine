// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"

class SSlider;

/**
 * A Slate VolumeControl is a mute icon/button and volume slider.
 */
class SVolumeControl : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnMuted, bool );

	SLATE_BEGIN_ARGS(SVolumeControl)
		: _Style( &FCoreStyle::Get().GetWidgetStyle<FVolumeControlStyle>("VolumeControl") )
		, _Volume(1.f)
		, _Muted(false)
		{}

		/** The style to use to draw this volume control */
		SLATE_STYLE_ARGUMENT( FVolumeControlStyle, Style )

		/** The volume to display */
		SLATE_ATTRIBUTE( float, Volume )
		/** Called when the volume is changed by slider or muting */
		SLATE_EVENT( FOnFloatValueChanged, OnVolumeChanged )

		/** The mute state to display */
		SLATE_ATTRIBUTE( bool, Muted )
		/** Called when the mute state is changed by clicking the button */
		SLATE_EVENT( FOnMuted, OnMuteChanged )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InDeclaration   A declaration from which to construct the widget
	 */
	SLATE_API void Construct( const SVolumeControl::FArguments& InDeclaration );

	float GetVolume() const { return VolumeAttribute.Get(); }
	bool IsMuted() const { return MutedAttribute.Get(); }

private:
	enum ESpeakerIcon
	{
		ES_Full,
		ES_Mid,
		ES_Low,
		ES_Off,
		ES_Muted,

		ES_MAX
	};

	TAttribute<float> VolumeAttribute;
	FOnFloatValueChanged OnVolumeChanged;

	TAttribute<bool> MutedAttribute;
	FOnMuted OnMutedChanged;

	TSharedPtr<SSlider> Slider;

	const FSlateBrush* SpeakerIcons[ES_MAX];

	float GetSliderPosition() const { return IsMuted() ? 0.f : GetVolume(); }
	SLATE_API const FSlateBrush* GetSpeakerImage() const;
	SLATE_API FReply OnMuteClicked();
	SLATE_API void OnWriteValue(float NewValue);
};
