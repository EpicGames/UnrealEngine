// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class SLayer
{

};

class SLayerManager
{
	//
	// PLACEHOLDER
	//
	// Will become class that managed pushing and popping
	// game UI layers, showing tooltips for games, etc.
	//
};

