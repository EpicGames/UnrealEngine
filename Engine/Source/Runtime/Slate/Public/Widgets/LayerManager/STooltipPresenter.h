// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

/** Presents its content at the cursor's position. Tooltip avoids widget edges. */
class STooltipPresenter : public SPanel
{
public:
	SLATE_BEGIN_ARGS(STooltipPresenter)
	{
		_Visibility = EVisibility::HitTestInvisible;
	}

		SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	/** Constructor */
	STooltipPresenter()
		: ChildSlot(this)
	{
		bCanSupportFocus = false;
	}

	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API void SetContent(TSharedPtr<SWidget> InWidget);

private:
	SLATE_API virtual bool ComputeVolatility() const override;
	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	SLATE_API virtual FChildren* GetChildren() override;

	TWeakChild<SWidget> ChildSlot;
};
