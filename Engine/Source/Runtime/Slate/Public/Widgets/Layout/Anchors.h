// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Anchors.generated.h"

/**
 * Describes how a widget is anchored.
 */
USTRUCT(BlueprintType)
struct FAnchors
{
	GENERATED_USTRUCT_BODY()

	/** Holds the minimum anchors, left + top. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FVector2D Minimum;

	/** Holds the maximum anchors, right + bottom. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FVector2D Maximum;

public:

	/**
	 * Default constructor.
	 *
	 * The default margin size is zero on all four sides..
	 */
	FAnchors()
		: Minimum(0.0f, 0.0f)
		, Maximum(0.0f, 0.0f)
	{ }

	/** Construct a Anchors with uniform space on all sides */
	FAnchors(float UnifromAnchors)
		: Minimum(UnifromAnchors, UnifromAnchors)
		, Maximum(UnifromAnchors, UnifromAnchors)
	{ }

	/** Construct a Anchors where Horizontal describes Left and Right spacing while Vertical describes Top and Bottom spacing */
	FAnchors(float Horizontal, float Vertical)
		: Minimum(Horizontal, Vertical)
		, Maximum(Horizontal, Vertical)
	{ }

	/** Construct Anchors where the spacing on each side is individually specified. */
	FAnchors(float MinX, float MinY, float MaxX, float MaxY)
		: Minimum(MinX, MinY)
		, Maximum(MaxX, MaxY)
	{ }

	/**
	* Compares this set of FAnchors with another for equality.
	*
	* @param Other The other FAnchors.
	* @return true if the two FAnchors are equal, false otherwise.
	*/
	bool operator==(const FAnchors& Other) const
	{
		return Minimum == Other.Minimum && Maximum == Other.Maximum;
	}

	/**
	* Compares this set of FAnchors with another for inequality.
	*
	* @param Other The other FAnchors.
	* @return true if the two FAnchors are not equal, false otherwise.
	*/
	bool operator!=(const FAnchors& Other) const
	{
		return !(*this == Other);
	}

	/** Returns true if the anchors represent a stretch along the vertical axis */
	bool IsStretchedVertical() const { return Minimum.Y != Maximum.Y; }

	/** Returns true if the anchors represent a stretch along the horizontal axis */
	bool IsStretchedHorizontal() const { return Minimum.X != Maximum.X; }
};
