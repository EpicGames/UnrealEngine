// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SScrollBar.h"

class SLinkableScrollBar : public SScrollBar
{
public:
	
	SLATE_API virtual void SetState(float InOffsetFraction, float InThumbSizeFraction, bool bCallOnUserScrolled = false) override;

	static SLATE_API void LinkScrollBars(TSharedRef<SLinkableScrollBar> Left, TSharedRef<SLinkableScrollBar> Right, TAttribute<TArray<FVector2f>> ScrollSyncRate);

private:
	TWeakPtr<SLinkableScrollBar> LinkedScrollBarRight;
	TWeakPtr<SLinkableScrollBar> LinkedScrollBarLeft;

	// list of scroll distance pairs that determine variable scroll rate.
	// scroll values of Left panel will match X components while the right panel will match Y components
	TAttribute<TArray<FVector2f>> ScrollSyncRateRight;
	TAttribute<TArray<FVector2f>> ScrollSyncRateLeft;
};
