// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
/**
* A BackgroundBlur is similar to a Border in that it can be used to contain other widgets.
* However, instead of surrounding the content with an image, it applies a post-process blur to
* everything (both actors and widgets) that is underneath it.
* 
* Note: For low-spec machines where the blur effect is too computationally expensive, 
* a user-specified fallback image is used instead (effectively turning this into a Border)
*/
class SBackgroundBlur : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SBackgroundBlur, SCompoundWidget, SLATE_API)

public:
	SLATE_BEGIN_ARGS(SBackgroundBlur)
		: _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Fill)
		, _Padding(FMargin(2.0f))
		, _bApplyAlphaToBlur(true)
		, _BlurStrength(0.f)
		, _BlurRadius()
		, _CornerRadius(FVector4(0,0,0,0))
		, _LowQualityFallbackBrush(nullptr)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		
		SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
		SLATE_ARGUMENT(EVerticalAlignment, VAlign)
		SLATE_ATTRIBUTE(FMargin, Padding)

		SLATE_ARGUMENT(bool, bApplyAlphaToBlur)
		SLATE_ATTRIBUTE(float, BlurStrength)
		SLATE_ATTRIBUTE(TOptional<int32>, BlurRadius)
		SLATE_ATTRIBUTE(FVector4, CornerRadius)
		SLATE_ARGUMENT(const FSlateBrush*, LowQualityFallbackBrush)
	SLATE_END_ARGS()

public:
	SLATE_API SBackgroundBlur();
	SLATE_API virtual ~SBackgroundBlur();
	SLATE_API void Construct(const FArguments& InArgs);
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	SLATE_API void SetContent(const TSharedRef<SWidget>& InContent);
	SLATE_API void SetApplyAlphaToBlur(bool bInApplyAlphaToBlur);
	SLATE_API void SetBlurRadius(TAttribute<TOptional<int32>> InBlurRadius);
	SLATE_API void SetBlurStrength(TAttribute<float> InStrength);
	SLATE_API void SetCornerRadius(TAttribute<FVector4> InCornerRadius);
	SLATE_API void SetLowQualityBackgroundBrush(const FSlateBrush* InBrush);
	
	SLATE_API void SetHAlign(EHorizontalAlignment HAlign);
	SLATE_API void SetVAlign(EVerticalAlignment VAlign);
	SLATE_API void SetPadding(TAttribute<FMargin> InPadding);

	SLATE_API bool IsUsingLowQualityFallbackBrush() const;

protected:
	SLATE_API void ComputeEffectiveKernelSize(float Strength, int32& OutKernelSize, int32& OutDownsampleAmount) const;


	/** @return an attribute reference of ColorAndOpacity */
	TSlateAttributeRef<float> GetBlurStrengthAttribute() const { return TSlateAttributeRef<float>{SharedThis(this), BlurStrengthAttribute}; }

	/** @return an attribute reference of ForegroundColor */
	TSlateAttributeRef<TOptional<int32>> GetBlurRadiusAttribute() const { return TSlateAttributeRef<TOptional<int32>>{SharedThis(this), BlurRadiusAttribute}; }

#if WITH_EDITORONLY_DATA
	TSlateDeprecatedTAttribute<float> BlurStrength;
	TSlateDeprecatedTAttribute<TOptional<int32>> BlurRadius;
#endif

	bool bApplyAlphaToBlur;
	const FSlateBrush* LowQualityFallbackBrush;

private:
	TSlateAttribute<float> BlurStrengthAttribute;
	TSlateAttribute<TOptional<int32>> BlurRadiusAttribute;
	TSlateAttribute<FVector4> CornerRadiusAttribute;
};
