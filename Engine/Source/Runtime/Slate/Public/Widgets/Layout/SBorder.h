// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"

class FPaintArgs;
class FSlateWindowElementList;

template <>
struct TWidgetTypeTraits<class SBorder>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A border is an widget that can be used to contain other widgets.
 * It has a BorderImage property, which allows it to take on different appearances.
 * Border also has a Content() slot as well as some parameters controlling the
 * arrangement of said content.
 */
class SBorder : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SBorder, SCompoundWidget, SLATE_API)

public:

	SLATE_BEGIN_ARGS(SBorder)
		: _Content()
		, _HAlign( HAlign_Fill )
		, _VAlign( VAlign_Fill )
		, _Padding( FMargin(2.0f) )
		, _OnMouseButtonDown()
		, _OnMouseButtonUp()
		, _OnMouseMove()
		, _OnMouseDoubleClick()
		, _BorderImage( FCoreStyle::Get().GetBrush( "Border" ) )
		, _ContentScale( FVector2D(1,1) )
		, _DesiredSizeScale( FVector2D(1,1) )
		, _ColorAndOpacity( FLinearColor(1,1,1,1) )
		, _BorderBackgroundColor( FLinearColor::White )
		, _ForegroundColor( FSlateColor::UseForeground() )
		, _ShowEffectWhenDisabled( true )
		, _FlipForRightToLeftFlowDirection(false)
		{}

		SLATE_DEFAULT_SLOT( FArguments, Content )

		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )
		SLATE_ATTRIBUTE( FMargin, Padding )
		
		SLATE_EVENT( FPointerEventHandler, OnMouseButtonDown )
		SLATE_EVENT( FPointerEventHandler, OnMouseButtonUp )
		SLATE_EVENT( FPointerEventHandler, OnMouseMove )
		SLATE_EVENT( FPointerEventHandler, OnMouseDoubleClick )

		SLATE_ATTRIBUTE( const FSlateBrush*, BorderImage )

		SLATE_ATTRIBUTE( FVector2D, ContentScale )

		SLATE_ATTRIBUTE( FVector2D, DesiredSizeScale )

		/** ColorAndOpacity is the color and opacity of content in the border */
		SLATE_ATTRIBUTE( FLinearColor, ColorAndOpacity )
		/** BorderBackgroundColor refers to the actual color and opacity of the supplied border image.*/
		SLATE_ATTRIBUTE( FSlateColor, BorderBackgroundColor )
		/** The foreground color of text and some glyphs that appear as the border's content. */
		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )
		/** Whether or not to show the disabled effect when this border is disabled */
		SLATE_ATTRIBUTE( bool, ShowEffectWhenDisabled )

		/** Flips the background image if the localization's flow direction is RightToLeft */
		SLATE_ARGUMENT(bool, FlipForRightToLeftFlowDirection)

		FArguments& Padding(float Uniform)
		{
			_Padding = FMargin(Uniform);
			return *this;
		}

		FArguments& Padding(float Horizontal, float Vertical)
		{
			_Padding = FMargin(Horizontal, Vertical);
			return *this;
		}

		FArguments& Padding(float Left, float Top, float Right, float Bottom)
		{
			_Padding = FMargin(Left, Top, Right, Bottom);
			return *this;
		}

	SLATE_END_ARGS()

	/**
	 * Default constructor.
	 */
	SLATE_API SBorder();
	SLATE_API virtual ~SBorder();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Sets the content for this border
	 *
	 * @param	InContent	The widget to use as content for the border
	 */
	SLATE_API virtual void SetContent( TSharedRef< SWidget > InContent );

	/**
	 * Gets the content for this border
	 *
	 * @return The widget used as content for the border
	 */
	SLATE_API const TSharedRef< SWidget >& GetContent() const;

	/** Clears out the content for the border */
	SLATE_API void ClearContent();

	/** Sets the color and opacity of the background image of this border. */
	SLATE_API void SetBorderBackgroundColor(TAttribute<FSlateColor> InColorAndOpacity);

	/** Gets the color and opacity of the background image of this border. */
	FSlateColor GetBorderBackgroundColor() const { return BorderBackgroundColorAttribute.Get(); }

	/** Set the desired size scale multiplier */
	SLATE_API void SetDesiredSizeScale(TAttribute<FVector2D> InDesiredSizeScale);
	
	/** See HAlign argument */
	SLATE_API void SetHAlign(EHorizontalAlignment HAlign);

	/** See VAlign argument */
	SLATE_API void SetVAlign(EVerticalAlignment VAlign);

	/** See Padding attribute */
	SLATE_API void SetPadding(TAttribute<FMargin> InPadding);

	/** Set whether or not to show the disabled effect when this border is disabled */
	SLATE_API void SetShowEffectWhenDisabled(TAttribute<bool> InShowEffectWhenDisabled);

	/** Set the image to draw for this border. */
	SLATE_API void SetBorderImage(TAttribute<const FSlateBrush*> InBorderImage);

	/** Get the image to draw for this border. */
	const FSlateBrush* GetBorderImage() const { return BorderImageAttribute.Get(); }

public:
	// SWidget interface
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	// End of SWidget interface

protected:
	//~Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~End SWidget overrides.

	/** Get whether or not to show the disabled effect when this border is disabled */
	bool GetShowDisabledEffect() const { return ShowDisabledEffectAttribute.Get(); }

	/** Get the desired size scale multiplier */
	FVector2D GetDesiredSizeScale() const { return DesiredSizeScaleAttribute.Get(); }

	TSlateAttributeRef<const FSlateBrush*> GetBorderImageAttribute() const { return TSlateAttributeRef<const FSlateBrush*>(SharedThis(this), BorderImageAttribute); }
	TSlateAttributeRef<FSlateColor> GetBorderBackgroundColorAttribute() const { return TSlateAttributeRef<FSlateColor>(SharedThis(this), BorderBackgroundColorAttribute); }
	TSlateAttributeRef<FVector2D> GetDesiredSizeScaleAttribute() const { return TSlateAttributeRef<FVector2D>(SharedThis(this), DesiredSizeScaleAttribute); }
	TSlateAttributeRef<bool> GetShowDisabledEffectAttribute() const { return TSlateAttributeRef<bool>(SharedThis(this), ShowDisabledEffectAttribute); }

 private:
	TSlateAttribute<const FSlateBrush*> BorderImageAttribute;
	TSlateAttribute<FSlateColor> BorderBackgroundColorAttribute;
	TSlateAttribute<FVector2D> DesiredSizeScaleAttribute;
	/** Whether or not to show the disabled effect when this border is disabled */
	TSlateAttribute<bool> ShowDisabledEffectAttribute;

	/** Flips the image if the localization's flow direction is RightToLeft */
	bool bFlipForRightToLeftFlowDirection;
 };
