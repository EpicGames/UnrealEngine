// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Layout/Children.h"
#include "Types/SlateStructs.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

template <>
struct TWidgetTypeTraits<class SBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * SBox is the simplest layout element.
 */
class SBox : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SBox, SPanel, SLATE_API)

public:
	SLATE_BEGIN_ARGS(SBox)
		: _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Fill)
		, _Padding(0.0f)
		, _Content()
		, _WidthOverride(FOptionalSize())
		, _HeightOverride(FOptionalSize())
		, _MinDesiredWidth(FOptionalSize())
		, _MinDesiredHeight(FOptionalSize())
		, _MaxDesiredWidth(FOptionalSize())
		, _MaxDesiredHeight(FOptionalSize())
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

	/** Horizontal alignment of content in the area allotted to the SBox by its parent */
	SLATE_ARGUMENT(EHorizontalAlignment, HAlign)

	/** Vertical alignment of content in the area allotted to the SBox by its parent */
	SLATE_ARGUMENT(EVerticalAlignment, VAlign)

	/** Padding between the SBox and the content that it presents. Padding affects desired size. */
	SLATE_ATTRIBUTE(FMargin, Padding)

	/** The widget content presented by the SBox */
	SLATE_DEFAULT_SLOT(FArguments, Content)

	/** When specified, ignore the content's desired size and report the WidthOverride as the Box's desired width. */
	SLATE_ATTRIBUTE(FOptionalSize, WidthOverride)

	/** When specified, ignore the content's desired size and report the HeightOverride as the Box's desired height. */
	SLATE_ATTRIBUTE(FOptionalSize, HeightOverride)

	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)

	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredHeight)

	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredWidth)

	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredHeight)

	SLATE_ATTRIBUTE(FOptionalSize, MinAspectRatio)

	SLATE_ATTRIBUTE(FOptionalSize, MaxAspectRatio)
	
	FArguments& Padding(float Uniform)
	{
		_Padding = FMargin(Uniform);
		return *this;
	}

	FArguments& Padding(float Horizontal, float Vertical)
	{
		_Padding = FMargin(Horizontal, Vertical);
		return *this;
	}

	FArguments& Padding(float Left, float Top, float Right, float Bottom)
	{
		_Padding = FMargin(Left, Top, Right, Bottom);
		return *this;
	}

	SLATE_END_ARGS()

	SLATE_API SBox();
	SLATE_API virtual ~SBox();

	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/**
		* See the Content slot.
		*/
	SLATE_API void SetContent(const TSharedRef< SWidget >& InContent);

	/** See HAlign argument */
	SLATE_API void SetHAlign(EHorizontalAlignment HAlign);

	/** See VAlign argument */
	SLATE_API void SetVAlign(EVerticalAlignment VAlign);

	/** See Padding attribute */
	SLATE_API void SetPadding(TAttribute<FMargin> InPadding);

	/** See WidthOverride attribute */
	SLATE_API void SetWidthOverride(TAttribute<FOptionalSize> InWidthOverride);

	/** See HeightOverride attribute */
	SLATE_API void SetHeightOverride(TAttribute<FOptionalSize> InHeightOverride);

	/** See MinDesiredWidth attribute */
	SLATE_API void SetMinDesiredWidth(TAttribute<FOptionalSize> InMinDesiredWidth);

	/** See MinDesiredHeight attribute */
	SLATE_API void SetMinDesiredHeight(TAttribute<FOptionalSize> InMinDesiredHeight);

	/** See MaxDesiredWidth attribute */
	SLATE_API void SetMaxDesiredWidth(TAttribute<FOptionalSize> InMaxDesiredWidth);

	/** See MaxDesiredHeight attribute */
	SLATE_API void SetMaxDesiredHeight(TAttribute<FOptionalSize> InMaxDesiredHeight);

	SLATE_API void SetMinAspectRatio(TAttribute<FOptionalSize> InMinAspectRatio);

	SLATE_API void SetMaxAspectRatio(TAttribute<FOptionalSize> InMaxAspectRatio);

protected:
	// Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API float ComputeDesiredWidth() const;
	SLATE_API float ComputeDesiredHeight() const;
	// End SWidget overrides.

protected:

	struct FBoxOneChildSlot : ::TSingleWidgetChildrenWithBasicLayoutSlot<EInvalidateWidgetReason::None> // we want to add it to the Attribute descriptor
	{
		friend SBox;
		using ::TSingleWidgetChildrenWithBasicLayoutSlot<EInvalidateWidgetReason::None>::TSingleWidgetChildrenWithBasicLayoutSlot;
	};
	FBoxOneChildSlot ChildSlot;

private:
	/** When specified, ignore the content's desired size and report the.WidthOverride as the Box's desired width. */
	TSlateAttribute<FOptionalSize> WidthOverride;

	/** When specified, ignore the content's desired size and report the.HeightOverride as the Box's desired height. */
	TSlateAttribute<FOptionalSize> HeightOverride;

	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	TSlateAttribute<FOptionalSize> MinDesiredWidth;

	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	TSlateAttribute<FOptionalSize> MinDesiredHeight;

	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	TSlateAttribute<FOptionalSize> MaxDesiredWidth;

	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	TSlateAttribute<FOptionalSize> MaxDesiredHeight;

	TSlateAttribute<FOptionalSize> MinAspectRatio;

	TSlateAttribute<FOptionalSize> MaxAspectRatio;
};
