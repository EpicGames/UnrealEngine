// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Layout/Geometry.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Widgets/Layout/Anchors.h"

#define UE_API SLATE_API

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * ConstraintCanvas is a layout widget that allows you to arbitrary position and size child widgets in a 
 * relative coordinate space.  Additionally it permits anchoring widgets.
 */
class SConstraintCanvas : public SPanel
{
public:

	/**
	 * ConstraintCanvas slots allow child widgets to be positioned and sized
	 */
	class FSlot : public TSlotBase<FSlot>
	{
	public:
		/** Default values for a slot. */
		FSlot()
			: TSlotBase<FSlot>()
			, OffsetAttr(FMargin(0, 0, 1, 1))
			, AnchorsAttr(FAnchors(0.0f, 0.0f))
			, AlignmentAttr(FVector2D(0.5f, 0.5f))
			, AutoSizeAttr(false)
			, ZOrder(0)
		{ }

		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
			SLATE_ATTRIBUTE(FMargin, Offset)
			SLATE_ATTRIBUTE(FAnchors, Anchors)
			SLATE_ATTRIBUTE(FVector2D, Alignment)
			SLATE_ATTRIBUTE(bool, AutoSize)
			SLATE_ARGUMENT(TOptional<float>, ZOrder)
		SLATE_SLOT_END_ARGS()

		UE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);

		void SetOffset( const TAttribute<FMargin>& InOffset )
		{
			SetAttribute(OffsetAttr, InOffset, EInvalidateWidgetReason::Layout);
		}

		FMargin GetOffset() const
		{
			return OffsetAttr.Get();
		}

		void SetAnchors( const TAttribute<FAnchors>& InAnchors )
		{
			SetAttribute(AnchorsAttr, InAnchors, EInvalidateWidgetReason::Layout);
		}

		FAnchors GetAnchors() const
		{
			return AnchorsAttr.Get();
		}

		void SetAlignment(const TAttribute<FVector2D>& InAlignment)
		{
			SetAttribute(AlignmentAttr, InAlignment, EInvalidateWidgetReason::Layout);
		}

		FVector2D GetAlignment() const
		{
			return AlignmentAttr.Get();
		}

		void SetAutoSize(const TAttribute<bool>& InAutoSize)
		{
			SetAttribute(AutoSizeAttr, InAutoSize, EInvalidateWidgetReason::Layout);
		}

		bool GetAutoSize() const
		{
			return AutoSizeAttr.Get();
		}

		UE_API void SetZOrder(float InZOrder);

		float GetZOrder() const
		{
			return ZOrder;
		}

	private:
		/** Offset */
		TAttribute<FMargin> OffsetAttr;
		/** Anchors */
		TAttribute<FAnchors> AnchorsAttr;
		/** Size */
		TAttribute<FVector2D> AlignmentAttr;
		/** Auto-Size */
		TAttribute<bool> AutoSizeAttr;
	private:
		/** Z-Order */
		float ZOrder;
	};

	SLATE_BEGIN_ARGS( SConstraintCanvas )
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		SLATE_SLOT_ARGUMENT( FSlot, Slots)

	SLATE_END_ARGS()

	SLATE_API SConstraintCanvas();
	SLATE_API ~SConstraintCanvas();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	static SLATE_API FSlot::FSlotArguments Slot();

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Adds a content slot.
	 *
	 * @return The added slot.
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot();

	/**
	 * Removes a particular content slot.
	 *
	 * @param SlotWidget The widget in the slot to remove.
	 */
	SLATE_API int32 RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	/**
	 * Removes all slots from the panel.
	 */
	SLATE_API void ClearChildren();

public:

	// Begin SWidget overrides
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	// End SWidget overrides

protected:
	// Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

private:

	/** An array matching the length and order of ArrangedChildren. True means the child must be placed in a layer in front of all previous children. */
	typedef TArray<bool, TInlineAllocator<16>> FArrangedChildLayers;

	/** Like ArrangeChildren but also generates an array of layering information (see FArrangedChildLayers). */
	void ArrangeLayeredChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, FArrangedChildLayers& ArrangedChildLayers) const;

protected:

	/** The ConstraintCanvas widget's children. */
	TPanelChildren< FSlot > Children;
};

#undef UE_API
