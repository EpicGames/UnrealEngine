// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

/**
 * Controls the DPI scale of its content. Can be used for zooming or shrinking arbitrary widget content.
 */
class SDPIScaler : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SDPIScaler, SPanel, SLATE_API)

public:
	SLATE_BEGIN_ARGS( SDPIScaler )
		: _DPIScale( 1.0f )
		, _Content()
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/**
		 * The scaling factor between Pixels and Slate Units (SU).
		 * For example, a value of 2 means 2 pixels for every linear Slate Unit and 4 pixels for every square Slate Unit.
		 */
		SLATE_ATTRIBUTE( float, DPIScale )

		/** The content being presented and scaled. */
		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	SLATE_API SDPIScaler();
	SLATE_API virtual ~SDPIScaler();

	SLATE_API void Construct( const FArguments& InArgs );

	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	SLATE_API virtual FChildren* GetChildren() override;

	/** See the Content attribute */
	SLATE_API void SetContent(TSharedRef<SWidget> InContent);

	/** See the DPIScale attribute */
	SLATE_API void SetDPIScale(TAttribute<float> InDPIScale);

protected:

	SLATE_API virtual float GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const override;

	TSlateAttributeRef<float> GetDPIScaleAttribute() const { return TSlateAttributeRef<float>(SharedThis(this), DPIScaleAttribute); }

	struct FDPIScalerOneChildSlot : ::TSingleWidgetChildrenWithBasicLayoutSlot<EInvalidateWidgetReason::None> // we want to add it to the Attribute descriptor
	{
		friend SDPIScaler;
		using ::TSingleWidgetChildrenWithBasicLayoutSlot<EInvalidateWidgetReason::None>::TSingleWidgetChildrenWithBasicLayoutSlot;
	};

	/** The content being scaled. */
	FDPIScalerOneChildSlot ChildSlot;

private:
	/**
	 * The scaling factor from 1:1 pixel to slate unit (SU) ratio.
	 * e.g. Value of 2 would mean that there are two pixels for every slate unit in every dimension.
	 *      This means that every 1x1 SU square is represented by 2x2=4 pixels.
	 */
	TSlateAttribute<float> DPIScaleAttribute;
};
