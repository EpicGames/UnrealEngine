// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * A SEnableBox contains a widget that is lied to about whether the parent hierarchy is enabled or not, being told the parent is always enabled
 */
class SEnableBox : public SBox
{
public:
	SLATE_BEGIN_ARGS(SEnableBox) {}
		/** The widget content to be presented as if the parent were enabled */
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SBox::Construct(SBox::FArguments().Content()[InArgs._Content.Widget]);
	}

public:

	// SWidget interface
	
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return SBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, /*bParentEnabled=*/ true);
	}
};
