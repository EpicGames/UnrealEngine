// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"

class SBorder;
class SButton;

template<>
struct TWidgetTypeTraits<class SExpandableArea>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Represents an expandable area of content            
 */
class SExpandableArea : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SExpandableArea )
		: _Style( &FCoreStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea") )
		, _BorderBackgroundColor( FLinearColor::White )
		, _BorderImage( FCoreStyle::Get().GetBrush("ExpandableArea.Border") )
		, _BodyBorderBackgroundColor(FLinearColor::White)
		, _BodyBorderImage()
		, _AreaTitle( )
		, _InitiallyCollapsed( false )
		, _MinWidth( 0.0f )
		, _MaxHeight( 0.0f )
		, _AreaTitlePadding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
		, _HeaderPadding(FMargin(4.0f, 2.0f))
		, _Padding( 1.0f )
		, _AreaTitleFont( FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont") )
		, _HeaderCursor( )
		, _AllowAnimatedTransition(true)
		{}
		
		/** Style used to draw this area */
		SLATE_STYLE_ARGUMENT( FExpandableAreaStyle, Style )
		/** Background color to apply to the border image */
		SLATE_ATTRIBUTE( FSlateColor, BorderBackgroundColor )
		/** Border to use around the area */
		SLATE_ATTRIBUTE( const FSlateBrush*, BorderImage )
		/** Background color to apply to the body's border image. Unspecified uses BorderBackgroundColor */
		SLATE_ATTRIBUTE( FSlateColor, BodyBorderBackgroundColor )
		/** Border to use around the body. Unspecified uses BorderImage */
		SLATE_ARGUMENT( const FSlateBrush*, BodyBorderImage )
		/** Content displayed next to the expansion arrow.  This is always visible */
		SLATE_NAMED_SLOT( FArguments, HeaderContent )
		/** Content displayed inside the area that is expanded */
		SLATE_NAMED_SLOT( FArguments, BodyContent )
		/** The title to display.  Not used if header content is supplied */
		SLATE_ATTRIBUTE( FText, AreaTitle )
		/** Whether or not the area is initially collapsed */
		SLATE_ARGUMENT( bool, InitiallyCollapsed )
		/** The minimum width of the area */
		SLATE_ARGUMENT( float, MinWidth )
		/** The maximum height of the area */
		SLATE_ARGUMENT( float, MaxHeight )
		/** The title padding */
		SLATE_ATTRIBUTE(FMargin, AreaTitlePadding)
		/** The header padding */
		SLATE_ATTRIBUTE( FMargin, HeaderPadding )
		/** The content padding */
		SLATE_ATTRIBUTE( FMargin, Padding )
		/** Called when the area is expanded or collapsed */
		SLATE_EVENT( FOnBooleanValueChanged, OnAreaExpansionChanged )
		/** Sets the font used to draw the title text */
		SLATE_ATTRIBUTE( FSlateFontInfo, AreaTitleFont )
		/** Override for Cursor, so you can specify a different cursor for the header */
		SLATE_ATTRIBUTE( TOptional<EMouseCursor::Type>, HeaderCursor )
		/** Whether or not the animated effect is used when opening or closing the area */
		SLATE_ARGUMENT(bool, AllowAnimatedTransition)
			
	SLATE_END_ARGS()


public:

	/**
	 * Constructs a new widget.
	 *
	 * @param InArgs Construction arguments.
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * @return true if the area is currently expanded
	 */
	virtual bool IsExpanded() const { return !bAreaCollapsed; }

	/**
	 * Instantly sets the expanded state of the area
	 */
	SLATE_API virtual void SetExpanded( bool bExpanded );

	SLATE_API virtual void SetExpanded_Animated( bool bExpanded );

	/**
	 * Returns true if area title is hovered, false otherwise.
	 */
	SLATE_API virtual bool IsTitleHovered() const;

	/**
	 * Sets the max height of the area
	 */
	SLATE_API void SetMaxHeight(float InMaxHeight);

	/**
	 * Sets the color of all borders
	 */
	SLATE_API void SetBorderBackgroundColor(const FSlateColor& InBorderColor);

	/**
	 * Sets the header padding
	 */
	SLATE_API void SetHeaderPadding(FMargin InHeaderPadding);

	/**
	 * Sets the body padding
	 */
	SLATE_API void SetAreaPadding(FMargin InAreaPadding);

	/**
	 * Sets the style of the widget
	 */
	SLATE_API void SetStyle(const FExpandableAreaStyle* InStyle);

	/**
	 * Invalidates the style
	 */
	UE_DEPRECATED(5.8, "Changing the style needs to update other properties. Use SetStyle if the style has changed")
	SLATE_API void InvalidateStyle();

	/**
	 * Invalidates the border brush
	 */
	UE_DEPRECATED(5.8, "No need to call this function anymore, use SetBorderBrush")
	SLATE_API void InvalidateBorderBrush();

	/**
	 * Sets the brush of all borders
	 */
	SLATE_API void SetBorderBrush(const FSlateBrush* InBorderBrush);

protected:

	/**
	 * Constructs the header area widget
	 * 
	 * @param InArgs	Construction arguments
	 * @param HeaderContent	User specified header content to display
	 */
	SLATE_API virtual TSharedRef<SWidget> ConstructHeaderWidget( const FArguments& InArgs, TSharedRef<SWidget> HeaderContent );

	/**
	 * @return The visibility of this section                   
	 */
	SLATE_API virtual EVisibility OnGetContentVisibility() const;

	SLATE_API FReply OnHeaderClicked();

	/** Toggles selection visibility when the panel is clicked. */
	SLATE_API virtual void OnToggleContentVisibility();

	/**
	 * @return The collapsed/expanded image we should show.                     
	 */
	SLATE_API virtual const FSlateBrush* OnGetCollapseImage() const;

	/**
	 * @return The scale of the content inside this section.  Animated 
	 */
	SLATE_API virtual FVector2D GetSectionScale() const;

	/**
	 * Computes the desired size of this area. Optionally clamping to Max height
	 */
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

protected:

	/** Style of the Expandable Area */
	const FExpandableAreaStyle* WidgetStyle;

	/** Border widget for the header area */
	TSharedPtr<SBorder> TitleBorder;

	/** Border widget if the body style is different from header */
	TSharedPtr<SBorder> WidgetBorder;

	/** Border widget for the body area */
	TSharedPtr<SBorder> BodyBorder;

	/** Button for expanding the are */
	TSharedPtr<SButton> ExpandingButton;

	/** Curved used to simulate a rollout of the section */
	FCurveSequence RolloutCurve;

	/** Delegate that fires when the area becomes expanded or collapsed the user */
	FOnBooleanValueChanged OnAreaExpansionChanged;

	/** The minimum width of this area */
	float MinWidth;

	/** The maximum height of this area */
	float MaxHeight;

	/** true if the section is visible */
	bool bAreaCollapsed;

	/** Image to use when the area is collapsed */
	const FSlateBrush* CollapsedImage;

	/** Image to use when the area is expanded */
	const FSlateBrush* ExpandedImage;

	/** If true expanding and collapsing the area will animate, otherwise it will expand/collapse instantly */
	bool bAllowAnimatedTransition;

};
