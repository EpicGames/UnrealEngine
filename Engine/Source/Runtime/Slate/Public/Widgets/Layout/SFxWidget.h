// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * Enables visual effects (zooming, sliding, fading, etc.) to be applied to arbitrary widget content.
 * 
 * Unless specified properties do not affect layout.
 */
class SFxWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SFxWidget )
		: _RenderScale( 1.0f )
		, _RenderScaleOrigin( FVector2D::ZeroVector )
		, _LayoutScale( 1.0f )
		, _VisualOffset( FVector2D::ZeroVector )
		, _IgnoreClipping( true )
		, _ColorAndOpacity( FLinearColor::White )
		, _HAlign( HAlign_Center )
		, _VAlign( VAlign_Center )
		, _Content()
	{}

		/** Scale the visuals of this widget. Geometry is not affected. */
		SLATE_ATTRIBUTE( float, RenderScale )

		/** The origin of the visual scale transform in the 0..1 range (0 being upper left, 1 being lower right)  */
		SLATE_ATTRIBUTE( FVector2D, RenderScaleOrigin )

		/** Just like visual scale only affects Geometry. */
		SLATE_ATTRIBUTE( float, LayoutScale )

		/** Offset the widget by some fraction of its size in either dimension. */
		SLATE_ATTRIBUTE( FVector2D, VisualOffset )

		/** Should the FX widget disable all clipping and show through regardless of its parents' bounds.*/
		SLATE_ATTRIBUTE( bool, IgnoreClipping )

		/** Multiply the contents of the SFxWidget by this color and opacity when drawing */
		SLATE_ATTRIBUTE( FLinearColor, ColorAndOpacity )

		/** The horizontal alignment of the child widget */
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )

		/** The vertical alignment of the child widget */
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )

		/** The content that should be modified. */
		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	SLATE_API SFxWidget();

	SLATE_API void Construct( const FArguments& InArgs );

	/** @see VisualOffset */
	SLATE_API void SetVisualOffset( TAttribute<FVector2D> InOffset );

	/** @see VisualOffset */
	SLATE_API void SetVisualOffset( FVector InOffset );

	/** @see RenderScale */
	SLATE_API void SetRenderScale( TAttribute<float> InScale );

	/** @see RenderScale */
	SLATE_API void SetRenderScale( float InScale );

	

protected:
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	TSlateAttribute<float, EInvalidateWidgetReason::Paint> RenderScale;
	TSlateAttribute<FVector2D, EInvalidateWidgetReason::Paint> RenderScaleOrigin;
	TSlateAttribute<float, EInvalidateWidgetReason::Layout> LayoutScale;
	TSlateAttribute<FVector2D, EInvalidateWidgetReason::Paint> VisualOffset;
	TSlateAttribute<bool, EInvalidateWidgetReason::Paint> bIgnoreClipping;
};
