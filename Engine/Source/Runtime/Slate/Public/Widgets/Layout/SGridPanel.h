// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

class SGridPanel : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SGridPanel, SPanel, SLATE_API)

public:
	// Used by the mandatory named parameter in FSlot
	class Layer
	{
	public:
		explicit Layer(int32 InLayer)
			: TheLayer(InLayer)
		{

		}

		int32 TheLayer;
	};

	class FSlot : public TBasicLayoutWidgetSlot<FSlot>
	{
		friend SGridPanel;

		public:
			/** Default values for a slot. */
			FSlot( int32 Column, int32 Row, int32 InLayer )
				: TBasicLayoutWidgetSlot<FSlot>(HAlign_Fill, VAlign_Fill)
				, ColumnParam( Column )
				, ColumnSpanParam( 1 )
				, RowParam( Row )
				, RowSpanParam( 1 )
				, LayerParam( InLayer )
				, NudgeParam( FVector2D::ZeroVector )
			{
			}

			SLATE_SLOT_BEGIN_ARGS(FSlot, TBasicLayoutWidgetSlot<FSlot>)
				/** Which column in the grid this cell belongs to */
				SLATE_ARGUMENT(TOptional<int32>, Column)
				/** How many columns this slot spans over */
				SLATE_ARGUMENT(TOptional<int32>, ColumnSpan)
				/** Which row in the grid this cell belongs to */
				SLATE_ARGUMENT(TOptional<int32>, Row)
				/** How many rows this this slot spans over */
				SLATE_ARGUMENT(TOptional<int32>, RowSpan)
				/** Positive values offset this cell to be hit-tested and drawn on top of others. Default is 0; i.e. no offset. */
				SLATE_ARGUMENT(TOptional<int32>, Layer)
				/** Offset this slot's content by some amount; positive values offset to lower right*/
				SLATE_ARGUMENT(TOptional<FVector2D>, Nudge)
			SLATE_SLOT_END_ARGS()

			SLATE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);

		public:
			/** Which column in the grid this cell belongs to */
			int32 GetColumn() const
			{
				return ColumnParam;
			}

			void SetColumn(int32 Column)
			{
				Column = FMath::Max(0, Column);
				if (Column != ColumnParam)
				{
					ColumnParam = Column;
					NotifySlotChanged();
				}
			}

			/** How many columns this slot spans over */
			int32 GetColumnSpan() const
			{
				return ColumnSpanParam;
			}

			void SetColumnSpan(int32 ColumnSpan)
			{
				// clamp span to a sensible size, otherwise computing slot sizes can slow down dramatically
				ColumnSpan = FMath::Clamp(ColumnSpan, 1, 10000);
				if (ColumnSpan != ColumnSpanParam)
				{
					ColumnSpanParam = ColumnSpan;
					NotifySlotChanged();
				}
			}

			/** Which row in the grid this cell belongs to */
			int32 GetRow() const
			{
				return RowParam;
			}

			void SetRow(int32 Row)
			{
				Row = FMath::Max(0, Row);
				if (Row != RowParam)
				{
					RowParam = Row;
					NotifySlotChanged();
				}
			}

			/** How many rows this this slot spans over */
			int32 GetRowSpan() const
			{
				return RowSpanParam;
			}

			void SetRowSpan(int32 RowSpan)
			{
				// clamp span to a sensible size, otherwise computing slots sizes can slow down dramatically
				RowSpan = FMath::Clamp(RowSpan, 1, 10000);
				if (RowSpan != RowSpanParam)
				{
					RowSpanParam = RowSpan;
					NotifySlotChanged();
				}
			}

			/** Positive values offset this cell to be hit-tested and drawn on top of others. Default is 0; i.e. no offset. */
			int32 GetLayer() const
			{
				return LayerParam;
			}

			void SetLayer(int32 Layer)
			{
				if (Layer != LayerParam)
				{
					LayerParam = Layer;
					const bool bSlotLayerChanged = true;
					NotifySlotChanged(bSlotLayerChanged);
				}
			}

			/** Offset this slot's content by some amount; positive values offset to lower right */
			FVector2D GetNudge() const
			{
				return NudgeParam;
			}

			void SetNudge(const FVector2D& Nudge)
			{
				NudgeParam = Nudge;
				Invalidate(EInvalidateWidgetReason::Paint);
			}

		private:
			/** The panel that contains this slot */
			TWeakPtr<SGridPanel> Panel;

			int32 ColumnParam;
			int32 ColumnSpanParam;
			int32 RowParam;
			int32 RowSpanParam;
			int32 LayerParam;
			FVector2D NudgeParam;

			/** Notify that the slot was changed */
			inline void NotifySlotChanged(bool bSlotLayerChanged = false)
			{
				if ( Panel.IsValid() )
				{
					Panel.Pin()->NotifySlotChanged(this, bSlotLayerChanged);
				}
			}
	};

	/**
	 * Used by declarative syntax to create a Slot in the specified Column, Row and Layer.
	 */
	static SLATE_API FSlot::FSlotArguments Slot( int32 Column, int32 Row, Layer InLayer = Layer(0) );

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Dynamically add a new slot to the UI at specified Column and Row. Optionally, specify a layer at which this slot should be added.
	 *
	 * @return A reference to the newly-added slot
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot( int32 Column, int32 Row, Layer InLayer = Layer(0) );

	/**
	* Removes a slot from this panel which contains the specified SWidget
	*
	* @param SlotWidget The widget to match when searching through the slots
	* @returns The true if the slot was removed and false if no slot was found matching the widget
	*/
	SLATE_API bool RemoveSlot(const TSharedRef<SWidget>& SlotWidget);

	SLATE_BEGIN_ARGS( SGridPanel )
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		SLATE_SLOT_ARGUMENT( FSlot, Slots )

		/** Specify a column to stretch instead of sizing to content. */
		FArguments& FillColumn( int32 ColumnId, const TAttribute<float>& Coefficient )
		{
			while (ColFillCoefficients.Num() <= ColumnId)
			{
				ColFillCoefficients.Emplace( 0 );
			}
			ColFillCoefficients[ColumnId] = Coefficient;
			return Me();
		}

		/** Specify a row to stretch instead of sizing to content. */
		FArguments& FillRow( int32 RowId, const TAttribute<float>& Coefficient )
		{
			while (RowFillCoefficients.Num() <= RowId)
			{
				RowFillCoefficients.Emplace( 0 );
			}
			RowFillCoefficients[RowId] = Coefficient;
			return Me();
		}

		/** Coefficients for columns that need to stretch instead of size to content */
		TArray<TAttribute<float>> ColFillCoefficients;
		
		/** Coefficients for rows that need to stretch instead of size to content */
		TArray<TAttribute<float>> RowFillCoefficients;
		
	SLATE_END_ARGS()

	SLATE_API SGridPanel();
	SLATE_API virtual ~SGridPanel();

	/** Removes all slots from the panel */
	SLATE_API void ClearChildren();
	
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * GetDesiredSize of a subregion in the graph.
	 *
	 * @param StartCell   The cell (inclusive) in the upper left of the region.
	 * @param Size        Number of cells in the X and Y directions to get the size for.
	 *
	 * @return FVector2D  The desired size of the region of cells specified.
	 */
	SLATE_API FVector2D GetDesiredRegionSize( const FIntPoint& StartCell, int32 Width, int32 Height ) const;

	/** Specify a column to stretch instead of sizing to content. */
	SLATE_API void SetColumnFill( int32 ColumnId, const TAttribute<float>& Coefficient );

	/** Specify a row to stretch instead of sizing to content. */
	SLATE_API void SetRowFill( int32 RowId, const TAttribute<float>& Coefficient );

	/** Clear the row and column fill rules. */
	SLATE_API void ClearFill();

public:

	// SWidget interface

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual void CacheDesiredSize(float) override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual FChildren* GetChildren() override;

private:

	/**
	 * Given an array of values, re-populate the array such that every contains the partial sums up to that element.
	 * i.e. Array[N] = Array.Sum(0 .. N-1)
	 *
	 * The resulting array is 1-element longer.
	 */
	static SLATE_API void ComputePartialSums( TArray<float>& TurnMeIntoPartialSums );
	
	/** Given a SizeContribution, distribute it to the elements in DistributeOverMe at indexes from [StartIndex .. UpperBound) */
	static SLATE_API void DistributeSizeContributions( float SizeContribution, TArray<float>& DistributeOverMe, int32 StartIndex, int32 UpperBound );

	/**
	 * Find the index where the given slot should be inserted into the list of Slots based on its LayerParam, such that Slots are sorter by layer.
	 *
	 * @param The newly-allocated slot to insert.
	 * @return The index where the slot should be inserted.
	 */
	SLATE_API int32 FindInsertSlotLocation( const FSlot* InSlot );

	/** Compute the sizes of columns and rows needed to fit all the slots in this grid. */
	SLATE_API void ComputeDesiredCellSizes( TArray<float>& OutColumns, TArray<float>& OutRows ) const;

	/** Draw the debug grid of rows and colummns; useful for inspecting the GridPanel's logic. See OnPaint() for parameter meaning */
	SLATE_API int32 LayoutDebugPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId ) const;
	
	/** 
	 * Callback used to resize our internal arrays if a slot (or slot span) is changed. 
	 *
	 * @param InSlot The slot that has just changed.
	 * @param bSlotLayerChanged Whether the slot layer changed.
	 */
	SLATE_API void NotifySlotChanged(const FSlot* InSlot, bool bSlotLayerChanged = false);

private:

	/** The slots that are placed into various grid locations */
	TPanelChildren<FSlot> Slots;

	/**
	 * Offsets of each column from the beginning of the grid.
	 * Includes a faux value at the end of the array for finding the size of the last cell.
	 */
	TArray<float> Columns;
	
	/**
	 * Offsets of each row from the beginning of the grid.
	 * Includes a faux value at the end of the array for finding the size of the last cell.
	 */
	TArray<float> Rows;

	/** Total desires size along each axis. */
	FVector2D TotalDesiredSizes;

protected:
	/** Fill coefficients for the columns */
	mutable TArray<TAttribute<float>> ColFillCoefficients;

	/** Fill coefficients for the rows */
	mutable TArray<TAttribute<float>> RowFillCoefficients;
};