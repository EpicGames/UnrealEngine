// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"

/**
 * A heading for a section of content.
 * Looks like this: ------- Content --------
 */
class SHeader : public SHorizontalBox
{
public:
	SLATE_BEGIN_ARGS(SHeader)
		: _HAlign( HAlign_Left )
		, _Content()
		{}

		/**
		 * How the content should be aligned:
		 * HAlign_Left : -- Content -----------
		 * HAlign_Right: ----------- Content --
		 * Otherwise: ----- Content -----
		 */
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )

		/**
		 * The content to show in the header;
		 * usually some text describing a section of content.
		 */
		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );
 };
