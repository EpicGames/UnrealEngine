// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Layout/SLinkedBox.h"

#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/Application/SlateApplication.h"


FLinkedBoxManager::FLinkedBoxManager()
: CachedUniformSize(FVector2D::ZeroVector)
{
}

FLinkedBoxManager::~FLinkedBoxManager()
{
}

void FLinkedBoxManager::RegisterLinkedBox(SLinkedBox* InBox)
{
	Siblings.Add(InBox);	
}

void FLinkedBoxManager::UnregisterLinkedBox(SLinkedBox* InBox)
{
	Siblings.Remove(InBox);	
}

FVector2D FLinkedBoxManager::GetUniformCellSize() const
{
	const uint32 CurrentDrawId = FSlateApplication::Get().GetCurrentDrawId();

	if (SlateApplicationDrawIdLastCached == CurrentDrawId)
	{
		return CachedUniformSize;
	}

	FVector2D MaxDesiredSize = FVector2D::ZeroVector;
	for (auto Sibling : Siblings)
	{
		if (Sibling->GetVisibility() != EVisibility::Collapsed)
		{
			Sibling->CustomChildPrepass();
			FVector2D SiblingDesiredSize = Sibling->GetChildrensDesiredSize();
			if (!SiblingDesiredSize.IsZero())
			{
				MaxDesiredSize.X = FMath::Max( MaxDesiredSize.X, SiblingDesiredSize.X );
				MaxDesiredSize.Y = FMath::Max( MaxDesiredSize.Y, SiblingDesiredSize.Y );
			}
		}
	}

	SlateApplicationDrawIdLastCached = CurrentDrawId;
	CachedUniformSize = MaxDesiredSize;
	return CachedUniformSize;
}

SLinkedBox::SLinkedBox()
{
	bHasCustomPrepass = true;
}

SLinkedBox::~SLinkedBox()
{
	Manager->UnregisterLinkedBox(this);
}

void SLinkedBox::Construct( const FArguments& InArgs, TSharedRef<FLinkedBoxManager> InManager )
{
	Manager = InManager;
	Manager->RegisterLinkedBox(this);

	SBox::Construct( SBox::FArguments()
		.HAlign(InArgs._HAlign)
		.VAlign(InArgs._VAlign)
		.Padding(InArgs._Padding)
		.WidthOverride(InArgs._WidthOverride)
		.HeightOverride(InArgs._HeightOverride)
		.MinDesiredWidth(InArgs._MinDesiredWidth)
		.MinDesiredHeight(InArgs._MinDesiredHeight)
		.MaxDesiredWidth(InArgs._MaxDesiredWidth)
		.MaxDesiredHeight(InArgs._MaxDesiredHeight)
		.MinAspectRatio(InArgs._MinAspectRatio)
		.MaxAspectRatio(InArgs._MaxAspectRatio)
		[
			InArgs._Content.Widget
		]
	);
}

void SLinkedBox::CustomChildPrepass()
{
	ChildSlot.GetWidget()->SlatePrepass();	
}

bool SLinkedBox::CustomPrepass(float LayoutScaleMultiplier) 
{ 
	return false; 
}

FVector2D SLinkedBox::GetChildrensDesiredSize() const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if ( ChildVisibility != EVisibility::Collapsed )
	{
		return ChildSlot.GetWidget()->GetDesiredSize() + ChildSlot.GetPadding().GetDesiredSize();
	}
	return FVector2D::ZeroVector;
}

FVector2D SLinkedBox::ComputeDesiredSize(float) const
{
	return Manager->GetUniformCellSize();
}

