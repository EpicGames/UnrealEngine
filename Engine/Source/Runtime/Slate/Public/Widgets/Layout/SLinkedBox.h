// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Containers/Ticker.h"
#include "Layout/Margin.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SBox.h"
	

class FArrangedChildren;

class SLinkedBox;

/** 
 * Manages a group of SLinkedBoxes that report the same size based on the largest size of any LinkedBoxes in this managed group.
 *  
 *  FLinkedBoxManager needs to be created and passed to each LinkedBox during construction.
 */
class FLinkedBoxManager : public TSharedFromThis<FLinkedBoxManager> 
{

public: 

	SLATE_API FLinkedBoxManager();
	SLATE_API ~FLinkedBoxManager();

	/* Add an SLinkedBox */
	SLATE_API void RegisterLinkedBox(SLinkedBox* InBox);

	/* Remove an SLinkedBox */
	SLATE_API void UnregisterLinkedBox(SLinkedBox* InBox);

	/* Used by the individual SLinkedBoxes to acquire the computed uniform size */
	SLATE_API FVector2D GetUniformCellSize() const;

protected:
	
	mutable uint32 SlateApplicationDrawIdLastCached = 0;
	mutable FVector2D CachedUniformSize;

private:

	TSet< SLinkedBox* > Siblings;

};

/** A panel that */
class SLinkedBox: public SBox
{

public:

	SLATE_BEGIN_ARGS(SLinkedBox)
		: _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Fill)
		, _Padding(0.0f)
		, _Content()
		, _WidthOverride(FOptionalSize())
		, _HeightOverride(FOptionalSize())
		, _MinDesiredWidth(FOptionalSize())
		, _MinDesiredHeight(FOptionalSize())
		, _MaxDesiredWidth(FOptionalSize())
		, _MaxDesiredHeight(FOptionalSize())
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

		/** Horizontal alignment of content in the area allotted to the SBox by its parent */
	SLATE_ARGUMENT(EHorizontalAlignment, HAlign)

	/** Vertical alignment of content in the area allotted to the SBox by its parent */
	SLATE_ARGUMENT(EVerticalAlignment, VAlign)

	/** Padding between the SBox and the content that it presents. Padding affects desired size. */
	SLATE_ATTRIBUTE(FMargin, Padding)

	/** The widget content presented by the SBox */
	SLATE_DEFAULT_SLOT(FArguments, Content)

	/** When specified, ignore the content's desired size and report the WidthOverride as the Box's desired width. */
	SLATE_ATTRIBUTE(FOptionalSize, WidthOverride)

	/** When specified, ignore the content's desired size and report the HeightOverride as the Box's desired height. */
	SLATE_ATTRIBUTE(FOptionalSize, HeightOverride)

	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)

	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredHeight)

	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredWidth)

	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredHeight)

	SLATE_ATTRIBUTE(FOptionalSize, MinAspectRatio)

	SLATE_ATTRIBUTE(FOptionalSize, MaxAspectRatio)

	SLATE_END_ARGS()

	SLATE_API FVector2D GetChildrensDesiredSize() const;

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	/**
	 * Most panels do not create widgets as part of their implementation, so
	 * they do not need to implement a Construct()
	 */
	SLATE_API void Construct( const FArguments& InArgs, TSharedRef<FLinkedBoxManager> InManager ); 

	SLATE_API SLinkedBox();
	SLATE_API ~SLinkedBox();
	
protected:

	/**
	 *  CustomPrepass - Returns false so instead of
	 *  each SLinkedBox prepass being called in the usual depth first order, 
	 *  the Manager can call a prepass on all of the sibling 
	 *  LinkedBoxes at once.  
	*/
	SLATE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;

	SLATE_API void CustomChildPrepass();

	friend class FLinkedBoxManager;

private:

	TSharedPtr<FLinkedBoxManager> Manager;

};
