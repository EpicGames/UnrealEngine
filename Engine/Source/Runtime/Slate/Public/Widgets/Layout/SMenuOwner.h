// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SMenuAnchor.h"

template <>
struct TWidgetTypeTraits<class SMenuOwner>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


/**
 * Any widget that wants to own lists of items that may have sub-menus that should be handled in the same way as multibox menus (mouse can cross other items without sub-menu closing)
 */
class SMenuOwner : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SMenuOwner )
	{}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs);

public:
	/**
	 * For menu widgets, tells the owner widget about a currently active pull-down menu
	 *
	 * @param	InMenuAnchor	Menu anchor for active pull-down menu or sub-menu
	 */
	SLATE_API void SetSummonedMenu( TSharedRef< SMenuAnchor > InMenuAnchor );
	

	/**
	 * For menu bar or sub-menu widgets, returns the currently open menu, if there is one open
	 *
	 * @return	Menu anchor, or null pointer
	 */
	SLATE_API TSharedPtr< const SMenuAnchor > GetOpenMenu() const;

	/**
	 * For menu bar widget, closes any open pull-down or sub menus
	 */
	SLATE_API void CloseSummonedMenus();

protected:
	/** For menu bar widgets, this stores a weak reference to the last pull-down or sub-menu that was summoned. */
	TWeakPtr< SMenuAnchor > SummonedMenuAnchor;
};
