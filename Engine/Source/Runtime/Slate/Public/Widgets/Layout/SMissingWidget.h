// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Like a null widget, but visualizes itself as being explicitly missing.
 */
class SMissingWidget
{
public:

	/**
	 * Creates a new instance.
	 *
	 * @return The widget.
	 */
	static SLATE_API TSharedRef<class SWidget> MakeMissingWidget( );
};
