// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

/** A popup's contents show up on top of other things. */
class SPopup : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPopup)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	SPopup()
	{
		SetCanTick(false);
		bCanSupportFocus = false;
	}

	SLATE_API void Construct(const FArguments& InArgs);

private:

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

};
