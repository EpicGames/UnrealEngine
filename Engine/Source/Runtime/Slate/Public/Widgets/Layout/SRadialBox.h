// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

/**
 * Arranges widgets in a circular fashion
 */
class SRadialBox : public SPanel
{

public:

	/** Basic Slot without padding or alignment */
	class FSlot : public TSlotBase<FSlot>
	{
	public:
		using TSlotBase<FSlot>::TSlotBase;

		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
		SLATE_SLOT_END_ARGS()
		using TSlotBase<FSlot>::Construct;
	};


	SLATE_BEGIN_ARGS(SRadialBox)
		: _PreferredWidth( 100.f )
		, _UseAllottedWidth( false )
		, _StartingAngle(0.f)
		, _bDistributeItemsEvenly(true)
		, _AngleBetweenItems(0.f)
		, _SectorCentralAngle(360.f)
		, _bClockwiseOrder(false)
		, _MarginSize(0.f)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** The slot supported by this panel */
		SLATE_SLOT_ARGUMENT( FSlot, Slots )

		/** The preferred width, if not set will fill the space */
		SLATE_ATTRIBUTE( float, PreferredWidth )

		/** if true, the PreferredWidth will always match the room available to the SRadialBox  */
		SLATE_ARGUMENT( bool, UseAllottedWidth )

		/** Offset of the first element in the circle in degrees */
		SLATE_ARGUMENT( float, StartingAngle)

		/** Ignore AngleBetweenItems and distribute items evenly inside the whole circle */
		SLATE_ARGUMENT(bool, bDistributeItemsEvenly)

		/** How many degrees apart should the elements be? */
		SLATE_ARGUMENT(float, AngleBetweenItems)

		/** If we need a section of a radial (for example half-a-radial) we can define a central angle < 360 (180 in case of half-a-radial). Used when bDistributeItemsEvenly is enabled. */
		SLATE_ARGUMENT(float, SectorCentralAngle)

		/** If true children will be arranged in a clockwise order from starting angle  */
		SLATE_ARGUMENT(bool, bClockwiseOrder)

		/** How much space to leave between the widgets and the edge of the panel */
		SLATE_ARGUMENT(float, MarginSize)

	SLATE_END_ARGS()

	SLATE_API SRadialBox();

	static SLATE_API FSlot::FSlotArguments Slot();

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	SLATE_API FScopedWidgetSlotArguments AddSlot();

	/** Removes a slot from this radial box which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The index in the children array where the slot was removed and -1 if no slot was found matching the widget
	 */
	SLATE_API int32 RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	SLATE_API void Construct( const FArguments& InArgs );

	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	SLATE_API void ClearChildren();

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	SLATE_API virtual FChildren* GetChildren() override;

	void SetStartingAngle(float InStartingAngle) { StartingAngle = InStartingAngle; }
	void SetAngleBetweenItems(float InAngleBetweenItems) { AngleBetweenItems = InAngleBetweenItems; }
	void SetDistributeItemsEvenly(bool bInDistributeItemsEvenly) { bDistributeItemsEvenly = bInDistributeItemsEvenly; }
	void SetClockwiseOrder(bool bInClockwiseOrder) { bClockwiseOrder = bInClockwiseOrder; }
	SLATE_API void SetSectorCentralAngle(float InSectorCentralAngle);
	void SetMarginSize(float InMarginSize) { MarginSize = InMarginSize; }

	SLATE_API void SetUseAllottedWidth(bool bInUseAllottedWidth);

private:

	/** Mods the angle so it's between 0-360 */
	float InternalNormalizeAngle(float Angle) const;

	/** The slots that contain this panel's children. */
	TPanelChildren<FSlot> Slots;

	/** How wide this panel should appear to be. */
	TSlateAttribute<float, EInvalidateWidgetReason::Layout> PreferredWidth;
	/** Offset of the first element in the circle in degrees */
	float StartingAngle;

	/** If we need a section of a radial (for example half-a-radial) we can define a central angle < 360 (180 in case of half-a-radial). Used when bDistributeItemsEvenly is enabled. */
	float SectorCentralAngle;

	/** How many degrees apart should the elements be? */
	float AngleBetweenItems;

	/** Ignore AngleBetweenItems and distribute items evenly inside the whole circle */
	bool bDistributeItemsEvenly;

	/** If true the box will have a preferred width equal to its alloted width  */
	bool bUseAllottedWidth;

	/** If true children will be arranged in a clockwise order from starting angle  */
	bool bClockwiseOrder;

	/** How much space to leave between the widgets and the edge of the panel */
	float MarginSize;

	class FChildArranger;
	friend class SRadialBox::FChildArranger;
};

