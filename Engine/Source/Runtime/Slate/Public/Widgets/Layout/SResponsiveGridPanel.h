// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 *  !!!!!!!!!!!!!!!!!   EXPERIMENTAL  !!!!!!!!!!!!!!!!!   
 * These sizes are subject to change in the future 
 */
namespace SResponsiveGridSize
{
	enum Type
	{
		Mobile = 0,
		Tablet = 768,
		MediumDevice = 992,
		LargeDevice = 1200,
	};
}

/**
 *  !!!!!!!!!!!!!!!!!   EXPERIMENTAL  !!!!!!!!!!!!!!!!!   
 * The SResponsiveGridPanel is still in development and the API may change drastically in the future
 * or maybe removed entirely.
 */
class SResponsiveGridPanel : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SResponsiveGridPanel, SPanel, SLATE_API)
public:

	class FSlot : public TBasicLayoutWidgetSlot< FSlot >
	{
	private:
		friend SResponsiveGridPanel;
		struct FColumnLayout
		{
			float LayoutSize;
			int32 Span;
			int32 Offset;
		};

	public:
		/** Default values for a slot. */
		FSlot(int32 Row)
			: TBasicLayoutWidgetSlot<FSlot>(HAlign_Fill, VAlign_Fill)
			, RowParam(FMath::Max(0, Row))
			, ColumnLayouts()
		{ }


		SLATE_SLOT_BEGIN_ARGS(FSlot, TBasicLayoutWidgetSlot<FSlot>)
			/** How many columns this slot spans over */
			FSlot::FSlotArguments& ColumnSpan(float LayoutSize, int32 ColumnSpan, int32 ColumnOffset = 0)
			{
				FColumnLayout ColumnLayout;
				ColumnLayout.LayoutSize = LayoutSize;
				ColumnLayout.Span = FMath::Max(0, ColumnSpan);
				ColumnLayout.Offset = ColumnOffset;

				bool Inserted = false;
				for (int32 Index = 0; Index < ColumnLayouts.Num(); Index++)
				{
					if (ColumnLayout.LayoutSize < ColumnLayouts[Index].LayoutSize)
					{
						ColumnLayouts.Insert(ColumnLayout, Index);
						Inserted = true;
						break;
					}
				}

				if (!Inserted)
				{
					ColumnLayouts.Add(ColumnLayout);
				}

				return *this;
			}
		
			/** Layout information for the column */
			TArray<FColumnLayout> ColumnLayouts;
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TBasicLayoutWidgetSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			ColumnLayouts = MoveTemp(InArgs.ColumnLayouts);
		}

	private:
		/** The panel that contains this slot */
		TWeakPtr<SResponsiveGridPanel> Panel;

		/** The row index*/
		int32 RowParam;

		/** Layout information for the column */
		TArray<FColumnLayout> ColumnLayouts;
	};

	/**
	 * Used by declarative syntax to create a Slot
	 */
	static FSlot::FSlotArguments Slot(int32 Row)
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>(Row));
	}

	SLATE_BEGIN_ARGS(SResponsiveGridPanel)
		: _ColumnGutter(0)
		, _RowGutter(0)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		SLATE_ARGUMENT(float, ColumnGutter)
		SLATE_ARGUMENT(float, RowGutter)

		SLATE_SLOT_ARGUMENT(FSlot, Slots)

		/** Specify a row to stretch instead of sizing to content. */
		FArguments& FillRow(int32 RowId, float Coefficient)
		{
			if (RowFillCoefficients.Num() <= RowId)
			{
				RowFillCoefficients.AddZeroed(RowId - RowFillCoefficients.Num() + 1);
			}
			RowFillCoefficients[RowId] = Coefficient;
			return Me();
		}

		/** Coefficients for rows that need to stretch instead of size to content */
		TArray<float> RowFillCoefficients;

	SLATE_END_ARGS()

	SLATE_API SResponsiveGridPanel();
	SLATE_API virtual ~SResponsiveGridPanel();

	/** Removes all rows from the panel */
	SLATE_API void ClearChildren();
	
	SLATE_API void Construct( const FArguments& InArgs, int32 TotalColumns );

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	SLATE_API FScopedWidgetSlotArguments AddSlot(int32 Row);

	SLATE_API bool RemoveSlot(const TSharedRef<SWidget>& SlotWidget);

	SLATE_API void SetRowFill(int32 RowId, float Coefficient);

public:

	//~ SWidget interface

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual void CacheDesiredSize(float) override;
	SLATE_API virtual FChildren* GetChildren() override;

protected:
	//~ Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SWidget overrides.

private:

	/**
	 * Given an array of values, re-populate the array such that every contains the partial sums up to that element.
	 * i.e. Array[N] = Array.Sum(0 .. N-1)
	 *
	 * The resulting array is 1-element longer.
	 */
	static SLATE_API void ComputePartialSums( TArray<float>& TurnMeIntoPartialSums );
	
	/** Given a SizeContribution, distribute it to the elements in DistributeOverMe at indexes from [StartIndex .. UpperBound) */
	static SLATE_API void DistributeSizeContributions( float SizeContribution, TArray<float>& DistributeOverMe, int32 StartIndex, int32 UpperBound );

	/**
	 * Return the location where to insert the slot. INDEX_NONE if we insert it at the end.
	 */
	SLATE_API int32 FindInsertSlotLocation(SResponsiveGridPanel::FSlot* InSlot) const;

	/** Compute the sizes of columns and rows needed to fit all the slots in this grid. */
	SLATE_API void ComputeDesiredCellSizes(float AvailableWidth, TArray<float>& OutColumns, TArray<float>& OutRows, TArray<float>& OutRowToSlot) const;

	/** Draw the debug grid of rows and colummns; useful for inspecting the GridPanel's logic. See OnPaint() for parameter meaning */
	SLATE_API int32 LayoutDebugPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId ) const;

private:

	/** The rows*/
	TPanelChildren<FSlot> Slots;

	int32 TotalColumns;
	float ColumnGutter;
	float RowGutter;

	/** Fill coefficients for the rows */
	TArray<float> RowFillCoefficients;

	/** Total desires size along each axis. */
	FVector2D TotalDesiredSizes;

	mutable float PreviousWidth;
};
