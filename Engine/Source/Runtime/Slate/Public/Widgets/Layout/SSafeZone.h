// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

class FArrangedChildren;

/**
	Here's how you could make use of TitleSafe and ActionSafe areas:

		SNew(SOverlay)
		+SOverlay::Slot()
		[
			// ActioSafe
			SNew(SSafeZone) .IsTitleSafe( false )
		]
		+SOverlay::Slot()
		[
			// TitleSafe
			SNew(SSafeZone) .IsTitleSafe( true )
		] 
*/

class SSafeZone : public SBox
{
	SLATE_BEGIN_ARGS(SSafeZone)
		: _HAlign( HAlign_Fill )
		, _VAlign( VAlign_Fill )
		, _Padding( 0.0f )
		, _Content()
		, _IsTitleSafe( false )
		, _SafeAreaScale(1,1,1,1)
		, _PadLeft( true )
		, _PadRight( true )
		, _PadTop( true )
		, _PadBottom( true )
#if WITH_EDITOR
		, _OverrideScreenSize()
		, _OverrideDpiScale()
#endif
		{}

		/** Horizontal alignment of content in the area allotted to the SBox by its parent */
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )

		/** Vertical alignment of content in the area allotted to the SBox by its parent */
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )

		/** Padding between the SBox and the content that it presents. Padding affects desired size. */
		SLATE_ATTRIBUTE( FMargin, Padding )

		/** The widget content presented by the SBox */
		SLATE_DEFAULT_SLOT( FArguments, Content )
	
		/** True if the zone is TitleSafe, otherwise it's ActionSafe */
		SLATE_ARGUMENT( bool, IsTitleSafe )

		/**
		 * The scalar to apply to each side we want to have safe padding for.  This is a handy way 
		 * to ignore the padding along certain areas by setting it to 0, if you only care about asymmetric safe areas.
		 */
		SLATE_ARGUMENT(FMargin, SafeAreaScale)

		/** If this safe zone should pad for the left side of the screen's safe zone */
		SLATE_ARGUMENT( bool, PadLeft )

		/** If this safe zone should pad for the right side of the screen's safe zone */
		SLATE_ARGUMENT( bool, PadRight )

		/** If this safe zone should pad for the top of the screen's safe zone */
		SLATE_ARGUMENT( bool, PadTop )
		
		/** If this safe zone should pad for the bottom of the screen's safe zone */
		SLATE_ARGUMENT( bool, PadBottom )

#if WITH_EDITOR
		/** Force a particular screen size to be used instead of the reported device size. */
		SLATE_ARGUMENT( TOptional<FVector2D>, OverrideScreenSize )

		/** Force a particular screen size to be used instead of the reported device size. */
		SLATE_ARGUMENT(TOptional<float>, OverrideDpiScale )
#endif

	SLATE_END_ARGS()

public:

	SLATE_API SSafeZone();
	SLATE_API virtual ~SSafeZone();

	SLATE_API void Construct( const FArguments& InArgs );
	
	SLATE_API void SetTitleSafe( bool bIsTitleSafe );
	SLATE_API void SetSafeAreaScale(FMargin InSafeAreaScale);

	SLATE_API void SetSidesToPad( bool InPadLeft, bool InPadRight, bool InPadTop, bool InPadBottom );

	SLATE_API FMargin GetSafeMargin(float InLayoutScale) const;

#if WITH_EDITOR
	SLATE_API void SetOverrideScreenInformation(TOptional<FVector2D> InScreenSize, TOptional<float> InOverrideDpiScale);
	SLATE_API void DebugSafeAreaUpdated(const FMargin& NewSafeZone, bool bShouldRecacheMetrics);
#endif

	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float LayoutScale) const override;

	static SLATE_API void SetGlobalSafeZoneScale(TOptional<float> InScale);
	static SLATE_API TOptional<float> GetGlobalSafeZoneScale();

private:

	void UpdateSafeMargin();
	FMargin ComputeScaledSafeMargin(float Scale) const;

	/** Cached values from the args */
	TSlateAttribute<FMargin, EInvalidateWidgetReason::Layout> Padding;
	FMargin SafeAreaScale;
	bool bIsTitleSafe;
	bool bPadLeft;
	bool bPadRight;
	bool bPadTop;
	bool bPadBottom;

#if WITH_EDITOR
	TOptional<FVector2D> OverrideScreenSize;
	TOptional<float> OverrideDpiScale;
#endif

	/** Does the SafeMargin need an update? */
	mutable bool bSafeMarginNeedsUpdate;

	/** Screen space margin */
	mutable FMargin SafeMargin;

	FDelegateHandle OnSafeFrameChangedHandle;
};
