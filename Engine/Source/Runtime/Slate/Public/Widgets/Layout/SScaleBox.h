// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "SScaleBox.generated.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

UENUM(BlueprintType)
namespace EStretchDirection
{
	enum Type : int
	{
		/** Will scale the content up or down. */
		Both,
		/** Will only make the content smaller, will never scale it larger than the content's desired size. */
		DownOnly,
		/** Will only make the content larger, will never scale it smaller than the content's desired size. */
		UpOnly
	};
}

UENUM(BlueprintType)
namespace EStretch
{
	enum Type : int
	{
		/** Does not scale the content. */
		None,
		/** Scales the content non-uniformly filling the entire space of the area. */
		Fill,
		/**
		 * Scales the content uniformly (preserving aspect ratio) 
		 * until it can no longer scale the content without clipping it.
		 */
		ScaleToFit,
		/**
		 * Scales the content uniformly (preserving aspect ratio) 
		 * until it can no longer scale the content without clipping it along the x-axis, 
		 * the y-axis can/will be clipped.
		 */
		ScaleToFitX,
		/**
		 * Scales the content uniformly (preserving aspect ratio) 
		 * until it can no longer scale the content without clipping it along the y-axis, 
		 * the x-axis can/will be clipped.
		 */
		ScaleToFitY,
		/**
		 * Scales the content uniformly (preserving aspect ratio), until all sides meet 
		 * or exceed the size of the area.  Will result in clipping the longer side.
		 */
		ScaleToFill,
		/** Scales the content according to the size of the safe zone currently applied to the viewport. */
		ScaleBySafeZone,
		/** Scales the content by the scale specified by the user. */
		UserSpecified,
		/** Scales the content by the scale specified by the user and also clips. */
		UserSpecifiedWithClipping
	};
}

/**
 * Allows you to place content with a desired size and have it scale to meet the constraints placed on this box's alloted area.  If
 * you needed to have a background image scale to fill an area but not become distorted with different aspect ratios, or if you need
 * to auto fit some text to an area, this is the control for you.
 */
class SScaleBox : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SScaleBox, SCompoundWidget, SLATE_API)

public:
	SLATE_BEGIN_ARGS(SScaleBox)
	: _Content()
	, _HAlign(HAlign_Center)
	, _VAlign(VAlign_Center)
	, _StretchDirection(EStretchDirection::Both)
	, _Stretch(EStretch::None)
	, _UserSpecifiedScale(1.0f)
	, _IgnoreInheritedScale(false)
	{}
		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		/** The horizontal alignment of the content */
		SLATE_ARGUMENT(EHorizontalAlignment, HAlign)

		/** The vertical alignment of the content */
		SLATE_ARGUMENT(EVerticalAlignment, VAlign)
		
		/** Controls in what direction content can be scaled */
		SLATE_ATTRIBUTE(EStretchDirection::Type, StretchDirection)
		
		/** The stretching rule to apply when content is stretched */
		SLATE_ATTRIBUTE(EStretch::Type, Stretch)

		/** Optional scale that can be specified by the User */
		SLATE_ATTRIBUTE(float, UserSpecifiedScale)

		/** Undo any inherited scale factor before applying this scale box's scale */
		SLATE_ATTRIBUTE(bool, IgnoreInheritedScale)

#if WITH_EDITOR
		/** Force a particular screen size to be used instead of the reported device size. */
		SLATE_ARGUMENT(TOptional<FVector2D>, OverrideScreenSize)
#endif

	SLATE_END_ARGS()

protected:
	/** Constructor */
	SLATE_API SScaleBox();

public:
	SLATE_API virtual ~SScaleBox();

	SLATE_API void Construct(const FArguments& InArgs);
	
	// SWidget interface
	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// End SWidget of interface

	/** See Content slot */
	SLATE_API void SetContent(TSharedRef<SWidget> InContent);

	/** See HAlign argument */
	SLATE_API void SetHAlign(EHorizontalAlignment HAlign);

	/** See VAlign argument */
	SLATE_API void SetVAlign(EVerticalAlignment VAlign);

	/** See StretchDirection argument */
	SLATE_API void SetStretchDirection(EStretchDirection::Type InStretchDirection);

	/** See Stretch argument */
	SLATE_API void SetStretch(EStretch::Type InStretch);

	/** See UserSpecifiedScale argument */
	SLATE_API void SetUserSpecifiedScale(float InUserSpecifiedScale);

	/** Set IgnoreInheritedScale argument */
	SLATE_API void SetIgnoreInheritedScale(bool InIgnoreInheritedScale);

#if WITH_EDITOR
	SLATE_API void SetOverrideScreenInformation(TOptional<FVector2D> InScreenSize);
#endif
	
protected:
	// Begin SWidget overrides.
	SLATE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float InScale) const override;
	SLATE_API virtual float GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const override;
	// End SWidget overrides.

	SLATE_API bool DoesScaleRequireNormalizingPrepassOrLocalGeometry() const;
	SLATE_API bool IsDesiredSizeDependentOnAreaAndScale() const;
	SLATE_API float ComputeContentScale(const FGeometry& PaintGeometry) const;

	SLATE_API void RefreshSafeZoneScale();
	SLATE_API void HandleSafeFrameChangedEvent();

#if WITH_EDITOR
	SLATE_API void DebugSafeAreaUpdated(const FMargin& NewSafeZone, bool bShouldRecacheMetrics);
#endif

private:
	/** The allowed direction of stretching of the content */
	TSlateAttribute<EStretchDirection::Type> StretchDirectionAttribute;

	/** The method of scaling that is applied to the content. */
	TSlateAttribute<EStretch::Type> StretchAttribute;

	/** Optional scale that can be specified by the User */
	TSlateAttribute<float> UserSpecifiedScaleAttribute;

	/** Optional bool to ignore the inherited scale */
	TSlateAttribute<bool> IgnoreInheritedScaleAttribute;

	/** Computed scale when scaled by safe zone padding */
	float SafeZoneScale;

	/** Delegate handle to unhook the safe frame changed. */
	FDelegateHandle OnSafeFrameChangedHandle;

	mutable TOptional<FVector2D> LastAllocatedArea;
	mutable TOptional<FGeometry> LastPaintGeometry;

	mutable TOptional<FVector2D> NormalizedContentDesiredSize;

	/**  */
	mutable TOptional<float> ComputedContentScale;




	/**  */
	mutable FVector2D LastFinalOffset;

	/**  */
	mutable FVector2D LastSlotWidgetDesiredSize;

#if WITH_EDITOR
	TOptional<FVector2D> OverrideScreenSize;
#endif
};
