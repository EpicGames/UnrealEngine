// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * A ScissorRectBox is a widget that clips all its children using a scissor rect that matches it clipping rect
 * in LAYOUT SPACE (the way it is provided in OnPaint).
 * This ensures that any child widgets with render transforms do not "bleed out" of this parent container.
 * Note there is a significant performance impact to using one of these widgets because any widgets inside 
 * this one end up in a separate set of batches using that scissor rect.
 * 
 * Keep in mind that this widget will use the LAYOUT clipping rect to clip children, since hardware clipping
 * rects do not support non-aligned rectangles. If a render transform is applied to this widget itself, the results
 * will not be as expected.
 * 
 * NOTE: This class is provided as a STOPGAP for design-time widgets (like the UMG designer itself)
 *       until a more robust clipping solution can be put in place.
 */
class SScissorRectBox : public SPanel
{

public:
	using FScissorRectSlot = FSingleWidgetChildrenWithSlot;

	SLATE_BEGIN_ARGS(SScissorRectBox)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
		_Clipping = EWidgetClipping::ClipToBounds;
	}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()
	
	SLATE_API SScissorRectBox();
	
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * See the Content slot.
     */
	SLATE_API void SetContent(const TSharedRef< SWidget >& InContent);

private:
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

	FScissorRectSlot ChildSlot;
};
