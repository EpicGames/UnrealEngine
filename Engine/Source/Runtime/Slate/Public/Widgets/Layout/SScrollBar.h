// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/SlateWidgetStyleAsset.h"

class SScrollBarTrack;
class SSpacer;
class SImage;

DECLARE_DELEGATE_OneParam(
	FOnUserScrolled,
	float );	/** ScrollOffset as a fraction between 0 and 1 */

DECLARE_DELEGATE_OneParam(FOnScrollBarVisibilityChanged, EVisibility);	/** changed scroll bar visibility */

class SScrollBarTrack;

template <>
struct TWidgetTypeTraits<class SScrollBar>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SScrollBar : public SBorder
{
	SLATE_DECLARE_WIDGET_API(SScrollBar, SBorder, SLATE_API)
public:

	inline static const float DefaultUniformPadding = 2.0f;

	SLATE_BEGIN_ARGS( SScrollBar )
		: _Style( &FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("Scrollbar") )
		, _OnUserScrolled()
		, _AlwaysShowScrollbar(false)
		, _AlwaysShowScrollbarTrack(true)
		, _ScrollbarDisabledVisibility(EVisibility::Collapsed)
#if PLATFORM_UI_HAS_MOBILE_SCROLLBARS
		, _HideWhenNotInUse(true)
#else
		, _HideWhenNotInUse(false)
#endif
		, _PreventThrottling(false)
		, _Orientation( Orient_Vertical )
		, _DragFocusCause( EFocusCause::Mouse )
		, _Thickness()
		, _Padding( DefaultUniformPadding )
		{}

		/** The style to use for this scrollbar */
		SLATE_STYLE_ARGUMENT( FScrollBarStyle, Style )
		SLATE_EVENT( FOnUserScrolled, OnUserScrolled )
		SLATE_EVENT( FOnScrollBarVisibilityChanged, OnScrollBarVisibilityChanged )
		SLATE_ARGUMENT( bool, AlwaysShowScrollbar )
		SLATE_ARGUMENT( bool, AlwaysShowScrollbarTrack )
		SLATE_ARGUMENT( EVisibility, ScrollbarDisabledVisibility )
		SLATE_ARGUMENT( bool, HideWhenNotInUse )
		SLATE_ARGUMENT( bool, PreventThrottling )
		SLATE_ARGUMENT( EOrientation, Orientation )
		SLATE_ARGUMENT( EFocusCause, DragFocusCause )
		/** The thickness of the scrollbar thumb */
		SLATE_ATTRIBUTE( FVector2D, Thickness )
		/** The margin around the scrollbar */
		SLATE_ATTRIBUTE( FMargin, Padding )
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	/**
	 * Set the handler to be invoked when the user scrolls.
	 *
	 * @param InHandler   Method to execute when the user scrolls the scrollbar
	 */
	SLATE_API void SetOnUserScrolled( const FOnUserScrolled& InHandler );

	/**
	 * Set the handler to be invoked when scroll bar visibility changes.
	 *
	 * @param InHandler   Method to execute when scroll bar visibility changed
	 */
	SLATE_API void SetOnScrollBarVisibilityChanged( const FOnScrollBarVisibilityChanged& InHandler );

	/**
	 * Set the offset and size of the track's thumb.
	 * Note that the maximum offset is 1.0-ThumbSizeFraction.
	 * If the user can view 1/3 of the items in a single page, the maximum offset will be ~0.667f
	 *
	 * @param InOffsetFraction     Offset of the thumbnail from the top as a fraction of the total available scroll space.
	 * @param InThumbSizeFraction  Size of thumbnail as a fraction of the total available scroll space.
	 * @param bCallOnUserScrolled  If true, OnUserScrolled will be called with InOffsetFraction
	 */
	SLATE_API virtual void SetState( float InOffsetFraction, float InThumbSizeFraction, bool bCallOnUserScrolled = false);

	// SWidget
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	// SWidget

	/** @return true if scrolling is possible; false if the view is big enough to fit all the content */
	SLATE_API bool IsNeeded() const;

	/** @return normalized distance from top */
	SLATE_API float DistanceFromTop() const;

	/** @return normalized distance from bottom */
	SLATE_API float DistanceFromBottom() const;

	/** @return normalized percentage of track covered by thumb bar */
	SLATE_API float ThumbSizeFraction() const;

	/** @return the scrollbar's visibility as a product of internal rules and user-specified visibility */
	SLATE_API EVisibility ShouldBeVisible() const;

	/** @return True if the user is scrolling by dragging the scroll bar thumb. */
	SLATE_API bool IsScrolling() const;

	/** @return the orientation in which the scrollbar is scrolling. */
	SLATE_API EOrientation GetOrientation() const; 

	/** Set argument Style */
	SLATE_API void SetStyle(const FScrollBarStyle* InStyle);

	/** Invalidate the style */
	SLATE_API void InvalidateStyle();

	/** Set UserVisibility attribute */
	SLATE_API void SetUserVisibility(TAttribute<EVisibility> InUserVisibility);

	/** Set DragFocusCause attribute */
	SLATE_API void SetDragFocusCause(EFocusCause InDragFocusCause);

	/** Set Thickness attribute */
	SLATE_API void SetThickness(TAttribute<FVector2D> InThickness);

	/** Set ScrollBarAlwaysVisible attribute */
	SLATE_API void SetScrollBarAlwaysVisible(bool InAlwaysVisible);

	/**
	 * Set ScrollBarTrackAlwaysVisible attribute.
	 * Even if true, tracks will only be visible if the brush from the style has an ImageSize.
	 * See FScrollBarStyle::VerticalBottomSlotImage and equivalents
	 */
	SLATE_API void SetScrollBarTrackAlwaysVisible(bool InAlwaysVisible);

	/** Set the visibility of the ScrollBar when it is not needed. The default value is EVisibility::Collapsed. */
	SLATE_API void SetScrollbarDisabledVisibility(EVisibility InVisibility);

	/** Returns True when the scrollbar should always be shown, else False */
	SLATE_API bool AlwaysShowScrollbar() const;

	/** Allows external scrolling panels to notify the scrollbar when scrolling begins. */
	SLATE_API virtual void BeginScrolling();

	/** Allows external scrolling panels to notify the scrollbar when scrolling ends. */
	SLATE_API virtual void EndScrolling();

	SLATE_API SScrollBar();
	SLATE_API ~SScrollBar();

protected:
	
	/** Execute the on user scrolled delegate */
	SLATE_API void ExecuteOnUserScrolled( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

	/** We fade the scroll track unless it is being hovered*/
	SLATE_API FSlateColor GetTrackOpacity() const;

	/** We always show a subtle scroll thumb, but highlight it extra when dragging */
	SLATE_API FLinearColor GetThumbOpacity() const;

	/** @return the name of an image for the scrollbar thumb based on whether the user is dragging or hovering it. */
	SLATE_API const FSlateBrush* GetDragThumbImage() const;

	TSlateAttributeRef<EVisibility> GetUserVisibilityAttribute() const
	{
		return TSlateAttributeRef<EVisibility>{SharedThis(this), UserVisibilityAttribute};
	}

#if WITH_EDITORONLY_DATA
	/** The scrollbar's visibility as specified by the user. Will be compounded with internal visibility rules. */
	UE_DEPRECATED(5.7, "Use SetUserVisibility / GetUserVisibilityAttribute")
	TSlateDeprecatedTAttribute<EVisibility> UserVisibility;
#endif

	const FScrollBarStyle* Style = nullptr;
	TSharedPtr<SBorder> BackgroundBorder;
	TSharedPtr<SImage> TopImage;
	TSharedPtr<SImage> BottomImage;
	TSharedPtr<SBorder> DragThumb;
	TSharedPtr<SSpacer> ThicknessSpacer;
	bool bDraggingThumb;
	TSharedPtr<SScrollBarTrack> Track;
	FOnUserScrolled OnUserScrolled;
	FOnScrollBarVisibilityChanged OnScrollBarVisibilityChanged;
	float DragGrabOffset;
	EOrientation Orientation;
	bool bAlwaysShowScrollbar;
	bool bAlwaysShowScrollbarTrack;
	EFocusCause DragFocusCause;
	bool bHideWhenNotInUse;
	EVisibility ScrollbarDisabledVisibility = EVisibility::Collapsed;
	/*
	 * Holds whether or not to prevent throttling during mouse capture
	 * When true, the viewport will be updated with every single change to the value during dragging
	 */
	bool bPreventThrottling;
	bool bIsScrolling;
	double LastInteractionTime;

	/** Image to use when the scrollbar thumb is in its normal state */
	const FSlateBrush* NormalThumbImage;
	/** Image to use when the scrollbar thumb is in its hovered state */
	const FSlateBrush* HoveredThumbImage;
	/** Image to use when the scrollbar thumb is in its dragged state */
	const FSlateBrush* DraggedThumbImage;
	/** Background brush */
	const FSlateBrush* BackgroundBrush;
	/** Top brush */
	const FSlateBrush* TopBrush;
	/** Bottom brush */
	const FSlateBrush* BottomBrush;

private:
	bool bIsThicknessAttrSet = false;

	/** The scrollbar's visibility as specified by the user. Will be compounded with internal visibility rules. */
	TSlateAttribute<EVisibility> UserVisibilityAttribute;
};
