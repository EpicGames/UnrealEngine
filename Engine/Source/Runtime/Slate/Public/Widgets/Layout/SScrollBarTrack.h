// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

template <>
struct TWidgetTypeTraits<class SScrollBarTrack>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Arrange 3 widgets: the top track, bottom track, and thumb. */
class SScrollBarTrack : public SPanel
{
public:
	/** A ListPanel slot is very simple - it just stores a widget. */
	class FSlot : public TSlotBase<FSlot>
	{
	public:
		FSlot()
			: TSlotBase<FSlot>()
		{}
	};

	SLATE_BEGIN_ARGS(SScrollBarTrack)
		: _TopSlot()
		, _ThumbSlot()
		, _BottomSlot()
		, _Orientation(Orient_Vertical)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

		SLATE_NAMED_SLOT(FArguments, TopSlot)
		SLATE_NAMED_SLOT(FArguments, ThumbSlot)
		SLATE_NAMED_SLOT(FArguments, BottomSlot)
		SLATE_ARGUMENT(EOrientation, Orientation)

	SLATE_END_ARGS()

	SScrollBarTrack()
		: Children(this)
	{
	}

	/**
	 * Construct the widget from a Declaration
	 *
	 * @param InArgs  Declaration from which to construct the widget.
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	struct FTrackSizeInfo
	{
		FTrackSizeInfo(const FGeometry& TrackGeometry, EOrientation InOrientation, float InMinThumbSize, float ThumbSizeAsFractionOfTrack, float ThumbOffsetAsFractionOfTrack)
		{
			BiasedTrackSize = ((InOrientation == Orient_Horizontal) ? TrackGeometry.GetLocalSize().X : TrackGeometry.GetLocalSize().Y) - InMinThumbSize;
			const float AccurateThumbSize = ThumbSizeAsFractionOfTrack * (BiasedTrackSize);
			ThumbStart = BiasedTrackSize * ThumbOffsetAsFractionOfTrack;
			ThumbSize = InMinThumbSize + AccurateThumbSize;

		}

		float BiasedTrackSize;
		float ThumbStart;
		float ThumbSize;
		float GetThumbEnd()
		{
			return ThumbStart + ThumbSize;
		}
	};

	SLATE_API FTrackSizeInfo GetTrackSizeInfo(const FGeometry& InTrackGeometry) const;

	/**
	 * Panels arrange their children in a space described by the AllottedGeometry parameter. The results of the arrangement
	 * should be returned by appending a FArrangedWidget pair for every child widget. See StackPanel for an example
	 *
	 * @param AllottedGeometry    The geometry allotted for this widget by its parent.
	 * @param ArrangedChildren    The array to which to add the WidgetGeometries that represent the arranged children.
	 */
	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const;

	/**
	 * A Panel's desired size in the space required to arrange of its children on the screen while respecting all of
	 * the children's desired sizes and any layout-related options specified by the user. See StackPanel for an example.
	 *
	 * @return The desired size.
	 */
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const;

	/** @return  The children of a panel in a slot-agnostic way. */
	SLATE_API virtual FChildren* GetChildren();

	SLATE_API void SetSizes(float InThumbOffsetFraction, float InThumbSizeFraction);

	SLATE_API bool IsNeeded() const;

	SLATE_API float DistanceFromTop() const;

	SLATE_API float DistanceFromBottom() const;

	SLATE_API float GetMinThumbSize() const;

	SLATE_API float GetThumbSizeFraction() const;

	SLATE_API void SetIsAlwaysVisible(bool InIsAlwaysVisible);

private:
	void InvalidateIsNeeded(bool bWasNeeded);

protected:

	static const int32 TOP_SLOT_INDEX = 0;
	static const int32 BOTTOM_SLOT_INDEX = 1;
	static const int32 THUMB_SLOT_INDEX = 2;

	TPanelChildren< FSlot > Children;

	float OffsetFraction;
	float ThumbSizeFraction;
	float MinThumbSize;
	bool bIsAlwaysVisible = false;
	EOrientation Orientation;
};
