// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

class IScrollableWidget;

template <>
struct TWidgetTypeTraits<class SScrollBorder>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Shows a border above and below a scrollable area.
 */
class SScrollBorder : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SScrollBorder, SCompoundWidget, SLATE_API)
public:
	SLATE_BEGIN_ARGS(SScrollBorder)
		: _Style( &FCoreStyle::Get().GetWidgetStyle<FScrollBorderStyle>("ScrollBorder") )
		, _BorderFadeDistance( FVector2D(0.01f, 0.01f) )
		{}

		/** Style used to draw this scroll border */
		SLATE_STYLE_ARGUMENT( FScrollBorderStyle, Style )

		/** The distance in normalized coordinate to begin fading the scroll border. */
		SLATE_ATTRIBUTE( FVector2D, BorderFadeDistance )

		/** Arbitrary content to be displayed in the overlay under the shadow layers; if not specified, the associated ScrollableWidget will be used instead. */
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	SLATE_API SScrollBorder();
	SLATE_API virtual ~SScrollBorder();

	/**
	  * Constructs a scrollable border overlay for the specified ScrollableWidget
	  */
	SLATE_API void Construct(const FArguments& InArgs, TSharedRef<IScrollableWidget> InScrollableWidget);
	
protected:
	SLATE_API void SetBorderFadeDistance(TAttribute<FVector2D> InBorderFadeDistance);
	TSlateAttributeRef<FVector2D> GetBorderFadeDistanceAttribute() const
	{
		return TSlateAttributeRef<FVector2D>{SharedThis(this), BorderFadeDistanceAttribute};
	}

	/** Gets the top border opacity. */
	SLATE_API FSlateColor GetTopBorderOpacity() const;

	/** Gets the bottom border opacity. */
	SLATE_API FSlateColor GetBottomBorderOpacity() const;

#if WITH_EDITORONLY_DATA
	/** The border fade distance in normalized coordinates. */
	UE_DEPRECATED(5.7, "Use SetBorderFadeDistance / GetBorderFadeDistanceAttribute")
	TSlateDeprecatedTAttribute<FVector2D> BorderFadeDistance;
#endif

	/** The widget that to watch for when the scrolling changes to show the border. */
	TAttribute< TWeakPtr< IScrollableWidget > > ScrollableWidget;

private:
	/** The border fade distance in normalized coordinates. */
	TSlateAttribute<FVector2D> BorderFadeDistanceAttribute;
};
