// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Layout/Geometry.h"
#include "Styling/SlateColor.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Input/NavigationReply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Framework/Layout/InertialScrollManager.h"
#include "Framework/Layout/Overscroll.h"

#include "SScrollBox.generated.h"

class FNavigationMetaData;
class FPaintArgs;
class FSlateWindowElementList;
class SScrollPanel;

DECLARE_DELEGATE(FOnScrollBoxFocusReceived);
DECLARE_DELEGATE(FOnScrollBoxFocusLost);
DECLARE_DELEGATE_TwoParams(FOnScrollBoxFocusChanging, const FWeakWidgetPath& /*PreviousFocusPath*/, const FWidgetPath& /*NewWidgetPath*/);

/** Where to scroll the descendant to */
UENUM(BlueprintType)
enum class EDescendantScrollDestination : uint8
{
	/**
	* Scroll the widget into view using the least amount of energy possible.  So if the new item
	* is above the visible set, it will stop as soon as it's in view at the top.  If it's below the
	* visible set, it stop it comes into view at the bottom.
	*/
	IntoView,

	/** Always scroll the widget so it appears at the top/Left of the scrollable area. */
	TopOrLeft,

	/**
	* Always scroll the widget so it appears at the center of the scrollable area, if possible.
	* This won't be possible for the first few items and the last few items, as there's not enough
	* slack.
	*/
	Center,

	/** Always scroll the widget so it appears at the bottom/Right of the scrollable area. */
	BottomOrRight,
};

/** Set behavior when user focus changes inside this scroll box */
UENUM(BlueprintType)
enum class EScrollWhenFocusChanges : uint8
{
	/** Don't automatically scroll, navigation or child widget will handle this */
	NoScroll,

	/** Instantly scroll using NavigationDestination rule */
	InstantScroll,

	/** Use animation to scroll using NavigationDestination rule */
	AnimatedScroll,
};

template <>
struct TWidgetTypeTraits<class SScrollBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** SScrollBox can scroll through an arbitrary number of widgets. */
class SScrollBox : public SCompoundWidget
{
public:
	/** A Slot that provides layout options for the contents of a scrollable box. */
	class FSlot : public TBasicLayoutWidgetSlot<FSlot>, public TResizingWidgetSlotMixin<FSlot>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TBasicLayoutWidgetSlot<FSlot>, TResizingWidgetSlotMixin<FSlot>)
			/** The widget's DesiredSize will be used as the space required. */
			FSlotArguments& AutoSize()
			{
				_SizeParam = FAuto();
				return Me();
			}

			/** The available space will be distributed proportionately. */
			FSlotArguments& FillSize(TAttribute<float> InStretchCoefficient)
			{
				_SizeParam = FStretch(MoveTemp(InStretchCoefficient));
				return Me();
			}

			/**
			 * The widget's content size is adjusted proportionally to fit the available space.
			 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
			 * @param InStretchCoefficient Stretch coefficient for this slot.
			 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.
			 */
			FSlotArguments& FillContentSize(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
			{
				_SizeParam = FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
				return Me();
			}

			/** Set the min size in SlateUnit this slot can be. */
			FSlotArguments& MinSize(TAttribute<float> InMinHeight)
			{
				_MinSize = MoveTemp(InMinHeight);
				return Me();
			}

			/** Set the max size in SlateUnit this slot can be. */
			FSlotArguments& MaxSize(TAttribute<float> InMaxHeight)
			{
				_MaxSize = MoveTemp(InMaxHeight);
				return Me();
			}
		SLATE_SLOT_END_ARGS()

		/** Default values for a slot. */
		FSlot()
			: TBasicLayoutWidgetSlot<FSlot>(HAlign_Fill, VAlign_Fill)
			, TResizingWidgetSlotMixin<SScrollBox::FSlot>()
		{
			this->SizeRule = FSizeParam::SizeRule_Auto;
		}

		SLATE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);
		static SLATE_API void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer);

	private:
		/**
		 * How much space this slot should occupy along scrollbox's direction.
		 * When SizeRule is SizeRule_Auto, the widget's DesiredSize will be used as the space required.
		 * When SizeRule is SizeRule_Stretch, the available space will be distributed proportionately between
		 * peer Widgets depending on the Value property. Available space is space remaining after all the
		 * peers' SizeRule_Auto requirements have been satisfied.
		 */

		/** Flag indicating if the ShrinkSizeValue value is set. */
		bool bIsShrinkSizeValueSet;
	};

	SLATE_BEGIN_ARGS(SScrollBox)
		: _Style( &FAppStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBox") )
		, _ScrollBarStyle( &FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar") )
		, _ExternalScrollbar()
		, _Orientation(Orient_Vertical)
		, _ScrollBarVisibility(EVisibility::Visible)
		, _ScrollBarAlwaysVisible(false)
		, _ScrollBarDragFocusCause(EFocusCause::Mouse)
		, _ScrollBarThickness(FVector2f(_Style->BarThickness, _Style->BarThickness))
		, _ScrollBarPadding(2.0f)
		, _AllowOverscroll(EAllowOverscroll::Yes)
		, _BackPadScrolling(false)
		, _FrontPadScrolling(false)
		, _AnimateWheelScrolling(false)
		, _ScrollAnimationInterpSpeed(15.f)
		, _WheelScrollMultiplier(1.f)
		, _EnableTouchScrolling(true)
		, _ConsumePointerInput (true)
		, _NavigationDestination(EDescendantScrollDestination::IntoView)
		, _NavigationScrollPadding(0.0f)
		, _ScrollWhenFocusChanges(EScrollWhenFocusChanges::NoScroll)
		, _ScrollBarRightClickDragAllowed(false)
		, _OnUserScrolled()
	    , _OnScrollBarVisibilityChanged()
		, _OnFocusReceived()
		, _OnFocusLost()
		, _OnFocusChanging()
		, _ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
		, _AllowContentToShrink(false)
		{
			_Clipping = EWidgetClipping::ClipToBounds;
		}
		
		SLATE_SLOT_ARGUMENT( FSlot, Slots )

		/** Style used to draw this scrollbox */
		SLATE_STYLE_ARGUMENT( FScrollBoxStyle, Style )

		/** Style used to draw this scrollbox's scrollbar */
		SLATE_STYLE_ARGUMENT( FScrollBarStyle, ScrollBarStyle )

		/** Custom scroll bar */
		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar )

		/** The direction that children will be stacked, and also the direction the box will scroll. */
		SLATE_ARGUMENT( EOrientation, Orientation )

		SLATE_ARGUMENT( EVisibility, ScrollBarVisibility )

		SLATE_ARGUMENT( bool, ScrollBarAlwaysVisible )

		SLATE_ARGUMENT( EFocusCause, ScrollBarDragFocusCause )

		SLATE_ARGUMENT( UE::Slate::FDeprecateVector2DParameter, ScrollBarThickness )

		/** This accounts for total internal scroll bar padding; default 2.0f padding from the scroll bar itself is removed */
		SLATE_ARGUMENT( FMargin, ScrollBarPadding )

		SLATE_ARGUMENT(EAllowOverscroll, AllowOverscroll);

		SLATE_ARGUMENT(bool, BackPadScrolling);

		SLATE_ARGUMENT(bool, FrontPadScrolling);

		SLATE_ARGUMENT(bool, AnimateWheelScrolling);

		SLATE_ARGUMENT(float, ScrollAnimationInterpSpeed);

		SLATE_ARGUMENT(float, WheelScrollMultiplier);

		SLATE_ARGUMENT(bool, EnableTouchScrolling);
		
		SLATE_ARGUMENT(bool, ConsumePointerInput );

		SLATE_ARGUMENT(EDescendantScrollDestination, NavigationDestination);

		/**
		 * The amount of padding to ensure exists between the item being navigated to, at the edge of the
		 * scrollbox.  Use this if you want to ensure there's a preview of the next item the user could scroll to.
		 */
		SLATE_ARGUMENT(float, NavigationScrollPadding);

		SLATE_ARGUMENT(EScrollWhenFocusChanges, ScrollWhenFocusChanges);

		SLATE_ARGUMENT(bool, ScrollBarRightClickDragAllowed);

		/** Called when the button is clicked */
		SLATE_EVENT(FOnUserScrolled, OnUserScrolled)

		/** Fired when scroll bar visibility changed */
		SLATE_EVENT(FOnScrollBarVisibilityChanged, OnScrollBarVisibilityChanged)

		/** If bIsFocusable is enabled, Called when the scrollbox focus is received */
		SLATE_EVENT(FOnScrollBoxFocusReceived, OnFocusReceived)
		
		/** If bIsFocusable is enabled, Called when the scrollbox focus is lost */
		SLATE_EVENT(FOnScrollBoxFocusLost, OnFocusLost)

		/** If bIsFocusable is enabled, Called when the scrollbox focus is changing */
		SLATE_EVENT(FOnScrollBoxFocusChanging, OnFocusChanging)

		SLATE_ARGUMENT(EConsumeMouseWheel, ConsumeMouseWheel);

		/** If enabled, content in the underlying panel will be allowed to shrink with the overall size of the ScrollBox instead of always taking up maximum space */
		SLATE_ARGUMENT(bool, AllowContentToShrink);

	SLATE_END_ARGS()

	SLATE_API SScrollBox();
	SLATE_API virtual ~SScrollBox();

	/** @return a new slot. Slots contain children for SScrollBox */
	static SLATE_API FSlot::FSlotArguments Slot();

	SLATE_API void Construct( const FArguments& InArgs );

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/** Adds a slot to SScrollBox */
	SLATE_API FScopedWidgetSlotArguments AddSlot();

	/** Insert a slot at a given position. */
	SLATE_API FScopedWidgetSlotArguments InsertSlot(int32 Index);

	/** Returns the slot at the given index. */
	SLATE_API const FSlot& GetSlot(int32 SlotIndex) const;
	SLATE_API FSlot& GetSlot(int32 SlotIndex);

	/** Removes the corresponding widget from the set of slots if it exists. */
	SLATE_API void RemoveSlot(const TSharedRef<SWidget>& WidgetToRemove);

	/** @return the number of slots. */
	SLATE_API int32 NumSlots() const;

	/** Removes all children from the box */
	SLATE_API void ClearChildren();

	/** @return Returns true if the user is currently interactively scrolling the view by holding
		        the right mouse button and dragging. */
	SLATE_API bool IsRightClickScrolling() const;

	SLATE_API EAllowOverscroll GetAllowOverscroll() const;

	SLATE_API void SetAllowOverscroll( EAllowOverscroll NewAllowOverscroll );

	SLATE_API void SetAnimateWheelScrolling(bool bInAnimateWheelScrolling);

	SLATE_API void SetScrollingAnimationInterpolationSpeed(float NewScrollingAnimationInterpolationSpeed);

	SLATE_API void SetWheelScrollMultiplier(float NewWheelScrollMultiplier);

	/** Enables/disables being able to scroll using touch input. */
	SLATE_API void SetIsTouchScrollingEnabled(const bool bInEnableTouchScrolling);
	
	void SetConsumePointerInput(bool bInConsumePointerInput) { bConsumePointerInput = bInConsumePointerInput; }
	
	SLATE_API void SetScrollWhenFocusChanges(EScrollWhenFocusChanges NewScrollWhenFocusChanges);

	SLATE_API float GetScrollOffset() const;

	SLATE_API UE::Slate::FDeprecateVector2DResult GetScrollPanelContentSize() const;
	
	SLATE_API float GetOverscrollOffset() const;
	
	SLATE_API float GetOverscrollPercentage() const;
	
	SLATE_API float GetViewFraction() const;

	SLATE_API float GetViewOffsetFraction() const;

	/** Gets the scroll offset of the bottom of the ScrollBox in Slate Units. */
	SLATE_API float GetScrollOffsetOfEnd() const;

	bool GetIsScrolling() const { return bIsScrolling; }

	SLATE_API void SetScrollOffset( float NewScrollOffset );

	SLATE_API void ScrollToStart();

	SLATE_API void ScrollToEnd();

	SLATE_API void EndInertialScrolling();

	/** 
	 * Attempt to scroll a widget into view, will safely handle non-descendant widgets 
	 *
	 * @param WidgetToFind The widget whose geometry we wish to discover.
	 * @param InAnimateScroll	Whether or not to animate the scroll
	 * @param InDestination		Where we want the child widget to stop.
	 */
	SLATE_API void ScrollDescendantIntoView(const TSharedPtr<SWidget>& WidgetToFind, bool InAnimateScroll = true, EDescendantScrollDestination InDestination = EDescendantScrollDestination::IntoView, float Padding = 0);

	/** Get the current orientation of the scrollbox. */
	SLATE_API EOrientation GetOrientation();

	SLATE_API void SetNavigationDestination(const EDescendantScrollDestination NewNavigationDestination);

	SLATE_API void SetConsumeMouseWheel(EConsumeMouseWheel NewConsumeMouseWheel);

	SLATE_API void SetAnalogMouseWheelKey(FKey InAnalogKey);

	SLATE_API void SetIsFocusable(bool bInIsFocusable);

	/** Sets the current orientation of the scrollbox and updates the layout */
	SLATE_API void SetOrientation(EOrientation InOrientation);

	SLATE_API void SetScrollBarVisibility(EVisibility InVisibility);

	SLATE_API void SetScrollBarAlwaysVisible(bool InAlwaysVisible);
	
	SLATE_API void SetScrollBarTrackAlwaysVisible(bool InAlwaysVisible);

	SLATE_API void SetScrollBarThickness(UE::Slate::FDeprecateVector2DParameter InThickness);

	SLATE_API void SetScrollBarPadding(const FMargin& InPadding);

	SLATE_API void SetScrollBarRightClickDragAllowed(bool bIsAllowed);

	SLATE_API void SetStyle(const FScrollBoxStyle* InStyle);
	
	SLATE_API void SetScrollBarStyle(const FScrollBarStyle* InBarStyle);

	SLATE_API void InvalidateStyle();

	SLATE_API void InvalidateScrollBarStyle();

public:

	// SWidget interface
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual bool ComputeVolatility() const override;
	SLATE_API virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnAnalogValueChanged( const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent) override;
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	SLATE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	SLATE_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;
	SLATE_API virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;
	SLATE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	SLATE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return bIsFocusable; }
	SLATE_API virtual FSlateRect GetHitTestBoundingRect() const override;
	SLATE_API virtual void OnAddedMetadata(const TSharedRef<ISlateMetaData>& AddedMetaData) override;
	SLATE_API virtual void OnRemovedMetadata(const TSharedRef<ISlateMetaData>& RemovedMetaData) override;
	// End of SWidget interface

private:
	FNavigationReply OnNavigation_Default(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent);

protected:
	SLATE_API void OnClippingChanged();

public:
	bool HasNavigationMetaData() const
	{
		return bNavigationDataExists;
	}

private:
	/** Builds a default Scrollbar */
	TSharedPtr<SScrollBar> ConstructScrollBar();

	/** Constructs internal layout widgets for scrolling vertically using the existing ScrollPanel and ScrollBar. */
	void ConstructVerticalLayout();

	/** Constructs internal layout widgets for scrolling horizontally using the existing ScrollPanel and ScrollBar. */
	void ConstructHorizontalLayout();

	/** Gets the component of a vector in the direction of scrolling based on the Orientation property. */
	inline float GetScrollComponentFromVector(FVector2f Vector) const
	{
		return float(Orientation == Orient_Vertical ? Vector.Y : Vector.X);
	}

	/** Sets the component of a vector in the direction of scrolling based on the Orientation property. */
	inline void SetScrollComponentOnVector(FVector2f& InVector, float Value) const
	{
		if (Orientation == Orient_Vertical)
		{
			InVector.Y = Value;
		}
		else
		{
			InVector.X = Value;
		}
	}

	/** Scroll offset that the user asked for. We will clamp it before actually scrolling there. */
	float DesiredScrollOffset;

	/**
	 * Scroll the view by ScrollAmount given its currently AllottedGeometry.
	 *
	 * @param AllottedGeometry  The geometry allotted for this SScrollBox by the parent
	 * @param LocalScrollAmount
	 * @param Overscroll      
	 * @param InAnimateScroll	Whether or not to animate the scroll
	 * @return Whether or not the scroll was fully handled
	 */
	bool ScrollBy(const FGeometry& AllottedGeometry, float LocalScrollAmount, EAllowOverscroll Overscroll, bool InAnimateScroll);

	/** Invoked when the user scroll via the scrollbar */
	void ScrollBar_OnUserScrolled( float InScrollOffsetFraction );

	void ScrollBar_OnScrollBarVisibilityChanged( EVisibility NewVisibility);

	/** Does the user need a hint that they can scroll to the start of the list? */
	FSlateColor GetStartShadowOpacity() const;
	
	/** Does the user need a hint that they can scroll to the end of the list? */
	FSlateColor GetEndShadowOpacity() const;

	/** Active timer to update inertial scrolling as needed */
	EActiveTimerReturnType UpdateInertialScroll(double InCurrentTime, float InDeltaTime);

	/** Check whether the current state of the table warrants inertial scroll by the specified amount */
	bool CanUseInertialScroll(float ScrollAmount) const;

	void BeginInertialScrolling();

	/** Padding to the scrollbox */
	FMargin ScrollBarSlotPadding;

	union
	{
		// vertical scroll bar is stored in horizontal box and vice versa
		SHorizontalBox::FSlot* VerticalScrollBarSlot; // valid when Orientation == Orient_Vertical
		SVerticalBox::FSlot* HorizontalScrollBarSlot; // valid when Orientation == Orient_Horizontal
	};

protected:
	/** Scrolls or begins scrolling a widget into view, only valid to call when we have layout geometry. */
	SLATE_API bool InternalScrollDescendantIntoView(const FGeometry& MyGeometry, const TSharedPtr<SWidget>& WidgetToFind, bool InAnimateScroll = true, EDescendantScrollDestination InDestination = EDescendantScrollDestination::IntoView, float Padding = 0);

	/** returns widget that can receive keyboard focus or nullptr **/
	SLATE_API TSharedPtr<SWidget> GetKeyboardFocusableWidget(TSharedPtr<SWidget> InWidget);

	/** The panel which stacks the child slots */
	TSharedPtr<class SScrollPanel> ScrollPanel;

	/** The scrollbar which controls scrolling for the scrollbox. */
	TSharedPtr<SScrollBar> ScrollBar;

	/** Flag if Navigation Data exists for the ScrollBox */
	bool bNavigationDataExists = false;

	/** The Top or Left Image of the shadow hint depending on orientation. */
	TSharedPtr<SImage> TopLeftShadowHint;
	/** The Bottom or Right Image of the shadow hint depending on orientation. */
	TSharedPtr<SImage> BottomRightShadowHint;

	/** The amount we have scrolled this tick cycle */
	float TickScrollDelta;

	/** Did the user start an interaction in this list? */
	TOptional<int32> bFingerOwningTouchInteraction;

	/** How much we scrolled while the rmb has been held */
	float AmountScrolledWhileRightMouseDown;

	/** The current deviation we've accumulated on scrol, once it passes the trigger amount, we're going to begin scrolling. */
	float PendingScrollTriggerAmount;

	/** Helper object to manage inertial scrolling */
	FInertialScrollManager InertialScrollManager;

	/** The overscroll state management structure. */
	FOverscroll Overscroll;

	/** Whether to permit overscroll on this scroll box */
	EAllowOverscroll AllowOverscroll;

	/** Whether to back pad this scroll box, allowing user to scroll backward until child contents are no longer visible */
	bool BackPadScrolling;

	/** Whether to front pad this scroll box, allowing user to scroll forward until child contents are no longer visible */
	bool FrontPadScrolling;

	/**
	 * The amount of padding to ensure exists between the item being navigated to, at the edge of the
	 * scrollbox.  Use this if you want to ensure there's a preview of the next item the user could scroll to.
	 */
	float NavigationScrollPadding;

	/** Sets where to scroll a widget to when using explicit navigation or if ScrollWhenFocusChanges is enabled */
	EDescendantScrollDestination NavigationDestination;

	/** Scroll behavior when user focus is given to a child widget */
	EScrollWhenFocusChanges ScrollWhenFocusChanges;

	/**	The current position of the software cursor */
	FVector2f SoftwareCursorPosition;

	/** Fired when the user scrolls the scrollbox */
	FOnUserScrolled OnUserScrolled;
	
	/** Fired when scroll bar visibility changed */
	FOnScrollBarVisibilityChanged OnScrollBarVisibilityChanged;

	/** If bIsFocusable is enabled, Called when the scrollbox focus is received */
	FOnScrollBoxFocusReceived OnScrollBoxFocusReceived;
		
	/** If bIsFocusable is enabled, Called when the scrollbox focus is lost */
	FOnScrollBoxFocusLost OnScrollBoxFocusLost;

	/** If bIsFocusable is enabled, Called when the scrollbox focus is changing */
	FOnScrollBoxFocusChanging OnScrollBoxFocusChanging;

	/** The scrolling and stacking orientation. */
	EOrientation Orientation;

	/** Style resource for the scrollbox */
	const FScrollBoxStyle* Style;

	/** Style resource for the scrollbar */
	const FScrollBarStyle* ScrollBarStyle;

	/** How we should handle scrolling with the mouse wheel */
	EConsumeMouseWheel ConsumeMouseWheel;

	/** Gamepad key used for scrolling. Will adhere to ConsumeMouseWheel */
	FKey AnalogMouseWheelKey;

	/** Cached geometry for use with the active timer */
	FGeometry CachedGeometry;

	/** Scroll into view request. */
	TFunction<void(FGeometry)> ScrollIntoViewRequest;

	TSharedPtr<FActiveTimerHandle> UpdateInertialScrollHandle;

	double LastScrollTime;

	/** Multiplier applied to each click of the scroll wheel (applied alongside the global scroll amount) */
	float WheelScrollMultiplier = 1.f;

	/** True to allow scrolling by using touch input. */
	bool bEnableTouchScrolling = true;

	/** If true, touch input events will pass through to widgets under the ScrollBox, while still being handled by the ScrollBox. */
	bool bConsumePointerInput = true;

	/** The speed of interpolation for the scrolling animation */
	float ScrollingAnimationInterpolationSpeed = 15.f;

	/** Whether to animate wheel scrolling */
	bool bAnimateWheelScrolling : 1;

	/**	Whether the software cursor should be drawn in the viewport */
	bool bShowSoftwareCursor : 1;

	/** Whether or not the user supplied an external scrollbar to control scrolling. */
	bool bScrollBarIsExternal : 1;

	/** Are we actively scrolling right now */
	bool bIsScrolling : 1;

	/** Should the current scrolling be animated or immediately jump to the desired scroll offer */
	bool bAnimateScroll : 1;

	/** If true, will scroll to the end next Tick */
	bool bScrollToEnd : 1;

	/** Whether the active timer to update the inertial scroll is registered */
	bool bIsScrollingActiveTimerRegistered : 1;

	bool bAllowsRightClickDragScrolling : 1;
	
	bool bTouchPanningCapture : 1;

	bool bIsFocusable : 1;

	/** Whether content in the underlying panel will be allowed to shrink with the overall size of the ScrollBox instead of always taking up maximum space */
	bool bAllowContentToShrink = false;
};


template <>
struct TWidgetTypeTraits<class SScrollPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SScrollPanel : public SPanel
{
	SLATE_DECLARE_WIDGET(SScrollPanel, SPanel)
public:

	SLATE_BEGIN_ARGS(SScrollPanel)
		: _Orientation(EOrientation::Orient_Horizontal)
		, _BackPadScrolling(false)
		, _FrontPadScrolling(false)
		, _AllowContentToShrink(false)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

	SLATE_ARGUMENT(EOrientation, Orientation)
	SLATE_ARGUMENT(bool, BackPadScrolling)
	SLATE_ARGUMENT(bool, FrontPadScrolling)
	SLATE_ARGUMENT(bool, AllowContentToShrink)

	SLATE_END_ARGS()

	SScrollPanel()
	: Children(this)
	{
	}

	void Construct(const FArguments& InArgs, TArray<SScrollBox::FSlot::FSlotArguments> InSlots, TSharedRef<SScrollBox> InScrollBox);

public:

	EOrientation GetOrientation()
	{
		return Orientation;
	}

	void SetOrientation(EOrientation InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}

	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	virtual FChildren* GetChildren() override
	{
		return &Children;
	}

	const FVector2D& GetArrangedSize() const
	{
		return ArrangedSize;
	}

	float PhysicalOffset;
	TPanelChildren<SScrollBox::FSlot> Children;

protected:
	// Begin SWidget overrides.
	virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

private:
	TWeakPtr<SScrollBox> OwningScrollBox;
	EOrientation Orientation;
	bool BackPadScrolling;
	bool FrontPadScrolling;
	bool bAllowContentToShrink;
	mutable FVector2D ArrangedSize;
};
