// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"

template <>
struct TWidgetTypeTraits<class SSeparator>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SSeparator)
		: _SeparatorImage( FCoreStyle::Get().GetBrush("Separator") )
		, _Orientation(Orient_Horizontal)
		, _Thickness(3.0f)
		, _ColorAndOpacity(FLinearColor::White)
		{}
		
		SLATE_ARGUMENT(const FSlateBrush*, SeparatorImage)
		/** A horizontal separator is used in a vertical list (orientation is direction of the line drawn) */
		SLATE_ARGUMENT( EOrientation, Orientation)

		SLATE_ARGUMENT(float, Thickness)

		/** Color and opacity */
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End of SWidget interface
private:
	EOrientation Orientation;
	float Thickness;
};
