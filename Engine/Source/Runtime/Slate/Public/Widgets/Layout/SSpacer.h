// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;

template <>
struct TWidgetTypeTraits<class SSpacer>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SSpacer : public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SSpacer, SLeafWidget, SLATE_API)

public:

	SLATE_BEGIN_ARGS( SSpacer )
		: _Size(FVector2D::ZeroVector)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		SLATE_ATTRIBUTE( FVector2D, Size )
	SLATE_END_ARGS()

	SLATE_API SSpacer();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	// End of SWidget interface

	FVector2D GetSize() const
	{
		if (bIsSpacerSizeBound)
		{
			SSpacer& MutableSelf = const_cast<SSpacer&>(*this);
			MutableSelf.SpacerSize.UpdateNow(MutableSelf);
		}
		return SpacerSize.Get();
	}

	void SetSize( TAttribute<FVector2D> InSpacerSize )
	{
		bIsSpacerSizeBound = InSpacerSize.IsBound();
		SpacerSize.Assign(*this, InSpacerSize);
	}

protected:
	//~ Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SWidget overrides.

private:
	TSlateAttribute<FVector2D> SpacerSize;
	bool bIsSpacerSizeBound;
};
