// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/LayoutGeometry.h"
#include "Widgets/SWidget.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SPanel.h"
#include "Styling/SlateTypes.h"

#define UE_API SLATE_API

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

namespace ESplitterResizeMode
{
	enum Type
	{
		/** Resize the selected slot. If space is needed, then resize the next resizable slot. */
		FixedPosition,
		/** Resize the selected slot. If space is needed, then resize the last resizable slot. */
		FixedSize,
		/** Resize the selected slot by redistributing the available space with the following resizable slots. */
		Fill,
	};
}

class FLayoutGeometry;


template <>
struct TWidgetTypeTraits<class SSplitter>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * SSplitter divides its allotted area into N segments, where N is the number of children it has.
 * It allows the users to resize the children along the splitters axis: that is, horizontally or vertically.
 */
class SSplitter : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SSplitter, SPanel, SLATE_API)
public:
	/** How should a child's size be determined */
	enum ESizeRule
	{
		/** Get the DesiredSize() of the content */
		SizeToContent,
		/** Use a fraction of the parent's size */
		FractionOfParent
	};

	DECLARE_DELEGATE_OneParam(
		FOnSlotResized,
		/** The new size coefficient of the slot */
		float );

	DECLARE_DELEGATE_RetVal_OneParam(FVector2D, FOnGetMaxSlotSize, int32);

public:
	class FSlot : public TWidgetSlotWithAttributeSupport<FSlot>
	{
	public:		
		FSlot();

		SLATE_SLOT_BEGIN_ARGS(FSlot, TWidgetSlotWithAttributeSupport<FSlot>)
			/** The size rule used by the slot. */
			SLATE_ATTRIBUTE(ESizeRule, SizeRule)
			/** When the SizeRule is set to FractionOfParent, the size of the slot is the Value percentage of its parent size. */
			SLATE_ATTRIBUTE(float, Value)
			/** Minimum slot size when resizing. */
			SLATE_ATTRIBUTE(float, MinSize)
			/** Can the slot be resized by the user. */
			SLATE_ARGUMENT(TOptional<bool>, Resizable)
			/** Callback when the slot is resized. */
			SLATE_EVENT(FOnSlotResized, OnSlotResized)
		SLATE_SLOT_END_ARGS()

		UE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);
		static UE_API void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer);

		/** When the SizeRule is set to FractionOfParent, the size of the slot is the Value percentage of its parent size. */
		UE_API void SetSizeValue( TAttribute<float> InValue );
		float GetSizeValue() const
		{
			return SizeValue.Get();
		}

		/**
		 * Can the slot be resized by the user.
		 * @see CanBeResized()
		 */
		void SetResizable(bool bInIsResizable)
		{
			bIsResizable = bInIsResizable;
		}
		bool IsResizable() const
		{
			return bIsResizable.Get(false);
		}

		/** Minimum slot size when resizing. */
		UE_API void SetMinSize( TAttribute<float> InMinSize );
		float GetMinSize() const
		{
			return MinSizeValue.Get();
		}
		
		/**
		 * Callback when the slot is resized.
		 * @see CanBeResized()
		 */
		FOnSlotResized& OnSlotResized()
		{
			return OnSlotResized_Handler;
		}
		const FOnSlotResized& OnSlotResized() const
		{
			return OnSlotResized_Handler;
		}

		/** The size rule used by the slot. */
		UE_API void SetSizingRule( TAttribute<ESizeRule> InSizeRule ); 
		ESizeRule GetSizingRule() const
		{
			return SizingRule.Get();
		}

	public:
		/** A slot can be resize if bIsResizable and the SizeRule is a FractionOfParent or the OnSlotResized delegate is set. */
		UE_API bool CanBeResized() const;

	private:
		TSlateSlotAttribute<ESizeRule> SizingRule;
		TSlateSlotAttribute<float> SizeValue;
		TSlateSlotAttribute<float> MinSizeValue;

		FOnSlotResized OnSlotResized_Handler;
		TOptional<bool> bIsResizable;
	};

	/** @return Add a new FSlot() */
	SLATE_API static FSlot::FSlotArguments Slot();
	
	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Add a slot to the splitter at the specified index
	 * Sample usage:
	 *     SomeSplitter->AddSlot()
	 *     [
	 *       SNew(SSomeWidget)
	 *     ];
	 *
	 * @return the new slot.
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot( int32 AtIndex = INDEX_NONE );

	DECLARE_DELEGATE_OneParam(FOnHandleHovered, int32);

	SLATE_BEGIN_ARGS(SSplitter)
		: _Style( &FCoreStyle::Get().GetWidgetStyle<FSplitterStyle>("Splitter") )
		, _Orientation( Orient_Horizontal )
		, _ResizeMode( ESplitterResizeMode::FixedPosition )
		, _PhysicalSplitterHandleSize( 5.0f )
		, _HitDetectionSplitterHandleSize( 5.0f )
		, _MinimumSlotHeight( 20.0f )
		, _OnSplitterFinishedResizing()
		{
		}

		SLATE_SLOT_ARGUMENT(FSlot, Slots)

		/** Style used to draw this splitter */
		SLATE_STYLE_ARGUMENT( FSplitterStyle, Style )

		SLATE_ARGUMENT( EOrientation, Orientation )

		SLATE_ARGUMENT( ESplitterResizeMode::Type, ResizeMode )

		SLATE_ARGUMENT( float, PhysicalSplitterHandleSize )

		SLATE_ARGUMENT( float, HitDetectionSplitterHandleSize )

		SLATE_ARGUMENT( float, MinimumSlotHeight )

		SLATE_ATTRIBUTE( int32, HighlightedHandleIndex )

		SLATE_EVENT( FOnHandleHovered, OnHandleHovered )

		SLATE_EVENT( FSimpleDelegate, OnSplitterFinishedResizing )
		
		SLATE_EVENT( FOnGetMaxSlotSize, OnGetMaxSlotSize )

	SLATE_END_ARGS()

	SLATE_API SSplitter();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

public:

	/**
	 * Get the slot at the specified index
	 *
	 * @param SlotIndex    Replace the child at this index.
	 *
	 * @return Slot at the index specified by SlotIndex
	 */
	SLATE_API SSplitter::FSlot& SlotAt( int32 SlotIndex );

	/** Removes a slot from this splitter panel which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The index in the children array where the slot was removed and -1 if no slot was found matching the widget
	 */
	SLATE_API int32 RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	/**
	 * Remove the child at IndexToRemove
	 *
	 * @param IndexToRemove     Remove the slot and child at this index.
	 */
	SLATE_API void RemoveAt( int32 IndexToRemove );

	/** Removes all children from the splitter. */
	SLATE_API void ClearChildren();

	/** @return the number of slots. */
	SLATE_API int32 NumSlots() const;

	/** @return if it's a valid index slot index. */
	SLATE_API bool IsValidSlotIndex(int32 Index) const;

public:

	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;


	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;


	/**
	 * A Panel's desired size in the space required to arrange of its children on the screen while respecting all of
	 * the children's desired sizes and any layout-related options specified by the user. See StackPanel for an example.
	 */
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	/**
	 * All widgets must provide a way to access their children in a layout-agnostic way.
	 * Panels store their children in Slots, which creates a dilemma. Most panels
	 * can store their children in a TPanelChildren<Slot>, where the Slot class
	 * provides layout information about the child it stores. In that case
	 * GetChildren should simply return the TPanelChildren<Slot>. See StackPanel for an example.
	 */
	SLATE_API virtual FChildren* GetChildren() override;

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	
	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	SLATE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;

	/**
	 * The system asks each widget under the mouse to provide a cursor. This event is bubbled.
	 * 
	 * @return FCursorReply::Unhandled() if the event is not handled; return FCursorReply::Cursor() otherwise.
	 */
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;

	/**
	 * Change the orientation of the splitter
	 *
	 * @param NewOrientation  Should the splitter be horizontal or vertical
	 */
	SLATE_API void SetOrientation( EOrientation NewOrientation );

	/**
	 * @return the current orientation of the splitter.
	 */
	SLATE_API EOrientation GetOrientation() const;

private:
	SLATE_API TArray<FLayoutGeometry> ArrangeChildrenForLayout( const FGeometry& AllottedGeometry ) const;

protected:

	/**
	 * Given the index of the dragged handle and the children, find a child above/left_of of the dragged handle that can be resized.
	 *
	 * @return INDEX_NONE if no such child can be found.
	 */
	static SLATE_API int32 FindResizeableSlotBeforeHandle( int32 DraggedHandle, const TPanelChildren<FSlot>& Children );

	/**
	 * Given the index of the dragged handle and the children, find a child below/right_of the dragged handle that can be resized
	 *
	 * @return Children.Num() if no such child can be found.
	 */
	static SLATE_API int32 FindResizeableSlotAfterHandle( int32 DraggedHandle, const TPanelChildren<FSlot>& Children );

	static SLATE_API void FindAllResizeableSlotsAfterHandle( int32 DraggedHandle, const TPanelChildren<FSlot>& Children, TArray<int32, FConcurrentLinearArrayAllocator>& OutSlotIndicies );

	/**
	 * Resizes the children based on user input. The template parameter Orientation corresponds to the splitter being horizontal or vertical.
	 * 
	 * @param DraggedHandle    The index of the handle that the user is dragging.
	 * @param LocalMousePos    The position of the mouse in this widgets local space.
	 * @param Children         A reference to this splitter's children array; we will modify the children's layout values.
	 * @param ChildGeometries  The arranged children; we need their sizes and positions so that we can perform a resizing.
	 */
	SLATE_API void HandleResizingByMousePosition(EOrientation Orientation, const float PhysicalSplitterHandleSize, const ESplitterResizeMode::Type ResizeMode, int32 DraggedHandle, const FVector2D& LocalMousePos, TPanelChildren<FSlot>& Children, const TArray<FLayoutGeometry>& ChildGeometries );
	SLATE_API void HandleResizingDelta(EOrientation Orientation, const float PhysicalSplitterHandleSize, const ESplitterResizeMode::Type ResizeMode, int32 DraggedHandle, float Delta, TPanelChildren<FSlot>& Children, const TArray<FLayoutGeometry>& ChildGeometries);
	SLATE_API void HandleResizingBySize(EOrientation Orientation, const float PhysicalSplitterHandleSize, const ESplitterResizeMode::Type ResizeMode, int32 DraggedHandle, const FVector2D& DesiredSize, TPanelChildren<FSlot>& Children, const TArray<FLayoutGeometry>& ChildGeometries);

	/**
	 * @param ProposedSize  A size that a child would like to be
	 *
	 * @return A size that is clamped against the minimum size allowed for children.
	 */
	SLATE_API float ClampChild(const FSlot& ChildSlot, float ProposedSize) const;

	/**
	 * Given a mouse position within the splitter, figure out which resize handle we are hovering (if any).
	 *

	 * @param LocalMousePos  The mouse position within this splitter.
	 * @param ChildGeometris The arranged children and their geometries; we need to test the mouse against them.
	 *
	 * @return The index of the handle being hovered, or INDEX_NONE if we are not hovering a handle.
	 */
	template<EOrientation SplitterOrientation>
	static int32 GetHandleBeingResizedFromMousePosition(  float PhysicalSplitterHandleSize, float HitDetectionSplitterHandleSize, FVector2D LocalMousePos, const TArray<FLayoutGeometry>& ChildGeometries );

	TPanelChildren< FSlot > Children;

	int32 HoveredHandleIndex;
	TSlateAttribute<int32, EInvalidateWidgetReason::Paint> HighlightedHandleIndex;
	bool bIsResizing;
	EOrientation Orientation;
	ESplitterResizeMode::Type ResizeMode;

	FSimpleDelegate OnSplitterFinishedResizing;
	FOnGetMaxSlotSize OnGetMaxSlotSize;
	FOnHandleHovered OnHandleHovered;

	/** The user is not allowed to make any of the splitter's children smaller than this. */
	float MinSplitterChildLength;

	/** The thickness of the grip area that the user uses to resize a splitter */
	float PhysicalSplitterHandleSize;
	float HitDetectionSplitterHandleSize;

	const FSplitterStyle* Style;
};

enum class EResizingAxis : uint8
{
	None = 0x00,
	LeftRightMask = 0x01,
	UpDownMask = 0x10,
	CrossMask = UpDownMask | LeftRightMask
};
ENUM_CLASS_FLAGS(EResizingAxis);

/**
 * SSplitter2x2													
 * A splitter which has exactly 4 children and allows simultaneous		
 * of all children along an axis as well as resizing all children
 * by dragging the center of the splitter.
 */
class SSplitter2x2 : public SPanel
{
private:
	class FSlot : public TSlotBase<FSlot>
	{
	public:	
		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
			SLATE_ATTRIBUTE(FVector2D, Percentage)
		SLATE_SLOT_END_ARGS()

		/**
		 * Sets the percentage attribute
		 *
		 * @param Value The new percentage value
		 */
		void SetPercentage( const FVector2D& Value )
		{
			PercentageAttribute.Set( Value );
		}

		FVector2D GetPercentage() const
		{
			return PercentageAttribute.Get();
		}

		SLATE_API FSlot(const TSharedRef<SWidget>& InWidget);

		SLATE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArg);

	private:
		/** The percentage of the alloted space of the splitter that this slot requires */
		TAttribute<FVector2D> PercentageAttribute;
	};

	SLATE_BEGIN_ARGS( SSplitter2x2 )
		: _Style(&FCoreStyle::Get().GetWidgetStyle<FSplitterStyle>("Splitter"))
		{
		}
		/** Style used to draw this splitter */
		SLATE_STYLE_ARGUMENT(FSplitterStyle, Style)
		SLATE_NAMED_SLOT( FArguments, TopLeft )
		SLATE_NAMED_SLOT( FArguments, BottomLeft )
		SLATE_NAMED_SLOT( FArguments, TopRight )
		SLATE_NAMED_SLOT( FArguments, BottomRight )
	SLATE_END_ARGS()

	SLATE_API SSplitter2x2();

	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Returns the widget displayed in the splitter top left area
	 *
	 * @return	Top left widget
	 */
	SLATE_API TSharedRef< SWidget > GetTopLeftContent();

	/**
	 * Returns the widget displayed in the splitter bottom left area
	 *
	 * @return	Bottom left widget
	 */
	SLATE_API TSharedRef< SWidget > GetBottomLeftContent();

	/**
	 * Returns the widget displayed in the splitter top right area
	 *
	 * @return	Top right widget
	 */
	SLATE_API TSharedRef< SWidget > GetTopRightContent();

	/**
	 * Returns the widget displayed in the splitter bottom right area
	 *
	 * @return	Bottom right widget
	 */
	SLATE_API TSharedRef< SWidget > GetBottomRightContent();

	/**
	 * Sets the widget to be displayed in the splitter top left area
	 *
	 * @param	TopLeftContent	The top left widget
	 */
	SLATE_API void SetTopLeftContent( TSharedRef< SWidget > TopLeftContent );

	/**
	 * Sets the widget to be displayed in the splitter bottom left area
	 *
	 * @param	BottomLeftContent	The bottom left widget
	 */
	SLATE_API void SetBottomLeftContent( TSharedRef< SWidget > BottomLeftContent );

	/**
	 * Sets the widget to be displayed in the splitter top right area
	 *
	 * @param	TopRightContent	The top right widget
	 */
	SLATE_API void SetTopRightContent( TSharedRef< SWidget > TopRightContent );

	/**
	 * Sets the widget to be displayed in the splitter bottom right area
	 *
	 * @param	BottomRightContent	The bottom right widget
	 */
	SLATE_API void SetBottomRightContent( TSharedRef< SWidget > BottomRightContent );

	/** Returns an array of size percentages for the children in this order: TopLeft, BottomLeft, TopRight, BottomRight */
	SLATE_API void GetSplitterPercentages( TArray< FVector2D >& OutPercentages ) const;
	
	/** Sets the size percentages for the children in this order: TopLeft, BottomLeft, TopRight, BottomRight */
	SLATE_API void SetSplitterPercentages( TArrayView< FVector2D > InPercentages );


private:

	TArray<FLayoutGeometry> ArrangeChildrenForLayout( const FGeometry& AllottedGeometry ) const;

	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/**
	 * A Panel's desired size in the space required to arrange of its children on the screen while respecting all of
	 * the children's desired sizes and any layout-related options specified by the user. See StackPanel for an example.
	 */
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	/**
	 * All widgets must provide a way to access their children in a layout-agnostic way.
	 * Panels store their children in Slots, which creates a dilemma. Most panels
	 * can store their children in a TPanelChildren<Slot>, where the Slot class
	 * provides layout information about the child it stores. In that case
	 * GetChildren should simply return the TPanelChildren<Slot>. See StackPanel for an example.
	 */
	SLATE_API virtual FChildren* GetChildren() override;

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
 	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	
	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
 	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	/**
	 * The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	/**
	 * @return The cursor that should be visible
	 */
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;

	/**
	 * Calculates the axis being resized
	 * 
	 * @param MyGeometry	The geometry of this widget
	 * @param LocalMousePos	The local space current mouse position
	 */
	EResizingAxis CalculateResizingAxis( const FGeometry& MyGeometry, const FVector2D& LocalMousePos ) const;

	/**
	 * Resizes all children based on a user moving the splitter handles
	 * 
	 * @param ArrangedChildren	The current geometry of all arranged children before the user moved the splitter
	 * @param LocalMousePos		The current mouse position		
	 */
	void ResizeChildren( const FGeometry& MyGeometry, const TArray<FLayoutGeometry>& ArrangedChildren, const FVector2D& LocalMousePos );


private:

	/** The children of the splitter. There can only be four */
	TPanelChildren<FSlot> Children;

	const FSplitterStyle* Style = nullptr;

	/** The axis currently being resized or INDEX_NONE if no resizing */
	EResizingAxis ResizingAxisMask;

	/** true if a splitter axis is currently being resized. */
	bool bIsResizing;

	float SplitterHandleSize;

	float MinSplitterChildLength;
};

#undef UE_API
