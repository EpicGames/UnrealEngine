// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Layout/Margin.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

/** A panel that evenly divides up available space between all of its children. */
class SUniformGridPanel : public SPanel
{
public:
	/** Stores the per-child info for this panel type */
	struct FSlot : public TSlotBase<FSlot>, public TAlignmentWidgetSlotMixin<FSlot>
	{
		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TSlotBase<FSlot>, TAlignmentWidgetSlotMixin<FSlot>)
		SLATE_SLOT_END_ARGS()

		FSlot( int32 InColumn, int32 InRow )
			: TSlotBase<FSlot>()
			, TAlignmentWidgetSlotMixin<FSlot>(HAlign_Fill, VAlign_Fill)
			, Column( InColumn )
			, Row( InRow )
			{
			}

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			TAlignmentWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
		}

		void SetColumn(int32 InColumn)
		{
			if (InColumn != Column)
			{
				Column = InColumn;
				Invalidate(EInvalidateWidgetReason::Layout);
			}
		}

		int32 GetColumn() const
		{
			return Column;
		}

		int32 GetRow() const
		{
			return Row;
		}

		void SetRow(int32 InRow)
		{
			if (InRow != Row)
			{
				Row = InRow;
				Invalidate(EInvalidateWidgetReason::Layout);
			}
		}

	private:
		int32 Column;
		int32 Row;
	};

	SLATE_API SUniformGridPanel();
	SLATE_API virtual ~SUniformGridPanel();

	/**
	 * Used by declarative syntax to create a Slot in the specified Column, Row.
	 */
	static SLATE_API FSlot::FSlotArguments Slot( int32 Column, int32 Row );

	SLATE_BEGIN_ARGS( SUniformGridPanel )
		: _SlotPadding( FMargin(0.0f) )
		, _MinDesiredSlotWidth( 0.0f )
		, _MinDesiredSlotHeight( 0.0f )
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** Slot type supported by this panel */
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
		
		/** Padding given to each slot */
		SLATE_ATTRIBUTE(FMargin, SlotPadding)

		/** The minimum desired width of the slots */
		SLATE_ATTRIBUTE(float, MinDesiredSlotWidth)

		/** The minimum desired height of the slots */
		SLATE_ATTRIBUTE(float, MinDesiredSlotHeight)

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	//~ Begin SPanel Interface	
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	//~ End SPanel Interface

	/** See SlotPadding attribute */
	SLATE_API void SetSlotPadding(TAttribute<FMargin> InSlotPadding);

	/** See MinDesiredSlotWidth attribute */
	SLATE_API void SetMinDesiredSlotWidth(TAttribute<float> InMinDesiredSlotWidth);

	/** See MinDesiredSlotHeight attribute */
	SLATE_API void SetMinDesiredSlotHeight(TAttribute<float> InMinDesiredSlotHeight);

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Dynamically add a new slot to the UI at specified Column and Row.
	 *
	 * @return A reference to the newly-added slot
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot( int32 Column, int32 Row );
	
	/**
	 * Removes a slot from this panel which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The true if the slot was removed and false if no slot was found matching the widget
	 */
	SLATE_API bool RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	/** Removes all slots from the panel */
	SLATE_API void ClearChildren();

protected:
	// Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

private:
	TPanelChildren<FSlot> Children;
	TSlateAttribute<FMargin, EInvalidateWidgetReason::Layout> SlotPadding;
	
	/** These values are recomputed and cached during compute desired size, as they may have changed since the previous frame. */
	mutable int32 NumColumns;
	mutable int32 NumRows;
	mutable TSet<int32> CollapsedColumns;
	mutable TSet<int32> CollapsedRows;

	TSlateAttribute<float, EInvalidateWidgetReason::Layout> MinDesiredSlotWidth;
	TSlateAttribute<float, EInvalidateWidgetReason::Layout> MinDesiredSlotHeight;
};
