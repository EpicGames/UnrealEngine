// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Layout/Margin.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Types/SlateEnums.h"

class FArrangedChildren;

template <>
struct TWidgetTypeTraits<class SUniformWrapPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A panel that evenly divides up available space between all of its children. */
class SUniformWrapPanel : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SUniformWrapPanel, SPanel, SLATE_API)

public:
	/** Stores the per-child info for this panel type */
	struct FSlot : public TWidgetSlotWithAttributeSupport<FSlot>, public TAlignmentWidgetSlotMixin<FSlot>
	{
		FSlot()
			: TWidgetSlotWithAttributeSupport<FSlot>()
			, TAlignmentWidgetSlotMixin<FSlot>(HAlign_Fill, VAlign_Fill)
		{ }

		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TWidgetSlotWithAttributeSupport<FSlot>, TAlignmentWidgetSlotMixin<FSlot>)
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			TAlignmentWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
		}
		static void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
		{
			TWidgetSlotWithAttributeSupport::RegisterAttributes(AttributeInitializer);
		}
	};

	SLATE_API SUniformWrapPanel();
	SLATE_API virtual ~SUniformWrapPanel();

	/**
	 * Used by declarative syntax to create a Slot.
	 */
	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	SLATE_BEGIN_ARGS( SUniformWrapPanel )
		: _SlotPadding( FMargin(0.0f) )
		, _NumColumnsOverride( 0 )
		, _MinDesiredSlotWidth( 0.0f )
		, _MinDesiredSlotHeight( 0.0f )
		, _MaxDesiredSlotWidth( FLT_MAX )
		, _MaxDesiredSlotHeight( FLT_MAX )
		, _EvenRowDistribution(false)
		, _HAlign(EHorizontalAlignment::HAlign_Left)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** Slot type supported by this panel */
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
		
		/** Padding given to each slot */
		SLATE_ATTRIBUTE(FMargin, SlotPadding)

		/** The number of columns for the wrapped panel */
		SLATE_ATTRIBUTE(int32, NumColumnsOverride)
	
		/** The minimum desired width of the slots */
		SLATE_ATTRIBUTE(float, MinDesiredSlotWidth)

		/** The minimum desired height of the slots */
		SLATE_ATTRIBUTE(float, MinDesiredSlotHeight)

		/** The minimum desired width of the slots */
		SLATE_ATTRIBUTE(float, MaxDesiredSlotWidth)

		/** The minimum desired height of the slots */
		SLATE_ATTRIBUTE(float, MaxDesiredSlotHeight)

		/** If the distribution to evenly distribute down rows */
		SLATE_ATTRIBUTE(bool, EvenRowDistribution)

		/** How to distribute the elements among any extra space in a given row */
		SLATE_ATTRIBUTE(EHorizontalAlignment, HAlign )

	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	//~ Begin SPanel Interface	
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	//~ End SPanel Interface

	/** See SlotPadding attribute */
	SLATE_API void SetSlotPadding(TAttribute<FMargin> InSlotPadding);

	/** See MinDesiredSlotWidth attribute */
	SLATE_API void SetMinDesiredSlotWidth(TAttribute<float> InMinDesiredSlotWidth);

	/** See MinDesiredSlotHeight attribute */
	SLATE_API void SetMinDesiredSlotHeight(TAttribute<float> InMinDesiredSlotHeight);

	/** See NumColumnsOverride attribute */
	SLATE_API void SetNumColumnsOverride(TAttribute<int32> InNumColumnsOverride);
	
	/** See MinDesiredSlotWidth attribute */
	SLATE_API void SetMaxDesiredSlotWidth(TAttribute<float> InMaxDesiredSlotWidth);

	/** See MinDesiredSlotHeight attribute */
	SLATE_API void SetMaxDesiredSlotHeight(TAttribute<float> InMaxDesiredSlotHeight);

	/** See HAlign attribute */
	SLATE_API void SetHorizontalAlignment(TAttribute<EHorizontalAlignment> InHAlignment);
	EHorizontalAlignment GetHorizontalAlignment() { return HAlign.Get(); }

	/** See EvenRowDistribution attribute */
	SLATE_API void SetEvenRowDistribution(TAttribute<bool> InEvenRowDistribution);
	bool GetEvenRowDistribution() { return EvenRowDistribution.Get(); }

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	SLATE_API FScopedWidgetSlotArguments AddSlot();
	
	/**
	 * Removes a slot from this panel which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The true if the slot was removed and false if no slot was found matching the widget
	 */
	SLATE_API bool RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	/** Removes all slots from the panel */
	SLATE_API void ClearChildren();

	/** @return the number of slots. */
	int32 NumSlots() const { return Children.Num(); }

protected:
	// Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

	SLATE_API FVector2D ComputeUniformCellSize() const;

private:
	TPanelChildren<FSlot> Children;
	TSlateAttribute<FMargin> SlotPadding;
	
	/** These values are recomputed and cached during compute desired size, as they may have changed since the previous frame. */
	mutable int32 NumColumns;
	mutable int32 NumRows;
	mutable int32 NumVisibleChildren;

	TSlateAttribute<int32> NumColumnsOverride;
	
	TSlateAttribute<float> MinDesiredSlotWidth;
	TSlateAttribute<float> MinDesiredSlotHeight;

	TSlateAttribute<float> MaxDesiredSlotWidth;
	TSlateAttribute<float> MaxDesiredSlotHeight;

	TSlateAttribute<EHorizontalAlignment> HAlign;
	TSlateAttribute<bool> EvenRowDistribution;
};
