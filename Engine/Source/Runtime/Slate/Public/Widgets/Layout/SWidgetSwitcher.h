// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

template <>
struct TWidgetTypeTraits<class SWidgetSwitcher>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Implements a widget switcher.
 *
 * A widget switcher is like a tab control, but without tabs. At most one widget is visible at time.
 */
class SWidgetSwitcher
	: public SPanel
{
	SLATE_DECLARE_WIDGET_API(SWidgetSwitcher, SPanel, SLATE_API)
public:

	using FSlot = FBasicLayoutWidgetSlot;

	SLATE_BEGIN_ARGS(SWidgetSwitcher)
		: _WidgetIndex(0)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		SLATE_SLOT_ARGUMENT(FSlot, Slots)

		/** Holds the index of the initial widget to be displayed (INDEX_NONE = default). */
		SLATE_ATTRIBUTE(int32, WidgetIndex)

	SLATE_END_ARGS()

	SLATE_API SWidgetSwitcher();
	SLATE_API virtual ~SWidgetSwitcher();

public:

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Adds a slot to the widget switcher at the specified location.
	 *
	 * @param SlotIndex The index at which to insert the slot, or INDEX_NONE to append.
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot( int32 SlotIndex = INDEX_NONE );

	/**
	 * Constructs the widget.
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Gets the active widget.
	 *
	 * @return Active widget.
	 */
	SLATE_API TSharedPtr<SWidget> GetActiveWidget( ) const;

	/**
	 * Gets the slot index of the currently active widget.
	 *
	 * @return The slot index, or INDEX_NONE if no widget is active.
	 */
	int32 GetActiveWidgetIndex( ) const
	{
		return WidgetIndex.Get();
	}

	/**
	 * Gets the number of widgets that this switcher manages.
	 *
	 * @return Number of widgets.
	 */
	int32 GetNumWidgets( ) const
	{
		return AllChildren.Num();
	}

	/**
	 * Gets the widget in the specified slot.
	 *
	 * @param SlotIndex The slot index of the widget to get.
	 * @return The widget, or nullptr if the slot does not exist.
	 */
	SLATE_API TSharedPtr<SWidget> GetWidget( int32 SlotIndex ) const;

	/**
	 * Gets the slot index of the specified widget.
	 *
	 * @param Widget The widget to get the index for.
	 * @return The slot index, or INDEX_NONE if the widget does not exist.
	 */
	SLATE_API int32 GetWidgetIndex( TSharedRef<SWidget> Widget ) const;

	/**
	 * Removes a slot with the corresponding widget in it.  Returns the index where the widget was found, otherwise -1.
	 *
	 * @param Widget The widget to find and remove.
	 */
	SLATE_API int32 RemoveSlot( TSharedRef<SWidget> WidgetToRemove );

	/**
	 * Sets the active widget.
	 *
	 * @param Widget The widget to activate.
	 */
	void SetActiveWidget( TSharedRef<SWidget> Widget )
	{
		SetActiveWidgetIndex(GetWidgetIndex(Widget));
	}

	/**
	 * Activates the widget at the specified index.
	 *
	 * @param Index The slot index.
	 */
	SLATE_API void SetActiveWidgetIndex( int32 Index );

	SLATE_API virtual bool ValidatePathToChild(SWidget* InChild) override;

public:

	/**
	 * Creates a new widget slot.
	 *
	 * @return A new slot.
	 */
	static SLATE_API FSlot::FSlotArguments Slot();

protected:
	SLATE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize( float ) const override;
	SLATE_API virtual FChildren* GetChildren( ) override;
	virtual void OnSlotAdded(int32 AddedIndex) {}
	virtual void OnSlotRemoved(int32 RemovedIndex, TSharedRef<SWidget> RemovedWidget, bool bWasActiveSlot) {}
	SLATE_API const FSlot* GetActiveSlot() const;

	TPanelChildren<FSlot>& GetTypedChildren() { return AllChildren; }

private:
	void SetActiveWidgetIndexInternal();

	/** Holds the desired widget index */
	TSlateAttribute<int32> WidgetIndex;
	int32 PreviousWidgetIndex = INDEX_NONE;

	/*
	 * Holds the collection of all child widget, however the only one with a valid parent pointer
	 * will be the one in the dynamic slot.
	 */
	TPanelChildren<FSlot> AllChildren;

	/** Required to implement GetChildren() in a way that can dynamically return the currently active child. */
	TOneDynamicChildBase<FSlot, TSlateAttribute<int32>> OneDynamicChild;

#if WITH_ACCESSIBILITY
	/** Used to detect when WidgetIndex changes while bound. */
	TWeakPtr<SWidget> LastActiveWidget;
#endif
};
