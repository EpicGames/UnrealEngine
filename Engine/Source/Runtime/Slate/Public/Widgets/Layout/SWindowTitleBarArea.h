// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SButton.h"


class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class SWindow;

template< typename ObjectType > class TAttribute;

class SWindowTitleBarArea
	: public SPanel
{
public:
	
	using FSlot = FSingleWidgetChildrenWithBasicLayoutSlot;

	SLATE_BEGIN_ARGS(SWindowTitleBarArea)
		: _HAlign( HAlign_Fill )
		, _VAlign( VAlign_Fill )
		, _Padding( 0.0f )
		, _Content()
		{ }

		/** Horizontal alignment of content in the area allotted to the SWindowTitleBarArea by its parent */
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )

		/** Vertical alignment of content in the area allotted to the SWindowTitleBarArea by its parent */
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )

		/** Padding between the SWindowTitleBarArea and the content that it presents. Padding affects desired size. */
		SLATE_ATTRIBUTE( FMargin, Padding )

		/** The widget content presented by the SWindowTitleBarArea */
		SLATE_DEFAULT_SLOT( FArguments, Content )

		/** Called when the maximize/restore button or double click needs to toggle fullscreen. We don't have access to GEngine in SWindowTitleBarArea, so the GameLayerManager or UMG widget will handle this. */
		SLATE_EVENT( FSimpleDelegate, RequestToggleFullscreen )

	SLATE_END_ARGS()

	SLATE_API SWindowTitleBarArea();
	SLATE_API virtual ~SWindowTitleBarArea();

	SLATE_API void Construct( const FArguments& InArgs );

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

	SLATE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	SLATE_API virtual EWindowZone::Type GetWindowZoneOverride() const override;

	/** See the Content slot. */
	SLATE_API void SetContent(const TSharedRef< SWidget >& InContent);

	/** See HAlign argument */
	SLATE_API void SetHAlign(EHorizontalAlignment HAlign);

	/** See VAlign argument */
	SLATE_API void SetVAlign(EVerticalAlignment VAlign);

	/** See Padding attribute */
	SLATE_API void SetPadding(TAttribute<FMargin> InPadding);

	void SetGameWindow(TSharedPtr<SWindow> Window)
	{
		GameWindow = Window;
	}

	void SetRequestToggleFullscreenCallback(FSimpleDelegate InRequestToggleFullscreen)
	{
		RequestToggleFullscreen = InRequestToggleFullscreen;
	}

	static void SetOnCloseButtonClickedDelegate(FSimpleDelegate InOnCloseCuttonClicked)
	{
		OnCloseButtonClicked = InOnCloseCuttonClicked;
	}

	static void SetIsCloseButtonActive(bool bIsAcive)
	{
		bIsCloseButtonActive = bIsAcive;
	}

	void SetWindowButtonsVisibility(bool bIsVisible)
	{
		WindowButtonsBox->SetVisibility(bIsVisible && PLATFORM_DESKTOP ? EVisibility::Visible : EVisibility::Collapsed);
	}

	static void SetCustomStyleForWindowButtons(const FButtonStyle& InMinimizeButtonStyle, const FButtonStyle& InMaximizeButtonStyle, const FButtonStyle& InRestoreButtonStyle, const FButtonStyle& InCloseButtonStyle)
	{
		MinimizeButtonStyle = InMinimizeButtonStyle;
		MaximizeButtonStyle = InMaximizeButtonStyle;
		RestoreButtonStyle = InRestoreButtonStyle;
		CloseButtonStyle = InCloseButtonStyle;
	}

protected:

	FSlot ChildSlot;

private:

	TSharedPtr<SWindow> GameWindow;
	FSimpleDelegate RequestToggleFullscreen;

	static SLATE_API FSimpleDelegate OnCloseButtonClicked;
	static SLATE_API bool bIsCloseButtonActive;

	static SLATE_API FButtonStyle MinimizeButtonStyle;
	static SLATE_API FButtonStyle MaximizeButtonStyle;
	static SLATE_API FButtonStyle RestoreButtonStyle;
	static SLATE_API FButtonStyle CloseButtonStyle;

	bool bIsMinimizeButtonEnabled;
	bool bIsMaximizeRestoreButtonEnabled;
	bool bIsCloseButtonEnabled;

	SLATE_API FReply MinimizeButton_OnClicked();
	SLATE_API FReply MaximizeRestoreButton_OnClicked();
	SLATE_API FReply CloseButton_OnClicked();

	SLATE_API const FSlateBrush* GetMinimizeImage() const;
	SLATE_API const FSlateBrush* GetMaximizeRestoreImage() const;
	SLATE_API const FSlateBrush* GetCloseImage() const;

	TSharedPtr<SButton> MinimizeButton;
	TSharedPtr<SButton> MaximizeRestoreButton;
	TSharedPtr<SButton> CloseButton;

	TSharedPtr<SVerticalBox> WindowButtonsBox;
};
