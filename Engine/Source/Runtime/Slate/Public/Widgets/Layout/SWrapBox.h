// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;

template <>
struct TWidgetTypeTraits<class SWrapBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * With EOrientation::Orient_Horizontal
 * Arranges widgets left-to-right.
 * When the widgets exceed the PreferredSize
 * the SWrapBox will place widgets on the next line.
 *
 * Illustration:
 *                      +-----Preferred Size
 *                      |
 *       [-----------][-|-]
 *       [--][------[--]|
 *       [--------------|]
 *       [---]          |
 */

 /**
  * With EOrientation::Orient_Vertical
  * Arranges widgets top-to-bottom.
  * When the widgets exceed the PreferredSize
  * the SVerticalWrapBox will place widgets on the next line.
  *
  * Illustration:
  *
  *      [___]  [___]
  *      [-1-]  [-3-]
  *
  *		 [___]  [___]
  *      [-2-]  [-4-]
  *
  *      [___]
  *==============================>--------Preferred Size
  *		 [-3-]
  */

class SWrapBox : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SWrapBox, SPanel, SLATE_API)
public:

	/** A slot that support alignment of content and padding */
	class FSlot : public TBasicLayoutWidgetSlot<FSlot>
	{
	public:
		FSlot()
			: TBasicLayoutWidgetSlot<FSlot>(HAlign_Fill, VAlign_Fill)
			, SlotFillLineWhenSizeLessThan()
			, bSlotFillEmptySpace(false)
			, bSlotForceNewLine(false)
		{
		}

		SLATE_SLOT_BEGIN_ARGS(FSlot, TBasicLayoutWidgetSlot<FSlot>)
			SLATE_ARGUMENT(TOptional<float>, FillLineWhenSizeLessThan)
			SLATE_ARGUMENT(TOptional<bool>, FillEmptySpace)
			SLATE_ARGUMENT(TOptional<bool>, ForceNewLine)
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TBasicLayoutWidgetSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			if (InArgs._FillLineWhenSizeLessThan.IsSet())
			{
				SlotFillLineWhenSizeLessThan = InArgs._FillLineWhenSizeLessThan;
			}
			bSlotFillEmptySpace = InArgs._FillEmptySpace.Get(bSlotFillEmptySpace);
			bSlotForceNewLine = InArgs._ForceNewLine.Get(bSlotForceNewLine);
		}

		/** Dependently of the Orientation, if the total available horizontal or vertical space in the wrap panel drops below this threshold, this slot will attempt to fill an entire line. */
		void SetFillLineWhenSizeLessThan(TOptional<float> InFillLineWhenSizeLessThan)
		{
			if (SlotFillLineWhenSizeLessThan != InFillLineWhenSizeLessThan)
			{
				SlotFillLineWhenSizeLessThan = InFillLineWhenSizeLessThan;
				FSlotBase::Invalidate(EInvalidateWidgetReason::Layout);
			}
		}

		TOptional<float> GetFillLineWhenSizeLessThan() const
		{
			return SlotFillLineWhenSizeLessThan;
		}

		/** Should this slot fill the remaining space on the line? */
		void SetFillEmptySpace(bool bInFillEmptySpace)
		{
			if (bSlotFillEmptySpace != bInFillEmptySpace)
			{
				bSlotFillEmptySpace = bInFillEmptySpace;
				FSlotBase::Invalidate(EInvalidateWidgetReason::Layout);
			}
		}

		bool GetFillEmptySpace() const
		{
			return bSlotFillEmptySpace;
		}

		void SetForceNewLine(bool bInForceNewLine)
		{
			if (bSlotForceNewLine != bInForceNewLine)
			{
				bSlotForceNewLine = bInForceNewLine;
				FSlotBase::Invalidate(EInvalidateWidgetReason::Layout);
			}
		}

		bool GetForceNewLine() const
		{
			return bSlotForceNewLine;
		}

	private:
		TOptional<float> SlotFillLineWhenSizeLessThan;
		bool bSlotFillEmptySpace;
		bool bSlotForceNewLine;
	};


	SLATE_BEGIN_ARGS(SWrapBox)
		: _PreferredSize(100.f)
		, _HAlign(HAlign_Left)
		, _InnerSlotPadding(FVector2D::ZeroVector)
		, _UseAllottedSize(false)
		, _Orientation(EOrientation::Orient_Horizontal)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** The slot supported by this panel */
		SLATE_SLOT_ARGUMENT( FSlot, Slots )

		/** The preferred size, if not set will fill the space */
		SLATE_ATTRIBUTE( float, PreferredSize )

		/** How to distribute the elements among any extra space in a given row */
		SLATE_ATTRIBUTE(EHorizontalAlignment, HAlign)

		/** The inner slot padding goes between slots sharing borders */
		SLATE_ARGUMENT( FVector2D, InnerSlotPadding )

		/** if true, the PreferredSize will always match the room available to the SWrapBox  */
		SLATE_ARGUMENT( bool, UseAllottedSize )

		/** Determines if the wrap box needs to arrange the slots left-to-right or top-to-bottom.*/
		SLATE_ARGUMENT(EOrientation, Orientation);
	SLATE_END_ARGS()

	SLATE_API SWrapBox();
	SLATE_API virtual ~SWrapBox();

	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	SLATE_API FScopedWidgetSlotArguments AddSlot();

	/** Removes a slot from this box panel which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The index in the children array where the slot was removed and -1 if no slot was found matching the widget
	 */
	SLATE_API int32 RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	SLATE_API void Construct( const FArguments& InArgs );

	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

	SLATE_API void ClearChildren();

	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	SLATE_API virtual FChildren* GetChildren() override;

	/** See InnerSlotPadding Attribute */
	SLATE_API void SetInnerSlotPadding(FVector2D InInnerSlotPadding);

	/** Set the size at which the wrap panel should wrap its content. */
	SLATE_API void SetWrapSize(TAttribute<float> InWrapSize );

	/** When true, use the WrapSize property to determine where to wrap to the next line. */
	SLATE_API void SetUseAllottedSize(bool bInUseAllottedSize);

	/** Set the Orientation to determine if the wrap box needs to arrange the slots left-to-right or top-to-bottom */
	SLATE_API void SetOrientation(EOrientation InOrientation);

	/** How to distribute the elements among any extra space in a given row */
	SLATE_API void SetHorizontalAlignment(TAttribute<EHorizontalAlignment> InHAlignment);

private:
	/** How wide or long, dependently of the orientation, this panel should appear to be. Any widgets past this line will be wrapped onto the next line. */
	TSlateAttribute<float> PreferredSize;

	/** How to distribute the elements among any extra space in a given row */
	TSlateAttribute< EHorizontalAlignment > HAlign;

	/** The slots that contain this panel's children. */
	TPanelChildren<FSlot> Slots;

	/** When two slots end up sharing a border, this will put that much padding between then, but otherwise wont. */
	FVector2D InnerSlotPadding;

	/** If true the box will have a preferred size equal to its alloted size  */
	bool bUseAllottedSize;

	/** Determines if the wrap box needs to arrange the slots left-to-right or top-to-bottom.*/
	EOrientation Orientation;

	class FChildArranger;
	friend class SWrapBox::FChildArranger;
};
