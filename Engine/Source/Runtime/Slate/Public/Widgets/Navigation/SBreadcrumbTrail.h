// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"

// Base class for breadcrumb trail holding methods which are not dependent on the crumb data type.
class SBreadcrumbTrailBase : public SCompoundWidget
{
public:
	SLATE_API void ScrollToStart();
	SLATE_API void ScrollToEnd();

protected:
	/** The horizontal box which contains all the breadcrumbs */
	TSharedPtr<SScrollBox> CrumbBox;
};

/**
 * A breadcrumb trail. Allows the user to see their currently selected path and navigate upwards.
 */
template <typename ItemType>
class SBreadcrumbTrail : public SBreadcrumbTrailBase
{
private:
	/** A container for data associated with a single crumb in the trail. */
	struct FCrumbItem
	{
		int32 CrumbID;
		TSharedRef<SButton> Button;
		TSharedRef<SMenuAnchor> Delimiter;
		TSharedRef<SVerticalBox> ButtonBox;
		TSharedRef<SVerticalBox> DelimiterBox;
		ItemType CrumbData;

		FCrumbItem(int32 InCrumbID, TSharedRef<SButton> InButton, TSharedRef<SMenuAnchor> InDelimiter, TSharedRef<SVerticalBox> InButtonBox, TSharedRef<SVerticalBox> InDelimiterBox, ItemType InCrumbData)
			: CrumbID(InCrumbID)
			, Button(InButton)
			, Delimiter(InDelimiter)
			, ButtonBox(InButtonBox)
			, DelimiterBox(InDelimiterBox)
			, CrumbData(InCrumbData)
		{}
	};

public:
	/** Callback for when a crumb has been pushed on the trail */
	DECLARE_DELEGATE_OneParam( FOnCrumbPushed, const ItemType& /*CrumbData*/ );

	/** Callback for when a crumb has been popped off the trail */
	DECLARE_DELEGATE_OneParam( FOnCrumbPopped, const ItemType& /*CrumbData*/ );

	/** Callback for when a crumb in the trail has been clicked */
	DECLARE_DELEGATE_OneParam( FOnCrumbClicked, const ItemType& /*CrumbData*/ );

	/** Callback for checking whether the menu to be displayed when clicking on a crumb's delimiter arrow would have any content */
	DECLARE_DELEGATE_RetVal_OneParam( bool, FHasCrumbMenuContent, const ItemType& /*CrumbData*/ );

	/** Callback for getting the menu content to be displayed when clicking on a crumb's delimiter arrow */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef< SWidget >, FGetCrumbMenuContent, const ItemType& /*CrumbData*/ );

	/** Callback for customizing the crumb button content */
	DECLARE_DELEGATE_RetVal_TwoParams( TSharedRef<SWidget>, FGetCrumbButtonContent, const ItemType& /*CrumbData*/, const FTextBlockStyle* InTextStyle );	

	SLATE_BEGIN_ARGS( SBreadcrumbTrail )
		: _ButtonStyle( &FCoreStyle::Get().GetWidgetStyle< FButtonStyle >( "BreadcrumbButton" ) )
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText") )
		, _ButtonContentPadding(FMargin(4.0, 2.0))
		, _DelimiterImage( FCoreStyle::Get().GetBrush("BreadcrumbTrail.Delimiter") )
		, _ShowLeadingDelimiter(false)
		, _PersistentBreadcrumbs(false)
		, _HasCrumbMenuContent()
		, _GetCrumbMenuContent()
	    {}

		/** The name of the style to use for the crumb buttons */
		SLATE_STYLE_ARGUMENT( FButtonStyle, ButtonStyle )
		
		/** The name of the style to use for the crumb button text */
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )

		/** The padding for the content in crumb buttons */
		SLATE_ATTRIBUTE( FMargin, ButtonContentPadding )

		/** The image to use between crumb trail buttons */
		SLATE_ATTRIBUTE( const FSlateBrush*, DelimiterImage )

		/** If true, a leading delimiter will be shown */
		SLATE_ATTRIBUTE( bool, ShowLeadingDelimiter )

		/** Called when a crumb is pushed */
		SLATE_EVENT( FOnCrumbPushed, OnCrumbPushed )

		/** Called when a crumb is popped */
		SLATE_EVENT( FOnCrumbPopped, OnCrumbPopped )

		/** Called when a crumb is clicked, after the later crumbs were popped */
		SLATE_EVENT( FOnCrumbClicked, OnCrumbClicked )

		/** If true, do not remove breadcrumbs when clicking */
		SLATE_ARGUMENT( bool, PersistentBreadcrumbs )

		SLATE_EVENT( FHasCrumbMenuContent, HasCrumbMenuContent )

		SLATE_EVENT( FGetCrumbMenuContent, GetCrumbMenuContent )

		SLATE_EVENT( FGetCrumbButtonContent, GetCrumbButtonContent )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs )
	{
		ButtonStyle = InArgs._ButtonStyle;
		TextStyle = InArgs._TextStyle;
		ButtonContentPadding = InArgs._ButtonContentPadding;
		DelimiterImage = InArgs._DelimiterImage;
		ShowLeadingDelimiter = InArgs._ShowLeadingDelimiter;
		OnCrumbPushed = InArgs._OnCrumbPushed;
		OnCrumbPopped = InArgs._OnCrumbPopped;
		OnCrumbClicked = InArgs._OnCrumbClicked;
		bHasStaticBreadcrumbs = InArgs._PersistentBreadcrumbs;
		HasCrumbMenuContentCallback = InArgs._HasCrumbMenuContent;
		GetCrumbMenuContentCallback = InArgs._GetCrumbMenuContent;
		GetCrumbButtonContentCallback = InArgs._GetCrumbButtonContent;

		NextValidCrumbID = 0;

		ChildSlot
		[
			SAssignNew(CrumbBox, SScrollBox)
			.Orientation(Orient_Horizontal)
			.ScrollBarVisibility(EVisibility::Collapsed)
		];

		AddLeadingDelimiter();
	}

	/** Adds a crumb to the end of the trail.*/
	void PushCrumb(const TAttribute<FText>& CrumbText, const ItemType& NewCrumbData)
	{
		// Create the crumb and add it to the crumb box
		TSharedPtr<SButton> NewButton;
		TSharedPtr<SMenuAnchor> NewDelimiter;

		TSharedPtr<SVerticalBox> NewButtonBox;
		TSharedPtr<SVerticalBox> NewDelimiterBox;

		TSharedRef<SWidget> ContentWidget = GetCrumbButtonContentCallback.IsBound() ? GetCrumbButtonContentCallback.Execute(NewCrumbData, TextStyle) : SNullWidget::NullWidget; 

		// Add the crumb button
		CrumbBox->AddSlot()
		[
			SAssignNew(NewButtonBox, SVerticalBox)

			+SVerticalBox::Slot()
			.FillHeight( 1.0f )
			[
				// Crumb Button
				SAssignNew(NewButton, SButton)
				.ButtonStyle(ButtonStyle)
				.ContentPadding(ButtonContentPadding)
				.TextStyle(TextStyle)
				.Text(CrumbText)
				.OnClicked( this, &SBreadcrumbTrail::CrumbButtonClicked, NextValidCrumbID )
				.Content()
				[
					ContentWidget
				]
			]
		];

		TSharedRef< SWidget > DelimiterContent = SNullWidget::NullWidget;
		if ( GetCrumbMenuContentCallback.IsBound() )
		{
			// Crumb Arrow
			DelimiterContent = SNew(SButton)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(ButtonStyle)
				.Visibility(this, &SBreadcrumbTrail::GetCrumbDelimiterVisibility, NextValidCrumbID)
				.OnClicked(this, &SBreadcrumbTrail::OnCrumbDelimiterClicked, NextValidCrumbID)
				.ContentPadding(FMargin(3, 0))
				[
					SNew(SImage)
					.Image(DelimiterImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}
		else
		{
			DelimiterContent = SNew(SButton)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(ButtonStyle)
				.Visibility( this, &SBreadcrumbTrail::GetDelimiterVisibility, NextValidCrumbID )
				.ContentPadding( FMargin(3, 0) )
				[
					SNew(SImage)
					.Image(DelimiterImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}

		CrumbBox->AddSlot()
		[
			SAssignNew(NewDelimiterBox, SVerticalBox)

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew( NewDelimiter, SMenuAnchor )
				.OnGetMenuContent( this, &SBreadcrumbTrail::GetCrumbMenuContent, NextValidCrumbID )
				[
					DelimiterContent
				]
			]
		];

		// Push the crumb data
		CrumbList.Emplace(NextValidCrumbID, NewButton.ToSharedRef(), NewDelimiter.ToSharedRef(), NewButtonBox.ToSharedRef(), NewDelimiterBox.ToSharedRef(), NewCrumbData);

		// Increment the crumb ID for the next crumb
		NextValidCrumbID = (NextValidCrumbID + 1) % (INT_MAX - 1);

		// Trigger event
		OnCrumbPushed.ExecuteIfBound(NewCrumbData);

		// We've added a new crumb so defer scrolling to the end to next tick
		CrumbBox->ScrollToEnd();
	}

	/** Pops a crumb off the end of the trail. Returns the crumb data. */
	ItemType PopCrumb()
	{
		check(HasCrumbs());

		// Remove from the crumb list and box
		const FCrumbItem& LastCrumbItem = CrumbList.Pop();

		CrumbBox->RemoveSlot(LastCrumbItem.ButtonBox);
		CrumbBox->RemoveSlot(LastCrumbItem.DelimiterBox);

		// Trigger event
		OnCrumbPopped.ExecuteIfBound(LastCrumbItem.CrumbData);

		// Return the popped crumb's data
		return LastCrumbItem.CrumbData;
	}

	/** Peeks at the end crumb in the trail. Returns the crumb data. */
	ItemType PeekCrumb() const
	{
		check(HasCrumbs());

		// Return the last crumb's text
		return CrumbList.Last().CrumbData;
	}

	/** Returns true if there are any crumbs in the trail. */
	bool HasCrumbs() const
	{
		return NumCrumbs() > 0;
	}

	/** Returns the number of crumbs in the trail. */
	int32 NumCrumbs() const
	{
		return CrumbList.Num();
	}

	/** Removes all crumbs from the crumb box */
	void ClearCrumbs(bool bPopAllCrumbsToClear = false)
	{
		if (bPopAllCrumbsToClear)
		{
			while ( HasCrumbs() )
			{
				PopCrumb();
			}
		}
		else
		{
			CrumbBox->ClearChildren();
			CrumbList.Empty();
			
			AddLeadingDelimiter();
		}
	}

	/** Gets all the crumb data in the trail */
	void GetAllCrumbData(TArray<ItemType>& CrumbData) const
	{
		for (int32 CrumbIdx = 0; CrumbIdx < NumCrumbs(); ++CrumbIdx)
		{
			CrumbData.Add(CrumbList[CrumbIdx].CrumbData);
		}
	}

private:

	EVisibility GetCrumbDelimiterVisibility(int32 CrumbID) const
	{
		if (HasCrumbs() && CrumbList.Last().CrumbID == CrumbID && HasCrumbMenuContentCallback.IsBound() && !HasCrumbMenuContentCallback.Execute(CrumbList.Last().CrumbData))
		{
			return EVisibility::Collapsed;
		}
		return EVisibility::Visible;
	}

	FReply OnCrumbDelimiterClicked( int32 CrumbID )
	{
		FReply Reply = FReply::Unhandled();

		if ( GetCrumbMenuContentCallback.IsBound() )
		{
			for (int32 CrumbListIdx = 0; CrumbListIdx < NumCrumbs(); ++CrumbListIdx)
			{
				if (CrumbList[CrumbListIdx].CrumbID == CrumbID)
				{
					CrumbList[CrumbListIdx].Delimiter->SetIsOpen( true );
					Reply = FReply::Handled();
					break;
				}
			}
		}

		return Reply;
	}

	TSharedRef< SWidget > GetCrumbMenuContent( int32 CrumbId )
	{
		if ( !GetCrumbMenuContentCallback.IsBound() )
		{
			return SNullWidget::NullWidget;
		}

		TSharedPtr< SWidget > MenuContent;

		for (int32 CrumbListIdx = 0; CrumbListIdx < NumCrumbs(); ++CrumbListIdx)
		{
			if (CrumbList[CrumbListIdx].CrumbID == CrumbId)
			{
				if (HasCrumbMenuContentCallback.IsBound() )
				{
					if (!HasCrumbMenuContentCallback.Execute( CrumbList[CrumbListIdx].CrumbData ))
					{
						return SNullWidget::NullWidget;
					}
				}

				MenuContent = GetCrumbMenuContentCallback.Execute( CrumbList[CrumbListIdx].CrumbData );
				break;
			}
		}

		return MenuContent.ToSharedRef();
	}

	/** Handler to determine the visibility of the arrows between crumbs */
	EVisibility GetDelimiterVisibility(int32 CrumbID) const
	{
		if ( HasCrumbs() && CrumbList.Last().CrumbID == CrumbID )
		{
			// Collapse the last delimiter
			return EVisibility::Collapsed;
		}

		return EVisibility::Visible;
	}

	/** Handler to determine the visibility of the arrow before all crumbs */
	EVisibility GetLeadingDelimiterVisibility() const
	{
		return ShowLeadingDelimiter.Get() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** Handler for when a crumb is clicked. Will pop crumbs down to the selected one. */
	FReply CrumbButtonClicked(int32 CrumbID)
	{
		int32 CrumbIdx = INDEX_NONE;

		if (bHasStaticBreadcrumbs)
		{
			for (int32 CrumbListIdx = 0; CrumbListIdx < NumCrumbs(); ++CrumbListIdx)
			{
				if (CrumbList[CrumbListIdx].CrumbID == CrumbID)
				{
					OnCrumbClicked.ExecuteIfBound(CrumbList[CrumbListIdx].CrumbData);
					break;
				}
			}
		}
		else
		{
			for (int32 CrumbListIdx = 0; CrumbListIdx < NumCrumbs(); ++CrumbListIdx)
			{
				if (CrumbList[CrumbListIdx].CrumbID == CrumbID)
				{
					CrumbIdx = CrumbListIdx;
					break;
				}
			}

			if ( CrumbIdx != INDEX_NONE )
			{
				while (NumCrumbs() - 1 > CrumbIdx)
				{
					PopCrumb();
				}

				OnCrumbClicked.ExecuteIfBound(CrumbList.Last().CrumbData);
			}
		}
		return FReply::Handled();
	}

	/** Adds a delimiter that is always visible */
	void AddLeadingDelimiter()
	{
		CrumbBox->AddSlot()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(DelimiterImage)
			.Visibility( this, &SBreadcrumbTrail::GetLeadingDelimiterVisibility )
		];
	}

private:

	/** The list of crumbs and their data */
	TArray<FCrumbItem> CrumbList;

	/** The next ID to assign to a crumb when it is created */
	int32 NextValidCrumbID;

	/** The button style to apply to all crumbs */
	const FButtonStyle* ButtonStyle;

	/** The text style to apply to all crumbs */
	const FTextBlockStyle* TextStyle;

	/** The padding for the content in crumb buttons */
	TAttribute<FMargin> ButtonContentPadding;

	/** The image to display between crumb trail buttons */
	TAttribute< const FSlateBrush* > DelimiterImage;

	/** Delegate to invoke when a crumb is pushed. */
	FOnCrumbPushed OnCrumbPushed;

	/** Delegate to invoke when a crumb is popped. */
	FOnCrumbPopped OnCrumbPopped;

	/** Delegate to invoke when selection changes. */
	FOnCrumbClicked OnCrumbClicked;

	/** Delegate to invoke to check whether the crumb's menu would have any content */
	FHasCrumbMenuContent HasCrumbMenuContentCallback;

	/** Delegate to invoke to retrieve the content for a crumb's menu */
	FGetCrumbMenuContent GetCrumbMenuContentCallback;

	/** Delegate to invoke to retrieve content a crumb's SButton */
	FGetCrumbButtonContent GetCrumbButtonContentCallback;

	/** If true, a leading delimiter will be added */
	TAttribute<bool> ShowLeadingDelimiter;

	/** If true, don't dynamically remove items when clicking */
	bool bHasStaticBreadcrumbs;

};
