// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"

/**
 * Class used to provide simple global notifications
 */
class FGlobalNotification
{
public:
	FGlobalNotification(const double InEnableDelayInSeconds = 1.0)
		: EnableDelayInSeconds(InEnableDelayInSeconds)
		, NextEnableTimeInSeconds(0)
	{
	}

	virtual ~FGlobalNotification() = default;

	/** Called to Tick this notification and update its state */
	SLATE_API void TickNotification(float DeltaTime);

protected:
	/** 
	 * Used to work out whether the notification should currently be visible
	 * (causes BeginNotification, EndNotification, and SetNotificationText to be called at appropriate points) 
	 */
	virtual bool ShouldShowNotification(const bool bIsNotificationAlreadyActive) const = 0;

	/** Called to update the text on the given notification */
	virtual void SetNotificationText(const TSharedPtr<SNotificationItem>& InNotificationItem) const = 0;
	
private:
	/** Begin the notification (make it visible, and mark it as pending) */
	SLATE_API TSharedPtr<SNotificationItem> BeginNotification();

	/** End the notification (hide it, and mark is at complete) */
	SLATE_API void EndNotification();

	/** Delay (in seconds) before we should actually show the notification (default is 1) */
	double EnableDelayInSeconds;

	/** Minimum time before the notification should actually appear (used to avoid spamming), or zero if no next time has been set */
	double NextEnableTimeInSeconds;

	/** The actual item being shown */
	TWeakPtr<SNotificationItem> NotificationPtr;
};
