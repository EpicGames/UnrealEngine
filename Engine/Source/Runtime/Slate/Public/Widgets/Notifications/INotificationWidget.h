// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Notifications/SNotificationList.h"

class INotificationWidget
{
public:
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) = 0;
	virtual TSharedRef< SWidget > AsWidget() = 0;
	virtual bool UseNotificationBackground() const { return true; }
};
