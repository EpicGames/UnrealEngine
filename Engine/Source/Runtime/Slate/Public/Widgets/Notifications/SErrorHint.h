// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Animation/CurveSequence.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Notifications/SErrorText.h"

class SErrorHint
	: public SCompoundWidget
	, public IErrorReportingWidget
{
public:

	SLATE_BEGIN_ARGS( SErrorHint )
		: _ErrorText()
		{}
		SLATE_ARGUMENT(FText, ErrorText)
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs);

public:

	// IErrorReportingWidget interface

	SLATE_API virtual void SetError( const FText& InErrorText ) override;
	SLATE_API virtual void SetError( const FString& InErrorText ) override;
	SLATE_API virtual bool HasError() const override;
	SLATE_API virtual TSharedRef<SWidget> AsWidget() override;

private:

	TAttribute<EVisibility> CustomVisibility;
	EVisibility MyVisibility() const;

	FVector2D GetDesiredSizeScale() const;
	FCurveSequence ExpandAnimation;

	TSharedPtr<SWidget> ImageWidget;
	FText ErrorText;
	FText GetErrorText() const;
};
