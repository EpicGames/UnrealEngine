// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Animation/CurveSequence.h"
#include "Widgets/Text/STextBlock.h"

/**
 * Interface for error reporting widgets.
 */
class IErrorReportingWidget
{
public:

	virtual void SetError( const FString& InErrorText ) = 0;
	virtual void SetError( const FText& InErrorText ) = 0;
	virtual bool HasError() const = 0;
	virtual TSharedRef<SWidget> AsWidget() = 0;
};


template <>
struct TWidgetTypeTraits<class SErrorText>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Implements a widget that displays an error text message.
 */
class SErrorText
	: public SBorder
	, public IErrorReportingWidget
{
public:

	SLATE_BEGIN_ARGS( SErrorText )
		: _ErrorText()
		, _BackgroundColor(FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor"))
		, _Font()
		, _AutoWrapText(false)
	{ }

		SLATE_ARGUMENT(FText, ErrorText)
		SLATE_ATTRIBUTE(FSlateColor, BackgroundColor)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_ATTRIBUTE(bool, AutoWrapText)
		
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs);

public:

	// IErrorReportingWidget interface

	SLATE_API virtual void SetError( const FText& InErrorText ) override;
	SLATE_API virtual void SetError( const FString& InErrorText ) override;
	SLATE_API virtual bool HasError() const override;
	SLATE_API virtual TSharedRef<SWidget> AsWidget() override;

private:
	TAttribute<EVisibility> CustomVisibility;
	EVisibility MyVisibility() const;

	TSharedPtr<class STextBlock> TextBlock;

	FVector2D GetDesiredSizeScale() const;
	FCurveSequence ExpandAnimation;
};
