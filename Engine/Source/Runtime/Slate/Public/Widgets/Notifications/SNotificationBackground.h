// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBorder.h"

struct FSlateBrush;

class SNotificationBackground : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SNotificationBackground) 
		: _ColorAndOpacity(FLinearColor::White)
		, _BorderBackgroundColor(FLinearColor::White)
		, _DesiredSizeScale(FVector2D(1.0f, 1.0f))
	{}
		SLATE_ATTRIBUTE(FMargin, Padding)
		SLATE_ATTRIBUTE(FLinearColor, ColorAndOpacity)
		SLATE_ATTRIBUTE(FSlateColor, BorderBackgroundColor)
		SLATE_ATTRIBUTE(FVector2D, DesiredSizeScale)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	SLATE_API void Construct(const FArguments& InArgs);
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	const FSlateBrush* WatermarkBrush = nullptr;
	const FSlateBrush* BorderBrush = nullptr;
	FLinearColor WatermarkTint;
};