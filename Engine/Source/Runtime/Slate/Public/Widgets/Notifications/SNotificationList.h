// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Types/SlateStructs.h"
#include "Layout/Visibility.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SCheckBox.h"

class INotificationWidget;
class SVerticalBox;
class SWindow;

/**
 * Interface class for an item in the event message list.
 * Real implementation is found in SEventMessageItemImpl
 */
class SNotificationItem
	: public SCompoundWidget
{
public:
	enum ECompletionState
	{
		CS_None,
		CS_Pending,
		CS_Success,
		CS_Fail,
	};

	/** Sets the text for the notification element */
	virtual void SetText(const TAttribute< FText >& InText) = 0;
	
	/** Sets the subtext the notification item. Sub text is used for longer text and is a smaller font */
	virtual void SetSubText(const TAttribute<FText>& InSubText) = 0;

	/** Sets the text and delegate for the hyperlink */
	virtual void SetHyperlink(const FSimpleDelegate& InHyperlink, const TAttribute< FText >& InHyperlinkText = TAttribute< FText >()) = 0;

	/** Sets the ExpireDuration */
	virtual void SetExpireDuration(float ExpireDuration) = 0;

	/** Sets the FadeInDuration */
	virtual void SetFadeInDuration(float FadeInDuration) = 0;

	/** Sets the FadeOutDuration */
	virtual void SetFadeOutDuration(float FadeOutDuration) = 0;

	/** Gets the visibility state of the throbber, success, and fail images */
	virtual ECompletionState GetCompletionState() const = 0;

	/** Sets the visibility state of the throbber, success, and fail images */
	virtual void SetCompletionState(ECompletionState State) = 0;

	/** Waits for the ExpireDuration then begins to fade out */
	virtual void ExpireAndFadeout() = 0;

	/** Begin the fade out */
	virtual void Fadeout() = 0;

	/** Pulse the notification with a certain outline glow color while it is still pending. */
	virtual void Pulse(const FLinearColor& GlowColor) = 0;
};


/**
 * Setup class to initialize buttons on a notification
 */
struct FNotificationButtonInfo
{
	using FVisibilityDelegate = TDelegate<EVisibility(SNotificationItem::ECompletionState)>;
	using FIsEnabledDelegate = TDelegate<bool(SNotificationItem::ECompletionState)>;

	FNotificationButtonInfo( const TAttribute<FText>& InText, const TAttribute<FText>& InToolTip, FSimpleDelegate InCallback, SNotificationItem::ECompletionState VisibleInState = SNotificationItem::CS_Pending )
		: Text( InText )
		, ToolTip( InToolTip )
		, Callback( MoveTemp(InCallback) )
		, VisibilityCallback()
		, IsEnabledCallback()
		, VisibilityOnNone( VisibleInState == SNotificationItem::CS_None ? EVisibility::Visible : EVisibility::Collapsed )
		, VisibilityOnPending( VisibleInState == SNotificationItem::CS_Pending ? EVisibility::Visible : EVisibility::Collapsed )
		, VisibilityOnSuccess( VisibleInState == SNotificationItem::CS_Success ? EVisibility::Visible : EVisibility::Collapsed )
		, VisibilityOnFail( VisibleInState == SNotificationItem::CS_Fail ? EVisibility::Visible : EVisibility::Collapsed )
	{ }

	FNotificationButtonInfo( const TAttribute<FText>& InText, const TAttribute<FText>& InToolTip, FSimpleDelegate InCallback, FVisibilityDelegate InVisibilityCallback, FIsEnabledDelegate InIsEnabledCallback = FIsEnabledDelegate() )
		: Text( InText )
		, ToolTip( InToolTip )
		, Callback( MoveTemp(InCallback) )
		, VisibilityCallback( MoveTemp(InVisibilityCallback) )
		, IsEnabledCallback( MoveTemp(InIsEnabledCallback) )
		, VisibilityOnNone( VisibilityCallback.IsBound() ? EVisibility::Collapsed : EVisibility::Visible )
		, VisibilityOnPending( VisibilityCallback.IsBound() ? EVisibility::Collapsed : EVisibility::Visible )
		, VisibilityOnSuccess( VisibilityCallback.IsBound() ? EVisibility::Collapsed : EVisibility::Visible )
		, VisibilityOnFail( VisibilityCallback.IsBound() ? EVisibility::Collapsed : EVisibility::Visible )
	{ }

	/** Message on the button */
	TAttribute<FText> Text;
	/** Tip displayed when moused over */
	TAttribute<FText> ToolTip;
	/** Method called when button clicked */
	FSimpleDelegate Callback;
	/** Method called to decide if the button if visible; overrides the EVisibility data below if bound */
	FVisibilityDelegate VisibilityCallback;
	/** Method called to decide if the button is enabled; assumes enabled if unbound */
	FIsEnabledDelegate IsEnabledCallback;

	/** Visibility of the button when the completion state of the button is SNotificationItem::ECompletionState::None and VisibilityCallback is unbound */
	EVisibility VisibilityOnNone;
	/** Visibility of the button when the completion state of the button is SNotificationItem::ECompletionState::Pending and VisibilityCallback is unbound */
	EVisibility VisibilityOnPending;
	/** Visibility of the button when the completion state of the button is SNotificationItem::ECompletionState::Success and VisibilityCallback is unbound */
	EVisibility VisibilityOnSuccess;
	/** Visibility of the button when the completion state of the button is SNotificationItem::ECompletionState::Fail and VisibilityCallback is unbound */
	EVisibility VisibilityOnFail;
};


/**
 * Setup class to initialize a notification.
 */
struct FNotificationInfo
{
	/**
	 * FNotifcationInfo initialization constructor
	 *
	 * @param	InText	Text string to display for this notification
	 */
	FNotificationInfo(const FText& InText)
		: ContentWidget(),
		Text(InText),
		SubText(),
		ButtonDetails(),
		Image(nullptr),
		FadeInDuration(0.5f),
		FadeOutDuration(2.0f),
		ExpireDuration(1.0f),
		bUseThrobber(true),
		bUseSuccessFailIcons(true),
		bUseLargeFont(true),
		bUseCopyToClipboard(false),
		WidthOverride(320.0f),
		bFireAndForget(true),
		DefaultState(SNotificationItem::CS_None),
		CheckBoxState(ECheckBoxState::Unchecked),
		CheckBoxStateChanged(),
		CheckBoxText(),
		Hyperlink(),
		HyperlinkText( NSLOCTEXT("EditorNotification", "DefaultHyperlinkText", "Show Log") ),
		bAllowThrottleWhenFrameRateIsLow(true)
	{ };

	/**
	* FNotifcationInfo initialization constructor
	*
	* @param	InContentWidget	The content of the notification
	*/
	FNotificationInfo(TSharedPtr<INotificationWidget> InContentWidget)
		: ContentWidget(InContentWidget),
		Text(),
		SubText(),
		ButtonDetails(),
		Image(nullptr),
		FadeInDuration(0.5f),
		FadeOutDuration(2.0f),
		ExpireDuration(1.0f),
		bUseThrobber(false),
		bUseSuccessFailIcons(false),
		bUseLargeFont(false),
		bUseCopyToClipboard(false),
		WidthOverride(320.0f),
		bFireAndForget(true),
		DefaultState(SNotificationItem::CS_None),
		CheckBoxState(ECheckBoxState::Unchecked),
		CheckBoxStateChanged(),
		CheckBoxText(),
		Hyperlink(),
		HyperlinkText(),
		bAllowThrottleWhenFrameRateIsLow(true)
	{ };

	/**
	 * Shows a "Copy to Clipboard" hyperlink that when clicked copies the text and sub-text to the operating system clipboard
	 * @note Text and SubText must be set prior to calling this method for them to be included
	 */
	UE_DEPRECATED(5.6, "Use the flag bUseCopyToClipboard instead.")
	SLATE_API void ShowCopyToClipboadHyperlink();

	/** If set, overrides the entire content of the notification with this widget */
	TSharedPtr<INotificationWidget> ContentWidget;

	/** The text displayed in this notification. Suitable for short notifications and titles. */
	TAttribute<FText> Text;

	/** Optional subtext displayed in this notification. Subtext is smaller than the default text field and is better for long descriptions.*/
	TAttribute<FText> SubText;

	/** Setup information for the buttons on the notification */ 
	TArray<FNotificationButtonInfo> ButtonDetails;

	/** The icon image to display next to the text */
	const FSlateBrush* Image;

	/** The fade in duration for this element */
	float FadeInDuration;

	/** The fade out duration for this element */
	float FadeOutDuration;

	/** The duration before a fadeout for this element */
	float ExpireDuration;

	/** Controls whether or not to add the animated throbber */
	bool bUseThrobber;

	/** Controls whether or not to display the success and fail icons */
	bool bUseSuccessFailIcons;

	/** When true the larger bolder font will be used to display the message */
	bool bUseLargeFont;

	/** Controls whether or not to display the copy to clipboard button */
	bool bUseCopyToClipboard;

	/** When set this forces the width of the box, used to stop resizing on text change */
	FOptionalSize WidthOverride;

	/** When true the notification will automatically time out after the expire duration. */ 
	bool bFireAndForget;

	/** Defines what state should this notification have by default */
	SNotificationItem::ECompletionState DefaultState;

	/** When set this will display a check box on the notification; handles getting the current check box state */
	TAttribute<ECheckBoxState> CheckBoxState;

	/** When set this will display a check box on the notification; handles setting the new check box state */
	FOnCheckStateChanged CheckBoxStateChanged;

	/** Text to display for the check box message */
	TAttribute<FText> CheckBoxText;

	/** When set this will display as a hyperlink on the right side of the notification. */
	FSimpleDelegate Hyperlink;

	/** Text to display for the hyperlink message */
	TAttribute<FText> HyperlinkText;

	/** A specific window to put the notification in. If this is null, the root window of the application will be used */
	TSharedPtr<SWindow> ForWindow;

	/** True if we should throttle the editor while the notification is transitioning and performance is poor, to make sure the user can see the animation */
	bool bAllowThrottleWhenFrameRateIsLow;
};


/**
 * A list of non-intrusive messages about the status of currently active work.
 */
class SNotificationList
	: public SCompoundWidget
{
	friend class SNotificationExtendable;
	friend class FSlateNotificationManager;

public:

	SLATE_BEGIN_ARGS( SNotificationList ){}
	SLATE_END_ARGS()
	
	/**
	 * Constructs this widget.
	 *
	 * @param InArgs    Declaration from which to construct the widget.
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Adds a floating notification.
	 *
	 * @param Info 		Contains various settings used to initialize the notification.
	 */
	SLATE_API virtual TSharedRef<SNotificationItem> AddNotification(const FNotificationInfo& Info);

protected:

	/**
	 * Invoked when a notification item has finished fading out. Called by the faded item.
	 *
	 * @param NotificationItem The item which finished fading out.
	 */
	SLATE_API virtual void NotificationItemFadedOut (const TSharedRef<SNotificationItem>& NotificationItem);

protected:

	/** The vertical box containing the list of message items. */
	TSharedPtr<SVerticalBox> MessageItemBoxPtr;

	/** The parent window of this list. */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** Flag to auto-destroy this list. */
	bool bDone;

private:

	/** The last notification added to the list. To avoid weak pointer pointing to a destroyed notification when notification are not allowed. */
	TSharedPtr<SNotificationItem> LastNotification;

};
