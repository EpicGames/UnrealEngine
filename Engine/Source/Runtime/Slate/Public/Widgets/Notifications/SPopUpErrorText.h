// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Input/SComboButton.h"

template <>
struct TWidgetTypeTraits<class SPopupErrorText>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SPopupErrorText : public SComboButton, public IErrorReportingWidget
{
public:
	SLATE_BEGIN_ARGS(SPopupErrorText)
		: _ShowInNewWindow( false )
		, _Font()
	{}
		/** The popup appears in a new window instead of in the same window that this widget is in */
		SLATE_ARGUMENT( bool, ShowInNewWindow )
 		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
	SLATE_END_ARGS()

	SLATE_API virtual void Construct( const FArguments& InArgs );

	// IErrorReportingWidget interface

	SLATE_API virtual void SetError( const FText& InErrorText ) override;
	SLATE_API virtual void SetError( const FString& InErrorText ) override;

	SLATE_API virtual bool HasError() const override;

	SLATE_API virtual TSharedRef<SWidget> AsWidget() override;

	// IErrorReportingWidget interface

private:
	TSharedPtr<SErrorText> HasErrorSymbol;
	TSharedPtr<SErrorText> ErrorText;
};
