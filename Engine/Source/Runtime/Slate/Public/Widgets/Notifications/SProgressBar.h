// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "SProgressBar.generated.h"

class FActiveTimerHandle;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * SProgressBar Fill Type 
 */
UENUM(BlueprintType)
namespace EProgressBarFillType
{
	enum Type : int
	{
		// will fill up from the left side to the right
		LeftToRight,
		// will fill up from the right side to the left side
		RightToLeft,
		// will scale up from the midpoint to the outer edges both vertically and horizontally
		FillFromCenter,
		// will fill up from the centerline to the outer edges horizontally
		FillFromCenterHorizontal,
		// will fill up from the centerline to the outer edges vertically
		FillFromCenterVertical,
		// will fill up from the top to the the bottom
		TopToBottom,
		// will fill up from the bottom to the the top
		BottomToTop,
	};
}

/**
 * SProgressBar Fill Style
 */
UENUM(BlueprintType)
namespace EProgressBarFillStyle
{
	enum Type : int
	{
		// a mask is used to paint the fill image
		Mask,
		// the fill image is scaled to the fill percentage
		Scale,
	};
}


template <>
struct TWidgetTypeTraits<class SProgressBar>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A progress bar widget.*/
class SProgressBar : public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SProgressBar, SLeafWidget, SLATE_API)
public:
	SLATE_BEGIN_ARGS(SProgressBar)
		: _Style( &FAppStyle::Get().GetWidgetStyle<FProgressBarStyle>("ProgressBar") )
		, _BarFillType(EProgressBarFillType::LeftToRight)
		, _BarFillStyle(EProgressBarFillStyle::Mask)
		, _Percent( TOptional<float>() )
		, _FillColorAndOpacity( FLinearColor::White )
		, _BorderPadding( FVector2D(0,0) )
		, _BackgroundImage(nullptr)
		, _FillImage(nullptr)
		, _MarqueeImage(nullptr)
		, _RefreshRate(2.0f)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		/** Style used for the progress bar */
		SLATE_STYLE_ARGUMENT( FProgressBarStyle, Style )

		/** Defines the direction in which the progress bar fills */
		SLATE_ARGUMENT( EProgressBarFillType::Type, BarFillType )

		/** Defines the visual style of the progress bar fill - scale or mask */
		SLATE_ARGUMENT( EProgressBarFillStyle::Type, BarFillStyle )

		/** Used to determine the fill position of the progress bar ranging 0..1 */
		SLATE_ATTRIBUTE( TOptional<float>, Percent )

		/** Fill Color and Opacity */
		SLATE_ATTRIBUTE( FSlateColor, FillColorAndOpacity )

		/** Border Padding around fill bar */
		SLATE_ATTRIBUTE( FVector2D, BorderPadding )
	
		/** The brush to use as the background of the progress bar */
		SLATE_ARGUMENT(const FSlateBrush*, BackgroundImage)
	
		/** The brush to use as the fill image */
		SLATE_ARGUMENT(const FSlateBrush*, FillImage)
	
		/** The brush to use as the marquee image */
		SLATE_ARGUMENT(const FSlateBrush*, MarqueeImage)

		/** Rate at which this widget is ticked when sleeping in seconds */
		SLATE_ARGUMENT_DEPRECATED(float, RefreshRate, 5.8, "RefreshRate will be ignored as the ProgressBar does not need to tick when sleeping anymore.")

	SLATE_END_ARGS()

	SLATE_API SProgressBar();
	SLATE_API ~SProgressBar();

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;

	/** See attribute Percent */
	SLATE_API void SetPercent(TAttribute< TOptional<float> > InPercent);
	
	/** See attribute Style */
	SLATE_API void SetStyle(const FProgressBarStyle* InStyle);
	
	/** See attribute BarFillType */
	SLATE_API void SetBarFillType(EProgressBarFillType::Type InBarFillType);

	/** See attribute BarFillStyle */
	SLATE_API void SetBarFillStyle(EProgressBarFillStyle::Type InBarFillStyle);
	
	/** See attribute SetFillColorAndOpacity */
	SLATE_API void SetFillColorAndOpacity(TAttribute< FSlateColor > InFillColorAndOpacity);
	
	/** See attribute BorderPadding */
	SLATE_API void SetBorderPadding(TAttribute< FVector2D > InBorderPadding);
	
	/** See attribute BackgroundImage */
	SLATE_API void SetBackgroundImage(const FSlateBrush* InBackgroundImage);
	
	/** See attribute FillImage */
	SLATE_API void SetFillImage(const FSlateBrush* InFillImage);
	
	/** See attribute MarqueeImage */
	SLATE_API void SetMarqueeImage(const FSlateBrush* InMarqueeImage);
	
private:

	void UpdateMarqueeActiveTimer();

	/** Widgets active tick */
	EActiveTimerReturnType ActiveTick(double InCurrentTime, float InDeltaTime);

	/** Gets the current background image. */
	const FSlateBrush* GetBackgroundImage() const;
	/** Gets the current fill image */
	const FSlateBrush* GetFillImage() const;
	/** Gets the current marquee image */
	const FSlateBrush* GetMarqueeImage() const;

private:
	
	/** The style of the progress bar */
	const FProgressBarStyle* Style;

	/** The text displayed over the progress bar */
	TSlateAttribute< TOptional<float> > Percent;

	EProgressBarFillType::Type BarFillType;

	EProgressBarFillStyle::Type BarFillStyle;

	/** Background image to use for the progress bar */
	const FSlateBrush* BackgroundImage;

	/** Foreground image to use for the progress bar */
	const FSlateBrush* FillImage;

	/** Image to use for marquee mode */
	const FSlateBrush* MarqueeImage;

	/** Fill Color and Opacity */
	TSlateAttribute<FSlateColor> FillColorAndOpacity;

	/** Border Padding */
	TSlateAttribute<FVector2D> BorderPadding;

	/** Value to drive progress bar animation */
	float MarqueeOffset;

	/** Reference to the widgets current active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	bool bNeedsMarqueeTimerUpdate;
};

