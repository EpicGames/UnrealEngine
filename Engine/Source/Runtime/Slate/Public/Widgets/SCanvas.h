// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * Canvas is a layout widget that allows you to arbitrary position and size child widgets in a relative coordinate space
 */
class SCanvas : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SCanvas, SPanel, SLATE_API)

public:

	/**
	 * Canvas slots allow child widgets to be positioned and sized
	 *
	 * Horizontal Alignment 
	 *  Given a top aligned slot, where '+' represents the 
	 *  anchor point defined by PositionAttr.
	 *  
	 *   Left				Center				Right
	 *	+ _ _ _ _            _ _ + _ _          _ _ _ _ +
	 *	|		  |		   | 		   |	  |		    |
	 *	| _ _ _ _ |        | _ _ _ _ _ |	  | _ _ _ _ |
	 * 
	 *  Note: FILL is NOT supported.
	 *
	 * Vertical Alignment 
	 *   Given a left aligned slot, where '+' represents the 
	 *   anchor point defined by PositionAttr.
	 *  
	 *   Top				Center			  Bottom
	 *	+_ _ _ _ _          _ _ _ _ _		 _ _ _ _ _ 
	 *	|		  |		   | 		 |		|		  |
	 *	| _ _ _ _ |        +		 |		|		  |
	 *					   | _ _ _ _ |		+ _ _ _ _ |
	 * 
	 *  Note: FILL is NOT supported.
	 */
	class FSlot : public TWidgetSlotWithAttributeSupport<FSlot>, public TAlignmentWidgetSlotMixin<FSlot>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TWidgetSlotWithAttributeSupport<FSlot>, TAlignmentWidgetSlotMixin<FSlot>)
			SLATE_ATTRIBUTE(FVector2D, Position)
			SLATE_ATTRIBUTE(FVector2D, Size)
		SLATE_SLOT_END_ARGS()

		void SetPosition( TAttribute<FVector2D> InPosition )
		{
			Position.Assign(*this, MoveTemp(InPosition));
		}
		FVector2D GetPosition() const
		{
			return Position.Get();
		}

		void SetSize( TAttribute<FVector2D> InSize )
		{
			Size.Assign(*this, MoveTemp(InSize));
		}
		FVector2D GetSize() const
		{
			return Size.Get();
		}

	public:
		/** Default values for a slot. */
		FSlot()
			: TWidgetSlotWithAttributeSupport<FSlot>()
			, TAlignmentWidgetSlotMixin<FSlot>(HAlign_Left, VAlign_Top)
			, Position(*this, FVector2D::ZeroVector)
			, Size(*this, FVector2D(1.0f, 1.0f))
		{ }

		SLATE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArg);
		static SLATE_API void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer);

	private:
		/** Position */
		TSlateSlotAttribute<FVector2D> Position;

		/** Size */
		TSlateSlotAttribute<FVector2D> Size;
	};

	SLATE_BEGIN_ARGS( SCanvas )
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}

		SLATE_SLOT_ARGUMENT( FSlot, Slots )

	SLATE_END_ARGS()

	SLATE_API SCanvas();
	SLATE_API virtual ~SCanvas();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	static SLATE_API FSlot::FSlotArguments Slot();

	using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	/**
	 * Adds a content slot.
	 *
	 * @return The added slot.
	 */
	SLATE_API FScopedWidgetSlotArguments AddSlot();

	/**
	 * Removes a particular content slot.
	 *
	 * @param SlotWidget The widget in the slot to remove.
	 */
	SLATE_API int32 RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	/**
	 * Removes all slots from the panel.
	 */
	SLATE_API void ClearChildren();

	/** @return the number of slots. */
	int32 NumSlots() const { return Children.Num(); }

public:

	//~ SWidget overrides
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FChildren* GetChildren() override;

protected:
	//~ Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SWidget overrides.

protected:

	/** The canvas widget's children. */
	TPanelChildren< FSlot > Children;
};
