// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/IToolTip.h"

class SWidget;
class SToolTip;

/** Called deferred to retrieve the tooltip widget when needed. */
DECLARE_DELEGATE_RetVal(TSharedPtr<IToolTip>, FOnGetDeferredToolTip);

/**
 * Interface for deferred tooltips. These do not create unless needed, so have lower startup cost
 */
class SDeferredToolTip : public IToolTip
{
public:

	SLATE_API SDeferredToolTip(FOnGetDeferredToolTip InOnGetDeferredToolTip);

public:

	// ~Begin IToolTip Interface 
	SLATE_API virtual TSharedRef<SWidget> AsWidget() override;
	SLATE_API virtual TSharedRef<SWidget> GetContentWidget() override;
	SLATE_API virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget) override;
	SLATE_API virtual void ResetContentWidget() override;
	SLATE_API virtual bool IsEmpty() const override;
	SLATE_API virtual bool IsInteractive() const override;
	SLATE_API virtual void OnOpening() override;
	SLATE_API virtual void OnClosed() override;
	SLATE_API virtual void OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const override;
	// ~End IToolTip Interface 

protected:

	/** Delegate to generate tooltip. Will be called deferred. */
	FOnGetDeferredToolTip OnGetDeferredToolTip = {};

private:

	/** Helper method to cache the tooltip if needed */
	void TryCacheToolTip() const;

private:

	/** Cached tooltip creation result */
	mutable TSharedPtr<IToolTip> CachedToolTip = nullptr;
};


/**
 * Specialized deferred tooltip just for text
 */
class SDeferredToolTipText : public IToolTip
{
public:

	SLATE_API SDeferredToolTipText(const TAttribute<FText>& InToolTipText);
	SLATE_API SDeferredToolTipText(const FText& InToolTipText);

public:

	// ~Begin IToolTip Interface 
	SLATE_API virtual TSharedRef<SWidget> AsWidget() override;
	SLATE_API virtual TSharedRef<SWidget> GetContentWidget() override;
	SLATE_API virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget) override;
	SLATE_API virtual void ResetContentWidget() override;
	SLATE_API virtual bool IsEmpty() const override;
	SLATE_API virtual bool IsInteractive() const override;
	SLATE_API virtual void OnOpening() override;
	SLATE_API virtual void OnClosed() override;
	SLATE_API virtual void OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const override;
	// ~End IToolTip Interface 

public:

	/** Text attribute we will use to make tooltip on demand */
	TAttribute<FText> ToolTipText = {};

private:

	/** Helper method to cache the tooltip if needed */
	void TryCacheToolTip() const;

private:

	/** Cached tooltip creation result */
	mutable TSharedPtr<SToolTip> CachedToolTip = nullptr;
};
