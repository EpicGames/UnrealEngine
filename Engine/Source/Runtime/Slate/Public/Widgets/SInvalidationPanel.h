// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "FastUpdate/SlateInvalidationRoot.h"
#include "Input/HittestGrid.h"

class FPaintArgs;
class FSlateRenderDataHandle;
class FSlateWindowElementList;
class SWindow;


template <>
struct TWidgetTypeTraits<class SInvalidationPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SInvalidationPanel : public SCompoundWidget, public FSlateInvalidationRoot
{
	SLATE_DECLARE_WIDGET_API(SInvalidationPanel, SCompoundWidget, SLATE_API)
public:
	SLATE_BEGIN_ARGS( SInvalidationPanel )
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
		: _UseDynamicInvalidation(false)
#endif
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
		SLATE_DEFAULT_SLOT(FArguments, Content)
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
		SLATE_ARGUMENT(bool, UseDynamicInvalidation)
#endif
#if !UE_BUILD_SHIPPING
		SLATE_ARGUMENT(FString, DebugName)
#endif
	SLATE_END_ARGS()

	SLATE_API SInvalidationPanel();
	SLATE_API ~SInvalidationPanel();

#if WITH_SLATE_DEBUGGING
	static SLATE_API bool AreInvalidationPanelsEnabled();
	static SLATE_API void EnableInvalidationPanels(bool bEnable);
#endif
	SLATE_API void Construct( const FArguments& InArgs );

	SLATE_API virtual FChildren* GetAllChildren() override;

	/** Returns true when the invalidation panel is able to cache the widgets. **/
	SLATE_API bool GetCanCache() const;

	SLATE_API void SetCanCache(bool InCanCache);

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	/**
	 * Returns true when the invalidation panel should use Dynamic Invalidation.
	 * Dynamic Invalidation means that the widget's content will only be cached if all descendant widgets support invalidation.
	 * Dynamic Invalidation can be globally disabled or forced. Use `CanUseDynamicInvalidation` to check if this invalidation panel will actually
	 * use Dynamic Invalidation
	**/
	SLATE_API bool GetUseDynamicInvalidation() const;
	SLATE_API void SetUseDynamicInvalidation(bool InUseDynamicInvalidation);

	/**
	 * Checks if this invalidation panel will actually use Dynamic Invalidation, depending on the CVar `Slate.DynamicInvalidation.Options`
	 * If Dynamic Invalidation is disabled, this will return false.
	 * If this invalidation panel is not dynamic, or if Dynamic Invalidation is forced Static, this will return false.
	 * If Dynamic Invalidation is enabled, this will return the same value as `GetUseDynamicInvalidation`.
	 * If Dynamic Invalidation is forced, this will return true.
	 */
	SLATE_API bool CanUseDynamicInvalidation() const;

	SLATE_API virtual bool SupportsInvalidationRecursive(EInvalidationStrategy InInvalidationStrategy) const override;
#endif

	SLATE_API virtual bool CanCacheThisFrame() const override;

	//~ SWidget overrides
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
#if WITH_SLATE_DEBUGGING
	SLATE_API virtual FChildren* Debug_GetChildrenForReflector() override;
#endif
	//~ End SWidget

	SLATE_API void SetContent(const TSharedRef< SWidget >& InContent);

protected:
	SLATE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
	SLATE_API virtual bool Advanced_IsInvalidationRoot() const override;
	SLATE_API virtual const FSlateInvalidationRoot* Advanced_AsInvalidationRoot() const override;
	SLATE_API virtual TSharedRef<SWidget> GetRootWidget() override;
	SLATE_API virtual int32 PaintSlowPath(const FSlateInvalidationContext& Context) override;

private:
	void OnGlobalInvalidationToggled(bool bGlobalInvalidationEnabled);
	bool UpdateCachePrequisites(FSlateWindowElementList& OutDrawElements, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, int32 LayerId, const FWidgetStyle& InWidgetStyle) const;

private:
	mutable TSharedRef<FHittestGrid> HittestGrid;

	mutable TOptional<FSlateClippingState> LastClippingState;
	mutable FGeometry LastAllottedGeometry;
	mutable FVector2D LastClipRectSize;
	mutable int32 LastIncomingLayerId;
	mutable FLinearColor LastIncomingColorAndOpacity;

	bool bCanCache;
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	bool bUseDynamicInvalidation;
#endif

	mutable bool bPaintedSinceLastPrepass;
#if SLATE_VERBOSE_NAMED_EVENTS
	FString DebugName;
	FString DebugTickName;
	FString DebugPaintName;
#endif
	mutable bool bWasCachable;
};
