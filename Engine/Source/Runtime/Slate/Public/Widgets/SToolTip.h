// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Widgets/IToolTip.h"

/**
 * Slate tool tip widget
 */
class SToolTip
	: public SCompoundWidget
	, public IToolTip
{
public:
	DECLARE_DELEGATE_OneParam(FOnSetInteractiveWindowLocation, FVector2D&);

	SLATE_BEGIN_ARGS( SToolTip )
		: _Text()
		, _Content()
		, _Font(FCoreStyle::Get().GetFontStyle("ToolTip.Font"))
		, _TextMargin(FMargin(8.0f))
		, _BorderImage(FCoreStyle::Get().GetBrush("ToolTip.Background"))
		, _IsInteractive(false)
		, _OnSetInteractiveWindowLocation()
	{ }

		/** The text displayed in this tool tip */
		SLATE_ATTRIBUTE(FText, Text)

		/** Arbitrary content to be displayed in the tool tip; overrides any text that may be set. */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		/** The font to use for this tool tip */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		
		/** Margin between the tool tip border and the text content */
		SLATE_ATTRIBUTE(FMargin, TextMargin)

		/** The background/border image to display */
		SLATE_ATTRIBUTE(const FSlateBrush*, BorderImage)

		/** Whether the tooltip should be considered interactive */
		SLATE_ATTRIBUTE(bool, IsInteractive)

		/** Hook to modify or override the desired location (in screen space) for interactive tooltip windows. By default, the previous frame's cursor position will be used. */
		SLATE_EVENT(FOnSetInteractiveWindowLocation, OnSetInteractiveWindowLocation)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

public:

	// IToolTip interface

	virtual TSharedRef<class SWidget> AsWidget( ) override
	{
		return AsShared();
	}

	virtual TSharedRef<SWidget> GetContentWidget( ) override
	{
		return ToolTipContent.ToSharedRef();
	}

	SLATE_API virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget) override;
	SLATE_API virtual void ResetContentWidget () override;

	SLATE_API virtual bool IsEmpty( ) const override;
	SLATE_API virtual bool IsInteractive( ) const override;
	virtual void OnOpening() override { }
	virtual void OnClosed() override { }
	SLATE_API virtual void OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const override;

	virtual const FText& GetTextTooltip() const
	{
		return TextContent.Get();
	}

public:

	static SLATE_API float GetToolTipWrapWidth();

private:

	// Text block widget.
	TAttribute<FText> TextContent;

	// Content widget.
	TWeakPtr<SWidget> WidgetContent;

	// Wrapped content within the widget;
	TSharedPtr<SWidget> ToolTipContent;

	// Font used for the text displayed (where applicable)
	TAttribute<FSlateFontInfo> Font;

	// Margin between the tool tip border and the text content
	TAttribute<FMargin> TextMargin;

	// The background/border image to display
	TAttribute<const FSlateBrush*> BorderImage;
	
	// Whether the tooltip should be considered interactive.
	TAttribute<bool> bIsInteractive;

	// Optional delegate to modify or override the desired location for an interactive tooltip.
	FOnSetInteractiveWindowLocation OnSetInteractiveWindowLocationDelegate;
};
