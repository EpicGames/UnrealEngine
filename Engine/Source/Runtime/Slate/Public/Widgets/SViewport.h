// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Input/NavigationReply.h"
#include "Input/PopupMethodReply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/SWindow.h"

class FActiveTimerHandle;
class FPaintArgs;
class FSlateWindowElementList;
class ICustomHitTestPath;


template <>
struct TWidgetTypeTraits<class SViewport>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SViewport
	: public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS( SViewport )
		: _Content()
		, _ShowEffectWhenDisabled(true)
		, _RenderDirectlyToWindow(false)
		, _EnableGammaCorrection(true)
		, _ReverseGammaCorrection(false)
		, _EnableBlending(false)
		, _EnableStereoRendering(false)
		, _PreMultipliedAlpha(true)
		, _IgnoreTextureAlpha(true)
		, _ViewportSize(GetDefaultViewportSize())
	{
		_Clipping = EWidgetClipping::ClipToBoundsAlways;
	}

		SLATE_DEFAULT_SLOT( FArguments, Content )

		/** Whether or not to show the disabled effect when this viewport is disabled */
		SLATE_ATTRIBUTE( bool, ShowEffectWhenDisabled )

		/** 
		 * Whether or not to render directly to the window's backbuffer or an offscreen render target that is applied to the window later 
		 * Rendering to an offscreen target is the most common option in the editor where there may be many frames which this viewport's interface may wish to not re-render but use a cached buffer instead
		 * Rendering directly to the backbuffer is the most common option in the game where you want to update each frame without the cost of writing to an intermediate target first.
		 */
		SLATE_ARGUMENT( bool, RenderDirectlyToWindow )

		/** Whether or not to enable gamma correction. Doesn't apply when rendering directly to a backbuffer. */
		SLATE_ARGUMENT( bool, EnableGammaCorrection )

		/** Whether or not to reverse the gamma correction done to the texture in this viewport.  Ignores the bEnableGammaCorrection setting */
		SLATE_ARGUMENT( bool, ReverseGammaCorrection )

		/** Allow this viewport to blend with its background. */
		SLATE_ARGUMENT( bool, EnableBlending )

		/** Whether or not to enable stereo rendering. */
		SLATE_ARGUMENT(bool, EnableStereoRendering )

		/** True if the viewport texture has pre-multiplied alpha */
		SLATE_ARGUMENT( bool, PreMultipliedAlpha )

		/**
		 * If true, the viewport's texture alpha is ignored when performing blending.  In this case only the viewport tint opacity is used
		 * If false, the texture alpha is used during blending
		 */
		SLATE_ARGUMENT( bool, IgnoreTextureAlpha )

		/** The interface to be used by this viewport for rendering and I/O. */
		SLATE_ARGUMENT(TSharedPtr<ISlateViewport>, ViewportInterface)

		/** Size of the viewport widget. */
		SLATE_ATTRIBUTE(FVector2D, ViewportSize);

		SLATE_API static UE::Slate::FDeprecateVector2DResult GetDefaultViewportSize();

	SLATE_END_ARGS()

	/** Default constructor. */
	SLATE_API SViewport();
	SLATE_API virtual ~SViewport();

	/**
	 * Construct the widget.
	 *
	 * @param InArgs  Declaration from which to construct the widget.
	 */
	SLATE_API void Construct(const FArguments& InArgs);

public:

	/** SViewport wants keyboard focus */
	virtual bool SupportsKeyboardFocus() const override { return true; }

	/** 
	 * Computes the ideal size necessary to display this widget.
	 *
	 * @return The desired width and height.
	 */
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return ViewportSize.Get();
	}

	/**
	 * Sets the interface to be used by this viewport for rendering and I/O
	 *
	 * @param InViewportInterface The interface to use
	 */
	SLATE_API void SetViewportInterface( TSharedRef<ISlateViewport> InViewportInterface );

	/**
	 * Sets the interface to be used by this viewport for rendering and I/O
	 *
	 * @param InViewportInterface The interface to use
	 */
	TWeakPtr<ISlateViewport> GetViewportInterface()
	{
		return ViewportInterface;
	}

	/**
	 * Sets the content for this widget
	 *
	 * @param InContent	The new content (can be null)
	 */
	SLATE_API void SetContent( TSharedPtr<SWidget> InContent );

	SLATE_API void SetCustomHitTestPath( TSharedPtr<ICustomHitTestPath> CustomHitTestPath );

	SLATE_API TSharedPtr<ICustomHitTestPath> GetCustomHitTestPath();

	const TSharedPtr<SWidget> GetContent() const { return ChildSlot.GetWidget(); }

	/**
	 * A delegate called when the viewports top level window is being closed
	 *
	 * @param InWindowBeingClosed	The window that is about to be closed
	 */
	SLATE_API void OnWindowClosed( const TSharedRef<SWindow>& InWindowBeingClosed );

	/**
	 * A delegate called when the viewports top level window is activated
	 */
	SLATE_API FReply OnViewportActivated(const FWindowActivateEvent& InActivateEvent);

	/**
	 * A delegate called when the viewports top level window is deactivated
	 */
	SLATE_API void OnViewportDeactivated(const FWindowActivateEvent& InActivateEvent);

	/** @return Whether or not this viewport renders directly to the backbuffer */
	bool ShouldRenderDirectly() const { return bRenderDirectlyToWindow; }

	/** @return Whether or not this viewport supports stereo rendering */
	bool IsStereoRenderingAllowed() const { return bEnableStereoRendering; }

	/**
	 * Sets whether this viewport can render directly to the back buffer.  Advanced use only
	 * 
	 * @param	bInRenderDirectlyToWindow	Whether we should be able to render to the back buffer
	 */
	SLATE_API void SetRenderDirectlyToWindow( const bool bInRenderDirectlyToWindow );

	/**
	 * If true, the viewport's texture alpha is ignored when performing blending.  In this case only the viewport tint opacity is used
	 * If false, the texture alpha is used during blending
	 * 
	 * @param bIgnoreTextureAlpha If texture alpha should be ignored when blending.
	 */
	SLATE_API void SetIgnoreTextureAlpha(const bool bInIgnoreTextureAlpha);

	/** @return Whether or not to ignore texture alpha when blending */
	bool GetIgnoreTextureAlpha(void) const
	{
		return bIgnoreTextureAlpha;
	}

	/**
	 * Sets whether stereo rendering is allowed for this viewport.  Advanced use only
	 * 
	 * @param	bInEnableStereoRendering	Whether stereo rendering should be allowed for this viewport
	 */
	void EnableStereoRendering( const bool bInEnableStereoRendering )
	{
		bEnableStereoRendering = bInEnableStereoRendering;
	}

	/** 
	 * Sets whether this viewport is active. 
	 * While active, a persistent Active Timer is registered and a Slate tick/paint pass is guaranteed every frame.
	 * @param bActive Whether to set the viewport as active
	 */
	SLATE_API void SetActive(bool bActive);

public:

	// SWidget interface

	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual FReply OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	SLATE_API virtual FReply OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	SLATE_API virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	SLATE_API virtual FReply OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent) override;
	SLATE_API virtual FReply OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent) override;
	SLATE_API virtual FReply OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent ) override;
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	SLATE_API virtual TOptional<TSharedRef<SWidget>> OnMapCursor(const FCursorReply& CursorReply) const override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent ) override;
	SLATE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent ) override;
	SLATE_API virtual FReply OnAnalogValueChanged( const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent ) override;
	SLATE_API virtual FReply OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& CharacterEvent ) override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnMotionDetected( const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent ) override;
	SLATE_API virtual TOptional<bool> OnQueryShowFocus( const EFocusCause InFocusCause ) const override;
	SLATE_API virtual FPopupMethodReply OnQueryPopupMethod() const override;
	SLATE_API virtual void OnFinishedPointerInput() override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual TOptional<FVirtualPointerPosition> TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& MyGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const override;
	SLATE_API virtual FNavigationReply OnNavigation( const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent ) override;

private:
	// Viewports shouldn't show focus
	virtual const FSlateBrush* GetFocusBrush() const override
	{
		return nullptr;
	}

protected:
	/** Empty active timer meant to ensure a tick/paint pass while this viewport is active */
	SLATE_API EActiveTimerReturnType EnsureTick(double InCurrentTime, float InDeltaTime);

	/** Interface to the rendering and I/O implementation of the viewport. */
	TWeakPtr<ISlateViewport> ViewportInterface;
	
private:

	/** The parent window during this viewport's last activation */
	TWeakPtr<SWindow> CachedParentWindow;

	/** The handle to the active EnsureTick() timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Whether or not to show the disabled effect when this viewport is disabled. */
	TSlateAttribute<bool, EInvalidateWidgetReason::Paint> ShowDisabledEffect;

	/** Size of the viewport. */
	TSlateAttribute<FVector2D, EInvalidateWidgetReason::Layout> ViewportSize;

	TSharedPtr<ICustomHitTestPath> CustomHitTestPath;

	/** Whether or not this viewport renders directly to the window back-buffer. */
	bool bRenderDirectlyToWindow;

	/** Whether or not to apply gamma correction on the render target supplied by the ISlateViewport. */
	bool bEnableGammaCorrection;

	/** Whether or not to reverse the gamma correction done by the texture in this viewport.  Ignores the bEnableGammaCorrection setting */
	bool bReverseGammaCorrection;

	/** Whether or not to blend this viewport with the background. */
	bool bEnableBlending;

	/** Whether or not to enable stereo rendering. */
	bool bEnableStereoRendering;

	/** Whether or not to allow texture alpha to be used in blending calculations. */
	bool bIgnoreTextureAlpha;

	/** True if the viewport texture has pre-multiplied alpha */
	bool bPreMultipliedAlpha;
};

namespace UE::Slate::Viewport
{
	/**
	 * Calculates the scaling factor needed to transform from local Slate geometry to viewport pixel geometry
	 * @param InGeometry The viewport's Slate geometry
	 * @param InViewportSize The viewport's render size
	 * @return A non-uniform scaling factor to convert from local Slate coordinates to viewport pixel coordinates
	 */
	SLATE_API FVector2D GetViewportLocalToPixelScale(const FGeometry& InGeometry, const FVector2D& InViewportSize);
}