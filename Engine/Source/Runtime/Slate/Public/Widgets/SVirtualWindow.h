// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Layout/SlateRect.h"
#include "Layout/Geometry.h"
#include "Input/PopupMethodReply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SWindow.h"

class FArrangedChildren;
class FSlateDrawBuffer;
class ISlate3DRenderer;
class STooltipPresenter;
class UTextureRenderTarget2D;

class SVirtualWindow : public SWindow
{
	SLATE_BEGIN_ARGS(SVirtualWindow)
		: _Size(FVector2D(100, 100))
	{}

	SLATE_ARGUMENT(FVector2D, Size)

		SLATE_END_ARGS()

public:
	SLATE_API void Construct(const FArguments& InArgs);

	/**
	 * We allow users to control the resolve policy for deferred content. When a virtual window is used in a retainer, we don't want it trying to resolve
	 * the deferred content, we instead need it to pass that content up to the non-retained window drawing the retainer. If a virtual window is used in
	 * a case like the WidgetComponent, we always want to resolve the deferred content because there won't be another opportunity since the 3D widget has
	 * no owner window that can draw it.
	 */
	SLATE_API void SetShouldResolveDeferred(bool bResolve);

	/**  */
	SLATE_API void SetIsFocusable(bool bFocusable);

	SLATE_API virtual FPopupMethodReply OnQueryPopupMethod() const override;
	SLATE_API virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent) override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	SLATE_API virtual bool Advanced_IsInvalidationRoot() const override;

private:
	TSharedPtr<class STooltipPresenter> TooltipPresenter;
	bool bIsFocusable;
	bool bShouldResolveDeferred;
};

