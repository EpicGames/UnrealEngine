// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * Implements a widget that holds a weak pointer to one child widget.
 *
 * Weak widgets encapsulate a piece of content without owning it.
 * e.g. A tooltip is owned by the hovered widget but displayed
 *      by a floating window. That window cannot own the tooltip
 *      and must therefore use an SWeakWidget.
 */
class SWeakWidget : public SWidget
{
public:

	SLATE_BEGIN_ARGS(SWeakWidget)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}
		SLATE_ARGUMENT( TSharedPtr<SWidget>, PossiblyNullContent )
	SLATE_END_ARGS()

public:

	SLATE_API SWeakWidget();
	SLATE_API virtual ~SWeakWidget();

	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API void SetContent(const TSharedRef<SWidget>& InWidget);

	SLATE_API bool ChildWidgetIsValid() const;

	SLATE_API TWeakPtr<SWidget> GetChildWidget() const;

public:

	// SWidget interface
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

protected:
	// Begin SWidget overrides.
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

private:

	TWeakChild<SWidget> WeakChild;
};
