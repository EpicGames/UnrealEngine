// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Framework/Application/IMenu.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"

#include "ISlateEditableTextWidget.generated.h"

struct FTextLocation;

enum class ECursorMoveMethod
{
	/** Move in one of the cardinal directions e.g. arrow left, right, up, down */
	Cardinal,
	/** Move the cursor to the correct character based on the given screen position. */
	ScreenPosition
};

enum class ECursorMoveGranularity
{
	/** Move one character at a time (e.g. arrow left, right, up, down). */
	Character,
	/** Move one word at a time (e.g. ctrl is held down and arrow left/right/up/down). */
	Word
};

enum class ECursorAction
{
	/** Just relocate the cursor. */
	MoveCursor,
	/** Move the cursor and select any text that it passes over. */
	SelectText
};

enum class ETextLocation
{
	/** Jump to the beginning of text. (e.g. Ctrl+Home) */
	BeginningOfDocument,
	/** Jump to the end of text. (e.g. Ctrl+End) */
	EndOfDocument,
	/** Jump to the beginning of the line or beginning of text within a line (e.g. Home) */
	BeginningOfLine,
	/** Jump to the end of line (e.g. End) */
	EndOfLine,
	/** Jump to the previous page in this document (e.g. PageUp) */
	PreviousPage,
	/** Jump to the next page in this document (e.g. PageDown) */
	NextPage,
};

UENUM(BlueprintType)
enum class EVirtualKeyboardTrigger : uint8
{
	/** Display the virtual keyboard when the widget gains keyboard focus by a pointer action. */
	OnFocusByPointer,
	/** Display the virtual keyboard when the widget gains keyboard focus by any means. */
	OnAllFocusEvents,
};

UENUM(BlueprintType)
enum class EVirtualKeyboardDismissAction : uint8
{
	/** Sends a text changed message when the virtual keyboard is dismissed by the user. */
	TextChangeOnDismiss,
	/** Send a text commit message if the user dismisses the keyboard by accepting text. Send a text changed message if the user cancels the virtual keyboard. */
	TextCommitOnAccept,
	/** Send a text commit message when the virtual keyboard is dismissed by the user. */
	TextCommitOnDismiss,
};

/**
 * Argument to the ITextEditorWidget::Move(); it decouples performing
 * cursor movement and text highlighting actions from event handling.
 * FMoveCursor describes how the cursor can be moved via keyboard,
 * mouse, and other mechanisms in the future.
 */
class FMoveCursor
{
public:
	/**
	 * Creates a MoveCursor action that describes moving by a single character in any of the cardinal directions.
	 *
	 * @param Granularity    Move one character at a time, or on word boundaries (e.g. Ctrl is held down, or double-click mouse+drag)
	 * @param Direction      Axis-aligned unit vector along which to move. e.g. Move right: (0,1) or Move Up: (-1, 0)
	 * @param Action         Just move or also select text?
	 */
	static FMoveCursor Cardinal(ECursorMoveGranularity Granularity, FIntPoint Direction, ECursorAction Action);

	/**
	 * Creates a MoveCursor action that describes moving the text cursor by selecting an arbitrary coordinate on the screen.
	 * e.g. User user touches a touch device screen or uses the mouse to point at text.
	 *
	 * @param LocalPosition          Position in the text widget where the user wants to move the cursor by pointing
	 * @param GeometryScale          DPI Scale of the widget geometry.
	 * @param Action                 Just move or also select text?
	 */
	static FMoveCursor ViaScreenPointer(FVector2D LocalPosition, float GeometryScale, ECursorAction Action);

	/** @see ECursorMoveMethod */
	ECursorMoveMethod GetMoveMethod() const;

	/** Is the cursor moving up/down; Only valid for word and character movement methods. */
	bool IsVerticalMovement() const;

	/** Is the cursor moving left/right; Only valid for word and character movement methods. */
	bool IsHorizontalMovement() const;

	/** @see ECursorAction */
	ECursorAction GetAction() const;

	/** When using directional movement (i.e. Character or Word granularity; not screen position, which way to move.) */
	FIntPoint GetMoveDirection() const;

	/** Which position in the widget where the user touched when using ScreenPosition mode. */
	FVector2D GetLocalPosition() const;

	/** Move one character at a time, or one word? */
	ECursorMoveGranularity GetGranularity() const;

	/** Geometry Scale at the time of the event that caused this action */
	float GetGeometryScale() const;

private:
	FMoveCursor(ECursorMoveGranularity InGranularity, ECursorMoveMethod InMethod, FVector2D InDirectionOrPosition, float InGeometryScale, ECursorAction InAction);

	ECursorMoveGranularity Granularity;
	ECursorMoveMethod Method;
	FVector2f DirectionOrPosition;
	ECursorAction Action;
	float GeometryScale;
};

/** Manages the state for an active context menu */
class FActiveTextEditContextMenu
{
public:
	FActiveTextEditContextMenu()
		: bIsPendingSummon(false)
		, ActiveMenu()
	{
	}

	/** Check to see whether this context is valid (either pending or active) */
	bool IsValid() const
	{
		return bIsPendingSummon || ActiveMenu.IsValid();
	}

	/** Called to reset the active context menu state */
	void Reset()
	{
		bIsPendingSummon = false;
		ActiveMenu.Reset();
	}

	/** Called before you summon your context menu */
	void PrepareToSummon()
	{
		bIsPendingSummon = true;
		ActiveMenu.Reset();
	}

	/** Called when you've successfully summoned your context menu */
	void SummonSucceeded(const TSharedRef<IMenu>& InMenu)
	{
		bIsPendingSummon = false;
		ActiveMenu = InMenu;
	}

	/** Called if your context menu summon fails */
	void SummonFailed()
	{
		bIsPendingSummon = false;
		ActiveMenu.Reset();
	}

	/** Called to dismiss the active context menu */
	void Dismiss()
	{
		if (ActiveMenu.IsValid())
		{
			auto ActiveMenuPin = ActiveMenu.Pin();
			ActiveMenuPin->Dismiss();
		}
		Reset();
	}

private:
	/** True if we are pending the summon of a context menu, but don't yet have an active window pointer */
	bool bIsPendingSummon;

	/** Handle to the active context menu (if any) */
	TWeakPtr<IMenu> ActiveMenu;
};

/** Interface to allow FSlateEditableTextLayout to notify its parent SEditableText/SMultiLineEditableText of changes, as well as query some widget specific state*/
class ISlateEditableTextWidget
{
public:
	/** Is the widget construction completed (did we called and returned from the Construct() function) */
	virtual bool IsWidgetConstructed() const = 0;

	/** Is the text currently read-only? */
	virtual bool IsTextReadOnly() const = 0;

	/** Is the text displaying a password and should be obscured? */
	virtual bool IsTextPassword() const = 0;

	/** Is the text edit multi-line aware? */
	virtual bool IsMultiLineTextEdit() const = 0;

	/** Should the cursor be jumped to the end of the document when the widget gains focus? */
	virtual bool ShouldJumpCursorToEndWhenFocused() const = 0;

	/** Should the text be selected when the widget gains focus? */
	virtual bool ShouldSelectAllTextWhenFocused() const = 0;

	/** Should the text clear its selection the widget loses focus? */
	virtual bool ShouldClearTextSelectionOnFocusLoss() const = 0;

	/** Should we revert the text back to its original state when the user presses escape? */
	virtual bool ShouldRevertTextOnEscape() const = 0;

	/** Should we clear the keyboard focus when the user commits text to this widget? */
	virtual bool ShouldClearKeyboardFocusOnCommit() const = 0;

	/** Should we select all text when the user commits text to this widget? */
	virtual bool ShouldSelectAllTextOnCommit() const = 0;

	/** Should we select word on mouse double click? */
	virtual bool ShouldSelectWordOnMouseDoubleClick() const = 0;

	/** Are we currently able to insert a carriage return? (some widgets have modifier keys that need to be pressed) */
	virtual bool CanInsertCarriageReturn() const = 0;

	/** Are we able to insert the given character into our text? */
	virtual bool CanTypeCharacter(const TCHAR InChar) const = 0;

	/** Is the widget currently has IntegratedKeyboard enabled? */
	virtual bool IsIntegratedKeyboardEnabled() const = 0;

	/**
	 * Ensure that we will get a Tick() soon (either due to having active focus, or something having changed progmatically and requiring an update)
	 * Does nothing if the active tick timer is already enabled
	 */
	virtual void EnsureActiveTick() = 0;

	/** Get the type of virtual keyboard to use for this widget */
	virtual EKeyboardType GetVirtualKeyboardType() const = 0;

	/** Get the options for the virtual keyboard used for this widget */
	virtual FVirtualKeyboardOptions GetVirtualKeyboardOptions() const = 0;

	/** Get the type of event that will trigger the display of the virtual keyboard */
	virtual EVirtualKeyboardTrigger GetVirtualKeyboardTrigger() const = 0;

	/** Get the message action to take when the virtual keyboard is dismissed by the user */
	virtual EVirtualKeyboardDismissAction GetVirtualKeyboardDismissAction() const = 0;

	/** Get the Slate widget this interface is representing (may not be called during destruction) */
	virtual TSharedRef<SWidget> GetSlateWidget() = 0;

	/** Get the Slate widget this interface is representing (may be null during destruction) */
	virtual TSharedPtr<SWidget> GetSlateWidgetPtr() = 0;

	/** Build the context menu content to use for this widget (if any) */
	virtual TSharedPtr<SWidget> BuildContextMenuContent() const = 0;
	
	/** Called when the text starts to be edited */
    virtual void OnBeginTextEdit(const FText& InText) = 0;

	/** Called when the text has been changed by an edit operation */
	virtual void OnTextChanged(const FText& InText) = 0;

	/** Called when the text control has committed its current edit changes */
	virtual void OnTextCommitted(const FText& InText, const ETextCommit::Type InTextAction) = 0;

	/** Called when the cursor is moved within the text area */
	virtual void OnCursorMoved(const FTextLocation& InLocation) = 0;

	/** Called when the fraction and offset of the horizontal scroll area has been recalculated. This function should apply the new values to any scrollbars, and return a clamped horizontal scroll value */
	virtual float UpdateAndClampHorizontalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisiblityOverride) = 0;

	/** Called when the fraction and offset of the vertical scroll area has been recalculated. This function should apply the new values to any scrollbars, and return a clamped vertical scroll value */
	virtual float UpdateAndClampVerticalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisiblityOverride) = 0;
};
