// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "SlateGlobals.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Text/TextLayout.h"

class FActiveTimerHandle;
class IBreakIterator;
class SEditableTextBox;
class SHorizontalBox;
class SMultiLineEditableTextBox;
class STextBlock;

template <>
struct TWidgetTypeTraits<class SInlineEditableTextBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Slate's InlineEditableTextBlock's are double selectable to go from a STextBlock to become SEditableTextBox.
 */
class SInlineEditableTextBlock: public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SInlineEditableTextBlock, SCompoundWidget, SLATE_API)

	SLATE_BEGIN_ARGS( SInlineEditableTextBlock )
		: _Text()
		, _HintText()
		, _Style( &FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle") )
		, _Font()
		, _ColorAndOpacity()
		, _ShadowOffset()
		, _ShadowColorAndOpacity()
		, _HighlightText()
		, _WrapTextAt(0.0f)
		, _AutoWrapNonEditText(false)
		, _AutoWrapMultilineEditText(false)
		, _Justification(ETextJustify::Left)
		, _MaximumLength(-1)
		, _LineBreakPolicy()
		, _IsReadOnly(false)
		, _MultiLine(false)
		, _DoubleSelectDelay(0.5f)
		, _DelayedLeftClickEntersEditMode(true)
		, _ModiferKeyForNewLine(EModifierKey::None)
		, _OverflowPolicy()
	{
	}

		/** The text displayed in this text block */
		SLATE_ATTRIBUTE( FText, Text )

		/** Hint text that appears when there is no text in the text box */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Pointer to a style of the inline editable text block, which dictates the font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT( FInlineEditableTextBlockStyle, Style )

		/** Sets the font used to draw the text (overrides style) */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Text color and opacity (overrides style) */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Drop shadow offset in pixels (overrides style) */
		SLATE_ATTRIBUTE( FVector2D, ShadowOffset )

		/** Shadow color and opacity (overrides style) */
		SLATE_ATTRIBUTE( FLinearColor, ShadowColorAndOpacity )

		/** Highlight this text in the text block */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
		SLATE_ATTRIBUTE( float, WrapTextAt )

		/** Whether the text should wrap automatically when in non-edit mode */
		SLATE_ATTRIBUTE( bool, AutoWrapNonEditText )

		/** Whether the multiline text should wrap automatically when in edit mode */
		SLATE_ATTRIBUTE( bool, AutoWrapMultilineEditText )

		/** How the text should be aligned with the margin. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)

		/** Maximum length of the text. Input will refuse text longer than that. */
		SLATE_ATTRIBUTE(int32, MaximumLength)

		/** The iterator to use to detect appropriate soft-wrapping points for lines (or null to use the default) */
		SLATE_ARGUMENT( TSharedPtr<IBreakIterator>, LineBreakPolicy )

		/** True if the editable text block is read-only. It will not be able to enter edit mode if read-only */
		SLATE_ATTRIBUTE( bool, IsReadOnly )

		/** True if the editable text block is multi-line */
		SLATE_ARGUMENT( bool, MultiLine )

		/** The length of delay to enter edit mode after selection, if zero, it will enter edit mode on double click */
		SLATE_ARGUMENT( float, DoubleSelectDelay )

		/** True if left-clicking will enter edit mode after a delay. */
		SLATE_ARGUMENT( bool, DelayedLeftClickEntersEditMode )

		/** The optional modifier key necessary to create a newline when typing into the editor. */
		SLATE_ARGUMENT(EModifierKey::Type, ModiferKeyForNewLine)

		/** Callback when the text starts to be edited */
		SLATE_EVENT( FOnBeginTextEdit, OnBeginTextEdit )

		/** Callback when the text is committed. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Callback when the text editing begins. */
		SLATE_EVENT(FSimpleDelegate, OnEnterEditingMode)

		/** Callback when the text editing ends. */
		SLATE_EVENT(FSimpleDelegate, OnExitEditingMode)

		/** Delegate to execute to check the status of if the widget is selected or not
			Only needs to be hooked up if an external widget is managing selection, such
			as a SListView or STreeView. */
		SLATE_EVENT( FIsSelected, IsSelected )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT( FOnVerifyTextChanged, OnVerifyTextChanged )

		/** Determines what happens to text that is clipped and doesn't fit within the clip rect for this widget */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)
	SLATE_END_ARGS()

	SLATE_API SInlineEditableTextBlock();

	SLATE_API ~SInlineEditableTextBlock();

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/**
	 * See SWidget::SupportsKeyboardFocus().
	 *
	 * @return  This widget can gain focus if IsSelected is not bound.
	 */
	SLATE_API virtual bool SupportsKeyboardFocus() const override;

	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	SLATE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Switches the widget to editing mode */
	SLATE_API void EnterEditingMode();

	/** Switches the widget to label mode. */
	SLATE_API void ExitEditingMode();

	/** Checks if the widget is in edit mode */
	SLATE_API bool IsInEditMode() const;

	SLATE_API void SetReadOnly(const TAttribute<bool>& bInIsReadOnly);

	/**
	 * Sets the maximum text length that will be accepted by the widget
	 *
	 * @param  InMaximumLength		Length to check for. Only values superior to 0 will be taken into account, otherwise no length check will occur.
	 */
	SLATE_API void SetMaximumLength(TAttribute<int32> InMaximumLength);
	TSlateAttributeRef<int32> GetMaximumLengthAttribute() const
	{
		return TSlateAttributeRef<int32>{SharedThis(this), MaximumLengthAttribute};
	}

	SLATE_API void SetText( const TAttribute< FText >& InText );
	SLATE_API void SetText( const FString& InText );

	/** Return the current text */
	SLATE_API FText GetText() const;

	SLATE_API void SetHintText( const TAttribute< FText >& InHintText );

	SLATE_API void SetHighlightText(const TAttribute<FText>& InText);

	/** Sets the wrap text at attribute.  See WrapTextAt attribute */
	SLATE_API void SetWrapTextAt(const TAttribute<float>& InWrapTextAt);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy);

	SLATE_API void SetOnTextCommitted(FOnTextCommitted InOnTextCommitted);

	SLATE_API void SetOnEnterEditingMode(FSimpleDelegate InOnEnterEditingMode);

	SLATE_API void SetOnExitEditingMode(FSimpleDelegate InOnExitEditingMode);

	SLATE_API void SetOnExitEditingMode(FIsSelected InIsSelected);

	SLATE_API void SetOnVerifyTextChanged(FOnVerifyTextChanged InOnVerifyTextChanged);

protected:
	/** Callback for the text box's OnTextChanged event */
	SLATE_API void OnTextChanged(const FText& InText);

	/** Validate the given text by verifying its length and by calling OnVerifyTextChanged if bound. */
	SLATE_API bool IsTextAllowed(const FText& InText, FText& OutErrorMessage) const;

	/** Callback when the text box is committed, switches back to label mode. */ 
	SLATE_API void OnTextBoxCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Cancels the edit mode and switches back to label mode */
	SLATE_API void CancelEditMode();

protected:
	/** The widget used when in label mode */
	TSharedPtr< STextBlock > TextBlock;

	/** The widget used when in editing mode */ 
	TSharedPtr< SEditableTextBox > TextBox;

#if WITH_FANCY_TEXT
	/** The widget used when in editing mode */ 
	TSharedPtr< SMultiLineEditableTextBox > MultiLineTextBox;
#endif //WITH_FANCY_TEXT

	FSimpleDelegate OnEnterEditingMode;
	FSimpleDelegate OnExitEditingMode;

	/** Delegate to execute when editing mode text is committed. */
	FOnTextCommitted OnTextCommittedDelegate;

	/** Delegate to execute to check the status of if the widget is selected or not
		Only needs to be hooked up if an external widget is managing selection, such
		as a SListView or STreeView. */
	FIsSelected IsSelected;

	/** Main horizontal box, used to dynamically add and remove the editable slot. */
	TSharedPtr< SHorizontalBox > HorizontalBox;

	/** Callback to verify text when changed. Will return an error message to denote problems. */
	FOnVerifyTextChanged OnVerifyTextChanged;

	/** Attribute for the text to use for the widget */
	TAttribute<FText> Text;

	/** Attribute for the hint text to use for the widget */
	TAttribute<FText> HintText;

	/** Attribute to look up if the widget is read-only */
	TAttribute< bool > bIsReadOnly;

#if WITH_EDITORONLY_DATA
	/** Maximum text length that will be accepted by the widget */
	UE_DEPRECATED(5.8, "Use SetMaximumLength / GetMaximumLengthAttribute")
	TSlateDeprecatedTAttribute<int32> MaximumLength;
#endif

	/** Widget to focus when we finish editing */
	TWeakPtr<SWidget> WidgetToFocus;

private:

	/** Get editable text widget */
	TSharedPtr<SWidget> GetEditableTextWidget() const;

	/** Set editable text */
	void SetEditableText( const TAttribute< FText >& InNewText );

	FText GetEditableText() const;

	/** Set textbox error */
	void SetTextBoxError( const FText& ErrorText );
	bool HasTextBoxError() const;

	/** Active timer to trigger entry into edit mode after a delay */
	EActiveTimerReturnType TriggerEditMode(double InCurrentTime, float InDeltaTime);

	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Maximum text length that will be accepted by the widget */
	TSlateAttribute<int32> MaximumLengthAttribute;

	mutable bool bNeedsMaximumLengthChecked = false;

protected:
	/** When selection of widget is managed by another widget, this delays the "double select" from 
	occurring immediately, offering a chance for double clicking to take action. */
	float DoubleSelectDelay = 0.5f;

	/** Attribute to look up if the widget is multiline */
	uint8 bIsMultiLine:1;

	/** Enable left-clicking the text block to enter edit mode. */
	uint8 bDelayedLeftClickEntersEditMode:1;
};
