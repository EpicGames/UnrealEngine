// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateEditableTextLayout.h"
#include "Misc/Attribute.h"
#include "SlateGlobals.h"
#include "Layout/Margin.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Framework/SlateDelegates.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#if WITH_FANCY_TEXT
	#include "Widgets/Text/ISlateEditableTextWidget.h"
	#include "Framework/Text/SlateTextLayoutFactory.h"
#endif

class FActiveTimerHandle;
class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class ITextLayoutMarshaller;
enum class ETextShapingMethod : uint8;

#if WITH_FANCY_TEXT

class ITextLayoutMarshaller;
class FSlateEditableTextLayout;

template <>
struct TWidgetTypeTraits<class SMultiLineEditableText>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** An editable text widget that supports multiple lines and soft word-wrapping. */
class SMultiLineEditableText : public SWidget, public ISlateEditableTextWidget
{
	SLATE_DECLARE_WIDGET_API(SMultiLineEditableText, SWidget, SLATE_API)
public:

	/** Used to merge multiple text edit transactions within a scope */
	struct FScopedEditableTextTransaction
	{
	public:
		FScopedEditableTextTransaction(TSharedPtr<SMultiLineEditableText> InText)
			: Text(MoveTemp(InText))
		{
			Text->BeginEditTransaction();
		}

		~FScopedEditableTextTransaction()
		{
			Text->EndEditTransaction();	
		};

	private:
		TSharedPtr<SMultiLineEditableText> Text;
	};
	
	/** Called when the cursor is moved within the text area */
	DECLARE_DELEGATE_OneParam( FOnCursorMoved, const FTextLocation& );

	SLATE_BEGIN_ARGS( SMultiLineEditableText )
		: _Text()
		, _HintText()
		, _SearchText()
		, _Marshaller()
		, _WrapTextAt( 0.0f )
		, _AutoWrapText(false)
		, _WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		, _Font()
		, _Margin( FMargin() )
		, _LineHeightPercentage( 1.0f )
		, _ApplyLineHeightToBottomLine( true )
		, _Justification( ETextJustify::Left )
		, _OnTextChanged()
		, _OnTextCommitted()
		, _AllowMultiLine(true)
		, _SelectAllTextWhenFocused(false)
		, _SelectWordOnMouseDoubleClick(true)
		, _ClearTextSelectionOnFocusLoss(true)
		, _RevertTextOnEscape(false)
		, _ClearKeyboardFocusOnCommit(true)
		, _AllowContextMenu(true)
		, _OnCursorMoved()
		, _ContextMenuExtender()
		, _ModiferKeyForNewLine(EModifierKey::None)
		, _VirtualKeyboardOptions( FVirtualKeyboardOptions() )
		, _VirtualKeyboardTrigger(EVirtualKeyboardTrigger::OnFocusByPointer)
		, _VirtualKeyboardDismissAction(EVirtualKeyboardDismissAction::TextChangeOnDismiss)
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _OverflowPolicy()
		, _FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}

		/** The initial text that will appear in the widget. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Hint text that appears when there is no text in the text box */
		SLATE_ATTRIBUTE(FText, HintText)

		/** Text to search for (a new search is triggered whenever this text changes) */
		SLATE_ATTRIBUTE(FText, SearchText)

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr< ITextLayoutMarshaller >, Marshaller)

		/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
		SLATE_ATTRIBUTE(float, WrapTextAt)

		/** Whether to wrap text automatically based on the widget's computed horizontal space.  IMPORTANT: Using automatic wrapping can result
			in visual artifacts, as the computed wrapped size will be at least one frame late!  Consider using WrapTextAt instead.  The initial
			desired size will not be clamped.  This works best in cases where the text block's size is not affecting other widget's layout. */
		SLATE_ATTRIBUTE(bool, AutoWrapText)

		/** The wrapping policy to use */
		SLATE_ATTRIBUTE(ETextWrappingPolicy, WrappingPolicy)

		/** Pointer to a style of the text block, which dictates the font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)

		/** Font color and opacity (overrides Style) */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** The amount of blank space left around the edges of text area. */
		SLATE_ATTRIBUTE(FMargin, Margin)

		/** The amount to scale each lines height by. */
		SLATE_ATTRIBUTE(float, LineHeightPercentage)

		/** The amount to scale each lines height by. */
		SLATE_ATTRIBUTE(bool, ApplyLineHeightToBottomLine)

		/** How the text should be aligned with the margin. */
		SLATE_ATTRIBUTE(ETextJustify::Type, Justification)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** The horizontal scroll bar widget */
		SLATE_ARGUMENT(TSharedPtr< SScrollBar >, HScrollBar)

		/** The vertical scroll bar widget */
		SLATE_ARGUMENT(TSharedPtr< SScrollBar >, VScrollBar)

		/**
		 * This is NOT for validating input!
		 * 
		 * Called whenever a character is typed.
		 * Not called for copy, paste, or any other text changes!
		 */
		SLATE_EVENT(FOnIsTypedCharValid, OnIsTypedCharValid)

		/** Callback when the text starts to be edited */
		SLATE_EVENT(FOnBeginTextEdit, OnBeginTextEdit )

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

		/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** Whether to allow multi-line text */
		SLATE_ATTRIBUTE(bool, AllowMultiLine)

		/** Whether to select all text when the user clicks to give focus on the widget */
		SLATE_ATTRIBUTE(bool, SelectAllTextWhenFocused)

		/** Whether to select word on mouse double click on the widget */
		SLATE_ATTRIBUTE(bool, SelectWordOnMouseDoubleClick)

		/** Whether to clear text selection when focus is lost */
		SLATE_ATTRIBUTE(bool, ClearTextSelectionOnFocusLoss)

		/** Whether to allow the user to back out of changes when they press the escape key */
		SLATE_ATTRIBUTE(bool, RevertTextOnEscape)

		/** Whether to clear keyboard focus when pressing enter to commit changes */
		SLATE_ATTRIBUTE(bool, ClearKeyboardFocusOnCommit)

		/** Whether to prevent the context menu from being displayed  */
		SLATE_ATTRIBUTE(bool, AllowContextMenu)

		/** Delegate to call before a context menu is opened. User returns the menu content or null to the disable context menu */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

		/** Called whenever the horizontal scrollbar is moved by the user */
		SLATE_EVENT(FOnUserScrolled, OnHScrollBarUserScrolled)

		/** Called whenever the vertical scrollbar is moved by the user */
		SLATE_EVENT(FOnUserScrolled, OnVScrollBarUserScrolled)

		/** Called when the cursor is moved within the text area */
		SLATE_EVENT(FOnCursorMoved, OnCursorMoved)

		/** Callback delegate to have first chance handling of the OnKeyChar event */
		SLATE_EVENT(FOnKeyChar, OnKeyCharHandler)

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

		/** Menu extender for the right-click context menu */
		SLATE_EVENT(FMenuExtensionDelegate, ContextMenuExtender)

		/** Delegate used to create text layouts for this widget. If none is provided then FSlateTextLayout will be used. */
		SLATE_EVENT(FCreateSlateTextLayout, CreateSlateTextLayout)

		/** The optional modifier key necessary to create a newline when typing into the editor. */
		SLATE_ARGUMENT(EModifierKey::Type, ModiferKeyForNewLine)

		/** Additional options for the virtual keyboard used by this widget */
		SLATE_ARGUMENT(FVirtualKeyboardOptions, VirtualKeyboardOptions)

		/** The type of event that will trigger the display of the virtual keyboard */
		SLATE_ATTRIBUTE(EVirtualKeyboardTrigger, VirtualKeyboardTrigger)

		/** The message action to take when the virtual keyboard is dismissed by the user */
		SLATE_ATTRIBUTE(EVirtualKeyboardDismissAction, VirtualKeyboardDismissAction)

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		
		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

		/** Determines what happens to text that is clipped and doesn't fit within the clip rect for this widget */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

		/** The paint policy to use for text if the font faces are still async loading. */
		SLATE_ARGUMENT(EFontFacesLoadingPaintPolicy, FontFacesLoadingPaintPolicy)

		/** Called when all font faces for the text have finished loading asynchronously */
		SLATE_EVENT(FSimpleDelegate, OnAllFontFacesFinishLoading)
	SLATE_END_ARGS()

	SLATE_API SMultiLineEditableText();

	SLATE_API void Construct(const FArguments& InArgs);

	/**
	 * Sets the text for this text block
	 */
	SLATE_API void SetText(TAttribute<FText> InText);

	/**
	 * Returns the text string
	 *
	 * @return  Text string
	 */
	SLATE_API FText GetText() const;

	/**
	 * Returns the plain text string without richtext formatting
	 * @return  Text string
	 */
	SLATE_API FText GetPlainText() const;

	/**
	 * Fill OutTextLine with the text line where the current cursor location is at
	 *
	 * @param OutTextLine   FString of the line
	 */
	SLATE_API void GetCurrentTextLine(FString& OutTextLine) const;

	/**
	 * Fill OutTextLine with the text line at the specified index
	 *
	 * @param InLineIndex   Index of the line
	 * @param OutTextLine   FString of the line
	 */
	SLATE_API void GetTextLine(const int32 InLineIndex, FString& OutTextLine) const;
	
	/**
	 * Sets the text that appears when there is no text in the text box
	 */
	SLATE_API void SetHintText(TAttribute<FText> InHintText);

	/** Get the text that appears when there is no text in the text box */
	SLATE_API FText GetHintText() const;

	/** Set the text that is currently being searched for (if any) */
	SLATE_API void SetSearchText(TAttribute<FText> InSearchText);

	/** Get the text that is currently being searched for (if any) */
	SLATE_API FText GetSearchText() const;

	/** Get the index of the search result (0 if none) */
	SLATE_API int32 GetSearchResultIndex() const;

	/** Get the total number of search results (0 if none) */
	SLATE_API int32 GetNumSearchResults() const;

	/** See attribute TextStyle */
	SLATE_API void SetTextStyle(const FTextBlockStyle* InTextStyle);

	/** See attribute Font */
	SLATE_API void SetFont(TAttribute<FSlateFontInfo> InNewFont);
	SLATE_API FSlateFontInfo GetFont() const;

	/** See TextShapingMethod attribute */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** See TextFlowDirection attribute */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** See WrapTextAt attribute */
	SLATE_API void SetWrapTextAt(TAttribute<float> InWrapTextAt);

	/** See AutoWrapText attribute */
	SLATE_API void SetAutoWrapText(TAttribute<bool> InAutoWrapText);

	/** Set WrappingPolicy attribute */
	SLATE_API void SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy);

	/** See LineHeightPercentage attribute */
	SLATE_API void SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage);

	/** See ApplyLineHeightToBottomLine attribute */
	SLATE_API void SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine);

	/** See Margin attribute */
	SLATE_API void SetMargin(TAttribute<FMargin> InMargin);

	/** See Justification attribute */
	SLATE_API void SetJustification(TAttribute<ETextJustify::Type> InJustification);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy);

	/** See the AllowContextMenu attribute */
	SLATE_API void SetAllowContextMenu(TAttribute<bool> InAllowContextMenu);

	/** Set the VirtualKeyboardDismissAction attribute */
	SLATE_API void SetVirtualKeyboardDismissAction(TAttribute<EVirtualKeyboardDismissAction> InVirtualKeyboardDismissAction);

	/**
	 * Sets whether to select word on the mouse double click
	 *
	 * @param  InSelectWordOnMouseDoubleClick		Select word on the mouse double click
	 */
	SLATE_API void SetSelectWordOnMouseDoubleClick(TAttribute<bool> InSelectWordOnMouseDoubleClick);
	
	/** Sets the ReadOnly attribute */
	SLATE_API void SetIsReadOnly(TAttribute<bool> InIsReadOnly);

	/** Get the number of Text Lines */
	UE_DEPRECATED(5.8, "Use the const version instead.")
	int32 GetTextLineCount()
	{
		return const_cast<const SMultiLineEditableText*>(this)->GetTextLineCount();
	}
	/** Get the number of Text Lines */
	SLATE_API int32 GetTextLineCount() const;

	/**
	 * Sets whether to select all text when the user clicks to give focus on the widget
	 *
	 * @param  InSelectAllTextWhenFocused	Select all text when the user clicks?
	 */
	SLATE_API void SetSelectAllTextWhenFocused(TAttribute<bool> InSelectAllTextWhenFocused);

	/**
	 * Sets whether to clear text selection when focus is lost
	 *
	 * @param  InClearTextSelectionOnFocusLoss	Clear text selection when focus is lost?
	 */
	SLATE_API void SetClearTextSelectionOnFocusLoss(TAttribute<bool> InClearTextSelectionOnFocusLoss);

	/**
	 * Sets whether to allow the user to back out of changes when they press the escape key
	 *
	 * @param  InRevertTextOnEscape			Allow the user to back out of changes?
	 */
	SLATE_API void SetRevertTextOnEscape(TAttribute<bool> InRevertTextOnEscape);

	/**
	 * Sets whether to clear keyboard focus when pressing enter to commit changes
	 *
	 * @param  InClearKeyboardFocusOnCommit		Clear keyboard focus when pressing enter?
	 */
	SLATE_API void SetClearKeyboardFocusOnCommit(TAttribute<bool> InClearKeyboardFocusOnCommit);

	/** Query to see if any text is selected within the document */
	SLATE_API bool AnyTextSelected() const;

	/** Select all the text in the document */
	SLATE_API void SelectAllText();

	/** Select a block of text */
	SLATE_API void SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation);
	
	/** Clear the active text selection */
	SLATE_API void ClearSelection();

	/** Get the currently selected text */
	SLATE_API FText GetSelectedText() const;

	/** Get the current selection */
	SLATE_API FTextSelection GetSelection() const;

	/** Delete any currently selected text */
	SLATE_API void DeleteSelectedText();

	/** Insert the given text at the current cursor position, correctly taking into account new line characters */
	SLATE_API void InsertTextAtCursor(const FText& InText);
	SLATE_API void InsertTextAtCursor(const FString& InString);

	/** Insert the given run at the current cursor position */
	SLATE_API void InsertRunAtCursor(TSharedRef<IRun> InRun);

	/** Move the cursor to the given location in the document (will also scroll to this point) */
	SLATE_API void GoTo(const FTextLocation& NewLocation);

	/** Move the cursor specified location */
	SLATE_API void GoTo(const ETextLocation NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const FTextLocation& NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const ETextLocation NewLocation);

	/** Apply the given style to the currently selected text (or insert a new run at the current cursor position if no text is selected) */
	SLATE_API void ApplyToSelection(const FRunInfo& InRunInfo, const FTextBlockStyle& InStyle);

	/** Begin a new text search (this is called automatically when the bound search text changes) */
	SLATE_API void BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase, const bool InReverse = false);

	/** Advance the current search to the next match (does nothing if not currently searching) */
	SLATE_API void AdvanceSearch(const bool InReverse = false);

	/** Get the run currently under the cursor, or null if there is no run currently under the cursor */
	SLATE_API TSharedPtr<const IRun> GetRunUnderCursor() const;

	/** Get the runs that are currently selected, some of which may be only partially selected */
	SLATE_API TArray<TSharedRef<const IRun>> GetSelectedRuns() const;

	/** Get the interaction position of the cursor (where to insert, delete, etc, text from/to) */
	SLATE_API FTextLocation GetCursorLocation() const;

	/** Get the character at Location */
	SLATE_API TCHAR GetCharacterAt(const FTextLocation& Location) const;

	/** Get the horizontal scroll bar widget */
	SLATE_API TSharedPtr<const SScrollBar> GetHScrollBar() const;

	/** Get the vertical scroll bar widget */
	SLATE_API TSharedPtr<const SScrollBar> GetVScrollBar() const;

	/** Refresh this editable text immediately, rather than wait for the usual caching mechanisms to take effect on the text Tick */
	SLATE_API void Refresh();

	/**
	 * Sets the OnKeyCharHandler to provide first chance handling of the OnKeyChar event
	 *
	 * @param InOnKeyCharHandler			Delegate to call during OnKeyChar event
	 */
	void SetOnKeyCharHandler(FOnKeyChar InOnKeyCharHandler)
	{
		OnKeyCharHandler = MoveTemp(InOnKeyCharHandler);
	}

	/**
	 * Sets the OnKeyDownHandler to provide first chance handling of the OnKeyDown event
	 *
	 * @param InOnKeyDownHandler			Delegate to call during OnKeyDown event
	 */
	void SetOnKeyDownHandler(FOnKeyDown InOnKeyDownHandler)
	{
		OnKeyDownHandler = MoveTemp(InOnKeyDownHandler);
	}

	/** Sets the paint policy to use for text if the font faces are still async loading. */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
	/**
	 *	Force a single scroll operation. 
	 */
	SLATE_API void ForceScroll(int32 UserIndex, float ScrollAxisMagnitude);

protected:
	/** Synchronize the text style currently set (including from overrides) and update the text layout if required */
	SLATE_API void SynchronizeTextStyle();

protected:
	//~ Begin SWidget Interface
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual void CacheDesiredSize(float LayoutScaleMultiplier) override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual FReply OnKeyChar( const FGeometry& MyGeometry,const FCharacterEvent& InCharacterEvent ) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	SLATE_API virtual bool IsInteractable() const override;
	SLATE_API virtual bool ComputeVolatility() const override;
	//~ End SWidget Interface

protected:
	SLATE_API void OnHScrollBarMoved(const float InScrollOffsetFraction);
	SLATE_API void OnVScrollBarMoved(const float InScrollOffsetFraction);

	/** Return whether a RMB+Drag scroll operation is taking place */
	SLATE_API bool IsRightClickScrolling() const;

public:
	//~ Begin ISlateEditableTextWidget Interface
	SLATE_API virtual bool IsWidgetConstructed() const override;
	SLATE_API virtual bool IsTextReadOnly() const override;
	SLATE_API virtual bool IsTextPassword() const override;
	SLATE_API virtual bool IsMultiLineTextEdit() const override;
	SLATE_API virtual bool IsIntegratedKeyboardEnabled() const override;
	//~ End ISlateEditableTextWidget Interface

protected:
	//~ Begin ISlateEditableTextWidget Interface
	SLATE_API virtual bool ShouldJumpCursorToEndWhenFocused() const override;
	SLATE_API virtual bool ShouldSelectAllTextWhenFocused() const override;
	SLATE_API virtual bool ShouldClearTextSelectionOnFocusLoss() const override;
	SLATE_API virtual bool ShouldRevertTextOnEscape() const override;
	SLATE_API virtual bool ShouldClearKeyboardFocusOnCommit() const override;
	SLATE_API virtual bool ShouldSelectAllTextOnCommit() const override;
	SLATE_API virtual bool ShouldSelectWordOnMouseDoubleClick() const override;
	SLATE_API virtual bool CanInsertCarriageReturn() const override;
	SLATE_API virtual bool CanTypeCharacter(const TCHAR InChar) const override;
	SLATE_API virtual void EnsureActiveTick() override;
	SLATE_API virtual EKeyboardType GetVirtualKeyboardType() const override;
	SLATE_API virtual FVirtualKeyboardOptions GetVirtualKeyboardOptions() const override;
	SLATE_API virtual EVirtualKeyboardTrigger GetVirtualKeyboardTrigger() const override;
	SLATE_API virtual EVirtualKeyboardDismissAction GetVirtualKeyboardDismissAction() const override;
	SLATE_API virtual TSharedRef<SWidget> GetSlateWidget() override;
	SLATE_API virtual TSharedPtr<SWidget> GetSlateWidgetPtr() override;
	SLATE_API virtual TSharedPtr<SWidget> BuildContextMenuContent() const override;
	SLATE_API virtual void OnBeginTextEdit(const FText& InText) override;
	SLATE_API virtual void OnTextChanged(const FText& InText) override;
	SLATE_API virtual void OnTextCommitted(const FText& InText, const ETextCommit::Type InTextAction) override;
	SLATE_API virtual void OnCursorMoved(const FTextLocation& InLocation) override;
	SLATE_API virtual float UpdateAndClampHorizontalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride) override;
	SLATE_API virtual float UpdateAndClampVerticalScrollBar(const float InViewOffset, const float InViewFraction, const EVisibility InVisibilityOverride) override;
	//~ End ISlateEditableTextWidget Interface

protected:
	/** Called to begin an undoable editable text transaction, marked as protected for use with FScopedEditableTextTransaction only */
	SLATE_API void BeginEditTransaction();

	/** Called to end an undoable editable text transaction, marked as protected for use with FScopedEditableTextTransaction only */
	SLATE_API void EndEditTransaction();

protected:
	TSlateAttributeRef<FSlateFontInfo> GetFontAttribute() const
	{
		return TSlateAttributeRef<FSlateFontInfo>{SharedThis(this), FontAttribute};
	}

protected:
#if WITH_EDITORONLY_DATA
	/** The text layout that deals with the editable text */
	UE_DEPRECATED(5.8, "Use SlateEditableTextLayout, this variable will never be set.")
	TUniquePtr<FSlateEditableTextLayout> EditableTextLayout;
#endif
	/** The text layout that deals with the editable text */
	FSlateEditableTextLayout SlateEditableTextLayout;

	/** Whether to allow multi-line text */
	TAttribute<bool> bAllowMultiLine;

	/** Whether to select all text when the user clicks to give focus on the widget */
	TAttribute<bool> bSelectAllTextWhenFocused;

	/** Whether to clear text selection when focus is lost */
	TAttribute<bool> bClearTextSelectionOnFocusLoss;

	/** Whether to select work on mouse double click */
	TAttribute<bool> bSelectWordOnMouseDoubleClick;

	/** True if any changes should be reverted if we receive an escape key */
	TAttribute<bool> bRevertTextOnEscape;

	/** True if we want the text control to lose focus on a text commit/revert events */
	TAttribute<bool> bClearKeyboardFocusOnCommit;

	/** Whether the context menu can be opened */
	TAttribute<bool> bAllowContextMenu;

#if WITH_EDITORONLY_DATA
	/** Sets whether this text box can actually be modified interactively by the user */
	UE_DEPRECATED(5.8, "Use SetIsReadOnly / IsTextReadOnly")
	TSlateDeprecatedTAttribute<bool> bIsReadOnly;
#endif

	/** Delegate to call before a context menu is opened */
	FOnContextMenuOpening OnContextMenuOpening;

	/** Called when a character is typed and we want to know if the text field supports typing this character. */
	FOnIsTypedCharValid OnIsTypedCharValid;

	/** Called when the text starts to be edited */
	FOnBeginTextEdit OnBeginTextEditCallback;

	/** Called whenever the text is changed programmatically or interactively by the user */
	FOnTextChanged OnTextChangedCallback;

	/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
	FOnTextCommitted OnTextCommittedCallback;

	/** Called when the cursor is moved within the text area */
	FOnCursorMoved OnCursorMovedCallback;

	/** The horizontal scroll bar widget */
	TSharedPtr<SScrollBar> HScrollBar;

	/** The vertical scroll bar widget */
	TSharedPtr<SScrollBar> VScrollBar;

	/** Called whenever the horizontal scrollbar is moved by the user */
	FOnUserScrolled OnHScrollBarUserScrolled;

	/** Called whenever the vertical scrollbar is moved by the user */
	FOnUserScrolled OnVScrollBarUserScrolled;

	/** Menu extender for right-click context menu */
	TSharedPtr<FExtender> MenuExtender;

	/** The optional modifier key necessary to create a newline when typing into the editor. */
	EModifierKey::Type ModiferKeyForNewLine;

	/** The timer that is actively driving this widget to Tick() even when Slate is idle */
	TWeakPtr<FActiveTimerHandle> ActiveTickTimer;

	/** How much we scrolled while RMB was being held */
	float AmountScrolledWhileRightMouseDown;

	/** Whether a software cursor is currently active */
	bool bIsSoftwareCursor;

	/**	The current position of the software cursor */
	FVector2f SoftwareCursorPosition;

	/** Callback delegate to have first chance handling of the OnKeyChar event */
	FOnKeyChar OnKeyCharHandler;

	/** Callback delegate to have first chance handling of the OnKeyDown event */
	FOnKeyDown OnKeyDownHandler;

	/** Options to use for the virtual keyboard summoned by this widget */
	FVirtualKeyboardOptions VirtualKeyboardOptions;

	/** The type of event that will trigger the display of the virtual keyboard */
	TAttribute<EVirtualKeyboardTrigger> VirtualKeyboardTrigger;

	/** The message action to take when the virtual keyboard is dismissed by the user */
	TAttribute<EVirtualKeyboardDismissAction> VirtualKeyboardDismissAction;

private:
	/** The font used to draw the text */
	TSlateAttribute<FSlateFontInfo> FontAttribute;
	bool bIsFontAttrSet = false;
};

#endif //WITH_FANCY_TEXT
