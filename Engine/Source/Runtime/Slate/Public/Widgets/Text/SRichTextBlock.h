// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "SlateGlobals.h"
#include "Layout/Margin.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/ITextDecorator.h"
#include "Framework/Text/TextDecorators.h"
#include "Framework/Text/SlateTextLayoutFactory.h"

class FArrangedChildren;
class FPaintArgs;
class FRichTextLayoutMarshaller;
class FSlateWindowElementList;
class FSlateTextBlockLayout;
class IRichTextMarkupParser;
enum class ETextShapingMethod : uint8;

#if WITH_FANCY_TEXT

/**
 * A rich static text widget. 
 * Through the use of markup and text decorators, text with different styles, embedded image and widgets can be achieved.
 */
class SRichTextBlock : public SWidget
{
public:

	SLATE_BEGIN_ARGS( SRichTextBlock )
		: _Text()
		, _HighlightText()
		, _WrapTextAt( 0.0f )
		, _AutoWrapText(false)
		, _WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
		, _TransformPolicy(ETextTransformPolicy::None)
		, _Marshaller()
		, _DecoratorStyleSet( &FCoreStyle::Get() )
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		, _Margin( FMargin() )
		, _LineHeightPercentage( 1.0f )
		, _ApplyLineHeightToBottomLine( true )
		, _Justification( ETextJustify::Left )
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _Decorators()
		, _Parser()
		, _OverflowPolicy()
		, _MinDesiredWidth()
		, _FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
		/** The text displayed in this text block */
		SLATE_ATTRIBUTE( FText, Text )

		/** Highlight this text in the text block */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
		SLATE_ATTRIBUTE( float, WrapTextAt )

		/** Whether to wrap text automatically based on the widget's computed horizontal space.  IMPORTANT: Using automatic wrapping can result
		    in visual artifacts, as the the wrapped size will computed be at least one frame late!  Consider using WrapTextAt instead.  The initial 
			desired size will not be clamped.  This works best in cases where the text block's size is not affecting other widget's layout. */
		SLATE_ATTRIBUTE( bool, AutoWrapText )

		/** The wrapping policy to use */
		SLATE_ATTRIBUTE( ETextWrappingPolicy, WrappingPolicy )

		/** The transform policy to use */
		SLATE_ATTRIBUTE( ETextTransformPolicy, TransformPolicy )

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr<class FRichTextLayoutMarshaller>, Marshaller)

		/** Delegate used to create text layouts for this widget. If none is provided then FSlateTextLayout will be used. */
		SLATE_EVENT(FCreateSlateTextLayout, CreateSlateTextLayout)

		/** The style set used for looking up styles used by decorators*/
		SLATE_ARGUMENT( const ISlateStyle*, DecoratorStyleSet )

		/** The style of the text block, which dictates the default font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )

		/** The amount of blank space left around the edges of text area. */
		SLATE_ATTRIBUTE( FMargin, Margin )

		/** The amount to scale each lines height by. */
		SLATE_ATTRIBUTE( float, LineHeightPercentage )

		/** Whether to leave extra space below the last line due to line height. */
		SLATE_ATTRIBUTE( bool, ApplyLineHeightToBottomLine )

		/** How the text should be aligned with the margin. */
		SLATE_ATTRIBUTE( ETextJustify::Type, Justification )

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		
		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

		/** Any decorators that should be used while parsing the text. */
		SLATE_ARGUMENT( TArray< TSharedRef< class ITextDecorator > >, Decorators )

		/** The parser used to resolve any markup used in the provided string. */
		SLATE_ARGUMENT( TSharedPtr< class IRichTextMarkupParser >, Parser )

		/** Determines what happens to text that is clipped and doesn't fit within the clip rect for this widget */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

		/** Minimum width that this text block should be */
		SLATE_ATTRIBUTE(float, MinDesiredWidth)

		/** The paint policy to use for text if the font faces are still async loading. */
		SLATE_ARGUMENT(EFontFacesLoadingPaintPolicy, FontFacesLoadingPaintPolicy)

		/** Called when all font faces for the text have finished loading asynchronously */
		SLATE_EVENT(FSimpleDelegate, OnAllFontFacesFinishLoading)

		/** Additional decorators can be append to the widget inline. Inline decorators get precedence over decorators not specified inline. */
		TArray< TSharedRef< ITextDecorator > > InlineDecorators;
		SRichTextBlock::FArguments& operator + (const TSharedRef< ITextDecorator >& DecoratorToAdd)
		{
			InlineDecorators.Add( DecoratorToAdd );
			return *this;
		}

	SLATE_END_ARGS()

	static TSharedRef< ITextDecorator > Decorator( const TSharedRef< ITextDecorator >& Decorator )
	{
		return Decorator;
	}

	static TSharedRef< ITextDecorator > WidgetDecorator( const FString RunName, const FWidgetDecorator::FCreateWidget& FactoryDelegate )
	{
		return FWidgetDecorator::Create( RunName, FactoryDelegate );
	}

	template< class UserClass >
	static TSharedRef< ITextDecorator >  WidgetDecorator( const FString RunName, UserClass* InUserObjectPtr, typename FWidgetDecorator::FCreateWidget::TConstMethodPtr< UserClass > InFunc )
	{
		return FWidgetDecorator::Create( RunName, FWidgetDecorator::FCreateWidget::CreateSP( InUserObjectPtr, InFunc ) );
	}

	static TSharedRef< ITextDecorator > ImageDecorator( FString RunName = TEXT("img"), const ISlateStyle* const OverrideStyle = NULL )
	{
		return FImageDecorator::Create( RunName, OverrideStyle );
	}

	static TSharedRef< ITextDecorator > HyperlinkDecorator( const FString Id, const FSlateHyperlinkRun::FOnClick& NavigateDelegate )
	{
		return FHyperlinkDecorator::Create( Id, NavigateDelegate );
	}

	template< class UserClass >
	static TSharedRef< ITextDecorator > HyperlinkDecorator( const FString Id, UserClass* InUserObjectPtr, typename FSlateHyperlinkRun::FOnClick::TMethodPtr< UserClass > NavigateFunc )
	{
		return FHyperlinkDecorator::Create( Id, FSlateHyperlinkRun::FOnClick::CreateSP( InUserObjectPtr, NavigateFunc ) );
	}

	/** Constructor */
	SLATE_API SRichTextBlock();

	/** Destructor */
	SLATE_API ~SRichTextBlock();

	//~ Begin SWidget Interface
	SLATE_API void Construct( const FArguments& InArgs );
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FChildren* GetChildren() override;
	SLATE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	//~ End SWidget Interface

	/**
	 * Gets the text assigned to this text block
	 */
	const FText& GetText() const
	{
		return BoundText.Get(FText::GetEmpty());
	}

	/**
	 * Sets the text for this text block
	 */
	SLATE_API void SetText( const TAttribute<FText>& InTextAttr );

	/** See HighlightText attribute */
	SLATE_API void SetHighlightText( const TAttribute<FText>& InHighlightText );

	/** See TextShapingMethod attribute */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** See TextFlowDirection attribute */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** See WrapTextAt attribute */
	SLATE_API void SetWrapTextAt(const TAttribute<float>& InWrapTextAt);

	/** See AutoWrapText attribute */
	SLATE_API void SetAutoWrapText(const TAttribute<bool>& InAutoWrapText);

	/** Set WrappingPolicy attribute */
	SLATE_API void SetWrappingPolicy(const TAttribute<ETextWrappingPolicy>& InWrappingPolicy);

	/** Set TransformPolicy attribute */
	SLATE_API void SetTransformPolicy(const TAttribute<ETextTransformPolicy>& InTransformPolicy);

	/** See LineHeightPercentage attribute */
	SLATE_API void SetLineHeightPercentage(const TAttribute<float>& InLineHeightPercentage);

	/** See ApplyLineHeightToBottomLine attribute */
	SLATE_API void SetApplyLineHeightToBottomLine(const TAttribute<bool>& InApplyLineHeightToBottomLine);

	/** See Margin attribute */
	SLATE_API void SetMargin(const TAttribute<FMargin>& InMargin);

	/** See Justification attribute */
	SLATE_API void SetJustification(const TAttribute<ETextJustify::Type>& InJustification);

	/** See TextStyle argument */
	SLATE_API void SetTextStyle(const FTextBlockStyle& InTextStyle);

	/** See MinDesiredWidth attribute */
	SLATE_API void SetMinDesiredWidth(const TAttribute<float>& InMinDesiredWidth);

	/**  */
	SLATE_API void SetDecoratorStyleSet(const ISlateStyle* NewDecoratorStyleSet);

	/** Replaces the decorators for this text block */
	SLATE_API void SetDecorators(TArrayView<TSharedRef<ITextDecorator>> InDecorators);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy);

	/**
	 * Causes the text to reflow it's layout
	 */
	SLATE_API void Refresh();

	/** set the scale value at the TextLayout*/
	SLATE_API void SetTextBlockScale(const float NewTextBlockScale);

	/** Set the paint policy for the text layout if font faces are still async loading.*/
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
protected:
	//~ SWidget interface
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATE_API virtual bool ComputeVolatility() const override;
	//~ End of SWidget interface

private:

	/** The text displayed in this text block */
	TAttribute< FText > BoundText;

	/** The wrapped layout for this text block */
	TUniquePtr< FSlateTextBlockLayout > TextLayoutCache;

	/** Default style used by the TextLayout */
	FTextBlockStyle TextStyle;

	/** Highlight this text in the textblock */
	TAttribute<FText> HighlightText;

	/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
	TAttribute<float> WrapTextAt;
	
	/** True if we're wrapping text automatically based on the computed horizontal space for this widget */
	TAttribute<bool> AutoWrapText;

	/** The wrapping policy we're using */
	TAttribute<ETextWrappingPolicy> WrappingPolicy;

	/** The transform policy we're using */
	TAttribute<ETextTransformPolicy> TransformPolicy;

	/** The amount of blank space left around the edges of text area. */
	TAttribute< FMargin > Margin;

	/** The amount to scale each lines height by. */
	TAttribute< ETextJustify::Type > Justification; 

	/** How the text should be aligned with the margin. */
	TAttribute< float > LineHeightPercentage;

	/** Whether to leave extra space below the last line due to line height. */
	TAttribute< bool > ApplyLineHeightToBottomLine;

	/** Prevents the text block from being smaller than desired in certain cases (e.g. when it is empty) */
	TAttribute<float> MinDesiredWidth;

	/** Use to Scale the entire Text Block*/
	float TextBlockScale = 1.0f;

	/**  */
	TSharedPtr<FRichTextLayoutMarshaller> Marshaller;
};

#endif //WITH_FANCY_TEXT
