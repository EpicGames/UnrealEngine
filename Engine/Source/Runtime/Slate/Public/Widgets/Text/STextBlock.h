// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"

class FPaintArgs;
class FSlateWindowElementList;
class FSlateTextBlockLayout;
class IBreakIterator;
enum class ETextShapingMethod : uint8;


namespace ETextRole
{
	enum Type
	{
		Custom,
		ButtonText,
		ComboText
	};
}

template <>
struct TWidgetTypeTraits<class STextBlock>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A simple static text widget
 */
class STextBlock : public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(STextBlock, SLeafWidget, SLATE_API)

public:

	SLATE_BEGIN_ARGS( STextBlock )
		: _Text()
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		, _Font()
		, _StrikeBrush()
		, _ColorAndOpacity()
		, _ShadowOffset()
		, _ShadowColorAndOpacity()
		, _HighlightColor()
		, _HighlightShape()
		, _HighlightText()
		, _WrapTextAt(0.0f)
		, _AutoWrapText(false)
		, _WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
		, _TransformPolicy()
		, _Margin()
		, _LineHeightPercentage(1.0f)
		, _ApplyLineHeightToBottomLine(true)
		, _Justification(ETextJustify::Left)
		, _MinDesiredWidth(0.0f)
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _LineBreakPolicy()
		, _OverflowPolicy()
		, _SimpleTextMode(false)
		, _FontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy::DoNotPaint)
		{
			_Clipping = EWidgetClipping::OnDemand;
		}

		/** The text displayed in this text block */
		SLATE_ATTRIBUTE( FText, Text )

		/** Pointer to a style of the text block, which dictates the font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )

		/** Sets the font used to draw the text */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Sets the brush used to strike through the text */
		SLATE_ATTRIBUTE( const FSlateBrush*, StrikeBrush )

		/** Text color and opacity */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Drop shadow offset in pixels */
		SLATE_ATTRIBUTE( FVector2D, ShadowOffset )

		/** Shadow color and opacity */
		SLATE_ATTRIBUTE( FLinearColor, ShadowColorAndOpacity )

		/** The color used to highlight the specified text */
		SLATE_ATTRIBUTE( FLinearColor, HighlightColor )

		/** The brush used to highlight the specified text */
		SLATE_ATTRIBUTE( const FSlateBrush*, HighlightShape )

		/** Highlight this text in the text block */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
		SLATE_ATTRIBUTE( float, WrapTextAt )

		/** Whether to wrap text automatically based on the widget's computed horizontal space.  IMPORTANT: Using automatic wrapping can result
		    in visual artifacts, as the the wrapped size will computed be at least one frame late!  Consider using WrapTextAt instead.  The initial 
			desired size will not be clamped.  This works best in cases where the text block's size is not affecting other widget's layout. */
		SLATE_ATTRIBUTE( bool, AutoWrapText )

		/** The wrapping policy to use */
		SLATE_ATTRIBUTE( ETextWrappingPolicy, WrappingPolicy )

		/** The transform policy to use */
		SLATE_ATTRIBUTE( ETextTransformPolicy, TransformPolicy )

		/** The amount of blank space left around the edges of text area. */
		SLATE_ATTRIBUTE( FMargin, Margin )

		/** The amount to scale each lines height by. */
		SLATE_ATTRIBUTE( float, LineHeightPercentage )

		/** Whether to leave extra space below the last line due to line height. */
		SLATE_ATTRIBUTE( bool, ApplyLineHeightToBottomLine )

		/** How the text should be aligned with the margin. */
		SLATE_ATTRIBUTE( ETextJustify::Type, Justification )

		/** Minimum width that a text block should be */
		SLATE_ATTRIBUTE( float, MinDesiredWidth )

		/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		
		/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

		/** The iterator to use to detect appropriate soft-wrapping points for lines (or null to use the default) */
		SLATE_ARGUMENT( TSharedPtr<IBreakIterator>, LineBreakPolicy )

		/** Determines what happens to text that is clipped and doesn't fit within the clip rect for this widget */
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

		/**
		 * If this is enabled, text shaping, wrapping, justification are disabled in favor of much faster text layout and measurement.
		 * This feature is suitable for numbers and text that changes often and impact performance.
		 * Enabling this setting may cause certain languages (such as Right to left languages) to not display properly.
		 */
		SLATE_ARGUMENT( bool, SimpleTextMode )

		/** Called when this text is double clicked */
		SLATE_EVENT(FPointerEventHandler, OnDoubleClicked)

		/** The paint policy to use for text if the font faces are still async loading. */
		SLATE_ARGUMENT(EFontFacesLoadingPaintPolicy, FontFacesLoadingPaintPolicy)

		/** Called when all font faces for the text have finished loading asynchronously */
		SLATE_EVENT(FSimpleDelegate, OnAllFontFacesFinishLoading)
	SLATE_END_ARGS()

	/** Constructor */
	SLATE_API STextBlock();

	/** Destructor */
	SLATE_API ~STextBlock();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATE_API void Construct( const FArguments& InArgs );

	/**
	 * Gets the text assigned to this text block
	 *
	 * @return	This text block's text string
	 */
	const FText& GetText() const
	{
		if (bIsAttributeBoundTextBound)
		{
			STextBlock& MutableSelf = const_cast<STextBlock&>(*this);
			MutableSelf.BoundText.UpdateNow(MutableSelf);
		}
		return BoundText.Get();
	}
	
public:
	/** Sets the text for this text block */
	SLATE_API void SetText(TAttribute<FText> InText);

	/** Sets the highlight text for this text block */
	SLATE_API void SetHighlightText(TAttribute<FText> InText);

	/** Sets the font used to draw the text	*/
	SLATE_API void SetFont(TAttribute<FSlateFontInfo> InFont);

	/** Sets the brush used to strike through the text */
	SLATE_API void SetStrikeBrush(TAttribute<const FSlateBrush*> InStrikeBrush);

	/** See ColorAndOpacity attribute */
	SLATE_API void SetColorAndOpacity(TAttribute<FSlateColor> InColorAndOpacity);

	/** See TextStyle argument */
	SLATE_API void SetTextStyle(const FTextBlockStyle* InTextStyle, bool bUpdateTextLayout = false);

	/** See TextShapingMethod attribute */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** See TextFlowDirection attribute */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** See WrapTextAt attribute */
	SLATE_API void SetWrapTextAt(TAttribute<float> InWrapTextAt);

	/** See AutoWrapText attribute */
	SLATE_API void SetAutoWrapText(TAttribute<bool> InAutoWrapText);

	/** Set WrappingPolicy attribute */
	SLATE_API void SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy);

	/** Set TransformPolicy attribute */
	SLATE_API void SetTransformPolicy(TAttribute<ETextTransformPolicy> InTransformPolicy);

	/** Sets the overflow policy for this text block */
	SLATE_API void SetOverflowPolicy(TOptional<ETextOverflowPolicy> InOverflowPolicy);

	/** See ShadowOffset attribute */
	SLATE_API void SetShadowOffset(TAttribute<FVector2D> InShadowOffset);

	/** See ShadowColorAndOpacity attribute */
	SLATE_API void SetShadowColorAndOpacity(TAttribute<FLinearColor> InShadowColorAndOpacity);

	/** See HighlightColor attribute */
	SLATE_API void SetHighlightColor(TAttribute<FLinearColor> InHighlightColor);
	
	/** See HighlightShape attribute */
	SLATE_API void SetHighlightShape(TAttribute<const FSlateBrush*> InHighlightShape);

	/** See MinDesiredWidth attribute */
	SLATE_API void SetMinDesiredWidth(TAttribute<float> InMinDesiredWidth);

	/** See LineHeightPercentage attribute */
	SLATE_API void SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage);

	/** See ApplyLineHeightToBottomLine attribute */
	SLATE_API void SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine);

	/** See Margin attribute */
	SLATE_API void SetMargin(TAttribute<FMargin> InMargin);

	/** See Justification attribute */
	SLATE_API void SetJustification(TAttribute<ETextJustify::Type> InJustification);

	/** See FontFacesLoadingPaintPolicy argument */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
	// SWidget interface
	SLATE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATE_API virtual FVector2D ComputeDesiredSize(float) const override;
#if WITH_ACCESSIBILITY
	SLATE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
	SLATE_API virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType = EAccessibleType::Main) const override;
#endif
	// End of SWidget interface

public:
	/** Gets the current foreground color */
	SLATE_API FSlateColor GetColorAndOpacity() const;

	/** Gets the current foreground color as ref */
	SLATE_API const FSlateColor& GetColorAndOpacityRef() const;

	/** Gets the current font */
	SLATE_API FSlateFontInfo GetFont() const;

	/** Gets the current font as ref */
	SLATE_API const FSlateFontInfo& GetFontRef() const;

	/** Gets the current strike brush */
	SLATE_API const FSlateBrush* GetStrikeBrush() const;

	/** Get TransformPolicy attribute */
	SLATE_API ETextTransformPolicy GetTransformPolicyImpl() const;
	
	/** Gets the current shadow offset */
	SLATE_API FVector2f GetShadowOffset() const;

	/** Gets the current shadow color and opacity */
	SLATE_API FLinearColor GetShadowColorAndOpacity() const;

	/** Gets the current shadow color and opacity as ref */
	SLATE_API const FLinearColor& GetShadowColorAndOpacityRef() const;

	/** Gets the current highlight color */
	SLATE_API FSlateColor GetHighlightColor() const;

	/** Gets the current highlight shape */
	SLATE_API const FSlateBrush* GetHighlightShape() const;

	/** Gets the current margin */
	SLATE_API FMargin GetMargin() const;

	/** Gets the current min desired width */
	SLATE_API float GetMinDesiredWidth() const;

private:
	/** Get the computed text style to use with the text marshaller */
	FTextBlockStyle GetComputedTextStyle() const;

	/** Update the TextBlock layout. */
	void UpdateTextBlockLayout(float LayoutScaleMultiplier) const;

	/** Call to invalidate this text block */
	void InvalidateText(EInvalidateWidgetReason InvalidateReason);

private:
	/** The text displayed in this text block */
	TSlateAttribute<FText> BoundText;

	/** The wrapped layout for this text block */
	TUniquePtr< FSlateTextBlockLayout > TextLayoutCache;

	/** Default style used by the TextLayout */
	FTextBlockStyle TextStyle;

	/** Sets the font used to draw the text */
	TSlateAttribute<FSlateFontInfo> Font;

	/** Sets the brush used to strike through the text */
	TSlateAttribute<const FSlateBrush*> StrikeBrush;

	/** Text color and opacity */
	TSlateAttribute<FSlateColor> ColorAndOpacity;

	/** Drop shadow offset in pixels */
	TSlateAttribute<FVector2D> ShadowOffset;

	/** Shadow color and opacity */
	TSlateAttribute<FLinearColor> ShadowColorAndOpacity;

	/** The color used to highlight the specified text */
	TSlateAttribute<FLinearColor> HighlightColor;

	/** The brush used to highlight the specified text */
	TSlateAttribute<const FSlateBrush*> HighlightShape;

	/** Highlight this text in the TextBlock */
	TSlateAttribute<FText> HighlightText;

	/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
	TSlateAttribute<float> WrapTextAt;

	/** True if we're wrapping text automatically based on the computed horizontal space for this widget */
	TSlateAttribute<bool> AutoWrapText;

	/** The wrapping policy we're using */
	TSlateAttribute<ETextWrappingPolicy> WrappingPolicy;

	/** The transform policy we're using */
	TSlateAttribute<ETextTransformPolicy> TransformPolicy;

	/** The amount of blank space left around the edges of text area. */
	TSlateAttribute<FMargin> Margin;

	/** The amount to scale each lines height by. */
	TSlateAttribute<ETextJustify::Type> Justification;

	/** How the text should be aligned with the margin. */
	TSlateAttribute<float> LineHeightPercentage;

	/** Whether to leave extra space below the last line due to line height. */
	TSlateAttribute<bool> ApplyLineHeightToBottomLine;

	/** Prevents the text block from being smaller than desired in certain cases (e.g. when it is empty) */
	TSlateAttribute<float> MinDesiredWidth;

	/** If this is enabled, text shaping, wrapping, justification are disabled in favor of much faster text layout and measurement. */
	mutable TOptional<FVector2f> CachedSimpleDesiredSize;

	/** Flags used to check if the SlateAttribute is set. */
	union
	{
		struct 
		{
			//~ for attribute
			uint16 bIsAttributeBoundTextBound : 1;
			uint16 bIsAttributeFontSet : 1;
			uint16 bIsAttributeStrikeBrushSet : 1;
			uint16 bIsAttributeColorAndOpacitySet : 1;
			uint16 bIsAttributeShadowOffsetSet : 1;
			uint16 bIsAttributeShadowColorAndOpacitySet : 1;
			uint16 bIsAttributeHighlightColorSet : 1;
			uint16 bIsAttributeHighlightShapeSet : 1;
			uint16 bIsAttributeWrapTextAtSet : 1;
			uint16 bIsAttributeTransformPolicySet : 1;
			//~ for TextBlockLayout
			mutable uint16 bTextLayoutUpdateTextStyle : 1;
			mutable uint16 bTextLayoutUpdateDesiredSize : 1;
		};
		uint16 Union_Flags;
	};

	bool bSimpleTextMode;
};
