// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Clipping.h"
#include "Widgets/SCompoundWidget.h"

struct FTextScrollerOptions
{
	/** Speed (in Slate units) to scroll per-second */
	float Speed = 25.0f;

	/** Delay (in seconds) to pause before scrolling the text */
	float StartDelay = 0.5f;

	/** Delay (in seconds) to pause after scrolling the text */
	float EndDelay = 0.5f;

	/** Duration (in seconds) to fade in before scrolling the text */
	float FadeInDelay = 0.0f;

	/** Duration (in seconds) to fade out after scrolling the text */
	float FadeOutDelay = 0.0f;
};

/**
 * Utility to wrap a single-line text widget (STextBlock or SRichTextBlock) and provide support for auto-scrolling the text if it's longer than the available space.
 */
class STextScroller : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextScroller)
		: _ScrollOrientation(Orient_Horizontal)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
		/** Options to apply when scrolling the text */
		SLATE_ARGUMENT(FTextScrollerOptions, ScrollOptions)
		/** Orientation this text will scroll if needed */
		SLATE_ARGUMENT(EOrientation, ScrollOrientation)
		/** Slot for the text widget to be scrolled */
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

public:
	SLATE_API void Construct(const FArguments& InArgs);

	SLATE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	SLATE_API void ResetScrollState();
	SLATE_API void StartScrolling();
	SLATE_API void SuspendScrolling();

	bool IsScrollingEnabled() const { return ActiveState != EActiveState::Suspend; }

private:
	SLATE_API EActiveTimerReturnType OnScrollTextTick(double CurrentTime, float DeltaTime);

private:
	enum class EActiveState : uint8
	{
		FadeIn = 0,
		Start,
		StartWait,
		Scrolling,
		Stop,
		StopWait,
		FadeOut,
		Suspend,
	};

	enum class ETickerState : uint8
	{
		None = 0,
		StartTicking,
		Ticking,
		StopTicking,
	};

	EActiveState ActiveState = EActiveState::Start;
	ETickerState TickerState = ETickerState::None;
	float TimeElapsed = 0.0f;
	float ScrollOffset = 0.0f;
	float FontAlpha = 1.0f;

	TSharedPtr<FActiveTimerHandle> ActiveTimerHandle;

	FTextScrollerOptions ScrollOptions;
	EOrientation ScrollOrientation = Orient_Horizontal;
};
