// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Framework/Application/IMenu.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "Widgets/Text/ISlateEditableTextWidget.h"
#include "Framework/Text/ITextLayoutMarshaller.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Widgets/Text/SlateEditableTextTypes.h"
#include "Framework/Text/SlateTextLayoutFactory.h"
#include "GenericPlatform/ITextInputMethodSystem.h"
#include "Types/SlateNestedAttribute.h"

class FArrangedChildren;
class FExtender;
class FPaintArgs;
class FSlateWindowElementList;
class FSlateTextBlockLayout;
class FUICommandList;
class IBreakIterator;
class ISlateEditableTextWidget;
class SWindow;
enum class ETextShapingMethod : uint8;
enum class EFontFacesLoadingPaintPolicy : uint8;

/** Class to handle the cached layout of SEditableText/SMultiLineEditableText by proxying around a FTextLayout */
class FSlateEditableTextLayout : public FSlateNestedAttributeContainer
{
public:
	template<auto InPointerToAttributeOwner>
	static void RegisterNestedAttributes(FSlateAttributeInitializer& AttributeInitializer)
	{
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, BoundText, EInvalidateWidgetReason::Layout)
			.OnValueChanged_Static(OnBoundTextChanged);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, HintText, EInvalidateWidgetReason::Layout);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, HintTextOpacity, EInvalidateWidgetReason::Paint);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, BoundSearchText, EInvalidateWidgetReason::Layout)
			.OnValueChanged_Static(OnBoundSearchTextChanged);

		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, bIsReadOnlyAttribute, EInvalidateWidgetReason::PaintAndVolatility)
			.OnValueChanged_Static(OnIsReadOnlyChanged);

		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, WrapTextAt, EInvalidateWidgetReason::Layout);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, AutoWrapText, EInvalidateWidgetReason::Layout);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, WrappingPolicy, EInvalidateWidgetReason::Layout);

		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, Margin, EInvalidateWidgetReason::Layout);

		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, Justification, EInvalidateWidgetReason::Layout);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, LineHeightPercentage, EInvalidateWidgetReason::Layout);
		SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToAttributeOwner, ApplyLineHeightToBottomLine, EInvalidateWidgetReason::Layout);
	}

	template<typename RealWidgetType>
	UE_DEPRECATED(5.8, "Use the other base constructor and call Construct during widget construction")
	FSlateEditableTextLayout(RealWidgetType& InOwnerWidget, TAttribute<FText> InInitialText, FTextBlockStyle InTextStyle, const TOptional<ETextShapingMethod> InTextShapingMethod, const TOptional<ETextFlowDirection> InTextFlowDirection, const FCreateSlateTextLayout& InCreateSlateTextLayout, TSharedRef<ITextLayoutMarshaller> InTextMarshaller, TSharedRef<ITextLayoutMarshaller> InHintTextMarshaller)
		: FSlateEditableTextLayout(InOwnerWidget)
	{
		Construct(MoveTemp(InInitialText), MoveTemp(InTextStyle), InTextShapingMethod, InTextFlowDirection, InCreateSlateTextLayout, MoveTemp(InTextMarshaller), MoveTemp(InHintTextMarshaller));
	}


	template<typename RealWidgetType>
	FSlateEditableTextLayout(RealWidgetType& InOwnerWidget)
		: OwnerWidget(&InOwnerWidget)
		, BoundText(InOwnerWidget, FText::GetEmpty())
		, bWasPasswordLastTick(false)
		, HintText(InOwnerWidget, FText::GetEmpty())
		, HintTextOpacity(InOwnerWidget, 0.35f)
		, BoundSearchText(InOwnerWidget, FText::GetEmpty())
		, SearchCase(ESearchCase::IgnoreCase)
		, CurrentSearchResultIndex(0)
		, bIsReadOnlyAttribute(InOwnerWidget, false)
		, WrapTextAt(InOwnerWidget, 0.0f)
		, AutoWrapText(InOwnerWidget, false)
		, WrappingPolicy(InOwnerWidget, ETextWrappingPolicy::DefaultWrapping)
		, Margin(InOwnerWidget, FMargin(0))
		, Justification(InOwnerWidget, ETextJustify::Left)
		, LineHeightPercentage(InOwnerWidget, 1.0f)
		, ApplyLineHeightToBottomLine(InOwnerWidget, false)
		, bHasRegisteredTextInputMethodContext(false)
		, PreferredCursorScreenOffsetInLine(0)
		, CurrentUndoLevel(0)
		, NumTransactionsOpened(0)
		, bIsDragSelecting(false)
		, bWasFocusedByLastMouseDown(false)
		, bHasDragSelectedSinceFocused(false)
		, bTextChangedByVirtualKeyboard(false)
		, bTextCommittedByVirtualKeyboard(false)
		, VirtualKeyboardTextCommitType()
		, bSelectionChangedExternally(false)
		, bForceBoundTextReview(false)
		, ExternalSelectionStart(0)
		, ExternalSelectionEnd(0)
	{
		static_assert(std::is_base_of_v<SWidget, RealWidgetType>, "RealWidgetType must be a Widget type.");
		static_assert(std::is_base_of_v<ISlateEditableTextWidget, RealWidgetType>, "RealWidgetType must derive from ISlateEditableTextWidget.");
	}

	SLATE_API ~FSlateEditableTextLayout();

	SLATE_API void Construct(TAttribute<FText> InInitialText, FTextBlockStyle InTextStyle, const TOptional<ETextShapingMethod>& InTextShapingMethod, const TOptional<ETextFlowDirection>& InTextFlowDirection, const FCreateSlateTextLayout& InCreateSlateTextLayout, TSharedRef<ITextLayoutMarshaller> InTextMarshaller, TSharedRef<ITextLayoutMarshaller> InHintTextMarshaller);

	SLATE_API void SetText(TAttribute<FText> InText);
	SLATE_API FText GetText() const;

	SLATE_API void SetHintText(TAttribute<FText> InHintText);
	SLATE_API FText GetHintText() const;

	SLATE_API void SetSearchText(TAttribute<FText> InSearchText);
	SLATE_API FText GetSearchText() const;

	/** Get the index of the search result (0 if none) */
	SLATE_API int32 GetSearchResultIndex() const;

	/** Get the total number of search results (0 if none) */
	SLATE_API int32 GetNumSearchResults() const;

	SLATE_API void SetTextStyle(const FTextBlockStyle& InTextStyle);
	SLATE_API const FTextBlockStyle& GetTextStyle() const;

	SLATE_API void SetHintTextOpacity(TAttribute<float> InOpacity);
	SLATE_API float GetHintTextOpacity() const;

	SLATE_API void SetIsReadOnly(TAttribute<bool> InIsReadOnly);
	SLATE_API bool GetIsReadOnly() const;

	/** Get the number of Text Lines */
	UE_DEPRECATED(5.8, "Use the const version instead.")
	int32 GetTextLineCount()
	{
		return const_cast<const FSlateEditableTextLayout*>(this)->GetTextLineCount();
	}
	/** Get the number of Text Lines */
	SLATE_API int32 GetTextLineCount() const;

	/** Set the brush to use when drawing the cursor */
	SLATE_API void SetCursorBrush(const TAttribute<const FSlateBrush*>& InCursorBrush);

	/** Set the brush to use when drawing the composition highlight */
	SLATE_API void SetCompositionBrush(const TAttribute<const FSlateBrush*>& InCompositionBrush);

	/** Get the plain text string without rich-text formatting */
	SLATE_API FText GetPlainText() const;

	/**
	 * Sets the current editable text for this text block
	 * Note: Doesn't update the value of BoundText, nor does it call OnTextChanged
	 *
	 * @param TextToSet	The new text to set in the internal TextLayout
	 * @param bForce	True to force the update, even if the text currently matches what's in the TextLayout
	 *
	 * @return true if the text was updated, false if the text wasn't update (because it was already up-to-date)
	 */
	SLATE_API bool SetEditableText(const FText& TextToSet, const bool bForce = false);

	/**
	 * Gets the current editable text for this text block
	 * Note: We don't store text in this form (it's stored as lines in the text layout) so every call to this function has to reconstruct it
	 */
	SLATE_API FText GetEditableText() const;

	/** Get the currently selected text */
	SLATE_API FText GetSelectedText() const;

	/** Get the current selection*/
	SLATE_API FTextSelection GetSelection() const;
	
	/** Set the text shaping method that the internal text layout should use */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/** Set the text flow direction that the internal text layout should use */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/** Set the wrapping to use for this document */
	SLATE_API void SetTextWrapping(TAttribute<float> InWrapTextAt, TAttribute<bool> InAutoWrapText, TAttribute<ETextWrappingPolicy> InWrappingPolicy);

	/** Set whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs */
	SLATE_API void SetWrapTextAt(TAttribute<float> InWrapTextAt);

	/** Set whether to wrap text automatically based on the widget's computed horizontal space */
	SLATE_API void SetAutoWrapText(TAttribute<bool> InAutoWrapText);

	/** Set the wrapping policy to use */
	SLATE_API void SetWrappingPolicy(TAttribute<ETextWrappingPolicy> InWrappingPolicy);

	/** Set the amount of blank space left around the edges of text area */
	SLATE_API void SetMargin(TAttribute<FMargin> InMargin);

	/** Set how the text should be aligned with the margin */
	SLATE_API void SetJustification(TAttribute<ETextJustify::Type> InJustification);
	/** Get how the text should be aligned with the margin */
	SLATE_API ETextJustify::Type GetJustification() const;

	/** Set the amount to scale each lines height by */
	SLATE_API void SetLineHeightPercentage(TAttribute<float> InLineHeightPercentage);

	/** Set whether to leave extra space below the last line due to line height */
	SLATE_API void SetApplyLineHeightToBottomLine(TAttribute<bool> InApplyLineHeightToBottomLine);

	/** Set the text overflow policy that should be used to determine what happens to clipped text */
	SLATE_API void SetOverflowPolicy(const TOptional<ETextOverflowPolicy>& InOverflowPolicy);

	/** Set the information used to help identify who owns this text layout in the case of an error */
	SLATE_API void SetDebugSourceInfo(const TAttribute<FString>& InDebugSourceInfo);

	/** Get the virtual keyboard handler for this text layout */
	SLATE_API TSharedRef<IVirtualKeyboardEntry> GetVirtualKeyboardEntry() const;

	/** Get the IME context for this text layout */
	SLATE_API TSharedRef<ITextInputMethodContext> GetTextInputMethodContext() const;

	/** Register and activate the IME context for this text layout */
	SLATE_API void EnableTextInputMethodContext();

	/** Refresh this editable text immediately, rather than wait for the usual caching mechanisms to take effect on the text Tick */
	SLATE_API bool Refresh();

	/** Force the text layout to be updated from the marshaller */
	SLATE_API void ForceRefreshTextLayout(const FText& CurrentText);

	/** Begin a new text search (this is called automatically when BoundSearchText changes) */
	SLATE_API void BeginSearch(const FText& InSearchText, const ESearchCase::Type InSearchCase = ESearchCase::IgnoreCase, const bool InReverse = false);

	/** Advance the current search to the next match (does nothing if not currently searching) */
	SLATE_API void AdvanceSearch(const bool InReverse = false);

	/** Update the horizontal scroll amount from the given fraction */
	SLATE_API UE::Slate::FDeprecateVector2DResult SetHorizontalScrollFraction(const float InScrollOffsetFraction);

	/** Update the vertical scroll amount from the given fraction */
	SLATE_API UE::Slate::FDeprecateVector2DResult SetVerticalScrollFraction(const float InScrollOffsetFraction);

	/** Set the absolute scroll offset value */
	SLATE_API UE::Slate::FDeprecateVector2DResult SetScrollOffset(const UE::Slate::FDeprecateVector2DParameter& InScrollOffset, const FGeometry& InGeometry);

	/** Get the absolute scroll offset value */
	SLATE_API UE::Slate::FDeprecateVector2DResult GetScrollOffset() const;

	/** Returns the computed wrap location for this layout */
	SLATE_API float GetComputedWrappingWidth() const;

	/** Returns whether or not we are auto wrapping text */
	SLATE_API bool GetAutoWrapText() const;

	/** Called when our parent widget receives focus */
	SLATE_API bool HandleFocusReceived(const FFocusEvent& InFocusEvent);

	/** Called when our parent widget loses focus */
	SLATE_API bool HandleFocusLost(const FFocusEvent& InFocusEvent);

	/** Called to handle an OnKeyChar event from our parent widget */
	SLATE_API FReply HandleKeyChar(const FCharacterEvent& InCharacterEvent);

	/** Called to handle an OnKeyDown event from our parent widget */
	SLATE_API FReply HandleKeyDown(const FKeyEvent& InKeyEvent);

	/** Called to handle an OnKeyUp event from our parent widget */
	SLATE_API FReply HandleKeyUp(const FKeyEvent& InKeyEvent);

	/** Called to handle an OnMouseButtonDown event from our parent widget */
	SLATE_API FReply HandleMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent);

	/** Called to handle an OnMouseButtonUp event from our parent widget */
	SLATE_API FReply HandleMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent);

	/** Called to handle an OnMouseMove event from our parent widget */
	SLATE_API FReply HandleMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Called to handle an OnMouseButtonDoubleClick event from our parent widget */
	SLATE_API FReply HandleMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Called to handle an escape action acting on the current selection */
	SLATE_API bool HandleEscape();

	/** Called to handle a backspace action acting on the current selection or at the cursor position */
	SLATE_API bool HandleBackspace();

	/** Called to handle a delete action acting on the current selection or at the cursor position */
	SLATE_API bool HandleDelete();

	/** Called to handle a typing a character on the current selection or at the cursor position */
	SLATE_API bool HandleTypeChar(const TCHAR InChar);

	/** Called to handle a carriage return action acting on the current selection or at the cursor position */
	SLATE_API bool HandleCarriageReturn(bool isRepeat, int32 UserIndex = 0);

	/** Are we able to delete the currently selected text? */
	SLATE_API bool CanExecuteDelete() const;

	/** Delete any currently selected text */
	SLATE_API void DeleteSelectedText();

	/** Query to see if any text is selected within the document */
	SLATE_API bool AnyTextSelected() const;

	/** Query to see if the text under the given position is currently selected */
	SLATE_API bool IsTextSelectedAt(const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& ScreenSpacePosition) const;

	/** Query to see if the text under the given position is currently selected (the position is local to the text layout space) */
	SLATE_API bool IsTextSelectedAt(const UE::Slate::FDeprecateVector2DParameter& InLocalPosition) const;

	/** Are we able to execute the "Select All" command? */
	SLATE_API bool CanExecuteSelectAll() const;

	/** Select all the text in the document */
	SLATE_API void SelectAllText();

	/** Select the word under the given position */
	SLATE_API void SelectWordAt(const FGeometry& MyGeometry, const UE::Slate::FDeprecateVector2DParameter& ScreenSpacePosition);

	/** Select the word under the given position (the position is local to the text layout space) */
	SLATE_API void SelectWordAt(const UE::Slate::FDeprecateVector2DParameter& InLocalPosition);

	/** Select a block of text */
	SLATE_API void SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation);

	/** Clear the active text selection */
	SLATE_API void ClearSelection();

	/** Are we able to cut the currently selected text? */
	SLATE_API bool CanExecuteCut() const;

	/** Cut the currently selected text and place it on the clipboard */
	SLATE_API void CutSelectedTextToClipboard();

	/** Are we able to copy the currently selected text? */
	SLATE_API bool CanExecuteCopy() const;

	/** Copy the currently selected text and place it on the clipboard */
	SLATE_API void CopySelectedTextToClipboard();

	/** Are we able to paste the text from the clipboard into this document? */
	SLATE_API bool CanExecutePaste() const;

	/** Paste the text from the clipboard into this document */
	SLATE_API void PasteTextFromClipboard();

	/** Insert the given text at the current cursor position, correctly taking into account new line characters */
	SLATE_API void InsertTextAtCursor(const FString& InString);

	/** Insert the given run at the current cursor position */
	SLATE_API void InsertRunAtCursor(TSharedRef<IRun> InRun);

	/** Move the cursor in the document using the specified move method */
	SLATE_API bool MoveCursor(const FMoveCursor& InArgs);

	/** Move the cursor to the given location in the document (will also scroll to this point) */
	SLATE_API void GoTo(const FTextLocation& NewLocation);

	/** Move the cursor specified location */
	SLATE_API void GoTo(const ETextLocation NewLocation);

	/** Jump the cursor to the given location in the document */
	SLATE_API void JumpTo(ETextLocation JumpLocation, ECursorAction Action);

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const FTextLocation& NewLocation);

	/** Scroll to the given location in the document (without moving the cursor) */
	SLATE_API void ScrollTo(const ETextLocation NewLocation);

	/** Update the active cursor highlight based on the state of the text layout */
	SLATE_API void UpdateCursorHighlight();

	/** Remove any active cursor highlights */
	SLATE_API void RemoveCursorHighlight();

	/** Update the preferred offset of the cursor, based on the current state of the text layout */
	SLATE_API void UpdatePreferredCursorScreenOffsetInLine();

	/** Apply the given style to the currently selected text (or insert a new run at the current cursor position if no text is selected) */
	SLATE_API void ApplyToSelection(const FRunInfo& InRunInfo, const FTextBlockStyle& InStyle);

	/** Get the run currently under the cursor, or null if there is no run currently under the cursor */
	SLATE_API TSharedPtr<const IRun> GetRunUnderCursor() const;

	/** Get the runs currently that are current selected, some of which may be only partially selected */
	SLATE_API TArray<TSharedRef<const IRun>> GetSelectedRuns() const;

	/** Get the interaction position of the cursor (where to insert, delete, etc, text from/to) */
	SLATE_API FTextLocation GetCursorLocation() const;

	/**
	 * Given a location and a Direction to offset, return a new location.
	 *
	 * @param CurrentLocation    Cursor location from which to offset
	 * @param Direction   Positive means right, negative means left.
	 */
	SLATE_API FTextLocation TranslatedLocation(const FTextLocation& CurrentLocation, int8 Direction) const;

	/**
	 * Given a location and a Direction to offset, return a new location.
	 *
	 * @param Location              Cursor location from which to offset
	 * @param NumLinesToMove        Number of lines to move in a given direction. Positive means down, negative means up.
	 * @param GeometryScale         Geometry DPI scale at which the widget is being rendered
	 * @param OutCursorPosition     Fill with the updated cursor position.
	 * @param OutCursorAlignment    Optionally fill with a new cursor alignment (will be auto-calculated if not set).
	 */
	SLATE_API void TranslateLocationVertical(const FTextLocation& Location, int32 NumLinesToMove, float GeometryScale, FTextLocation& OutCursorPosition, TOptional<SlateEditableTextTypes::ECursorAlignment>& OutCursorAlignment) const;

	/** Find the closest word boundary */
	SLATE_API FTextLocation ScanForWordBoundary(const FTextLocation& Location, int8 Direction) const;

	/** Get the character at Location */
	SLATE_API TCHAR GetCharacterAt(const FTextLocation& Location) const;

	/** Are we at the beginning of all the text. */
	SLATE_API bool IsAtBeginningOfDocument(const FTextLocation& Location) const;

	/** Are we at the end of all the text. */
	SLATE_API bool IsAtEndOfDocument(const FTextLocation& Location) const;

	/** Is this location the beginning of a line */
	SLATE_API bool IsAtBeginningOfLine(const FTextLocation& Location) const;

	/** Is this location the end of a line. */
	SLATE_API bool IsAtEndOfLine(const FTextLocation& Location) const;

	/** Are we currently at the beginning of a word */
	SLATE_API bool IsAtWordStart(const FTextLocation& Location) const;

	/** Restores the text to the original state */
	SLATE_API void RestoreOriginalText();

	/** Returns whether the current text varies from the original */
	SLATE_API bool HasTextChangedFromOriginal() const;

	UE_DEPRECATED(5.8, "Use the properly spelled version BeginEditTransaction")
	void BeginEditTransation()
	{
		BeginEditTransaction();
	}
	/** Called to begin an undoable editable text transaction */
	SLATE_API void BeginEditTransaction();

	/** Called to end an undoable editable text transaction */
	SLATE_API void EndEditTransaction();

	/** Push the given undo state onto the undo stack */
	SLATE_API void PushUndoState(const SlateEditableTextTypes::FUndoState& InUndoState);

	/** Clear the current undo stack */
	SLATE_API void ClearUndoStates();

	/** Create an undo state that reflects the current state of the document */
	SLATE_API void MakeUndoState(SlateEditableTextTypes::FUndoState& OutUndoState);

	/** Are we currently able to execute an undo action? */
	SLATE_API bool CanExecuteUndo() const;
	
	/** Execute an undo action */
	SLATE_API void Undo();

	/** Are we currently able to execute a redo action? */
	SLATE_API bool CanExecuteRedo() const;

	/** Execute a redo action */
	SLATE_API void Redo();

	SLATE_API void SaveText(const FText& TextToSave);

	SLATE_API void LoadText();

	SLATE_API bool ComputeVolatility() const;

	SLATE_API void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	UE_DEPRECATED(5.8, "Use the const version instead.")
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled)
	{
		return const_cast<const FSlateEditableTextLayout*>(this)->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
	SLATE_API int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	SLATE_API void SetCachedSize(const FGeometry& AllottedGeometry);

	SLATE_API void CacheDesiredSize(float LayoutScaleMultiplier);

	SLATE_API FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const;

	SLATE_API FChildren* GetChildren();

	SLATE_API void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const;

	SLATE_API UE::Slate::FDeprecateVector2DResult GetSize() const;

	SLATE_API TSharedRef<SWidget> BuildDefaultContextMenu(const TSharedPtr<FExtender>& InMenuExtender) const;

	SLATE_API bool HasActiveContextMenu() const;

	/**
	 * Fill OutTextLine with the text line where the current cursor location is at
	 *
	 * @param OutTextLine   FString of the line
	 */
	SLATE_API void GetCurrentTextLine(FString& OutTextLine) const;

	/**
	 * Fill OutTextLine with the text line at the specified index
	 *
	 * @param InLineIndex   Index of the line
	 * @param OutTextLine   FString of the line
	 */
	SLATE_API void GetTextLine(const int32 InLineIndex, FString& OutTextLine) const;

	/* Function to spawn virtual keyboard on demand */
	SLATE_API void ToggleVirtualKeyboard(bool bShow, const int32 UserIndex);

	/** Sets the delegate for the internal text layout that is Called when all font faces for the text have finished loading asynchronously. */
	SLATE_API void SetOnAllFontFacesFinishLoadingDelegate(const FSimpleDelegate& InAllFontFacesFinishLoadingDelegate);

	/** Sets the paint policy to use for text if the font faces are still async loading. */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);

private:
	/** Function to call OnTextChanged on the owner widget and to properly set the PreviousText. */
	void BroadcastOnTextChanged(const FText& InText);

	/** Function called when the BoundText attribute changes. */
	static void OnBoundTextChanged(SWidget& Widget, FSlateEditableTextLayout& SlateTextLayout)
	{
		SlateTextLayout.OnBoundTextChanged();
	}

	/** Function called when the BoundText attribute changes or called manually from SetText. Will call the Widget's OnTextChanged */
	void OnBoundTextChanged();

	/** Function called when the BoundSearchText attribute changes */
	static void OnBoundSearchTextChanged(SWidget& Widget, FSlateEditableTextLayout& SlateTextLayout);

	/** Function called when the bIsReadOnlyAttribute attribute changes */
	static void OnIsReadOnlyChanged(SWidget& Widget, FSlateEditableTextLayout& SlateTextLayout);

	/** Insert the given text at the current cursor position, correctly taking into account new line characters */
	void InsertTextAtCursorImpl(const FString& InString);

	/** Insert a new-line at the current cursor position */
	void InsertNewLineAtCursorImpl();

	/** Implementation of Refresh that actually updates the layout. Optionally takes text to set, or will use the current editable text if none if provided */
	bool RefreshImpl(const FText* InTextToSet, const bool bForce = false);

	/** Create a text or password run using the given text and style */
	TSharedRef<IRun> CreateTextOrPasswordRun(const FRunInfo& InRunInfo, const TSharedRef<const FString>& InText, const FTextBlockStyle& InStyle);

	/** Called when the active context menu is closed */
	void OnContextMenuClosed(TSharedRef<IMenu> Menu);
	
	void UpdateTextChangedByVirtualKeyboard();

private:
	/** Virtual keyboard handler for an editable text layout */
	friend class FVirtualKeyboardEntry;
	class FVirtualKeyboardEntry : public IVirtualKeyboardEntry
	{
	public:
		static TSharedRef<FVirtualKeyboardEntry> Create(FSlateEditableTextLayout& InOwnerLayout);

		virtual void SetTextFromVirtualKeyboard(const FText& InNewText, ETextEntryType TextEntryType) override;
		virtual void SetSelectionFromVirtualKeyboard(int InSelStart, int InSelEnd) override;
		virtual bool GetSelection(int& OutSelStart, int& OutSelEnd) override;

		virtual FText GetText() const override;
		virtual FText GetHintText() const override;
		virtual EKeyboardType GetVirtualKeyboardType() const override;
		virtual FVirtualKeyboardOptions GetVirtualKeyboardOptions() const override;
		virtual bool IsMultilineEntry() const override;
		virtual bool IsIntegratedKeyboardEnabled() const override;

	private:
		FVirtualKeyboardEntry(FSlateEditableTextLayout& InOwnerLayout);
		FSlateEditableTextLayout* OwnerLayout;
	};

private:
	/**
	 * Note: The IME interface for the multiline editable text uses the pre-flowed version of the string since the IME APIs are designed to work with flat strings
	 *		 This means we have to do a bit of juggling to convert between the two
	 */
	friend class FTextInputMethodContext;
	class FTextInputMethodContext : public ITextInputMethodContext
	{
	public:
		static TSharedRef<FTextInputMethodContext> Create(FSlateEditableTextLayout& InOwnerLayout);

		void CacheWindow();

		inline void KillContext()
		{
			OwnerLayout = nullptr;
			bIsComposing = false;
		}

		inline FTextRange GetCompositionRange() const
		{
			return FTextRange(CompositionBeginIndex, CompositionBeginIndex + CompositionLength);
		}

		bool UpdateCachedGeometry(const FGeometry& InAllottedGeometry)
		{
			if (CachedGeometry != InAllottedGeometry)
			{
				CachedGeometry = InAllottedGeometry;
				return true;
			}
			return false;
		}

		virtual bool IsComposing() override;
		virtual bool IsReadOnly() override;
		virtual uint32 GetTextLength() override;
		virtual void GetSelectionRange(uint32& BeginIndex, uint32& Length, ECaretPosition& CaretPosition) override;
		virtual void SetSelectionRange(const uint32 BeginIndex, const uint32 Length, const ECaretPosition CaretPosition) override;
		virtual void GetTextInRange(const uint32 BeginIndex, const uint32 Length, FString& OutString) override;
		virtual void SetTextInRange(const uint32 BeginIndex, const uint32 Length, const FString& InString) override;
		virtual int32 GetCharacterIndexFromPoint(const FVector2D& Point) override;
		virtual bool GetTextBounds(const uint32 BeginIndex, const uint32 Length, FVector2D& Position, FVector2D& Size) override;
		virtual void GetScreenBounds(FVector2D& Position, FVector2D& Size) override;
		virtual TSharedPtr<FGenericWindow> GetWindow() override;
		virtual void BeginComposition() override;
		virtual void UpdateCompositionRange(const int32 InBeginIndex, const uint32 InLength) override;
		virtual void EndComposition() override;

	private:
		FTextInputMethodContext(FSlateEditableTextLayout& InOwnerLayout);
		FSlateEditableTextLayout* OwnerLayout;
		TWeakPtr<SWindow> CachedParentWindow;

		FGeometry CachedGeometry;
		bool bIsComposing;
		int32 CompositionBeginIndex;
		uint32 CompositionLength;
	};

private:
	/** Pointer to the interface for our owner widget */
	ISlateEditableTextWidget* OwnerWidget;

	/** The iterator to use to detect grapheme cluster boundaries */
	TSharedPtr<IBreakIterator> GraphemeBreakIterator;

	/** The marshaller used to get/set the BoundText text to/from the text layout. */
	TSharedPtr<ITextLayoutMarshaller> Marshaller;

	/** The marshaller used to get/set the HintText text to/from the text layout. */
	TSharedPtr<ITextLayoutMarshaller> HintMarshaller;

	/** Delegate used to create internal text layouts. */
	FCreateSlateTextLayout CreateSlateTextLayout;

	/** In control of the layout and wrapping of the BoundText */
	TSharedPtr<FSlateTextLayout> TextLayout;

	/** In control of the layout and wrapping of the HintText */
	TUniquePtr<FSlateTextBlockLayout> HintTextLayout;

	/** Default style used by the TextLayout */
	FTextBlockStyle TextStyle;

	/** Style used to draw the hint text (only valid when HintTextLayout is set) */
	mutable FTextBlockStyle HintTextStyle;

	/** The text displayed in this text block */
	TSlateNestedAttribute<FText> BoundText;
	FText PreviousText;

	/** The state of BoundText last Tick() (used to allow updates when the text is changed) */
	FTextSnapshot BoundTextLastTick;

	/** Was the editable text showing a password last Tick() (allows a forcible text layout update when changing state) */
	bool bWasPasswordLastTick;

	/** The text that appears when there is no text in the text box */
	TSlateNestedAttribute<FText> HintText;

	/** Opacity of the hint text */
	TSlateNestedAttribute<float> HintTextOpacity;

	/** The text to be searched for */
	TSlateNestedAttribute<FText> BoundSearchText;

	/** The state of BoundSearchText last Tick() (used to allow updates when the text is changed) */
	FTextSnapshot BoundSearchTextLastTick;

	/** The active search text (set from BeginSearch) */
	FText SearchText;

	/** The case-sensitivity of the active search (set from BeginSearch) */
	ESearchCase::Type SearchCase;

	/** The map to look up the index of each search result (key is the starting location of each matched string)*/
	TMap<FTextLocation, int32> SearchResultToIndexMap;

	/** The active search result index */ 
	int32 CurrentSearchResultIndex;

	/** Sets whether this text box can actually be modified interactively by the user */
	TSlateNestedAttribute<bool> bIsReadOnlyAttribute;

	/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
	TSlateNestedAttribute<float> WrapTextAt;

	/** True if we're wrapping text automatically based on the computed horizontal space for this widget */
	TSlateNestedAttribute<bool> AutoWrapText;

	/** The wrapping policy we're using */
	TSlateNestedAttribute<ETextWrappingPolicy> WrappingPolicy;

	/** The amount of blank space left around the edges of text area */
	TSlateNestedAttribute<FMargin> Margin;

	/** How the text should be aligned with the margin */
	TSlateNestedAttribute<ETextJustify::Type> Justification;

	/** The amount to scale each lines height by */
	TSlateNestedAttribute<float> LineHeightPercentage;

	/** Whether to leave extra space below the last line due to line height */
	TSlateNestedAttribute<bool> ApplyLineHeightToBottomLine;

	/** The information used to help identify who owns this text layout in the case of an error */
	TAttribute<FString> DebugSourceInfo;

	/** Virtual keyboard handler for this text layout */
	TSharedPtr<FVirtualKeyboardEntry> VirtualKeyboardEntry;

	/** True if the IME context for this text layout has been registered with the input method manager */
	bool bHasRegisteredTextInputMethodContext;

	/** IME context for this text layout */
	TSharedPtr<FTextInputMethodContext> TextInputMethodContext;

	/** Notification interface object for IMEs */
	TSharedPtr<ITextInputMethodChangeNotifier> TextInputMethodChangeNotifier;

	/** Layout highlighter used to draw the cursor */
	TSharedPtr<SlateEditableTextTypes::FCursorLineHighlighter> CursorLineHighlighter;

	/** Layout highlighter used to draw an active text composition */
	TSharedPtr<SlateEditableTextTypes::FTextCompositionHighlighter> TextCompositionHighlighter;

	/** Layout highlighter used to draw the active text selection */
	TSharedPtr<SlateEditableTextTypes::FTextSelectionHighlighter> TextSelectionHighlighter;

	/** Layout highlighter used to draw the active search selection */
	TSharedPtr<SlateEditableTextTypes::FTextSearchHighlighter> SearchSelectionHighlighter;

	/** Line highlights that have been added from this editable text layout (used for cleanup without removing) */
	TArray<FTextLineHighlight> ActiveLineHighlights;

	/** The scroll offset (in unscaled Slate units) for this text */
	FVector2f ScrollOffset;

	/** If set, the pending data containing a position that should be scrolled into view */
	TOptional<SlateEditableTextTypes::FScrollInfo> PositionToScrollIntoView;

	/** That start of the selection when there is a selection. The end is implicitly wherever the cursor happens to be. */
	TOptional<FTextLocation> SelectionStart;

	/** The user probably wants the cursor where they last explicitly positioned it horizontally. */
	float PreferredCursorScreenOffsetInLine;

	/** Current cursor data */
	SlateEditableTextTypes::FCursorInfo CursorInfo;

	/** Undo states */
	TArray<SlateEditableTextTypes::FUndoState> UndoStates;

	/** Current undo state level that we've rolled back to, or INDEX_NONE if we haven't undone. Used for 'Redo'. */
	int32 CurrentUndoLevel;

	/** Undo state that will be pushed if text is actually changed between calls to BeginEditTransaction() and EndEditTransaction() */
	TOptional<SlateEditableTextTypes::FUndoState> StateBeforeChangingText;

	/** Track the number transactions that are opened */
	int32 NumTransactionsOpened;

	/** Original text undo state */
	SlateEditableTextTypes::FUndoState OriginalText;

	/** True if we're currently selecting text by dragging the mouse cursor with the left button held down */
	bool bIsDragSelecting;

	/** True if the last mouse down caused us to receive keyboard focus */
	bool bWasFocusedByLastMouseDown;

	/** True if characters were selected by dragging since the last keyboard focus.  Used for text selection. */
	bool bHasDragSelectedSinceFocused;

	/** Whether the text has been changed by a virtual keyboard */
	bool bTextChangedByVirtualKeyboard;

	/** Whether the text has been committed by a virtual keyboard */
	bool bTextCommittedByVirtualKeyboard;

	/** What text was submitted by a virtual keyboard */
	FText VirtualKeyboardText;

	/** How the text was committed by the virtual keyboard */
	ETextCommit::Type VirtualKeyboardTextCommitType;

	/** Override for the overflow policy. If this is not set the text style setting is used */
	TOptional<ETextOverflowPolicy> OverflowPolicyOverride;

	/** The last known size of the widget from the previous OnPaint, used to recalculate wrapping */
	mutable FVector2f CachedSize;

	/** A list commands to execute if a user presses the corresponding key-binding in the text box */
	TSharedPtr<FUICommandList> UICommandList;

	/** Information about any active context menu widgets */
	FActiveTextEditContextMenu ActiveContextMenu;

	/** Whether the cursor position has been changed externally*/
	bool bSelectionChangedExternally;

	/** Used in OnBoundTextChanged to assess if we need to force some text updates */
	bool bForceBoundTextReview;

	/** The boundaries of the external selection */
	int ExternalSelectionStart;
	int ExternalSelectionEnd;

};
