// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateLineHighlighter.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;
struct FSlateBrush;
struct FTextBlockStyle;

namespace SlateEditableTextTypes
{

enum class ECursorAlignment : uint8
{
	/** Visually align the cursor to the left of the character its placed at, and insert text before the character */
	Left,

	/** Visually align the cursor to the right of the character its placed at, and insert text after the character */
	Right,
};

/** Store the information about the current cursor position */
class FCursorInfo
{
public:
	FCursorInfo()
		: CursorPosition()
		, CursorAlignment(ECursorAlignment::Left)
		, CursorTextDirection(TextBiDi::ETextDirection::LeftToRight)
		, LastCursorInteractionTime(0)
	{
	}

	/** Get the literal position of the cursor (note: this may not be the correct place to insert text to, use GetCursorInteractionLocation for that) */
	inline FTextLocation GetCursorLocation() const
	{
		return CursorPosition;
	}

	/** Get the alignment of the cursor */
	inline ECursorAlignment GetCursorAlignment() const
	{
		return CursorAlignment;
	}

	/** Get the direction of the text under the cursor */
	inline TextBiDi::ETextDirection GetCursorTextDirection() const
	{
		return CursorTextDirection;
	}

	/** Get the interaction position of the cursor (where to insert, delete, etc, text from/to) */
	inline FTextLocation GetCursorInteractionLocation() const
	{
		// If the cursor is right aligned, we treat it as if it were one character further along for insert/delete purposes
		return FTextLocation(CursorPosition, (CursorAlignment == ECursorAlignment::Right) ? 1 : 0);
	}

	inline double GetLastCursorInteractionTime() const
	{
		return LastCursorInteractionTime;
	}

	/** Update the last cursor interaction time to now */
	SLATE_API void UpdateLastCursorInteractionTime();

	/** Set the position of the cursor, and then work out the correct alignment based on the current text layout */
	SLATE_API void SetCursorLocationAndCalculateAlignment(const FTextLayout& InTextLayout, const FTextLocation& InCursorPosition);

	/** Set the literal position and alignment of the cursor */
	SLATE_API void SetCursorLocationAndAlignment(const FTextLayout& InTextLayout, const FTextLocation& InCursorPosition, const ECursorAlignment InCursorAlignment);

	/** Create an undo for this cursor data */
	SLATE_API FCursorInfo CreateUndo() const;

	/** Restore this cursor data from an undo */
	SLATE_API void RestoreFromUndo(const FCursorInfo& UndoData);

private:
	/** Current cursor position; there is always one. */
	FTextLocation CursorPosition;

	/** Cursor alignment (horizontal) within its position. This affects the rendering and behavior of the cursor when adding new characters */
	ECursorAlignment CursorAlignment;

	/** The direction of the text under the cursor */
	TextBiDi::ETextDirection CursorTextDirection;

	/** Last time the user did anything with the cursor.*/
	double LastCursorInteractionTime;
};

/** Stores a single undo level for editable text */
struct FUndoState
{
	/** Text */
	FText Text;

	/** Selection state */
	TOptional<FTextLocation> SelectionStart;

	/** Cursor data */
	FCursorInfo CursorInfo;
};

/** Information needed to be able to scroll to a given point */
struct FScrollInfo
{
	FScrollInfo()
		: Position()
		, Alignment(ECursorAlignment::Left)
	{
	}

	FScrollInfo(const FTextLocation InPosition, const ECursorAlignment InAlignment)
		: Position(InPosition)
		, Alignment(InAlignment)
	{
	}

	/** The location in the document to scroll to (in line model space) */
	FTextLocation Position;

	/** The alignment at the given position. This may affect which line view the character maps to when converted from line model space */
	ECursorAlignment Alignment;
};

/** Run highlighter used to draw the cursor */
class FCursorLineHighlighter : public ISlateLineHighlighter
{
public:
	static SLATE_API TSharedRef< FCursorLineHighlighter > Create(const FCursorInfo* InCursorInfo);

	SLATE_API void SetCursorBrush(const TAttribute<const FSlateBrush*>& InCursorBrush);

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Set to false to skip drawing the cursor */
	void SetDrawCursor(bool bInDrawCursor) { bDrawCursor = bInDrawCursor; }
	/** Is the cursor drawn? */
	bool GetDrawCursor() const { return bDrawCursor; }

protected:
	SLATE_API FCursorLineHighlighter(const FCursorInfo* InCursorInfo);

	/** Cursor data that this highlighter is tracking */
	const FCursorInfo* CursorInfo;

	/** Brush used to draw the cursor */
	TAttribute<const FSlateBrush*> CursorBrush;

	/** Set to false to skip drawing the cursor */
	bool bDrawCursor = true;
};

/** Run highlighter used to draw the composition range */
class FTextCompositionHighlighter : public ISlateLineHighlighter
{
public:
	static SLATE_API TSharedRef< FTextCompositionHighlighter > Create();

	SLATE_API void SetCompositionBrush(const TAttribute<const FSlateBrush*>& InCompositionBrush);

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

protected:
	SLATE_API FTextCompositionHighlighter();

	/** Brush used to draw the composition highlight */
	TAttribute<const FSlateBrush*> CompositionBrush;
};

/** Run highlighter used to draw selection ranges */
class FTextSelectionHighlighter : public ISlateLineHighlighter
{
public:
	static SLATE_API TSharedRef< FTextSelectionHighlighter > Create();

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	void SetHasKeyboardFocus(const bool bInHasKeyboardFocus)
	{
		bHasKeyboardFocus = bInHasKeyboardFocus;
	}

protected:
	SLATE_API FTextSelectionHighlighter();

	/** true if the parent widget has keyboard focus, false otherwise */
	bool bHasKeyboardFocus;
};

/** Run highlighter used to draw search ranges */
class FTextSearchHighlighter : public ISlateLineHighlighter
{
public:
	static SLATE_API TSharedRef< FTextSearchHighlighter > Create();

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	void SetHasKeyboardFocus(const bool bInHasKeyboardFocus)
	{
		bHasKeyboardFocus = bInHasKeyboardFocus;
	}

protected:
	SLATE_API FTextSearchHighlighter();

	/** true if the parent widget has keyboard focus, false otherwise */
	bool bHasKeyboardFocus;
};

} // namespace SlateEditableTextTypes
