// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/ITextLayoutMarshaller.h"
#include "Framework/Text/SlateTextLayoutFactory.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class IBreakIterator;
class ISlateRunRenderer;
enum class ETextShapingMethod : uint8;
enum class EFontFacesLoadingPaintPolicy : uint8;

/** Class to handle the cached layout of STextBlock/SRichTextBlock by proxying around a FTextLayout */
class FSlateTextBlockLayout
{
public:
	struct FWidgetDesiredSizeArgs
	{
		inline FWidgetDesiredSizeArgs(
			const FText& InText,
			const FText& InHighlightText,
			const float InWrapTextAt,
			const bool InAutoWrapText,
			const ETextWrappingPolicy InWrappingPolicy,
			const ETextTransformPolicy InTransformPolicy,
			const FMargin& InMargin,
			const float InLineHeightPercentage,
			const bool InApplyLineHeightToBottomLine,
			const ETextJustify::Type InJustification
		)
			: Text(InText)
			, HighlightText(InHighlightText)
			, Margin(InMargin)
			, WrapTextAt(InWrapTextAt)
			, LineHeightPercentage(InLineHeightPercentage)
			, ApplyLineHeightToBottomLine(InApplyLineHeightToBottomLine)
			, WrappingPolicy(InWrappingPolicy)
			, TransformPolicy(InTransformPolicy)
			, Justification(InJustification)
			, AutoWrapText(InAutoWrapText)
		{
		}

		inline FWidgetDesiredSizeArgs(
			const FText& InText,
			const FText& InHighlightText,
			const float InWrapTextAt,
			const bool InAutoWrapText,
			const ETextWrappingPolicy InWrappingPolicy,
			const ETextTransformPolicy InTransformPolicy,
			const FMargin& InMargin,
			const float InLineHeightPercentage,
			const ETextJustify::Type InJustification
		)
			: Text(InText)
			, HighlightText(InHighlightText)
			, Margin(InMargin)
			, WrapTextAt(InWrapTextAt)
			, LineHeightPercentage(InLineHeightPercentage)
			, ApplyLineHeightToBottomLine(true)
			, WrappingPolicy(InWrappingPolicy)
			, TransformPolicy(InTransformPolicy)
			, Justification(InJustification)
			, AutoWrapText(InAutoWrapText)
		{
		}

		const FText Text = FText::GetEmpty();
		const FText HighlightText = FText::GetEmpty();
		const FMargin Margin;
		const float WrapTextAt = 0.f;
		const float LineHeightPercentage;
		const bool ApplyLineHeightToBottomLine;
		const ETextWrappingPolicy WrappingPolicy;
		const ETextTransformPolicy TransformPolicy;
		const ETextJustify::Type Justification;
		const bool AutoWrapText = false;
	};

	struct
		UE_DEPRECATED(5.0, "FWidgetArgs is deprecated. Upgrade to FWidgetDesiredSizeArgs instead.")
		FWidgetArgs
	{
		inline FWidgetArgs(
			const TAttribute<FText>& InText, 
			const TAttribute<FText>& InHighlightText, 
			const TAttribute<float>& InWrapTextAt, 
			const TAttribute<bool>& InAutoWrapText, 
			const TAttribute<ETextWrappingPolicy>& InWrappingPolicy, 
			const TAttribute<ETextTransformPolicy>& InTransformPolicy,
			const TAttribute<FMargin>& InMargin, 
			const TAttribute<float>& InLineHeightPercentage, 
			const TAttribute<ETextJustify::Type>& InJustification
		)
			: Text(InText)
			, HighlightText(InHighlightText)
			, WrapTextAt(InWrapTextAt)
			, AutoWrapText(InAutoWrapText)
			, WrappingPolicy(InWrappingPolicy)
			, TransformPolicy(InTransformPolicy)
			, Margin(InMargin)
			, LineHeightPercentage(InLineHeightPercentage)
			, Justification(InJustification)
		{
		}

		const TAttribute<FText>& Text;
		const TAttribute<FText>& HighlightText;
		const TAttribute<float>& WrapTextAt;
		const TAttribute<bool>& AutoWrapText;
		const TAttribute<ETextWrappingPolicy> WrappingPolicy;
		const TAttribute<ETextTransformPolicy> TransformPolicy;
		const TAttribute<FMargin>& Margin;
		const TAttribute<float>& LineHeightPercentage;
		const TAttribute<ETextJustify::Type>& Justification;
	};

	/**
	 * Constructor
	 */
	SLATE_API FSlateTextBlockLayout(SWidget* InOwner, FTextBlockStyle InDefaultTextStyle, const TOptional<ETextShapingMethod> InTextShapingMethod, const TOptional<ETextFlowDirection> InTextFlowDirection, const FCreateSlateTextLayout& InCreateSlateTextLayout, TSharedRef<ITextLayoutMarshaller> InMarshaller, TSharedPtr<IBreakIterator> InLineBreakPolicy);

	/**
	 * Conditionally update the text style if needed
	 */
	SLATE_API void ConditionallyUpdateTextStyle(const FTextBlockStyle& InTextStyle);

	/**
	 * Conditionally update the text style if needed
	 */
	SLATE_API void ConditionallyUpdateTextStyle(const FTextBlockStyle::CompareParams& InNewStyleParams);
	
	/**
	 * Update the text style.
	 */
	SLATE_API void UpdateTextStyle(const FTextBlockStyle& InTextStyle);

	/**
	 * Update the text style.
	 */
	SLATE_API void UpdateTextStyle(const FTextBlockStyle::CompareParams& InNewStyleParams);

	/**
	 * Get the computed desired size for this layout, updating the internal cache as required
	 */
	SLATE_API FVector2D ComputeDesiredSize(const FWidgetDesiredSizeArgs& InWidgetArgs, const float InScale);

	/**
	 * Get the computed desired size for this layout, updating the internal cache as required
	 */
	SLATE_API FVector2D ComputeDesiredSize(const FWidgetDesiredSizeArgs& InWidgetArgs, const float InScale, const FTextBlockStyle& InTextStyle);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/**
	 * Get the computed desired size for this layout, updating the internal cache as required
	 */
	UE_DEPRECATED(5.0, "FWidgetArgs is deprecated. Upgrade to FWidgetDesiredSizeArgs instead.")
	SLATE_API FVector2D ComputeDesiredSize(const FWidgetArgs& InWidgetArgs, const float InScale, const FTextBlockStyle& InTextStyle);

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Gets the last computed desired size.
	 */
	SLATE_API FVector2D GetDesiredSize() const;


	/**
	 * Get the TextLayout scale.
	 */
	SLATE_API float GetLayoutScale() const;


	/**
	 * Paint this layout, updating the internal cache as required
	 */
	SLATE_API int32 OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled);

	/**
	 * Force dirty the layout due to an external change that can't be picked up automatically by this cache
	 */
	SLATE_API void DirtyLayout();

	/**
	 * Force dirty the content due to an external change that can't be picked up automatically by this cache.
	 * Also force dirties the layout. Will cause the contained text to be re-parsed by the marshaller next frame.
	 */
	SLATE_API void DirtyContent();

	/**
	 * Override the text style used and immediately update the text layout (if required).
	 * This can be used to override the text style after calling ComputeDesiredSize (eg, if you can only compute your text style in OnPaint)
	 * Please note that changing the size or font used by the text may causing clipping issues until the next call to ComputeDesiredSize
	 */
	SLATE_API void OverrideTextStyle(const FTextBlockStyle& InTextStyle);

	/**
	 * Set the text shaping method that the internal text layout should use
	 */
	SLATE_API void SetTextShapingMethod(const TOptional<ETextShapingMethod>& InTextShapingMethod);

	/**
	 * Set the text flow direction that the internal text layout should use
	 */
	SLATE_API void SetTextFlowDirection(const TOptional<ETextFlowDirection>& InTextFlowDirection);

	/**
	 * Set the text overflow policy that the internal text layout should use
	 */
	SLATE_API void SetTextOverflowPolicy(const TOptional<ETextOverflowPolicy> InTextOverflowPolicy);

	/**
	 * Set the information used to help identify who owns this text layout in the case of an error
	 */
	SLATE_API void SetDebugSourceInfo(const TAttribute<FString>& InDebugSourceInfo);

	/**
	 * Get the child widgets of this layout
	 */
	SLATE_API FChildren* GetChildren();

	/**
	 * Arrange any child widgets in this layout
	 */
	SLATE_API void ArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const;

	/**
	 * Sets the delegate for the internal text layout that is Called when all font faces for the text have finished loading asynchronously.
	 */
	SLATE_API void SetOnAllFontFacesFinishLoading(const FSimpleDelegate& InAllFontFacesFinishLoading);

	/**
	 * Sets the paint policy to use for text if the font faces are still async loading.
	 */
	SLATE_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
private:
	/** Updates the text layout to contain the given text */
	void UpdateTextLayout(const FText& InText);

	/** Updates the text layout to contain the given string */
	void UpdateTextLayout(const FString& InText);

	/** Update the text highlights */
	void UpdateTextHighlights(const FText& InHighlightText);

	/** Is the style used by the text marshaller up-to-date? */
	bool IsStyleUpToDate(const FTextBlockStyle& NewStyle) const;

	/** Is the style used by the text marshaller up-to-date? */
	bool IsStyleUpToDate(const FTextBlockStyle::CompareParams& InNewStyleParams) const;

	/** Calculate the wrapping width based on the given fixed wrap width, and whether we're auto-wrapping */
	float CalculateWrappingWidth() const;

	/** In control of the layout and wrapping of the text */
	TSharedPtr<FSlateTextLayout> TextLayout;

	/** The marshaller used to get/set the text to/from the text layout. */
	TSharedPtr<ITextLayoutMarshaller> Marshaller;

	/** Used to render the current highlights in the text layout */
	TSharedPtr<ISlateRunRenderer> TextHighlighter;

	/** The last known size of the layout from the previous OnPaint, used to guess at an auto-wrapping width in ComputeDesiredSize */
	FVector2f CachedSize;

	/** Cache where to wrap text at? */
	float CachedWrapTextAt;

	/** Cache the auto wrap text value */
	bool bCachedAutoWrapText;

	/** The state of the text the last time it was updated (used to allow updates when the text is changed) */
	FTextSnapshot TextLastUpdate;

	/** The state of the highlight text the last time it was updated (used to allow updates when the text is changed) */
	FTextSnapshot HighlightTextLastUpdate;
};
