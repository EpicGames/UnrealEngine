// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Containers/ObservableArray.h"
#include "Widgets/Views/STableViewBase.h"

namespace UE::Slate::ItemsSource
{
template<typename ArgType>
struct ForwardedSlateItemsSourceArgument
{
	const TArray<ArgType>* ArrayPointer;
	::UE::Slate::Containers::TObservableArray<ArgType>* ObservableArrayPointer;
	TSharedPtr<::UE::Slate::Containers::TObservableArray<ArgType>> SharedObservableArray;
};

#define SLATE_ITEMS_SOURCE_ARGUMENT( ArgType, ArgName ) \
	private: \
		const TArray<ArgType>* _##ArgName##_ArrayPointer = nullptr; \
		::UE::Slate::Containers::TObservableArray<ArgType>* _##ArgName##_ObservableArrayPointer = nullptr; \
		TSharedPtr<::UE::Slate::Containers::TObservableArray<ArgType>> _##ArgName##_SharedObservableArray; \
		void _Reset##ArgName() \
		{ \
			_##ArgName##_ArrayPointer = nullptr; \
			_##ArgName##_ObservableArrayPointer = nullptr; \
			_##ArgName##_SharedObservableArray.Reset(); \
		} \
	public: \
		WidgetArgsType& ArgName(const TArray<ArgType>* InArg) \
		{ \
			_Reset##ArgName(); \
			_##ArgName##_ArrayPointer = InArg; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		WidgetArgsType& ArgName(::UE::Slate::Containers::TObservableArray<ArgType>* InArg) \
		{ \
			_Reset##ArgName(); \
			_##ArgName##_ObservableArrayPointer = InArg; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		WidgetArgsType& ArgName(TSharedPtr<::UE::Slate::Containers::TObservableArray<ArgType>> InArg) \
		{ \
			_Reset##ArgName(); \
			_##ArgName##_SharedObservableArray = InArg; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		WidgetArgsType& ArgName(TSharedRef<::UE::Slate::Containers::TObservableArray<ArgType>> InArg) \
		{ \
			_Reset##ArgName(); \
			_##ArgName##_SharedObservableArray = InArg; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		WidgetArgsType& ArgName(::UE::Slate::ItemsSource::ForwardedSlateItemsSourceArgument<ArgType> InArg) \
		{ \
			_Reset##ArgName(); \
			_##ArgName##_ArrayPointer = InArg.ArrayPointer; \
			_##ArgName##_ObservableArrayPointer = InArg.ObservableArrayPointer; \
			_##ArgName##_SharedObservableArray = InArg.SharedObservableArray; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		::UE::Slate::ItemsSource::ForwardedSlateItemsSourceArgument<ArgType> Get##ArgName() const \
		{ \
			return {_##ArgName##_ArrayPointer, _##ArgName##_ObservableArrayPointer, _##ArgName##_SharedObservableArray}; \
		} \
		TUniquePtr<::UE::Slate::ItemsSource::IItemsSource<ArgType>> Make##ArgName(TSharedRef<STableViewBase> InWidget) const \
		{ \
			if (_##ArgName##_ArrayPointer) \
			{ \
				return MakeUnique<::UE::Slate::ItemsSource::FArrayPointer<ArgType>>(_##ArgName##_ArrayPointer); \
			} \
			else if (_##ArgName##_ObservableArrayPointer) \
			{ \
				return MakeUnique<::UE::Slate::ItemsSource::FObservableArrayPointer<ArgType>>(InWidget, _##ArgName##_ObservableArrayPointer); \
			} \
			else if (_##ArgName##_SharedObservableArray) \
			{ \
				return MakeUnique<::UE::Slate::ItemsSource::FSharedObservableArray<ArgType>>(InWidget, _##ArgName##_SharedObservableArray.ToSharedRef()); \
			} \
			return TUniquePtr<::UE::Slate::ItemsSource::IItemsSource<ArgType>>(); \
		}
		
		
 /*
  * A generic container for TableView items.
  */
 template <typename ItemType>
class IItemsSource
{
public:
	virtual ~IItemsSource() = default;
	/** Returns all the items in the source. */
	virtual const TArrayView<const ItemType> GetItems() const = 0;
	/** Test if the source has the same origin. */
	virtual bool IsSame(const void* RawPointer) const = 0;
};


/*
 *
 */
template <typename ItemType>
class FArrayPointer : public IItemsSource<ItemType>
{
public:
	explicit FArrayPointer(const TArray<ItemType>* InItemsSource)
		: ItemsSource(InItemsSource)
	{
	}

	virtual const TArrayView<const ItemType> GetItems() const override
	{
		return *ItemsSource;
	}

	virtual bool IsSame(const void* RawPointer) const override
	{
		return RawPointer == reinterpret_cast<const void*>(ItemsSource);
	}

private:
	const TArray<ItemType>* ItemsSource;
};


/*
 *
 */
template<typename InItemType>
class FObservableArrayPointer : public IItemsSource<InItemType>
{
public:
	using WidgetType = STableViewBase;
	using ItemType = InItemType;

	explicit FObservableArrayPointer(TSharedRef<WidgetType> InListView, ::UE::Slate::Containers::TObservableArray<ItemType>* InItemsSource)
		: ItemsSource(InItemsSource)
		, ListViewOwner(InListView)
	{
		ArrayChangedHandle = InItemsSource->OnArrayChanged().AddRaw(this, &FObservableArrayPointer::HandleArrayChanged);
	}

	virtual ~FObservableArrayPointer()
	{
		/**
		 * This is likely due to
		 * class SMyWidget : SCoumpoundWidget
		 * {
		 *   TObservableArray<TSharedPtr<int32>> MyArray;
		 *   SListView<TSharedPtr<int32>> MyList;
		 *   void Construct(const FArguments&)
		 *   {
		 *       MyList = SNew(SListView<TSharedPtr<int32>>)
		 *              .ItemsSource(&MyArray);
		 *   }
		 *   virtual ~SMyWidget()
		 *   {
		 *       // Remove the source to clear the binding before the ~SListView is called on MyList.
		 *       MyList->SetItemsSource(nullptr);
		 *   }
		 * };
		 * If you can do this pattern, uses a TSharedPtr<TObservableArray<TSharedPtr<>> instead.
		 */
		checkf(ListViewOwner.IsValid(), TEXT("The View widget has a source needed to be released to prevent bad memory access."));
		ItemsSource->OnArrayChanged().Remove(ArrayChangedHandle);
	}

	virtual const TArrayView<const ItemType> GetItems() const override
	{
		return TArrayView<const ItemType>(ItemsSource->GetData(), ItemsSource->Num());
	}

	virtual bool IsSame(const void* RawPointer) const override
	{
		return RawPointer == reinterpret_cast<const void*>(&ItemsSource);
	}

private:
	void HandleArrayChanged(typename ::UE::Slate::Containers::TObservableArray<ItemType>::ObservableArrayChangedArgsType Args)
	{
		if (TSharedPtr<WidgetType> ListViewOwnerPin = ListViewOwner.Pin())
		{
			ListViewOwnerPin->RequestListRefresh();
		}
	}

private:
	UE::Slate::Containers::TObservableArray<ItemType>* ItemsSource;
	TWeakPtr<WidgetType> ListViewOwner;
	FDelegateHandle ArrayChangedHandle;
};

/*
 *
 */
template<typename InItemType>
class FSharedObservableArray : public IItemsSource<InItemType>
{
public:
	using WidgetType = STableViewBase;
	using ItemType = InItemType;

	explicit FSharedObservableArray(TSharedRef<WidgetType> InListView, TSharedRef<::UE::Slate::Containers::TObservableArray<ItemType>> InItemsSource)
		: ItemsSource(InItemsSource)
		, ListViewOwner(InListView)
	{
		ArrayChangedHandle = InItemsSource->OnArrayChanged().AddRaw(this, &FSharedObservableArray::HandleArrayChanged);
	}

	virtual ~FSharedObservableArray()
	{
		ItemsSource->OnArrayChanged().Remove(ArrayChangedHandle);
	}

	virtual const TArrayView<const ItemType> GetItems() const override
	{
		return TArrayView<const ItemType>(ItemsSource->GetData(), ItemsSource->Num());
	}

	virtual bool IsSame(const void* RawPointer) const override
	{
		UE::Slate::Containers::TObservableArray<ItemType>* ValueToTest = &(ItemsSource.Get());
		return RawPointer == reinterpret_cast<const void*>(ValueToTest);
	}

private:
	void HandleArrayChanged(typename ::UE::Slate::Containers::TObservableArray<ItemType>::ObservableArrayChangedArgsType Args)
	{
		if (TSharedPtr<WidgetType> ListViewOwnerPin = ListViewOwner.Pin())
		{
			ListViewOwnerPin->RequestListRefresh();
		}
	}

private:
	TSharedRef<UE::Slate::Containers::TObservableArray<ItemType>> ItemsSource;
	TWeakPtr<WidgetType> ListViewOwner;
	FDelegateHandle ArrayChangedHandle;
};

} //UE::Slate::ItemsSource
