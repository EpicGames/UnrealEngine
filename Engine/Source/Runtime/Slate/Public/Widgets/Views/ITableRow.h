// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Framework/Views/ITypedTableView.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

/**
 * Interface for table views to talk to their rows.
 */
class ITableRow
{
public:

	/**
	* Called when the row has been generated and associated with an item in the owning table.
	* Any attempts to access the item associated with the row prior to this (i.e. in Construct()) will fail, as the association is not yet established in the owning table.
	*/
	virtual void InitializeRow() = 0;

	/**
	* Called when the row has been released from the owning table and is no longer associated with any items therein.
	* Only relevant if the row widgets are pooled or otherwise referenced/kept alive outside the owning table. Otherwise, the row is destroyed.
	*/
	virtual void ResetRow() = 0;

	/**
	* @param InIndexInList  The index of the item for which this widget was generated
	*/
	virtual void SetIndexInList(int32 InIndexInList) = 0;

	/** @return  The index of the item for which this widget was generated */
	virtual int32 GetIndexInList() = 0;

	/** @return true if the corresponding item is expanded; false otherwise*/
	virtual bool IsItemExpanded() const = 0;

	/** Toggle the expansion of the item associated with this row */
	virtual void ToggleExpansion() = 0;

	/** @return True if the corresponding item is selected; false otherwise */
	virtual bool IsItemSelected() const = 0;

	/** @return how nested the item associated with this row when it is in a TreeView */
	virtual int32 GetIndentLevel() const = 0;

	/** @return Does this item have children? */
	virtual int32 DoesItemHaveChildren() const = 0;

	/** @return BitArray where each entry corresponds to whether this item needs a vertical wire draw for that depth. */
	virtual TBitArray<> GetWiresNeededByDepth() const = 0;

	/** @return true if this item is the last direct descendant of its parent. */
	virtual bool IsLastChild() const = 0;
		
	/** @return this table row as a widget */
	virtual TSharedRef<SWidget> AsWidget() = 0;

	/** @return the content of this table row */
	virtual TSharedPtr<SWidget> GetContent() = 0;

	/** Called when the expander arrow for this row is shift+clicked */
	virtual void Private_OnExpanderArrowShiftClicked() = 0;

	/** @return the size for the specified column name */
	virtual FVector2D GetRowSizeForColumn(const FName& InColumnName) const = 0;

	/** @return the UUserWidget that backs this row, or nullptr if the row is not a UMG object row. */
	virtual class UUserWidget* GetUserWidget() const { return nullptr; }

protected:
	/** Called to query the selection mode for the row */
	virtual ESelectionMode::Type GetSelectionMode() const = 0;
};
