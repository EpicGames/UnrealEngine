// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"

class ITableRow;
class SButton;

template<>
struct TWidgetTypeTraits<class SExpanderArrow>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Expander arrow and indentation component that can be placed in a TableRow
 * of a TreeView. Intended for use by TMultiColumnRow in TreeViews.
 */
class SExpanderArrow : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SExpanderArrow, SCompoundWidget, SLATE_API)
public:

	SLATE_BEGIN_ARGS( SExpanderArrow )
		: _StyleSet(&FCoreStyle::Get())
		, _IndentAmount(10)
		, _BaseIndentLevel(0)
		, _ShouldDrawWires(false)
	{ }
		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		/** How many Slate Units to indent for every level of the tree. */
		SLATE_ATTRIBUTE(float, IndentAmount)
		/** The level that the root of the tree should start (e.g. 2 will shift the whole tree over by `IndentAmount*2`) */
		SLATE_ATTRIBUTE(int32, BaseIndentLevel)
		/** Whether to draw the wires that visually reinforce the tree hierarchy. */
		SLATE_ATTRIBUTE(bool, ShouldDrawWires)
	SLATE_END_ARGS()

	SLATE_API SExpanderArrow();
	SLATE_API ~SExpanderArrow();
	SLATE_API void Construct( const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow );

protected:

	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Invoked when the expanded button is clicked (toggle item expansion) */
	SLATE_API FReply OnArrowClicked();

	SLATE_API void SetIndentAmount(TAttribute<float> InIndentAmount);
	SLATE_API void SetBaseIndentLevel(TAttribute<int32> InBaseIndentLevel);
	float GetIndentAmount() const { return IndentAmountAttribute.Get(); }
	int32 GetBaseIndentLevel() const { return BaseIndentLevelAttribute.Get(); }

	TSlateAttributeRef<float> GetIndentAmountAttribute() const
	{
		return TSlateAttributeRef<float>{SharedThis(this), IndentAmountAttribute};
	}

	TSlateAttributeRef<int32> GetBaseIndentLevelAttribute() const
	{
		return TSlateAttributeRef<int32>{SharedThis(this), BaseIndentLevelAttribute};
	}

	/** @return Visible when has children; invisible otherwise */
	SLATE_API EVisibility GetExpanderVisibility() const;

	/** @return the margin corresponding to how far this item is indented */
	SLATE_API FMargin GetExpanderPadding() const;

	/** @return the name of an image that should be shown as the expander arrow */
	SLATE_API const FSlateBrush* GetExpanderImage() const;

	TWeakPtr<class ITableRow> OwnerRowPtr;

	/** A reference to the expander button */
	TSharedPtr<SButton> ExpanderArrow;

	/** The slate style to use */
	const ISlateStyle* StyleSet;

#if WITH_EDITORONLY_DATA
	/** The amount of space to indent at each level */
	UE_DEPRECATED(5.7, "Use SetIndentAmount / GetIndentAmount / GetIndentAmountAttribute")
	TSlateDeprecatedTAttribute<float> IndentAmount;

	/** The level in the tree that begins the indention amount */
	UE_DEPRECATED(5.7, "Use SetBaseIndentLevel / GetBaseIndentLevel / GetBaseIndentLevelAttribute")
	TSlateDeprecatedTAttribute<int32> BaseIndentLevel;
#endif

	/** Whether to draw the wires that visually reinforce the tree hierarchy. */
	TSlateAttribute<bool> ShouldDrawWires;

private:
	/** The amount of space to indent at each level */
	TSlateAttribute<float> IndentAmountAttribute;
	/** The level in the tree that begins the indention amount */
	TSlateAttribute<int32> BaseIndentLevelAttribute;
};
