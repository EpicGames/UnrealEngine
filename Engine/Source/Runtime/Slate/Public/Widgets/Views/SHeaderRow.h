// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/IndirectArray.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/Layout/SSplitter.h"
#include "Framework/SlateDelegates.h"

class FMenuBuilder;
class SScrollBar;
enum class ECheckBoxState : uint8;

namespace EColumnSortPriority
{
	enum Type
	{
		None,
		Primary = 0,
		Secondary = 1,
		Max,
	};
};


namespace EColumnSortMode
{
	enum Type
	{
		/** Unsorted */
		None = 0,

		/** Ascending */
		Ascending = 1,

		/** Descending */
		Descending = 2
	};
};


namespace EColumnSizeMode
{
	enum Type
	{
		/** Column stretches to a fraction of the header row */
		Fill = 0,
		
		/**	Column is fixed width and cannot be resized */
		Fixed = 1,

		/** Column is set to a width which can be user-sized */
		Manual = 2,

		/** Column stretches as Fill but is initialized with a width */
		FillSized = 3,
	};
};


enum class EHeaderComboVisibility
{
	/** Always show the drop down at full opacity */
	Always,

	/** Always show the drop down, but in a ghosted way when not hovered */
	Ghosted,

	/** Only show the drop down when hovered */
	OnHover,

	/** Never show the drop down. Context Menu can still be opened with a context click (such as right-click) */
	Never
};


/** Callback when sort mode changes */
DECLARE_DELEGATE_ThreeParams( FOnSortModeChanged, EColumnSortPriority::Type, const FName&, EColumnSortMode::Type );

/**	Callback when the width of the column changes */
DECLARE_DELEGATE_OneParam( FOnWidthChanged, float );

/**	Callback to fetch the max row width for a specified column id */
DECLARE_DELEGATE_RetVal_TwoParams(FVector2D, FOnGetMaxRowSizeForColumn, const FName&, EOrientation);

template <>
struct TWidgetTypeTraits<class SHeaderRow>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * The header that appears above lists and trees when they are showing multiple columns.
 */
class SHeaderRow : public SBorder
{
public:
	/** Describes a single column header */
	class FColumn
	{
	public:

		SLATE_BEGIN_ARGS(FColumn)
			: _ColumnId()
			, _DefaultLabel()
			, _DefaultTooltip()
			, _ToolTip()
			, _FillWidth( 1.0f )
			, _FixedWidth()
			, _ManualWidth()
			, _OnWidthChanged()
			, _MinSize( 0.0f )
			, _HeaderContent()
			, _HAlignHeader( HAlign_Fill )
			, _VAlignHeader( VAlign_Fill )
			, _HeaderContentPadding()
			, _HeaderComboVisibility(EHeaderComboVisibility::OnHover)
			, _MenuContent()
			, _HAlignCell( HAlign_Fill )
			, _VAlignCell( VAlign_Fill )
			, _InitialSortMode( EColumnSortMode::Ascending )
			, _SortMode( EColumnSortMode::None )
			, _IsSorting( false )
			, _CanManuallySort(true)
			, _OnSort()
			, _OverflowPolicy( ETextOverflowPolicy::Clip)
			{}
			SLATE_ARGUMENT( FName, ColumnId )
			SLATE_ATTRIBUTE( FText, DefaultLabel )
			SLATE_ATTRIBUTE( FText, DefaultTooltip )
			SLATE_ATTRIBUTE( TSharedPtr< IToolTip >, ToolTip )

			/** Set the Column Size Mode to Fill. It's a fraction between 0 and 1 */
			SLATE_ATTRIBUTE( float, FillWidth )
			/** Set the Column Size Mode to Fixed. */
			SLATE_ARGUMENT( TOptional< float >, FixedWidth )
			/** Set the Column Size Mode to Manual. */
			SLATE_ATTRIBUTE( float, ManualWidth )
			/** Set the Column Size Mode to Fill Sized. */
			SLATE_ARGUMENT(TOptional< float >, FillSized)
			SLATE_EVENT( FOnWidthChanged, OnWidthChanged )

			/** Set the minimum column size when resizing. */
			SLATE_ARGUMENT(float, MinSize)

			SLATE_DEFAULT_SLOT( FArguments, HeaderContent )
			SLATE_ARGUMENT( EHorizontalAlignment, HAlignHeader )
			SLATE_ARGUMENT( EVerticalAlignment, VAlignHeader )
			SLATE_ARGUMENT( TOptional< FMargin >, HeaderContentPadding )
			SLATE_ARGUMENT( EHeaderComboVisibility, HeaderComboVisibility )

			SLATE_NAMED_SLOT( FArguments, MenuContent )
			SLATE_EVENT( FOnGetContent, OnGetMenuContent )

			/** Allow to provide a custom handler when a splitter is double-clicked in Manual column size mode. */
			SLATE_EVENT( FPointerEventHandler, OnColumnSplitterDoubleClick )

			SLATE_ARGUMENT( EHorizontalAlignment, HAlignCell )
			SLATE_ARGUMENT( EVerticalAlignment, VAlignCell )

			SLATE_ATTRIBUTE( EColumnSortMode::Type, InitialSortMode )
			SLATE_ATTRIBUTE( EColumnSortMode::Type, SortMode )
			SLATE_ATTRIBUTE( EColumnSortPriority::Type, SortPriority )
			SLATE_ATTRIBUTE( bool, IsSorting )
			SLATE_ATTRIBUTE( bool, CanManuallySort)
			SLATE_EVENT( FOnSortModeChanged, OnSort )

			SLATE_ATTRIBUTE(bool, ShouldGenerateWidget)
			SLATE_ATTRIBUTE(bool, ShouldGenerateEmptyWidgetForSpacing)
			SLATE_ARGUMENT(ETextOverflowPolicy, OverflowPolicy)
		SLATE_END_ARGS()

		FColumn( const FArguments& InArgs )
			: ColumnId( InArgs._ColumnId )
			, DefaultText( InArgs._DefaultLabel )
			, DefaultTooltip( InArgs._DefaultTooltip )
			, ToolTip(InArgs._ToolTip)
			, Width( 1.0f )
			, DefaultWidth( 1.0f )
			, OnWidthChanged( InArgs._OnWidthChanged)
			, MinSize( InArgs._MinSize )
			, SizeRule( EColumnSizeMode::Fill )
			, HeaderContent( InArgs._HeaderContent )
			, HeaderMenuContent( InArgs._MenuContent )
			, OnGetMenuContent( InArgs._OnGetMenuContent )
			, OnColumnSplitterDoubleClick( InArgs._OnColumnSplitterDoubleClick )
			, HeaderHAlignment( InArgs._HAlignHeader )
			, HeaderVAlignment( InArgs._VAlignHeader )
			, HeaderContentPadding( InArgs._HeaderContentPadding )
			, HeaderComboVisibility (InArgs._HeaderComboVisibility )
			, CellHAlignment( InArgs._HAlignCell )
			, CellVAlignment( InArgs._VAlignCell )
			, OverflowPolicy(InArgs._OverflowPolicy)
			, InitialSortMode( InArgs._InitialSortMode )
			, SortMode( InArgs._SortMode )
			, SortPriority( InArgs._SortPriority )
			, IsSorting( InArgs._IsSorting)
			, CanManuallySort(InArgs._CanManuallySort)
			, OnSortModeChanged( InArgs._OnSort )
			, ShouldGenerateWidget(InArgs._ShouldGenerateWidget)
			, ShouldGenerateEmptyWidgetForSpacing(InArgs._ShouldGenerateEmptyWidgetForSpacing)
			, bIsVisible(true)
			{
			if ( InArgs._FixedWidth.IsSet() )
			{
				Width = InArgs._FixedWidth.GetValue();
				SizeRule = EColumnSizeMode::Fixed;
			}
			else if ( InArgs._ManualWidth.IsSet() )
			{
				Width = InArgs._ManualWidth;
				SizeRule = EColumnSizeMode::Manual;
			}
			else if ( InArgs._FillSized.IsSet() )
			{
				Width = InArgs._FillSized.GetValue();
				SizeRule = EColumnSizeMode::FillSized;
			}
			else
			{
				Width = InArgs._FillWidth;
				SizeRule = EColumnSizeMode::Fill;
			}

			DefaultWidth = Width.Get();
		}

	public:

		void SetWidth( float NewWidth )
		{
			if ( OnWidthChanged.IsBound() )
			{
				OnWidthChanged.Execute( NewWidth );
			}
			else
			{
				Width = NewWidth;
			}
		}

		float GetWidth() const
		{
			return Width.Get();
		}

		/** A unique ID for this column, so that it can be saved and restored. */
		FName ColumnId;

		/** Default text to use if no widget is passed in. */
		TAttribute< FText > DefaultText;

		/** Default tooltip to use if no widget is passed in */
		TAttribute< FText > DefaultTooltip;

		/** Custom tooltip to use */
		TAttribute< TSharedPtr< IToolTip > > ToolTip;

		/** A column width in Slate Units */
		TAttribute< float > Width;

		/** A original column width in Slate Units */
		float DefaultWidth;

		FOnWidthChanged OnWidthChanged;

		/** The minimum column size when resizing. */
		float MinSize;

		EColumnSizeMode::Type SizeRule;

		TAlwaysValidWidget HeaderContent;	
		TAlwaysValidWidget HeaderMenuContent;

		FOnGetContent OnGetMenuContent;
		FPointerEventHandler OnColumnSplitterDoubleClick;

		EHorizontalAlignment HeaderHAlignment;
		EVerticalAlignment HeaderVAlignment;
		TOptional< FMargin > HeaderContentPadding;
		EHeaderComboVisibility HeaderComboVisibility;

		EHorizontalAlignment CellHAlignment;
		EVerticalAlignment CellVAlignment;

		ETextOverflowPolicy OverflowPolicy;

		TAttribute< EColumnSortMode::Type > InitialSortMode;
		TAttribute< EColumnSortMode::Type > SortMode;
		TAttribute< EColumnSortPriority::Type > SortPriority;
		TAttribute< bool > IsSorting;
		TAttribute< bool > CanManuallySort;
		FOnSortModeChanged OnSortModeChanged;

		TAttribute<bool> ShouldGenerateWidget;
		TAttribute<bool> ShouldGenerateEmptyWidgetForSpacing;
		bool bIsVisible;
	};

	/** Create a column with the specified ColumnId */
	static FColumn::FArguments Column( const FName& InColumnId )
	{
		FColumn::FArguments NewArgs;
		NewArgs.ColumnId( InColumnId ); 
		return NewArgs;
	}

	DECLARE_EVENT_OneParam( SHeaderRow, FColumnsChanged, const TSharedRef< SHeaderRow >& );
	FColumnsChanged* OnColumnsChanged() { return &ColumnsChanged; }

	SLATE_BEGIN_ARGS(SHeaderRow)
		: _Style( &FAppStyle::Get().GetWidgetStyle<FHeaderRowStyle>("TableView.Header") )
		, _ResizeMode(ESplitterResizeMode::Fill)
		, _CanSelectGeneratedColumn(false)
		{}

		SLATE_STYLE_ARGUMENT( FHeaderRowStyle, Style )
		SLATE_SUPPORTS_SLOT_WITH_ARGS( FColumn )	
		SLATE_EVENT( FColumnsChanged::FDelegate, OnColumnsChanged )
		SLATE_EVENT( FOnGetMaxRowSizeForColumn, OnGetMaxRowSizeForColumn )
		SLATE_ARGUMENT( ESplitterResizeMode::Type, ResizeMode )
		SLATE_ARGUMENT( TOptional<float>, SplitterHandleSize )
		/**
		 * Can select the columns generated by right clicking on the header menu.
		 * FColumn with ShouldGenerateWidget set can not be selected.
		 * FColumn with MenuContent will still be displayed.
		 */
		SLATE_ARGUMENT( bool, CanSelectGeneratedColumn )
		/** The list of column ids that should not be generated by default. */
		SLATE_ARGUMENT( TArray<FName>, HiddenColumnsList )
		/** Triggered when columns visibility changed. */
		SLATE_EVENT( FSimpleDelegate, OnHiddenColumnsListChanged )
	SLATE_END_ARGS()

	SLATE_API void Construct( const FArguments& InArgs );

	/** Restore the columns to their original width */
	SLATE_API void ResetColumnWidths();

	/** @return the Columns driven by the column headers */
	SLATE_API const TIndirectArray<FColumn>& GetColumns() const;

	/** @return the index of the column associated to the specified name */
	SLATE_API int32 FindColumnIndex( const FName& InColumnId ) const;

	/** Adds a column to the header */
	SLATE_API void AddColumn( const FColumn::FArguments& NewColumnArgs );
	SLATE_API void AddColumns( TConstArrayView<FColumn::FArguments> NewColumns );
	SLATE_API void AddColumn( FColumn& NewColumn );

	/** Inserts a column at the specified index in the header */
	SLATE_API void InsertColumn( const FColumn::FArguments& NewColumnArgs, int32 InsertIdx );
	SLATE_API void InsertColumn( FColumn& NewColumn, int32 InsertIdx );

	/** Removes a column from the header */
	SLATE_API void RemoveColumn( const FName& InColumnId );
	SLATE_API void RemoveColumns( const int32 StartIdx, const int32 Count );

	/** Swap two columns at the specified indices */
	SLATE_API void SwapColumns( const int32 FirstColumnIdx, const int32 SecondColumnIdx );

	/** Force refreshing of the column widgets*/
	SLATE_API void RefreshColumns();

	/** Removes all columns from the header */
	SLATE_API void ClearColumns();

	SLATE_API void SetAssociatedVerticalScrollBar( const TSharedRef< SScrollBar >& ScrollBar, const float ScrollBarSize );

	/** Sets the column, with the specified name, to the desired width */
	SLATE_API void SetColumnWidth( const FName& InColumnId, float InWidth );

	/** Will return the size for this row at the specified slot index */
	SLATE_API FVector2D GetRowSizeForSlotIndex(int32 SlotIndex) const;

	/** Simple function to set the delegate to fetch the max row size for column id */
	void SetOnGetMaxRowSizeForColumn(const FOnGetMaxRowSizeForColumn& Delegate) { OnGetMaxRowSizeForColumn = Delegate; }

	/** @return The columns id of the header widgets that were not generated */
	SLATE_API TArray<FName> GetHiddenColumnIds() const;

	/** @return is the header widget should be generated */
	SLATE_API bool ShouldGeneratedColumn( const FName& InColumnId ) const;

	/** @return if we should generate this column even if empty, some users want to skip this to allow for large headers */
	SLATE_API bool ShouldGenerateEmptyColumnForSpacing( const FName& InColumnId ) const;

	/** @return is the header widget with the column id is generated */
	SLATE_API bool IsColumnGenerated( const FName& InColumnId ) const;

	/** @return true if the header widget with the column id is visible */
	SLATE_API bool IsColumnVisible( const FName& InColumnId ) const;

	//~ Begin SWidget interface
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	//~ End SWidget interface

	/** Show or Hide a generated column */
	SLATE_API void SetShowGeneratedColumn(const FName& InColumnIdm, bool InShow = true);


private:

	/** Regenerates all widgets in the header */
	SLATE_API void RegenerateWidgets();

	void ToggleAllColumns();
	bool CanToggleAllColumns() const;
	ECheckBoxState GetSelectAllColumnsCheckState() const;
	FText GetToggleAllColumnsText() const;

	void OnGenerateSelectColumnsSubMenu(FMenuBuilder& InSubMenuBuilder);
	SLATE_API void ToggleGeneratedColumn(FName ColumnId);
	SLATE_API ECheckBoxState GetGeneratedColumnCheckedState(FName ColumnId) const;

	/** Information about the various columns */
	TIndirectArray<FColumn> Columns;
	TArray<TSharedPtr<class STableColumnHeader>> HeaderWidgets;

	FVector2f ScrollBarThickness;
	TAttribute< EVisibility > ScrollBarVisibility;
	const FHeaderRowStyle* Style;
	FColumnsChanged ColumnsChanged;
	ESplitterResizeMode::Type ResizeMode;
	float SplitterHandleSize;
	FOnGetMaxRowSizeForColumn OnGetMaxRowSizeForColumn;

	FSimpleDelegate OnHiddenColumnsListChanged;
	bool bCanSelectGeneratedColumn;
};
