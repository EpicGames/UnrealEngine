// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Containers/ObservableArray.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Layout/Overscroll.h"
#include "Framework/Views/ITypedTableView.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Types/SlateConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/IItemsSource.h"
#include "Application/SlateApplicationBase.h"
#include "Framework/Application/SlateApplication.h"
#if WITH_ACCESSIBILITY
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleWidgetCache.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

template <typename ItemType>
class SListView;

template <typename ItemType>
struct TWidgetTypeTraits<SListView<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

namespace UE::Slate::Private
{
	template <typename ItemType>
	class SListViewPinnedRowWidget;
}

template <typename ItemType>
struct TWidgetTypeTraits<class UE::Slate::Private::SListViewPinnedRowWidget<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


namespace UE::Slate::Private
{
	/**
	 * Shared navigation context written by SListView::OnNavigation and read by both
	 * SListView::OnNavigation (entry with no selection) and SCommonListView::OnFocusReceived.
	 *
	 * Feature 1 (cross-list): written when a list escapes its boundary with
	 * bEnableProximateEntryNavigation = true, consumed by the receiving list to select the spatially
	 * nearest boundary item instead of always defaulting to item 0.
	 *
	 * Feature 2 (intra-entry): the SourceWidget field is also used by SCommonListView::OnNavigation
	 * to seed FindNextFocusableWidget when routing focus into a non-CommonButtonBase entry.
	 */

	/**
	 * Navigation context payload written by SListView::OnNavigation when escaping a boundary.
	 * Set via FSlateApplication::SetPendingNavigationContext; read during OnFocusReceived on
	 * the destination list. Cleared automatically at the end of each navigation event.
	 */
	struct FListViewNavigationContext : ISlateMetaData
	{
		SLATE_METADATA_TYPE(FListViewNavigationContext, ISlateMetaData)

		/** The widget that held focus on the departing list when it escaped its boundary. */
		TWeakPtr<SWidget> SourceWidget;

		/** Navigation direction that triggered the escape. */
		EUINavigation NavigationType = EUINavigation::Invalid;
	};
} // namespace UE::Slate::Private

namespace UE::Slate::Private
{
	// Private class that acts as a wrapper around PinnedRows, to allow for customized styling
	template <typename ItemType>
	class SListViewPinnedRowWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SListViewPinnedRowWidget) {}

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<ITableRow> InPinnedItemRow, TSharedRef<SListView<ItemType>> InOwnerListView, const int32 ItemIndex, const int32 NumPinnedItems)
		{
			PinnedItemRow = InPinnedItemRow;
			OwnerListView = InOwnerListView;

			TSharedPtr<STableRow<ItemType>> PinnedRow = StaticCastSharedPtr<STableRow<ItemType>>(InPinnedItemRow);

			// If the PinnedRow inherits from STableRow (i.e has a custom border already), remove it since we will be adding our own border
			// Also set the expander arrow to Hidden so it is not available for pinned rows, but still occupies the same space
			if (PinnedRow.IsValid())
			{
				PinnedRow->SetBorderImage(FAppStyle::Get().GetBrush("NoBrush"));
				PinnedRow->SetExpanderArrowVisibility(EVisibility::Hidden);
			}

			TSharedRef<SWidget> InPinnedItemRowWidget = InPinnedItemRow->AsWidget();
			InPinnedItemRowWidget->SetVisibility(EVisibility::HitTestInvisible);

			ChildSlot
				[
					SNew(SOverlay)
					.Visibility(TAttribute<EVisibility>::CreateSP(this, &SListViewPinnedRowWidget::SetPinnedItemVisibility, ItemIndex, NumPinnedItems))

					+ SOverlay::Slot()
					.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(SBorder)
						.BorderImage_Lambda([this]()
							{
								return this->IsHovered() ? FAppStyle::Get().GetBrush("Brushes.Hover") : FAppStyle::Get().GetBrush("Brushes.Header");
							})
						.Padding(0.f)
						.Content()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								InPinnedItemRowWidget
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(2.0f, 2.0f, 0.0f, 0.0f)
							[
								// Text Block for ellipses, shows up when some items in the pinned list are collapsed when the number of items > MaxPinnedItems
								SNew(STextBlock).Text(NSLOCTEXT("SListView", "Ellipses", "..."))
								.Visibility(this, &SListViewPinnedRowWidget::SetPinnedItemEllipsesVisibility, ItemIndex)
							]
						]
					]
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						// A shadow to indicate parent/child relationship
						SNew(SImage)
						.Visibility(EVisibility::HitTestInvisible)
						.Image(FAppStyle::Get().GetBrush("ListView.PinnedItemShadow"))
					]
					
				];

		}

	protected:

		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				const TObjectPtrWrapTypeOf<ItemType>* PinnedItem = OwnerListView->ItemFromWidget(PinnedItemRow.Get());

				if (PinnedItem)
				{
					// Select the pinned item on click 
					OwnerListView->NavigationSelect(*PinnedItem, MouseEvent);
					return FReply::Handled();
				}
			}

			return FReply::Unhandled();
		}

	private:

		EVisibility SetPinnedItemVisibility(const int32 IndexInList, const int32 NumPinnedItems) const
		{
			// If the hierarchy is not collapsed (i.e all items are visible)
			if (!OwnerListView->bIsHierarchyCollapsed)
			{
				return EVisibility::Visible;
			}

			int32 CurrentMaxPinnedItems = OwnerListView->GetMaxPinnedItems();

			// If this is the last item, it is visible
			if (IndexInList == NumPinnedItems - 1)
			{
				return EVisibility::Visible;
			}
			// Only show a limited number of items depending on MaxPinnedItems
			else if (IndexInList < CurrentMaxPinnedItems - 1)
			{
				return EVisibility::Visible;
			}

			return EVisibility::Collapsed;
		}

		EVisibility SetPinnedItemEllipsesVisibility(const int32 IndexInList) const
		{
			// If all items are visible, the ...'s are never visible
			if (!OwnerListView->bIsHierarchyCollapsed)
			{
				return EVisibility::Collapsed;
			}

			int32 CurrentMaxPinnedItems = OwnerListView->GetMaxPinnedItems();

			// If the hierarchy is collapsed, the last item before the collapsed items gets the ellipses
			return IndexInList == CurrentMaxPinnedItems - 2 ? EVisibility::Visible : EVisibility::Collapsed;
		}

	private:
		// A pointer to the ListView that owns this widget
		TSharedPtr<SListView<ItemType>> OwnerListView;

		// A pointer to the row contained by this widget
 		TSharedPtr<ITableRow> PinnedItemRow;
	};
}


/**
 * A ListView widget observes an array of data items and creates visual representations of these items.
 * ListView relies on the property that holding a reference to a value ensures its existence. In other words,
 * neither SListView<FText> nor SListView<FText*> are valid, while SListView< TSharedPtr<FText> > and
 * SListView< UObject* > are valid.
 *
 * A trivial use case appear below:
 *
 *   Given: TArray< TSharedPtr<FText> > Items;
 *
 *   SNew( SListView< TSharedPtr<FText> > )
 *     .ListItemsSource( &Items )
 *     .OnGenerateRow(this, &MyClass::GenerateItemRow)
 *
 * The ListView will create widgets based on data items
 * in the Items TArray. When the ListView needs to generate an item, it will do so using the specified OnGenerateRow method.
 *
 * A sample implementation of MyClass::GenerateItemRow has to return a STableRow with optional content:
 *
 * TSharedRef<ITableRow> MyClass::GenerateItemRow(TSharedPtr<FText> Item, const TSharedRef<STableViewBase>& OwnerTable)
 * {
 *     	return SNew(STableRow<TSharedPtr<FText>>, OwnerTable)
 *		[
 *			SNew(STextBlock)
 *			.Text(*Item)
 *		];
 * }
 */

template <typename ItemType>
class SListView : public STableViewBase, TListTypeTraits<ItemType>::SerializerType, public ITypedTableView< ItemType >
{
	SLATE_DECLARE_WIDGET(SListView, STableViewBase)
	static inline SLATE_IMPLEMENT_WIDGET(SListView)
public:
	using NullableItemType  = typename TListTypeTraits< ItemType >::NullableType;
	using MapKeyFuncs       = typename TListTypeTraits<ItemType>::MapKeyFuncs;
	using MapKeyFuncsSparse = typename TListTypeTraits<ItemType>::MapKeyFuncsSparse;
	
	using TItemSet          = TSet< TObjectPtrWrapTypeOf<ItemType>, typename TListTypeTraits< TObjectPtrWrapTypeOf<ItemType> >::SetKeyFuncs >;

	using FOnGenerateRow            = typename TSlateDelegates< ItemType >::FOnGenerateRow;
	using FOnRefreshRow				= typename TSlateDelegates< ItemType >::FOnRefreshRow;
	using FOnItemScrolledIntoView   = typename TSlateDelegates< ItemType >::FOnItemScrolledIntoView;
	using FOnSelectionChanged       = typename TSlateDelegates< NullableItemType >::FOnSelectionChanged;
	using FIsSelectableOrNavigable	= typename TSlateDelegates< ItemType >::FIsSelectableOrNavigable;
	using FOnMouseButtonClick       = typename TSlateDelegates< ItemType >::FOnMouseButtonClick;
	using FOnMouseButtonDoubleClick = typename TSlateDelegates< ItemType >::FOnMouseButtonDoubleClick;

	typedef typename TSlateDelegates< ItemType >::FOnItemToString_Debug FOnItemToString_Debug;

	DECLARE_DELEGATE_OneParam( FOnWidgetToBeRemoved, const TSharedRef<ITableRow>& );

	DECLARE_DELEGATE_TwoParams( FOnEntryInitialized, ItemType, const TSharedRef<ITableRow>& );

public:
	SLATE_BEGIN_ARGS(SListView<ItemType>)
		: _ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView"))
		, _bEnableShadowBoxStyle(false)
		, _OnGenerateRow()
		, _OnGeneratePinnedRow()
		, _OnRefreshRow()
		, _OnEntryInitialized()
		, _OnRowReleased()
		, _MaxPinnedItems(6)
		, _OnContextMenuOpening()
		, _OnItemsRebuilt()
		, _OnMouseButtonClick()
		, _OnMouseButtonDoubleClick()
		, _OnSelectionChanged()
		, _OnIsSelectableOrNavigable()
		, _SelectionMode(ESelectionMode::Multi)
		, _ClearSelectionOnClick(true)
		, _ExternalScrollbar()
		, _Orientation(Orient_Vertical)
		, _EnableAnimatedScrolling(false)
		, _ScrollbarDragFocusCause(EFocusCause::Mouse)
		, _AllowOverscroll(EAllowOverscroll::Yes)
		, _ScrollBarStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
		, _PreventThrottling(false)
		, _ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
		, _WheelScrollMultiplier(GetGlobalScrollAmount())
		, _NavigationScrollOffset(0.5f)
		, _ScrollBarPadding(0.0f)
		, _HandleGamepadEvents( true )
		, _HandleDirectionalNavigation( true )
		, _HandleSpacebarSelection(false)
		, _IsFocusable(true)
		, _ReturnFocusToSelection()
		, _ClearScrollVelocityOnSelection(true)
		, _OnItemToString_Debug()
		, _OnEnteredBadState()
		{
			this->_Clipping = EWidgetClipping::ClipToBounds;
		}

		SLATE_STYLE_ARGUMENT( FTableViewStyle, ListViewStyle )
		
		SLATE_STYLE_ARGUMENT( FScrollBoxStyle, ShadowBoxStyle )

		SLATE_ARGUMENT( bool, bEnableShadowBoxStyle)

		SLATE_EVENT( FOnGenerateRow, OnGenerateRow )

		SLATE_EVENT( FOnGenerateRow, OnGeneratePinnedRow )
		
		SLATE_EVENT( FOnRefreshRow, OnRefreshRow )

		SLATE_EVENT( FOnEntryInitialized, OnEntryInitialized )

		SLATE_EVENT( FOnWidgetToBeRemoved, OnRowReleased )

		SLATE_EVENT( FOnTableViewScrolled, OnListViewScrolled )

		SLATE_EVENT( FOnItemScrolledIntoView, OnItemScrolledIntoView )
		
		SLATE_EVENT( FOnFinishedScrolling, OnFinishedScrolling )

		SLATE_EVENT( FOnTouchStart, OnTouchStart )

		SLATE_EVENT( FOnTouchMove, OnTouchMove )

		SLATE_EVENT( FOnTouchEnd, OnTouchEnd )

		SLATE_ITEMS_SOURCE_ARGUMENT( ItemType, ListItemsSource )

		SLATE_ATTRIBUTE_DEPRECATED( float, ItemHeight, 5.5, "The ItemHeight is only used for Tile. See ShouldArrangeAsTiles")

		SLATE_ATTRIBUTE(int32, MaxPinnedItems)

		SLATE_EVENT( FOnContextMenuOpening, OnContextMenuOpening )

		SLATE_EVENT( FSimpleDelegate, OnItemsRebuilt )

		SLATE_EVENT(FOnMouseButtonClick, OnMouseButtonClick)

		SLATE_EVENT( FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick )

		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )

		SLATE_EVENT( FIsSelectableOrNavigable, OnIsSelectableOrNavigable)

		SLATE_ATTRIBUTE( ESelectionMode::Type, SelectionMode )

		SLATE_ARGUMENT( TSharedPtr<SHeaderRow>, HeaderRow )

		SLATE_ARGUMENT ( bool, ClearSelectionOnClick )

		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar )

		SLATE_ARGUMENT( EOrientation, Orientation )

		SLATE_ARGUMENT( EScrollIntoViewAlignment, ScrollIntoViewAlignment )

		SLATE_ARGUMENT( bool, EnableAnimatedScrolling)

		SLATE_ARGUMENT( TOptional<double>, FixedLineScrollOffset )

		SLATE_ATTRIBUTE( EVisibility, ScrollbarVisibility)
		
		SLATE_ARGUMENT( EFocusCause, ScrollbarDragFocusCause )

		SLATE_ARGUMENT( EAllowOverscroll, AllowOverscroll );
		
		SLATE_STYLE_ARGUMENT( FScrollBarStyle, ScrollBarStyle );

		SLATE_ARGUMENT(bool, PreventThrottling);

		SLATE_ARGUMENT( EConsumeMouseWheel, ConsumeMouseWheel );

		SLATE_ARGUMENT( float, WheelScrollMultiplier );

		SLATE_ARGUMENT( float, NavigationScrollOffset );

		SLATE_ARGUMENT( FMargin, ScrollBarPadding );

		SLATE_ARGUMENT( bool, HandleGamepadEvents );

		SLATE_ARGUMENT( bool, HandleDirectionalNavigation );

		SLATE_ARGUMENT(bool, HandleSpacebarSelection);

		SLATE_ATTRIBUTE(bool, IsFocusable)

		SLATE_ARGUMENT(bool, ReturnFocusToSelection)

		/**
		 * When true, enables spatially proximate focus routing for cross-list navigation
		 * (awarding focus to the nearest entry) and intra-entry navigation (routing focus to
		 * the nearest sub-widget within multi-focus entries). Defaults to false.
		 */
		SLATE_ARGUMENT(bool, EnableProximateEntryNavigation)

		SLATE_ARGUMENT(bool, ClearScrollVelocityOnSelection)

		/** Assign this to get more diagnostics from the list view. */
		SLATE_EVENT(FOnItemToString_Debug, OnItemToString_Debug)

		SLATE_EVENT(FOnTableViewBadState, OnEnteredBadState);

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const typename SListView<ItemType>::FArguments& InArgs)
	{
		this->Clipping = InArgs._Clipping;

		this->OnGenerateRow = InArgs._OnGenerateRow;
		this->OnGeneratePinnedRow = InArgs._OnGeneratePinnedRow;
		this->OnRefreshRow = InArgs._OnRefreshRow;
		this->OnEntryInitialized = InArgs._OnEntryInitialized;
		this->OnRowReleased = InArgs._OnRowReleased;
		this->OnItemScrolledIntoView = InArgs._OnItemScrolledIntoView;
		this->OnFinishedScrolling = InArgs._OnFinishedScrolling;
		this->OnTouchStart = InArgs._OnTouchStart;
		this->OnTouchMove = InArgs._OnTouchMove;
		this->OnTouchEnd = InArgs._OnTouchEnd;

		this->SetItemsSource(InArgs.MakeListItemsSource(this->SharedThis(this)));
		this->OnContextMenuOpening = InArgs._OnContextMenuOpening;
		this->OnItemsRebuilt = InArgs._OnItemsRebuilt;
		this->OnClick = InArgs._OnMouseButtonClick;
		this->OnDoubleClick = InArgs._OnMouseButtonDoubleClick;
		this->OnSelectionChanged = InArgs._OnSelectionChanged;
		this->OnIsSelectableOrNavigable = InArgs._OnIsSelectableOrNavigable;
		this->SetSelectionMode(InArgs._SelectionMode);

		this->bClearSelectionOnClick = InArgs._ClearSelectionOnClick;

		this->AllowOverscroll = InArgs._AllowOverscroll;
		this->ConsumeMouseWheel = InArgs._ConsumeMouseWheel;
		this->WheelScrollMultiplier = InArgs._WheelScrollMultiplier;
		this->NavigationScrollOffset = InArgs._NavigationScrollOffset;

		this->bHandleGamepadEvents = InArgs._HandleGamepadEvents;
		this->bHandleDirectionalNavigation = InArgs._HandleDirectionalNavigation;
		this->bHandleSpacebarSelection = InArgs._HandleSpacebarSelection;
		this->IsFocusable = InArgs._IsFocusable;

		this->bReturnFocusToSelection = InArgs._ReturnFocusToSelection;
		this->bEnableProximateEntryNavigation = InArgs._EnableProximateEntryNavigation;
			
		this->bClearScrollVelocityOnSelection = InArgs._ClearScrollVelocityOnSelection;

		this->bEnableAnimatedScrolling = InArgs._EnableAnimatedScrolling;
		this->FixedLineScrollOffset = InArgs._FixedLineScrollOffset;
		this->ScrollIntoViewAlignment = InArgs._ScrollIntoViewAlignment;

		this->OnItemToString_Debug = InArgs._OnItemToString_Debug.IsBound()
			? InArgs._OnItemToString_Debug
			: SListView< ItemType >::GetDefaultDebugDelegate();
		this->OnEnteredBadState = InArgs._OnEnteredBadState;

		this->OnKeyDownHandler = InArgs._OnKeyDownHandler;

		this->SetStyle(InArgs._ListViewStyle);

		this->bShouldUseShadowBoxStyle = InArgs._bEnableShadowBoxStyle;

		this->SetShadowBoxStyle(InArgs._ShadowBoxStyle);
			
		this->SetMaxPinnedItems(InArgs._MaxPinnedItems);
		this->DefaultMaxPinnedItems = InArgs._MaxPinnedItems;
		this->ScrollBarSlotPadding = InArgs._ScrollBarPadding;

		// Check for any parameters that the coder forgot to specify.
		FString ErrorString;
		{
			if ( !this->OnGenerateRow.IsBound() )
			{
				ErrorString += TEXT("Please specify an OnGenerateRow. \n");
			}

			if ( !this->HasValidItemsSource() )
			{
				ErrorString += TEXT("Please specify a ListItemsSource. \n");
			}
		}

		if (ErrorString.Len() > 0)
		{
			// Let the coder know what they forgot
			this->ChildSlot
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(ErrorString))
				];
		}
		else
		{
			// Make the ListView
			ConstructChildren( 0.0f, 0.0f, EListItemAlignment::LeftAligned, InArgs._HeaderRow, InArgs._ExternalScrollbar, InArgs._Orientation, InArgs._OnListViewScrolled, InArgs._ScrollBarStyle, InArgs._PreventThrottling );
			if(this->ScrollBar.IsValid())
			{
				this->ScrollBar->SetDragFocusCause(InArgs._ScrollbarDragFocusCause);
				this->ScrollBar->SetUserVisibility(InArgs._ScrollbarVisibility);
			}
			this->AddMetadata(MakeShared<TTableViewMetadata<ItemType>>(this->SharedThis(this)));
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SListView(ETableViewMode::Type InListMode = ETableViewMode::List)
		: STableViewBase(InListMode)
		, WidgetGenerator(this)
		, PinnedWidgetGenerator(this)
		, SelectorItem(TListTypeTraits<ItemType>::MakeNullPtr())
		, RangeSelectionStart(TListTypeTraits<ItemType>::MakeNullPtr())
		, ItemsSource(nullptr)
		, ItemToScrollIntoView(TListTypeTraits<ItemType>::MakeNullPtr())
		, UserRequestingScrollIntoView(0)
		, ItemToNotifyWhenInView(TListTypeTraits<ItemType>::MakeNullPtr())
		, ScrollIntoViewAlignment(EScrollIntoViewAlignment::CenterAligned)
		, IsFocusable(true)
		, MaxPinnedItemsAttribute(*this, 6)
	{
#if WITH_ACCESSIBILITY
		AccessibleBehavior = EAccessibleBehavior::Auto;
		bCanChildrenBeAccessible = true;
#endif
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:

	//~ SWidget overrides

	virtual bool SupportsKeyboardFocus() const override
	{
		return IsFocusable.Get();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if (OnKeyDownHandler.IsBound())
		{
			FReply Reply = OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
		return OnKeyDown_Internal(MyGeometry, InKeyEvent);
	}

protected:
	FReply OnKeyDown_Internal(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		const TArrayView<const ItemType> ItemsSourceRef = GetItems();

		// Don't respond to key-presses containing "Alt" as a modifier
		if ( ItemsSourceRef.Num() > 0 && !InKeyEvent.IsAltDown() )
		{
			bool bWasHandled = false;
			NullableItemType ItemNavigatedTo = TListTypeTraits<ItemType>::MakeNullPtr();

			// Check for selection manipulation keys (Up, Down, Home, End, PageUp, PageDown)
			if ( InKeyEvent.GetKey() == EKeys::Home )
			{
				// Select the first item
				ItemNavigatedTo = ItemsSourceRef[0];
				bWasHandled = true;
			}
			else if ( InKeyEvent.GetKey() == EKeys::End )
			{
				// Select the last item
				ItemNavigatedTo = ItemsSourceRef.Last();
				bWasHandled = true;
			}
			else if ( InKeyEvent.GetKey() == EKeys::PageUp )
			{
				int32 SelectionIndex = 0;
				if( TListTypeTraits<ItemType>::IsPtrValid(SelectorItem) )
				{
					SelectionIndex = ItemsSourceRef.Find( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( SelectorItem ) );
				}

				int32 NumItemsInAPage = FMath::TruncToInt(GetNumLiveWidgets());
				int32 Remainder = NumItemsInAPage % GetNumItemsPerLine();
				NumItemsInAPage -= Remainder;

				if ( SelectionIndex >= NumItemsInAPage )
				{
					// Select an item on the previous page
					ItemNavigatedTo = ItemsSourceRef[SelectionIndex - NumItemsInAPage];
				}
				else
				{
					ItemNavigatedTo = ItemsSourceRef[0];
				}

				bWasHandled = true;
			}
			else if ( InKeyEvent.GetKey() == EKeys::PageDown )
			{
				int32 SelectionIndex = 0;
				if( TListTypeTraits<ItemType>::IsPtrValid(SelectorItem) )
				{
					SelectionIndex = ItemsSourceRef.Find( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( SelectorItem ) );
				}

				int32 NumItemsInAPage = FMath::TruncToInt(GetNumLiveWidgets());
				int32 Remainder = NumItemsInAPage % GetNumItemsPerLine();
				NumItemsInAPage -= Remainder;

				if ( SelectionIndex < ItemsSourceRef.Num() - NumItemsInAPage )
				{
					// Select an item on the next page
					ItemNavigatedTo = ItemsSourceRef[SelectionIndex + NumItemsInAPage];
				}
				else
				{
					ItemNavigatedTo = ItemsSourceRef.Last();
				}

				bWasHandled = true;
			}

			if( TListTypeTraits<ItemType>::IsPtrValid(ItemNavigatedTo) )
			{
				ItemType ItemToSelect( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( ItemNavigatedTo ) );
				NavigationSelect( ItemToSelect, InKeyEvent );
			}
			else
			{
				// Change selected status of item.
				if (bHandleSpacebarSelection && TListTypeTraits<ItemType>::IsPtrValid(SelectorItem) && InKeyEvent.GetKey() == EKeys::SpaceBar)
				{
					ItemType SelectorItemDereference(TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(SelectorItem));

					// Deselect.
					if (InKeyEvent.IsControlDown() || GetSelectionMode() == ESelectionMode::SingleToggle)
					{
						this->Private_SetItemSelection(SelectorItemDereference, !(this->Private_IsItemSelected(SelectorItemDereference)), true);
						this->Private_SignalSelectionChanged(ESelectInfo::OnKeyPress);
						bWasHandled = true;
					}
					else
					{
						// Already selected, don't handle.
						if (this->Private_IsItemSelected(SelectorItemDereference))
						{
							bWasHandled = false;
						}
						// Select.
						else
						{
							this->Private_SetItemSelection(SelectorItemDereference, true, true);
							this->Private_SignalSelectionChanged(ESelectInfo::OnKeyPress);
							bWasHandled = true;
						}
					}

					RangeSelectionStart = SelectorItem;

					// If the selector is not in the view, scroll it into view.
					TSharedPtr<ITableRow> WidgetForItem = this->WidgetGenerator.GetWidgetForItem(SelectorItemDereference);
					if (!WidgetForItem.IsValid())
					{
						this->RequestScrollIntoView(SelectorItemDereference, InKeyEvent.GetUserIndex());
					}
				}
				// Select all items
				else if ( (!InKeyEvent.IsShiftDown() && !InKeyEvent.IsAltDown() && InKeyEvent.IsControlDown() && InKeyEvent.GetKey() == EKeys::A) && GetSelectionMode() == ESelectionMode::Multi )
				{
					this->Private_ClearSelection();

					for ( int32 ItemIdx = 0; ItemIdx < ItemsSourceRef.Num(); ++ItemIdx )
					{
						this->Private_SetItemSelection( ItemsSourceRef[ItemIdx], true );
					}

					this->Private_SignalSelectionChanged(ESelectInfo::OnKeyPress);

					bWasHandled = true;
				}
			}

			if (bWasHandled)
			{
				return FReply::Handled();
			}
		}

		return STableViewBase::OnKeyDown(MyGeometry, InKeyEvent);
	}
public:

	virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override
	{
		NavigationTypeOnScrollIntoView = EUINavigation::Invalid;

		if (this->HasValidItemsSource() && this->bHandleDirectionalNavigation && (this->bHandleGamepadEvents || InNavigationEvent.GetNavigationGenesis() != ENavigationGenesis::Controller))
		{
			const TArrayView<const ItemType> ItemsSourceRef = this->GetItems();
			const int32 NumItems = ItemsSourceRef.Num();

			const int32 NumItemsPerLine = GetNumItemsPerLine();
			const int32 CurSelectionIndex = (!TListTypeTraits<ItemType>::IsPtrValid(SelectorItem)) ? -1 : ItemsSourceRef.Find(TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(SelectorItem));
			int32 AttemptSelectIndex = -1;

			const EUINavigation NavType = InNavigationEvent.GetNavigationType();
			const bool bNavBackward = (Orientation == Orient_Vertical   && NavType == EUINavigation::Up)   ||
			                         (Orientation == Orient_Horizontal  && NavType == EUINavigation::Left);
			const bool bNavForward  = (Orientation == Orient_Vertical   && NavType == EUINavigation::Down)  ||
			                         (Orientation == Orient_Horizontal  && NavType == EUINavigation::Right);
			// Any of the four directional nav types - includes cross-axis directions (e.g. Down on a
			// horizontal list) that leave bNavForward and bNavBackward both false.
			const bool bIsAnyDirectionalNav = NavType == EUINavigation::Up   || NavType == EUINavigation::Down ||
			                                  NavType == EUINavigation::Left  || NavType == EUINavigation::Right;

			if (bNavBackward)
			{
				// Nav backward by a line
				AttemptSelectIndex = CurSelectionIndex - NumItemsPerLine;
			}
			else if (bNavForward)
			{
				AttemptSelectIndex = CurSelectionIndex + NumItemsPerLine;

				// The list might be jagged so attempt to determine if there's a partially filled row we can move to
				if (!ItemsSourceRef.IsValidIndex(AttemptSelectIndex))
				{
					if (NumItems > 0)
					{
						// NumItemsWide should never be 0, ensuring for sanity
						ensure(NumItemsPerLine > 0);

						// calculate total number of rows and row of current index (1 index)
						const int32 NumLines = FMath::CeilToInt((float)NumItems / (float)NumItemsPerLine);
						const int32 CurLine = FMath::CeilToInt((float)(CurSelectionIndex + 1) / (float)NumItemsPerLine);

						// if not on final row, assume a jagged list and select the final item
						if (CurLine < NumLines)
						{
							AttemptSelectIndex = NumItems - 1;
						}
					}
				}
			}

			// Cross-list entry with no current selection: select the proximate boundary item.
			if (bEnableProximateEntryNavigation && CurSelectionIndex == -1 && bIsAnyDirectionalNav && NumItems > 0)
			{
				ensure(NumItemsPerLine > 0);
				const TSharedPtr<UE::Slate::Private::FListViewNavigationContext> NavContext = FSlateApplication::Get().GetPendingNavigationContext<UE::Slate::Private::FListViewNavigationContext>(InNavigationEvent.GetUserIndex());
				if (NavContext.IsValid())
				{
					const bool bNavForwardForBoundary = NavType == EUINavigation::Down || NavType == EUINavigation::Right;
					AttemptSelectIndex = FindProximateBoundaryItem(NavContext->SourceWidget.Pin(), NavContext->NavigationType, ItemsSourceRef, NumItems, NumItemsPerLine, bNavForwardForBoundary);
					FSlateApplication::Get().ClearPendingNavigationContext(InNavigationEvent.GetUserIndex());
				}
				else if (NavType == EUINavigation::Up || NavType == EUINavigation::Left)
				{
					// Backward entry with no context: land on the first item of the last row.
					AttemptSelectIndex = ((NumItems - 1) / NumItemsPerLine) * NumItemsPerLine;
				}
				// Forward without context: falls through to CurSelectionIndex + NumItemsPerLine
				// (computed above), which equals NumItemsPerLine - 1 when CurSelectionIndex is -1.
			}

			// If it's valid we'll scroll it into view and return an explicit widget in the FNavigationReply.
			if (ItemsSourceRef.IsValidIndex(AttemptSelectIndex) && this->bIsGamepadScrollingEnabled)
			{
				TOptional<ItemType> ItemToSelect = Private_FindNextSelectableOrNavigableWithIndexAndDirection(ItemsSourceRef[AttemptSelectIndex], AttemptSelectIndex, AttemptSelectIndex >= CurSelectionIndex);
				if (ItemToSelect.IsSet())
				{
					NavigationSelect(ItemToSelect.GetValue(), InNavigationEvent);
					return FNavigationReply::Explicit(nullptr);
				}
			}

			// Escaping this list's boundary: write the navigation context for the receiving list.
			if (bEnableProximateEntryNavigation && bIsAnyDirectionalNav)
			{
				TSharedRef<UE::Slate::Private::FListViewNavigationContext> NavContext = MakeShared<UE::Slate::Private::FListViewNavigationContext>();
				NavContext->NavigationType = NavType;

				// Capture the source widget for cross-axis proximity calculations.
				if (TListTypeTraits<ItemType>::IsPtrValid(SelectorItem))
				{
					const ItemType CurrentItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(SelectorItem);
					if (const TSharedPtr<ITableRow> SelectedRow = this->WidgetFromItem(CurrentItem))
					{
						NavContext->SourceWidget = SelectedRow->AsWidget();
					}
				}
				FSlateApplication::Get().SetPendingNavigationContext(InNavigationEvent.GetUserIndex(), NavContext);
			}
		}

		return STableViewBase::OnNavigation(MyGeometry, InNavigationEvent);
	}

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if ( bClearSelectionOnClick
			&& GetSelectionMode() != ESelectionMode::None
			&& MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton
			&& !MouseEvent.IsControlDown()
			&& !MouseEvent.IsShiftDown()
			)
		{
			// Left clicking on a list (but not an item) will clear the selection on mouse button down.
			// Right clicking is handled on mouse up.
			if ( this->Private_GetNumSelectedItems() > 0 )
			{
				this->Private_ClearSelection();
				this->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
			}

			return FReply::Handled();
		}

		return STableViewBase::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if ( bClearSelectionOnClick
			&& GetSelectionMode() != ESelectionMode::None
			&& MouseEvent.GetEffectingButton() == EKeys::RightMouseButton
			&& !MouseEvent.IsControlDown()
			&& !MouseEvent.IsShiftDown()
			&& !this->IsRightClickScrolling()
			)
		{
			// Right clicking on a list (but not an item) will clear the selection on mouse button up.
			// Left clicking is handled on mouse down
			if ( this->Private_GetNumSelectedItems() > 0 )
			{
				this->Private_ClearSelection();
				this->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
			}
		}

		return STableViewBase::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

#if WITH_ACCESSIBILITY
protected:
	friend class FSlateAccessibleListView;
	/**
	* An accessible implementation for SListView to be exposed to platform accessibility APIs.
	* For subclasses of SListView, inherit from this class and override and functions to 
	* give desired behavior. 
	*/
	class FSlateAccessibleListView
		: public FSlateAccessibleWidget
		, public IAccessibleTable
	{
	public:
		FSlateAccessibleListView(TWeakPtr<SWidget> InWidget, EAccessibleWidgetType InWidgetType)
			: FSlateAccessibleWidget(InWidget, InWidgetType)
		{

		}

		// IAccessibleWidget
		virtual IAccessibleTable* AsTable() 
		{ 
			return this; 
		}
		// ~

		// IAccessibleTable
		virtual TArray<TSharedPtr<IAccessibleWidget>> GetSelectedItems() const 
		{ 
			TArray<TSharedPtr<IAccessibleWidget>> SelectedItemsArray;
			if (Widget.IsValid())
			{
				TSharedPtr<SListView<ItemType>> ListView = StaticCastSharedPtr<SListView<ItemType>>(Widget.Pin());
				SelectedItemsArray.Empty(ListView ->SelectedItems.Num());
				for (typename TItemSet::TConstIterator SelectedItemIt(ListView ->SelectedItems); SelectedItemIt; ++SelectedItemIt)
				{
					const ItemType& CurrentItem = *SelectedItemIt;
					// This is only valid if the item is visible
					TSharedPtr < ITableRow> TableRow = ListView->WidgetGenerator.GetWidgetForItem(CurrentItem);
					if (TableRow.IsValid())
					{
						TSharedPtr<SWidget> TableRowWidget = TableRow->AsWidget();
						TSharedPtr<IAccessibleWidget> AccessibleTableRow = FSlateAccessibleWidgetCache::GetAccessibleWidgetChecked(TableRowWidget);
						// it's possible for the accessible widget to still be in the process of generating 
						// in the slate accessibility tree 
						if (AccessibleTableRow.IsValid())
						{
							SelectedItemsArray.Add(AccessibleTableRow);
						}
					}
				}
			}
			return SelectedItemsArray;
		}

		virtual bool CanSupportMultiSelection() const 
		{ 
			if (Widget.IsValid())
			{
				TSharedPtr<SListView<ItemType>> ListView = StaticCastSharedPtr<SListView<ItemType>>(Widget.Pin());
				return ListView->GetSelectionMode() == ESelectionMode::Multi;
			}
			return false; 
		}

		virtual bool IsSelectionRequired() const 
		{ 
			return false; 
		}
		// ~
	};
public:
	virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override
	{
		// @TODOAccessibility: Add support for the different types of tables e.g tree and tile 
		// We hardcode to list for now 
		EAccessibleWidgetType WidgetType = EAccessibleWidgetType::List;
		return MakeShareable<FSlateAccessibleWidget>(new SListView<ItemType>::FSlateAccessibleListView(SharedThis(this), WidgetType));
	}

	virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType) const override
	{
		// current behaviour will red out the  templated type of the listwhich is verbose and unhelpful 
		// This will read out list twice, but it's the best we can do for now if no label is found 
		// @TODOAccessibility: Give a better name 
		static FString Name("List");
		return FText::FromString(Name);
	}
#endif
private:

	friend class FWidgetGenerator;
	/**
	 * A WidgetGenerator is a component responsible for creating widgets from data items.
	 * It also provides mapping from currently generated widgets to the data items which they
	 * represent.
	 */
	class FWidgetGenerator
	{
	public:
		FWidgetGenerator(SListView<ItemType>* InOwnerList)
			: OwnerList(InOwnerList)
		{
		}

		/**
		 * Find a widget for this item if it has already been constructed.
		 *
		 * @param Item  The item for which to find the widget.
		 * @return A pointer to the corresponding widget if it exists; otherwise nullptr.
		 */
		[[nodiscard]] TSharedPtr<ITableRow> GetWidgetForItem( const ItemType& Item ) const
		{
			const TSharedRef<ITableRow>* LookupResult = ItemToWidgetMap.Find(Item);
			return LookupResult ? TSharedPtr<ITableRow>(*LookupResult) : TSharedPtr<ITableRow>(nullptr);
		}

		/**
		 * Keep track of every item and corresponding widget during a generation pass.
		 *
		 * @param InItem             The DataItem which is in view.
		 * @param InGeneratedWidget  The widget generated for this item; it may have been newly generated.
		 */
		void OnItemSeen( ItemType InItem, TSharedRef<ITableRow> InGeneratedWidget)
		{
			ensure(TListTypeTraits<ItemType>::IsPtrValid(InItem));
			TSharedRef<ITableRow>* LookupResult = ItemToWidgetMap.Find( InItem );
			const bool bWidgetIsNewlyGenerated = (LookupResult == nullptr);
			if ( bWidgetIsNewlyGenerated )
			{
				// It's a newly generated item!
				ItemToWidgetMap.Add( InItem, InGeneratedWidget );
				WidgetMapToItem.Add( &InGeneratedWidget.Get(), InItem );

				// Now that the item-widget association is established, the generated row can fully initialize itself
				InGeneratedWidget->InitializeRow();
				OwnerList->Private_OnEntryInitialized(InItem, InGeneratedWidget);
			}

			// We should not clean up this item's widgets because it is in view.
			ItemsToBeCleanedUp.Remove(InItem);
			ItemsWithGeneratedWidgets.Add(InItem);
		}

		/**
		 * Called at the beginning of the generation pass.
		 * Begins tracking of which widgets were in view and which were not (so we can clean them up)
		 * 
		 * @param InNumDataItems   The total number of data items being observed
		 */
		void OnBeginGenerationPass()
		{
			// Assume all the previously generated items need to be cleaned up.
			ItemsToBeCleanedUp = ItemsWithGeneratedWidgets;
			ItemsWithGeneratedWidgets.Empty();
		}

		/**
		 * Called at the end of the generation pass.
		 * Cleans up any widgets associated with items that were not in view this frame.
		 */
		void OnEndGenerationPass()
		{
			ProcessItemCleanUp();
			ValidateWidgetGeneration();
		}

		/** Clear everything so widgets will be regenerated */
		void Clear()
		{
			ItemsToBeCleanedUp = ItemsWithGeneratedWidgets;
			ItemsWithGeneratedWidgets.Empty();
			ProcessItemCleanUp();
		}

		void ProcessItemCleanUp()
		{

			for (int32 ItemIndex = 0; ItemIndex < ItemsToBeCleanedUp.Num(); ++ItemIndex)
			{
				ItemType ItemToBeCleanedUp = ItemsToBeCleanedUp[ItemIndex];
				const TSharedRef<ITableRow>* FindResult = ItemToWidgetMap.Find(ItemToBeCleanedUp);
				if (FindResult != nullptr)
				{
					const TSharedRef<ITableRow> WidgetToCleanUp = *FindResult;
					ItemToWidgetMap.Remove(ItemToBeCleanedUp);
					WidgetMapToItem.Remove(&WidgetToCleanUp.Get());

					if (ensureMsgf(OwnerList, TEXT("OwnerList is null, something is wrong.")))
					{
						WidgetToCleanUp->ResetRow();
						OwnerList->OnRowReleased.ExecuteIfBound(WidgetToCleanUp);
					}
				}
				else if(!TListTypeTraits<ItemType>::IsPtrValid(ItemToBeCleanedUp))
				{
					// If we get here, it means we have an invalid object. We will need to remove that object from both maps.
					// This may happen for example when ItemType is a UObject* and the object is garbage collected.
					auto Widget = WidgetMapToItem.FindKey(ItemToBeCleanedUp);
					if (Widget != nullptr)
					{
						for (auto WidgetItemPair = ItemToWidgetMap.CreateIterator(); WidgetItemPair; ++WidgetItemPair)
						{
							const ITableRow* Item = &(WidgetItemPair.Value().Get());
							if (Item == *Widget)
							{
								WidgetItemPair.RemoveCurrent();
								break;
							}
						}

						WidgetMapToItem.Remove(*Widget);
					}
				}
			}

			ItemsToBeCleanedUp.Reset();
		}

		void ValidateWidgetGeneration()
		{
			const bool bMapsMismatch = ItemToWidgetMap.Num() != WidgetMapToItem.Num();
			const bool bGeneratedWidgetsSizeMismatch = WidgetMapToItem.Num() != ItemsWithGeneratedWidgets.Num();
			if (bMapsMismatch)
			{
				UE_LOGF(LogSlate, Warning, "ItemToWidgetMap length (%d) does not match WidgetMapToItem length (%d) in %ls. Diagnostics follow. ",
					ItemToWidgetMap.Num(),
					WidgetMapToItem.Num(),
					OwnerList ? *OwnerList->ToString() : TEXT("null"));
			}
			
			if (bGeneratedWidgetsSizeMismatch)
			{
				UE_LOGF(LogSlate, Warning,
					"WidgetMapToItem length (%d) does not match ItemsWithGeneratedWidgets length (%d). This is often because the same item is in the list more than once in %ls. Diagnostics follow.",
					WidgetMapToItem.Num(),
					ItemsWithGeneratedWidgets.Num(),
					OwnerList ? *OwnerList->ToString() : TEXT("null"));
			}

			if (bMapsMismatch || bGeneratedWidgetsSizeMismatch)
			{
				if (OwnerList->OnItemToString_Debug.IsBound())
				{
					UE_LOGF(LogSlate, Warning, "");
					UE_LOGF(LogSlate, Warning, "ItemToWidgetMap :");
					for (auto ItemWidgetPair = ItemToWidgetMap.CreateConstIterator(); ItemWidgetPair; ++ItemWidgetPair)
					{
						const TSharedRef<SWidget> RowAsWidget = ItemWidgetPair.Value()->AsWidget();
						UE_LOG(LogSlate, Warning, TEXT("%s -> 0x%08" UPTRINT_x_FMT " @ %s"), *OwnerList->OnItemToString_Debug.Execute(ItemWidgetPair.Key()), &RowAsWidget.Get(), *RowAsWidget->ToString() );
					}

					UE_LOGF(LogSlate, Warning, "");
					UE_LOGF(LogSlate, Warning, "WidgetMapToItem:")
					for (auto WidgetItemPair = WidgetMapToItem.CreateConstIterator(); WidgetItemPair; ++WidgetItemPair)
					{
						UE_LOG(LogSlate, Warning, TEXT("0x%08" UPTRINT_x_FMT " -> %s"), WidgetItemPair.Key(), *OwnerList->OnItemToString_Debug.Execute(WidgetItemPair.Value()) );
					}

					UE_LOGF(LogSlate, Warning, "");
					UE_LOGF(LogSlate, Warning, "ItemsWithGeneratedWidgets:");
					for (int i = 0; i < ItemsWithGeneratedWidgets.Num(); ++i)
					{
						UE_LOGF(LogSlate, Warning, "[%d] %ls",i, *OwnerList->OnItemToString_Debug.Execute(ItemsWithGeneratedWidgets[i]));
					}
				}
				else
				{
					UE_LOGF(LogSlate, Warning, "Provide custom 'OnItemToString_Debug' for diagnostics dump.");
				}

				OwnerList->OnEnteredBadState.ExecuteIfBound();

				checkf( false, TEXT("%s detected a critical error. See diagnostic dump above. Provide a custom 'OnItemToString_Debug' for more detailed diagnostics."), *OwnerList->ToString() );
			}			
		}

	public:
		/** We store a pointer to the owner list for error purposes, so when asserts occur we can report which list it happened for. */
		SListView<ItemType>* OwnerList;

		/** Map of DataItems to corresponding SWidgets */
		TMap< ItemType, TSharedRef<ITableRow>, FDefaultSetAllocator, MapKeyFuncs > ItemToWidgetMap;

		/** Map of SWidgets to DataItems from which they were generated */
		TMap< const ITableRow*, TObjectPtrWrapTypeOf<ItemType> > WidgetMapToItem;

		/** A set of Items that currently have a generated widget */
		TArray< TObjectPtrWrapTypeOf<ItemType> > ItemsWithGeneratedWidgets;

		/** Total number of DataItems the last time we performed a generation pass. */
		int32 TotalItemsLastGeneration;

		/** Items that need their widgets destroyed because they are no longer on screen. */
		TArray<ItemType> ItemsToBeCleanedUp;
	};

public:

	// A low-level interface for use various widgets generated by ItemsWidgets(Lists, Trees, etc).
	// These handle selection, expansion, and other such properties common to ItemsWidgets.
	//

	void Private_OnEntryInitialized(ItemType TheItem, const TSharedRef<ITableRow>& TableRow)
	{
		OnEntryInitialized.ExecuteIfBound(TheItem, TableRow);
	}

	virtual void Private_SetItemSelection( ItemType TheItem, bool bShouldBeSelected, bool bWasUserDirected = false ) override
	{
		if ( GetSelectionMode() == ESelectionMode::None )
		{
			return;
		}

		if ( bShouldBeSelected )
		{
			SelectedItems.Add( TheItem );
		}
		else
		{
			SelectedItems.Remove( TheItem );
		}

		// Move the selector item and range selection start if the user directed this change in selection or if the list view is single selection
		if( bWasUserDirected || GetSelectionMode() == ESelectionMode::Single || GetSelectionMode() == ESelectionMode::SingleToggle )
		{
			SelectorItem = TheItem;
			RangeSelectionStart = TheItem;
		}

		if (bClearScrollVelocityOnSelection)
		{
			this->InertialScrollManager.ClearScrollVelocity();
		}

#if WITH_ACCESSIBILITY
		// On certain platforms, we need to pass accessibility focus to a widget for screen readers to announce 
		// accessibility information. STableRows cannot accept keyboard focus, 
		// so we have to manually raise a focus change event for the selected table row 
		if (bShouldBeSelected)
		{
			TSharedPtr<ITableRow> TableRow = WidgetFromItem(TheItem);
			if (TableRow.IsValid())
			{
				TSharedRef<SWidget> TableRowWidget = TableRow->AsWidget();
				// We don't need to worry about raising a focus change event for the 
				// widget with accessibility focus  as FSlateAccessibleMessageHandler will take care of signalling a focus lost event
				// @TODOAccessibility: Technically we need to pass in the user Id that selected the row so the event can be routed to the correct user.
				// But we don't want to change the Slate API drastically right now
				FSlateApplicationBase::Get().GetAccessibleMessageHandler()->OnWidgetEventRaised(FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(TableRowWidget, EAccessibleEvent::FocusChange, false, true));
			}
		}
#endif
	}

	virtual void Private_ClearSelection() override
	{
		SelectedItems.Empty();

		if (bClearScrollVelocityOnSelection)
		{
			this->InertialScrollManager.ClearScrollVelocity();
		}
	}

	virtual void Private_SelectRangeFromCurrentTo( ItemType InRangeSelectionEnd ) override
	{
		if ( GetSelectionMode() == ESelectionMode::None )
		{
			return;
		}

		const TArrayView<const ItemType> ItemsSourceRef = GetItems();
		// The InRangeSelectionEnd come from the WidgetGenerator (previous tick). Maybe it is not in the current ItemsSource list.
		//RangeSelectionStart, maybe it is not in the current ItemsSource list.
		if (ItemsSourceRef.Num() != 0)
		{
			int32 RangeStartIndex = 0;
			if( TListTypeTraits<ItemType>::IsPtrValid(RangeSelectionStart) )
			{
				RangeStartIndex = ItemsSourceRef.Find( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( RangeSelectionStart ) );
			}

			int32 RangeEndIndex = ItemsSourceRef.Find( InRangeSelectionEnd );

			RangeStartIndex = FMath::Clamp(RangeStartIndex, 0, ItemsSourceRef.Num()-1);
			RangeEndIndex = FMath::Clamp(RangeEndIndex, 0, ItemsSourceRef.Num()-1);

			// Respect the direction of selection when ordering, ie if selecting upwards then make sure the top element is last-selected
			const int32 Direction = (RangeEndIndex > RangeStartIndex) ? 1 : -1;

			int32 ItemIndex = RangeStartIndex;
			for (; ItemIndex != RangeEndIndex; ItemIndex += Direction)
			{
				SelectedItems.Add(ItemsSourceRef[ItemIndex]);
			}
			// The above loop won't add the last item, so manually add it here
			SelectedItems.Add(ItemsSourceRef[ItemIndex]);
		}
		
		if (bClearScrollVelocityOnSelection)
		{
			this->InertialScrollManager.ClearScrollVelocity();
		}
	}

	virtual void Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo) override
	{
		if ( GetSelectionMode() == ESelectionMode::None )
		{
			return;
		}

		if( OnSelectionChanged.IsBound() )
		{
			TObjectPtrWrapTypeOf<NullableItemType> SelectedItem = (SelectedItems.Num() > 0)
				? (*typename TItemSet::TIterator(SelectedItems))
				: TListTypeTraits< TObjectPtrWrapTypeOf<ItemType> >::MakeNullPtr();

			OnSelectionChanged.ExecuteIfBound(SelectedItem, SelectInfo );
		}
	}

	virtual const TObjectPtrWrapTypeOf<ItemType>* Private_ItemFromWidget( const ITableRow* TheWidget ) const override
	{
		const TObjectPtrWrapTypeOf<ItemType>* LookupResult = WidgetGenerator.WidgetMapToItem.Find( TheWidget );
		return LookupResult == nullptr ? PinnedWidgetGenerator.WidgetMapToItem.Find(TheWidget) : LookupResult;
	}

	virtual bool Private_UsesSelectorFocus() const override
	{
		return true;
	}

	virtual bool Private_HasSelectorFocus( const ItemType& TheItem ) const override
	{
		return SelectorItem == TheItem;
	}

	virtual bool Private_IsItemSelected( const ItemType& TheItem ) const override
	{
		return nullptr != SelectedItems.Find(TheItem);
	}

	virtual bool Private_IsItemHighlighted(const ItemType& TheItem) const override
	{
		return nullptr != HighlightedItems.Find(TheItem);
	}

	virtual bool Private_IsItemExpanded( const ItemType& TheItem ) const override
	{
		// List View does not support item expansion.
		return false;	
	}

	virtual bool Private_IsItemSelectableOrNavigable(const ItemType& TheItem) const override
	{
		return OnIsSelectableOrNavigable.IsBound() ? OnIsSelectableOrNavigable.Execute(TheItem) : true;
	}

	virtual void Private_SetItemExpansion( ItemType TheItem, bool bShouldBeExpanded ) override
	{
		// Do nothing; you cannot expand an item in a list!
	}

	virtual void Private_OnExpanderArrowShiftClicked( ItemType TheItem, bool bShouldBeExpanded ) override
	{
		// Do nothing; you cannot expand an item in a list!
	}

	virtual bool Private_DoesItemHaveChildren( int32 ItemIndexInList ) const override
	{
		// List View items cannot have children
		return false;
	}

	virtual int32 Private_GetNumSelectedItems() const override
	{
		return SelectedItems.Num();
	}

	virtual void Private_SetItemHighlighted(ItemType TheItem, bool bShouldBeHighlighted) override
	{
		if (bShouldBeHighlighted)
		{
			HighlightedItems.Add(TheItem);
		}
		else
		{
			HighlightedItems.Remove(TheItem);
		}
	}

	virtual void Private_ClearHighlightedItems() override
	{
		HighlightedItems.Empty();
	}

	virtual int32 Private_GetNestingDepth( int32 ItemIndexInList ) const override
	{
		// List View items are not indented
		return 0;
	}

	virtual const TBitArray<>& Private_GetWiresNeededByDepth( int32 ItemIndexInList ) const override
	{
		return TableViewHelpers::GetEmptyBitArray();
	}

	virtual bool Private_IsLastChild(int32 ItemIndexInList) const override
	{
		return false;
	}

	virtual ESelectionMode::Type Private_GetSelectionMode() const override
	{
		return GetSelectionMode();
	}

	virtual EOrientation Private_GetOrientation() const override
	{
		return Orientation;
	}

	virtual bool Private_IsPendingRefresh() const override
	{
		return IsPendingRefresh();
	}

	virtual void Private_OnItemRightClicked( ItemType TheItem, const FPointerEvent& MouseEvent ) override
	{
		this->OnRightMouseButtonUp( MouseEvent );
	}

	virtual bool Private_OnItemClicked(ItemType TheItem) override
	{
		if (OnClick.ExecuteIfBound(TheItem))
		{
			return true;	// Handled
		}

		return false;	// Not handled
	}
	
	virtual bool Private_OnItemDoubleClicked( ItemType TheItem ) override
	{
		if( OnDoubleClick.ExecuteIfBound( TheItem ) )
		{
			return true;	// Handled
		}

		return false;	// Not handled
	}

	virtual ETableViewMode::Type GetTableViewMode() const override
	{
		return TableViewMode;
	}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		return SharedThis(this);
	}

private:
	typedef UE::Slate::Private::SListViewPinnedRowWidget<ItemType> SListViewPinnedRowWidget;
	friend SListViewPinnedRowWidget;

public:	

	void SetStyle(const FTableViewStyle* InStyle)
	{
		Style = InStyle;
		SetBackgroundBrush( Style != nullptr ? &Style->BackgroundBrush : FStyleDefaults::GetNoBrush() );
	}

	/** Sets the OnEntryInitializer delegate. This delegate is invoked after initializing an entry being generated, before it may be added to the actual widget hierarchy. */
	void SetOnEntryInitialized(const FOnEntryInitialized& Delegate)
	{
		OnEntryInitialized = Delegate;
	}

	/**
	 * Remove any items that are no longer in the list from the selection set.
	 */
	virtual void UpdateSelectionSet() override
	{
		// Trees take care of this update in a different way.
		if ( TableViewMode != ETableViewMode::Tree )
		{
			bool bSelectionChanged = false;
			if ( !HasValidItemsSource() )
			{
				// We are no longer observing items so there is no more selection.
				this->Private_ClearSelection();
				bSelectionChanged = true;
			}
			else
			{
				// We are observing some items; they are potentially different.
				// Unselect any that are no longer being observed.
				TItemSet NewSelectedItems;
				const TArrayView<const ItemType> Items = GetItems();
				for ( int32 ItemIndex = 0; ItemIndex < Items.Num(); ++ItemIndex )
				{
					ItemType CurItem = Items[ItemIndex];
					const bool bItemIsSelected = ( nullptr != SelectedItems.Find( CurItem ) );
					if ( bItemIsSelected )
					{
						NewSelectedItems.Add( CurItem );
					}
				}

				// Look for items that were removed from the selection.
				TItemSet SetDifference = SelectedItems.Difference( NewSelectedItems );
				bSelectionChanged = (SetDifference.Num()) > 0;

				// Update the selection to reflect the removal of any items from the ItemsSource.
				SelectedItems = NewSelectedItems;
			}

			if (bSelectionChanged)
			{
				Private_SignalSelectionChanged(ESelectInfo::Direct);
			}
		}
	}

	/**
	 * Update generate Widgets for Items as needed and clean up any Widgets that are no longer needed.
	 * Re-arrange the visible widget order as necessary.
	 */
	virtual FReGenerateResults ReGenerateItems( const FGeometry& MyGeometry ) override
	{
		auto DoubleFractional = [](double Value) -> double
		{
			return Value - FMath::TruncToDouble(Value);
		};

		// Clear all the items from our panel. We will re-add them in the correct order momentarily.
		this->ClearWidgets();

		// Ensure that we always begin and clean up a generation pass.
		FGenerationPassGuard GenerationPassGuard(WidgetGenerator);

		const TArrayView<const ItemType> Items = GetItems();
		if (Items.Num() > 0)
		{
			// Items in view, including fractional items
			float ItemsInView = 0.0f;

			// Total length of widgets generated so far (height for vertical lists, width for horizontal)
			float LengthGeneratedSoFar = 0.0f;

			// Length of generated widgets that in the bounds of the view.
			float ViewLengthUsedSoFar = 0.0f;

			// Index of the item at which we start generating based on how far scrolled down we are
			// Note that we must generate at LEAST one item.
			int32 StartIndex = FMath::Clamp( (int32)(FMath::FloorToDouble(CurrentScrollOffset)), 0, Items.Num() - 1 );

			// Length of the first item that is generated. This item is at the location where the user requested we scroll
			float FirstItemLength = 0.0f;

			// Generate widgets assuming scenario a.
			bool bHasFilledAvailableArea = false;
			bool bAtEndOfList = false;

			const float LayoutScaleMultiplier = MyGeometry.GetAccumulatedLayoutTransform().GetScale();
			FTableViewDimensions MyDimensions(this->Orientation, MyGeometry.GetLocalSize());
			
			for( int32 ItemIndex = StartIndex; !bHasFilledAvailableArea && ItemIndex < Items.Num(); ++ItemIndex )
			{
				const ItemType& CurItem = Items[ItemIndex];

				if (!TListTypeTraits<ItemType>::IsPtrValid(CurItem))
				{
					// Don't bother generating widgets for invalid items
					continue;
				}

				const float ItemLength = GenerateWidgetForItem(CurItem, ItemIndex, StartIndex, LayoutScaleMultiplier);

				const bool bIsFirstItem = ItemIndex == StartIndex;

				if (bIsFirstItem)
				{
					FirstItemLength = ItemLength;
				}

				// Track the number of items in the view, including fractions.
				if (bIsFirstItem)
				{
					// The first item may not be fully visible (but cannot exceed 1)
					// FirstItemFractionScrolledIntoView is the fraction of the item that is visible after taking into account anything that may be scrolled off the top/left of the list view
					const float FirstItemFractionScrolledIntoView = 1.0f - (float)FMath::Max(DoubleFractional(CurrentScrollOffset), 0.0);
					
					// FirstItemLengthScrolledIntoView is the length of the item, ignoring anything that is scrolled off the top/left of the list view
					const float FirstItemLengthScrolledIntoView = ItemLength * FirstItemFractionScrolledIntoView;
					
					// FirstItemVisibleFraction is either: The visible item length as a fraction of the available list view length (if the item size is larger than the available size, otherwise this will be >1), or just FirstItemFractionScrolledIntoView (which can never be >1)
					const float FirstItemVisibleFraction = FMath::Min(MyDimensions.ScrollAxis / FirstItemLengthScrolledIntoView, FirstItemFractionScrolledIntoView);
					
					ItemsInView += FirstItemVisibleFraction;
				}
				else if (ViewLengthUsedSoFar + ItemLength > MyDimensions.ScrollAxis)
				{
					// The last item may not be fully visible either
					ItemsInView += (MyDimensions.ScrollAxis - ViewLengthUsedSoFar) / ItemLength;
				}
				else
				{
					ItemsInView += 1;
				}

				LengthGeneratedSoFar += ItemLength;

				ViewLengthUsedSoFar += (bIsFirstItem)
					? ItemLength * ItemsInView	// For the first item, ItemsInView <= 1.0f
					: ItemLength;

				bAtEndOfList = ItemIndex >= Items.Num() - 1;

				if (bIsFirstItem && ViewLengthUsedSoFar >= MyDimensions.ScrollAxis)
				{
					// Since there was no sum of floating points, make sure we correctly detect the case where one element
					// fills up the space.
					bHasFilledAvailableArea = true;
				}
				else
				{
					// Note: To account for accrued error from floating point truncation and addition in our sum of dimensions used, 
					//	we pad the allotted axis just a little to be sure we have filled the available space.
					const float FloatPrecisionOffset = 0.001f;
					bHasFilledAvailableArea = ViewLengthUsedSoFar >= MyDimensions.ScrollAxis + FloatPrecisionOffset;
				}
			}

			// Handle scenario b.
			// We may have stopped because we got to the end of the items, but we may still have space to fill!
			if (bAtEndOfList && !bHasFilledAvailableArea)
			{
				double NewScrollOffsetForBackfill = static_cast<double>(StartIndex) + (LengthGeneratedSoFar - MyDimensions.ScrollAxis) / FirstItemLength;

				for (int32 ItemIndex = StartIndex - 1; LengthGeneratedSoFar < MyDimensions.ScrollAxis && ItemIndex >= 0; --ItemIndex)
				{
					const ItemType& CurItem = Items[ItemIndex];
					if (TListTypeTraits<ItemType>::IsPtrValid(CurItem))
					{
						const float ItemLength = GenerateWidgetForItem(CurItem, ItemIndex, StartIndex, LayoutScaleMultiplier);

						if (LengthGeneratedSoFar + ItemLength > MyDimensions.ScrollAxis && ItemLength > 0.f)
						{
							// Generated the item that puts us over the top.
							// Count the fraction of this item that will stick out beyond the list
							NewScrollOffsetForBackfill = static_cast<double>(ItemIndex) + (LengthGeneratedSoFar + ItemLength - MyDimensions.ScrollAxis) / ItemLength;
						}

						// The widget used up some of the available vertical space.
						LengthGeneratedSoFar += ItemLength;
					}
				}

				return FReGenerateResults(NewScrollOffsetForBackfill, LengthGeneratedSoFar, Items.Num() - NewScrollOffsetForBackfill, true);
			}

			return FReGenerateResults(CurrentScrollOffset, LengthGeneratedSoFar, ItemsInView, false);
		}

		return FReGenerateResults(0.0f, 0.0f, 0.0f, false);
	}

	float GenerateWidgetForItem( const ItemType& CurItem, int32 ItemIndex, int32 StartIndex, float LayoutScaleMultiplier )
	{
		ensure(TListTypeTraits<ItemType>::IsPtrValid(CurItem));
		// Find a previously generated Widget for this item, if one exists.
		TSharedPtr<ITableRow> WidgetForItem = WidgetGenerator.GetWidgetForItem( CurItem );
		if ( !WidgetForItem.IsValid() )
		{
			// We couldn't find an existing widgets, meaning that this data item was not visible before.
			// Make a new widget for it.
			WidgetForItem = this->GenerateNewWidget(CurItem);
		}
		else
		{
			if (OnRefreshRow.IsBound())
			{
				OnRefreshRow.Execute(CurItem);
			}
		}

		// It is useful to know the item's index that the widget was generated from.
		// Helps with even/odd coloring
		WidgetForItem->SetIndexInList(ItemIndex);

		// Let the item generator know that we encountered the current Item and associated Widget.
		WidgetGenerator.OnItemSeen( CurItem, WidgetForItem.ToSharedRef() );

		// We rely on the widgets desired size in order to determine how many will fit on screen.
		const TSharedRef<SWidget> NewlyGeneratedWidget = WidgetForItem->AsWidget();
		NewlyGeneratedWidget->MarkPrepassAsDirty();
		NewlyGeneratedWidget->SlatePrepass(LayoutScaleMultiplier);

		// We have a widget for this item; add it to the panel so that it is part of the UI.
		if (ItemIndex >= StartIndex)
		{
			// Generating widgets downward
			this->AppendWidget( WidgetForItem.ToSharedRef() );
		}
		else
		{
			// Backfilling widgets; going upward
			this->InsertWidget( WidgetForItem.ToSharedRef() );
		}

		const bool bIsVisible = NewlyGeneratedWidget->GetVisibility().IsVisible();
		FTableViewDimensions GeneratedWidgetDimensions(this->Orientation, bIsVisible ? NewlyGeneratedWidget->GetDesiredSize() : FVector2D::ZeroVector);
		return GeneratedWidgetDimensions.ScrollAxis;
	}

	void ReGeneratePinnedItems(const TArray<ItemType>& InItems, const FGeometry& MyGeometry, int32 MaxPinnedItemsOverride = -1)
	{
		const float LayoutScaleMultiplier = MyGeometry.GetAccumulatedLayoutTransform().GetScale();

		ClearPinnedWidgets();

		// Ensure that we always begin and clean up a generation pass.
		FGenerationPassGuard GenerationPassGuard(PinnedWidgetGenerator);

		// Check if the User provided an override for MaxPinnedItems, valid until the next time ReGeneratePinnedItems is called
		if (MaxPinnedItemsOverride != -1)
		{
			SetMaxPinnedItems(MaxPinnedItemsOverride);
		}
		else
		{
			// Reset it back to the default value if there is no override
			SetMaxPinnedItems(DefaultMaxPinnedItems);
		}

		int32 CurrentMaxPinnedItems = MaxPinnedItemsAttribute.Get();
		// There are more items than what we allow to show
		if (InItems.Num() > CurrentMaxPinnedItems)
		{
			bIsHierarchyCollapsed = true;
		}
		else
		{
			bIsHierarchyCollapsed = false;
		}
		

		const TArrayView<const ItemType> ItemsSourceRef = this->GetItems();

		for (int32 ItemIndex = 0; ItemIndex < InItems.Num(); ++ItemIndex)
		{
			GeneratePinnedWidgetForItem(InItems[ItemIndex], ItemIndex, InItems.Num(), LayoutScaleMultiplier);

			// Deselect any pinned items that were previously selected, since pinned items can only be navigated to on click and not selected
			if (TListTypeTraits<ItemType>::IsPtrValid(SelectorItem))
			{
				if (InItems[ItemIndex] == SelectorItem)
				{
					TListTypeTraits<ItemType>::ResetPtr(SelectorItem);
				}
			}

		}
		
	}

	void GeneratePinnedWidgetForItem(const ItemType& CurItem, int32 ItemIndex, int32 NumPinnedItems, float LayoutScaleMultiplier)
	{
		ensure(TListTypeTraits<ItemType>::IsPtrValid(CurItem));
		// Find a previously generated Widget for this item, if one exists.
		TSharedPtr<ITableRow> WidgetForItem = PinnedWidgetGenerator.GetWidgetForItem(CurItem);
		if (!WidgetForItem.IsValid())
		{
			// We couldn't find an existing widgets, meaning that this data item was not visible before.
			// Make a new widget for it.
			WidgetForItem = this->GenerateNewPinnedWidget(CurItem, ItemIndex, NumPinnedItems);
		}

		// It is useful to know the item's index that the widget was generated from.
		// Helps with even/odd coloring
		WidgetForItem->SetIndexInList(ItemIndex);

		// Let the item generator know that we encountered the current Item and associated Widget.
		PinnedWidgetGenerator.OnItemSeen(CurItem, WidgetForItem.ToSharedRef());

		// We wrap the row widget around an SListViewPinnedRowWidget for custom styling
		TSharedRef< SWidget > NewListItemWidget = SNew(SListViewPinnedRowWidget, WidgetForItem, SharedThis(this), ItemIndex, NumPinnedItems);
		NewListItemWidget->MarkPrepassAsDirty();
		NewListItemWidget->SlatePrepass(LayoutScaleMultiplier);

		// We have a widget for this item; add it to the panel so that it is part of the UI.
		this->AppendPinnedWidget(NewListItemWidget);
	}

	/** @return how many items there are in the TArray being observed */
	virtual int32 GetNumItemsBeingObserved() const override
	{
		return GetItems().Num();
	}

	virtual TSharedRef<ITableRow> GenerateNewPinnedWidget(ItemType InItem, const int32 ItemIndex, const int32 NumPinnedItems)
	{
		if (OnGeneratePinnedRow.IsBound())
		{
			return OnGeneratePinnedRow.Execute(InItem, SharedThis(this));
		}
		else
		{
			// The programmer did not provide an OnGeneratePinnedRow() handler; let them know.
			TSharedRef< STableRow<ItemType> > NewListItemWidget =
				SNew(STableRow<ItemType>, SharedThis(this))
				.Content()
				[
					SNew(STextBlock).Text(NSLOCTEXT("SListView", "OnGeneratePinnedRowNotAssignedMessage", "OnGeneratePinnedRow() not assigned."))
				];

			return NewListItemWidget;
		}

	}

	/**
	 * Given an data item, generate a widget corresponding to it.
	 *
	 * @param InItem  The data item which to visualize.
	 *
	 * @return A new Widget that represents the data item.
	 */
	virtual TSharedRef<ITableRow> GenerateNewWidget(ItemType InItem)
	{
		if ( OnGenerateRow.IsBound() )
		{
			return OnGenerateRow.Execute( InItem, SharedThis(this) );
		}
		else
		{
			// The programmer did not provide an OnGenerateRow() handler; let them know.
			TSharedRef< STableRow<ItemType> > NewListItemWidget = 
				SNew( STableRow<ItemType>, SharedThis(this) )
				.Content()
				[
					SNew(STextBlock) .Text( NSLOCTEXT("SListView", "OnGenerateWidgetNotAssignedMessage", "OnGenerateWidget() not assigned.") )
				];

			return NewListItemWidget;
		}

	}

	/**
	 * Establishes a wholly new list of items being observed by the list.
	 * Wipes all existing state and requests and will fully rebuild on the next tick.
	 */
	void SetItemsSource(const TArray<ItemType>* InListItemsSource)
	{
		ensureMsgf(InListItemsSource, TEXT("The ListItems is invalid."));
		if (ViewSource == nullptr || !ViewSource->IsSame(reinterpret_cast<const void*>(InListItemsSource)))
		{
			if (InListItemsSource)
			{
				SetItemsSource(MakeUnique<UE::Slate::ItemsSource::FArrayPointer<ItemType>>(InListItemsSource));
			}
			else
			{
				ClearItemsSource();
			}
		}
	}

	/**
	 * Establishes a wholly new list of items being observed by the list.
	 * Wipes all existing state and requests and will fully rebuild on the next tick.
	 * The ObservableArray will notify the Widget when it needs to refresh.
	 */
	void SetItemsSource(TSharedRef<::UE::Slate::Containers::TObservableArray<ItemType>> InListItemsSource)
	{
		if (ViewSource == nullptr || !ViewSource->IsSame(reinterpret_cast<const void*>(&InListItemsSource.Get())))
		{
			SetItemsSource(MakeUnique<UE::Slate::ItemsSource::FSharedObservableArray<ItemType>>(SharedThis(this), MoveTemp(InListItemsSource)));
		}
	}

	/**
	 * Establishes a wholly new list of items being observed by the list.
	 * Wipes all existing state and requests and will fully rebuild on the next tick.
	 */
	void SetItemsSource(TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>> Provider)
	{
		if (IsConstructed())
		{
			Private_ClearSelection();
			CancelScrollIntoView();
			ClearWidgets();

			ViewSource = MoveTemp(Provider);

			RebuildList();
		}
		else
		{
			ViewSource = MoveTemp(Provider);
		}
	}

	void ClearItemsSource()
	{
		SetItemsSource(TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>>());
	}

	void BindToRefreshRow(const FOnRefreshRow& Binding)
	{
		OnRefreshRow = Binding;
	}

public:

	UE_DEPRECATED(5.2, "SetListItemsSource is deprecated. Please use the correct SetItemsSource implementation.")
	void SetListItemsSource(const TArray<ItemType>& InListItemsSource)
	{
		SetItemsSource(&InListItemsSource);
	}

	bool HasValidItemsSource() const
	{
		return ViewSource != nullptr;
	}

	TArrayView<const ItemType> GetItems() const
	{
		return ViewSource ? ViewSource->GetItems() : TArrayView<const ItemType>();
	}

	/**
	 * Given a Widget, find the corresponding data item.
	 * 
	 * @param WidgetToFind  An widget generated by the list view for some data item.
	 *
	 * @return the data item from which the WidgetToFind was generated
	 */
	const TObjectPtrWrapTypeOf<ItemType>* ItemFromWidget( const ITableRow* WidgetToFind ) const
	{
		return Private_ItemFromWidget( WidgetToFind );
	}

	/**
	 * Test if the current item is selected.
	 *
	 * @param InItem The item to test.
	 *
	 * @return true if the item is selected in this list; false otherwise.
	 */
	bool IsItemSelected( const ItemType& InItem ) const
	{
		if ( GetSelectionMode() == ESelectionMode::None )
		{
			return false;
		}

		return Private_IsItemSelected( InItem );
	}

	/**
	 * Set the selection state of an item.
	 *
	 * @param InItem      The Item whose selection state to modify
	 * @param bSelected   true to select the item; false to unselect
	 * @param SelectInfo  Provides context on how the selection changed
	 */
	void SetItemSelection( const ItemType& InItem, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct )
	{
		if ( GetSelectionMode() == ESelectionMode::None )
		{
			return;
		}

		Private_SetItemSelection(InItem, bSelected, SelectInfo != ESelectInfo::Direct);
		Private_SignalSelectionChanged(SelectInfo);
	}

	/**
	 * Set the selection state of multiple items.
	 *
	 * @param InItems     The Items whose selection state to modify
	 * @param bSelected   true to select the items; false to unselect
	 * @param SelectInfo  Provides context on how the selection changed
	 */
	void SetItemSelection(TConstArrayView<ItemType> InItems, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct)
	{
		if ( InItems.Num() == 0 || GetSelectionMode() == ESelectionMode::None )
		{
			return;
		}

		for (const ItemType & Item : InItems)
		{
			Private_SetItemSelection(Item, bSelected, SelectInfo != ESelectInfo::Direct);

			// Any item after the first one selected will be direct
			SelectInfo = ESelectInfo::Direct;
		}
		Private_SignalSelectionChanged(SelectInfo);
	}

	/**
	 * Empty the selection set.
	 */
	void ClearSelection()
	{
		if ( SelectedItems.Num() == 0 )
		{
			return;
		}

		Private_ClearSelection();
		Private_SignalSelectionChanged(ESelectInfo::Direct);
	}



	/**
	* Set the highlighted state of an item.
	*
	* @param TheItem      The Item whose highlight state you wish to modify
	* @param bHighlighted True to enable the soft parent highlight, false to disable it.
	*/
	void SetItemHighlighted(const ItemType& TheItem, bool bHighlighted)
	{
		Private_SetItemHighlighted(TheItem, bHighlighted);
	}

	/**
	* Empty the highlighted item set.
	*/
	void ClearHighlightedItems()
	{
		Private_ClearHighlightedItems();
	}

	/**
	 * Gets the number of selected items.
	 *
	 * @return Number of selected items.
	 */
	int32 GetNumItemsSelected() const
	{
		return SelectedItems.Num();
	}

	virtual void RebuildList() override
	{
		WidgetGenerator.Clear();
		PinnedWidgetGenerator.Clear();
		RequestListRefresh();
	}

	/**
	 * Returns a list of selected item indices, or an empty array if nothing is selected
	 *
	 * @return	List of selected item indices (in no particular order)
	 */
	virtual TArray< ItemType > GetSelectedItems() const override
	{
		TArray< ItemType > SelectedItemArray;
		SelectedItemArray.Empty( SelectedItems.Num() );
		for( typename TItemSet::TConstIterator SelectedItemIt( SelectedItems ); SelectedItemIt; ++SelectedItemIt )
		{
			SelectedItemArray.Add( *SelectedItemIt );
		}
		return SelectedItemArray;
	}

	int32 GetSelectedItems(TArray< ItemType >& SelectedItemArray) const
	{
		SelectedItemArray.Empty(SelectedItems.Num());
		for (typename TItemSet::TConstIterator SelectedItemIt(SelectedItems); SelectedItemIt; ++SelectedItemIt)
		{
			SelectedItemArray.Add(*SelectedItemIt);
		}
		return SelectedItems.Num();
	}

	/**
	 * Checks whether the specified item is currently visible in the list view.
	 *
	 * @param Item - The item to check.
	 *
	 * @return true if the item is visible, false otherwise.
	 */
	bool IsItemVisible( ItemType Item ) const
	{
		return WidgetGenerator.GetWidgetForItem(Item).IsValid();
	}

	/**
	 * Scroll an item into view. If the item is not found, fails silently.
	 *
	 * @param ItemToView  The item to scroll into view on next tick.
	 */
	void RequestScrollIntoView( ItemType ItemToView, const uint32 UserIndex = 0)
	{
		ItemToScrollIntoView = ItemToView; 
		UserRequestingScrollIntoView = UserIndex;
		RequestLayoutRefresh();
	}

	UE_DEPRECATED(4.20, "RequestScrollIntoView no longer takes parameter bNavigateOnScrollIntoView. Call RequestNavigateToItem instead of RequestScrollIntoView if navigation is required.")
	void RequestScrollIntoView(ItemType ItemToView, const uint32 UserIndex, const bool NavigateOnScrollIntoView)
	{
		if (bNavigateOnScrollIntoView)
		{
			RequestNavigateToItem(ItemToView, UserIndex);
		}
		else
		{
			RequestScrollIntoView(ItemToView, UserIndex);
		}
	}

	/**
	 * Navigate to a specific item, scrolling it into view if needed. If the item is not found, fails silently.
	 *
	 * @param Item The item to navigate to on the next tick.
	 */
	void RequestNavigateToItem(ItemType Item, const uint32 UserIndex = 0)
	{
		TOptional<ItemType> FirstValidItem = Private_FindNextSelectableOrNavigable(Item);
		if (FirstValidItem.IsSet())
		{
			Private_RequestNavigateToItem(FirstValidItem.GetValue(), UserIndex);
		}
	}

private:
	void Private_RequestNavigateToItem(ItemType Item, const uint32 UserIndex)
	{
		bNavigateOnScrollIntoView = true;
		RequestScrollIntoView(Item, UserIndex);
	}

public:

	/**
	 * Cancels a previous request to scroll an item into view (cancels navigation requests as well).
	 */
	void CancelScrollIntoView()
	{
		UserRequestingScrollIntoView = 0;
		bNavigateOnScrollIntoView = false;
		TListTypeTraits<ItemType>::ResetPtr(ItemToScrollIntoView);
	}

	/**
	 * Set the currently selected Item.
	 *
	 * @param SoleSelectedItem   Sole item that should be selected.
	 * @param SelectInfo Provides context on how the selection changed
	 */
	void SetSelection(ItemType SoleSelectedItem, ESelectInfo::Type SelectInfo = ESelectInfo::Direct)
	{
		TOptional<ItemType> FirstValidItem = Private_FindNextSelectableOrNavigable(SoleSelectedItem);
		if (FirstValidItem.IsSet())
		{
			Private_SetSelection(FirstValidItem.GetValue(), SelectInfo);
		}
	}

private:
	void Private_SetSelection(ItemType SoleSelectedItem, ESelectInfo::Type SelectInfo)
	{
		SelectedItems.Empty();
		SetItemSelection( SoleSelectedItem, true, SelectInfo );
	}

public:

	/** 
	 * Set the current selection mode of the list.
	 * If going from multi-select to a type of single-select and one item is selected, it will be maintained (otherwise all will be cleared).
	 * If disabling selection, any current selections will be cleared.
	 */
	void SetSelectionMode(TAttribute<ESelectionMode::Type> NewSelectionMode)
	{
		Private_SetSelectionMode(MoveTemp(NewSelectionMode));
	}

	/* Sets ScrollIntoViewAlignment which allows to stick the selected item to either side or center */
	void SetScrollIntoViewAlignment (EScrollIntoViewAlignment NewScrollIntoViewAlignment)
	{
		ScrollIntoViewAlignment = NewScrollIntoViewAlignment;
	}

	void SetReturnFocusToSelection(bool bEnabled)
	{
		bReturnFocusToSelection = bEnabled;
	}

	/** Sets whether spatially proximate focus routing is enabled for cross-list and intra-entry navigation. */
	void SetEnableProximateEntryNavigation(bool bEnabled)
	{
		bEnableProximateEntryNavigation = bEnabled;
	}

	/**
	 * Find a widget for this item if it has already been constructed.
	 *
	 * @param InItem  The item for which to find the widget.
	 *
	 * @return A pointer to the corresponding widget if it exists; otherwise nullptr.
	*/
	virtual TSharedPtr<ITableRow> WidgetFromItem( const ItemType& InItem ) const override
	{
		TSharedPtr<ITableRow> ItemWidget = WidgetGenerator.GetWidgetForItem(InItem);

		return ItemWidget != nullptr ? ItemWidget : PinnedWidgetGenerator.GetWidgetForItem(InItem);
	}

	/**
	 * Lists and Trees serialize items that they observe because they rely on the property
	 * that holding a reference means it will not be garbage collected.
	 *
	 * @param Ar The archive to serialize with
	 */
	virtual void AddReferencedObjects( FReferenceCollector& Collector )
	{
		TListTypeTraits<ItemType>::AddReferencedObjects( Collector, WidgetGenerator.ItemsWithGeneratedWidgets, SelectedItems, WidgetGenerator.WidgetMapToItem );
	}
	virtual FString GetReferencerName() const
	{
		return TEXT("SListView");
	}

	/**
	* Will determine the max row size for the specified column id
	*
	* @param ColumnId  Column Id
	* @param ColumnOrientation  Orientation that is main axis you want to query
	*
	* @return The max size for a column Id.
	*/
	FVector2D GetMaxRowSizeForColumn(const FName& ColumnId, EOrientation ColumnOrientation)
	{
		FVector2D MaxSize = FVector2D::ZeroVector;

		for (auto It = WidgetGenerator.WidgetMapToItem.CreateConstIterator(); It; ++It)
		{
			const ITableRow* TableRow = It.Key();
			FVector2D NewMaxSize = TableRow->GetRowSizeForColumn(ColumnId);

			// We'll return the full size, but we only take into consideration the asked axis for the calculation of the size
			if (NewMaxSize.Component(ColumnOrientation) > MaxSize.Component(ColumnOrientation))
			{
				MaxSize = NewMaxSize;
			}
		}

		return MaxSize;
	}

protected:
	virtual void OnSelectionModeChanged(ESelectionMode::Type OldSelectionMode, ESelectionMode::Type NewSelectionMode) override
	{
		if (OldSelectionMode != NewSelectionMode)
		{
			if (NewSelectionMode == ESelectionMode::None)
			{
				ClearSelection();
			}
			else if (OldSelectionMode == ESelectionMode::Multi)
			{
				// We've gone to a single-selection mode, so if we already had a single item selected, preserve it
				if (SelectedItems.Num() == 1)
				{
					SetSelection(*SelectedItems.CreateIterator());
				}
				else
				{
					// Otherwise, there's no way to know accurately which item was selected most recently, so just wipe it all
					// The caller responsible for changing the mode can decide themselves which item they want to be selected
					ClearSelection();
				}
			}
		}
	}

	static FOnItemToString_Debug GetDefaultDebugDelegate()
	{
		return FOnItemToString_Debug::CreateStatic(TListTypeTraits<ItemType>::DebugDump);
	}

	/**
	 * If there is a pending request to scroll an item into view, do so.
	 * 
	 * @param ListViewGeometry  The geometry of the listView; can be useful for centering the item.
	 */
	virtual EScrollIntoViewResult ScrollIntoView( const FGeometry& ListViewGeometry ) override
	{
		bool bHandledByPixelScroll = false;
		if (PendingProximateNavType == EUINavigation::Invalid)
		{
			if (TSharedPtr<SWidget> ClippedWidget = PendingScrollContextWidget.Pin())
			{
				PendingScrollContextWidget.Reset();

				const FGeometry& WidgetGeo = ClippedWidget->GetCachedGeometry();
				const float ScrollDelta = ComputeScrollDeltaForAlignment(ListViewGeometry, WidgetGeo);
				if (ScrollDelta != 0.f)
				{
					this->ScrollBy(ListViewGeometry, ScrollDelta, EAllowOverscroll::No);
					bHandledByPixelScroll = true;
				}
			}
		}

		if (bHandledByPixelScroll && TListTypeTraits<ItemType>::IsPtrValid(ItemToScrollIntoView))
		{
			TListTypeTraits<ItemType>::ResetPtr(ItemToScrollIntoView);
		}
		else if (HasValidItemsSource() && TListTypeTraits<ItemType>::IsPtrValid(ItemToScrollIntoView))
		{
			const TArrayView<const ItemType> Items = GetItems();
			const int32 IndexOfItem = Items.Find( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( ItemToScrollIntoView ) );
			if (IndexOfItem != INDEX_NONE)
			{
				double NumLiveWidgets = GetNumLiveWidgets();
				if (NumLiveWidgets == 0. && IsPendingRefresh())
				{
					// Use the last number of widgets on screen to estimate if we actually need to scroll.
					NumLiveWidgets = LastGenerateResults.ExactNumLinesOnScreen;

					// If we still don't have any widgets, we're not in a situation where we can scroll an item into view
					// (probably as nothing has been generated yet), so we'll defer this again until the next frame
					if (NumLiveWidgets == 0)
					{
						return EScrollIntoViewResult::Deferred;
					}
				}

				EndInertialScrolling();

				// Pixel-based scroll path for proximate entry navigation. Uses actual widget
				// geometry instead of item-index math, which is unreliable for polymorphic views
				// with varying entry heights. Respects ScrollIntoViewAlignment preference.
				bool bHandledByPixelAlignment = false;
				if (bEnableProximateEntryNavigation)
				{
					if (TSharedPtr<ITableRow> TableRow = WidgetFromItem(Items[IndexOfItem]))
					{
						const FGeometry& WidgetGeo = TableRow->AsWidget()->GetCachedGeometry();
						const float ScrollDelta = ComputeScrollDeltaForAlignment(ListViewGeometry, WidgetGeo);

						if (ScrollDelta != 0.f)
						{
							this->ScrollBy(ListViewGeometry, ScrollDelta, EAllowOverscroll::No);
						}
						bHandledByPixelAlignment = true;
					}
				}

				// Standard item-index scroll path. Used when proximate entry navigation is off,
				// or when the widget is not yet generated (fall back to item-index positioning
				// so the list scrolls to the item's range and generates it).
				if (!bHandledByPixelAlignment)
				{
					const int32 NumFullEntriesInView = (int32)(FMath::FloorToDouble(CurrentScrollOffset + NumLiveWidgets) - FMath::CeilToDouble(CurrentScrollOffset));

				// Only scroll the item into view if it's not already in the visible range
				// When navigating, we don't want to scroll partially visible existing rows all the way to the center, so we count partially displayed indices in the displayed range
				const double MinDisplayedIndex = bNavigateOnScrollIntoView ? FMath::FloorToDouble(CurrentScrollOffset) : FMath::CeilToDouble(CurrentScrollOffset);
				const double MaxDisplayedIndex = bNavigateOnScrollIntoView ? FMath::CeilToDouble(CurrentScrollOffset + NumFullEntriesInView) : FMath::FloorToDouble(CurrentScrollOffset + NumFullEntriesInView);
				const bool bForceAlignmentScroll = (ScrollIntoViewAlignment == EScrollIntoViewAlignment::TopOrLeft || ScrollIntoViewAlignment == EScrollIntoViewAlignment::BottomOrRight);

				const bool bTargetNotGeneratedForNav = bNavigateOnScrollIntoView && !WidgetFromItem(Items[IndexOfItem]).IsValid();

				if (IndexOfItem < MinDisplayedIndex || IndexOfItem > MaxDisplayedIndex || bForceAlignmentScroll || bTargetNotGeneratedForNav)
					{
						// Scroll the top of the listview to the item in question
						double NewScrollOffset = IndexOfItem;

						switch (ScrollIntoViewAlignment)
						{
						case EScrollIntoViewAlignment::CenterAligned:
							if (bTargetNotGeneratedForNav && IndexOfItem > MaxDisplayedIndex)
							{
								NewScrollOffset -= (NumLiveWidgets - 1.0);
							}
							else
							{
								NewScrollOffset -= (NumLiveWidgets / 2.0);
							}
							break;

						case EScrollIntoViewAlignment::IntoView:
							if (IndexOfItem > MaxDisplayedIndex)
							{
								// Bring the new item in question to the bottom
								NewScrollOffset -= (NumLiveWidgets - 1.0);
							}

							// The alternative is that IndexOfItem < MinDisplayedIndex, and NewScrollOffset is already correct for that case
							break;

						case EScrollIntoViewAlignment::TopOrLeft:
							// Set NewScrollOffset to exactly IndexOfItem to ensure it's at the top/left
							NewScrollOffset = IndexOfItem;
							break;

						case EScrollIntoViewAlignment::BottomOrRight:
							// Set NewScrollOffset to position the item at the bottom/right
							NewScrollOffset = IndexOfItem - (NumLiveWidgets - 1.0);
							break;
						}

						// Limit offset to top and bottom of the list.
						const double MaxScrollOffset = FMath::Max(0.0, static_cast<double>(Items.Num()) - NumLiveWidgets);
						NewScrollOffset = FMath::Clamp<double>(NewScrollOffset, 0.0, MaxScrollOffset);

						SetScrollOffset((float)NewScrollOffset);
					}
					else if (bNavigateOnScrollIntoView)
					{
						if (TSharedPtr<ITableRow> TableRow = WidgetFromItem(Items[IndexOfItem]))
						{
							const FGeometry& WidgetGeometry = TableRow->AsWidget()->GetCachedGeometry();
							const FTableViewDimensions WidgetTopLeft(this->Orientation, WidgetGeometry.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector));
							const FTableViewDimensions ListViewTopLeft(this->Orientation, ListViewGeometry.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector));

							double NewScrollOffset = DesiredScrollOffset;
							if (WidgetTopLeft.ScrollAxis < ListViewTopLeft.ScrollAxis)
							{
								NewScrollOffset = static_cast<double>(IndexOfItem) - NavigationScrollOffset;
							}
							else
							{
								const FVector2D BottomRight(1.f, 1.f);
								const FTableViewDimensions WidgetBottomRight(this->Orientation, WidgetGeometry.GetAbsolutePositionAtCoordinates(BottomRight));
								const FTableViewDimensions ListViewBottomRight(this->Orientation, ListViewGeometry.GetAbsolutePositionAtCoordinates(BottomRight));
								
								if (WidgetBottomRight.ScrollAxis > ListViewBottomRight.ScrollAxis)
								{
									float DistanceRemaining = WidgetBottomRight.ScrollAxis - ListViewBottomRight.ScrollAxis;
									float AdditionalOffset = 0.f;
									for (const ItemType& ItemWithWidget : WidgetGenerator.ItemsWithGeneratedWidgets)
									{
										FTableViewDimensions WidgetAbsoluteDimensions(this->Orientation, WidgetGenerator.GetWidgetForItem(ItemWithWidget)->AsWidget()->GetCachedGeometry().GetAbsoluteSize());
										if (WidgetAbsoluteDimensions.ScrollAxis < DistanceRemaining)
										{
											DistanceRemaining -= WidgetAbsoluteDimensions.ScrollAxis;
											AdditionalOffset += 1.f;
										}
										else
										{
											AdditionalOffset += DistanceRemaining / WidgetAbsoluteDimensions.ScrollAxis;
											DistanceRemaining = 0.f;
											break;
										}
									}

									NewScrollOffset = DesiredScrollOffset + AdditionalOffset;
								}
							}

							if (NewScrollOffset != DesiredScrollOffset)
							{
								SetScrollOffset((float)NewScrollOffset);
							}
						}
					}
				}

				RequestLayoutRefresh();

				ItemToNotifyWhenInView = ItemToScrollIntoView;
			}

			TListTypeTraits<ItemType>::ResetPtr(ItemToScrollIntoView);
		}

		if (TListTypeTraits<ItemType>::IsPtrValid(ItemToNotifyWhenInView))
		{
			if (this->bEnableAnimatedScrolling)
			{
				// When we have a target item we're shooting for, we haven't succeeded with the scroll until a widget for it exists
				const bool bHasWidgetForItem = WidgetFromItem(TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(ItemToNotifyWhenInView)).IsValid();
				return bHasWidgetForItem ? EScrollIntoViewResult::Success : EScrollIntoViewResult::Deferred;
			}
		}
		else
		{
			return EScrollIntoViewResult::Failure;
		}
		return EScrollIntoViewResult::Success;
	}

	virtual void NotifyItemScrolledIntoView() override
	{
		// Notify that an item that came into view
		if ( TListTypeTraits<ItemType>::IsPtrValid( ItemToNotifyWhenInView ) )
		{
			ItemType NonNullItemToNotifyWhenInView = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( ItemToNotifyWhenInView );
			TSharedPtr<ITableRow> Widget = WidgetGenerator.GetWidgetForItem(NonNullItemToNotifyWhenInView);
			
			if (Widget.IsValid())
			{
				if (bNavigateOnScrollIntoView)
				{
					SelectorItem = NonNullItemToNotifyWhenInView;
					NavigateToWidget(UserRequestingScrollIntoView, Widget->AsWidget(), ENavigationSource::FocusedWidget, NavigationTypeOnScrollIntoView);
					NavigationTypeOnScrollIntoView = EUINavigation::Invalid;
				}
			
				OnItemScrolledIntoView.ExecuteIfBound(NonNullItemToNotifyWhenInView, Widget);
			}

			bNavigateOnScrollIntoView = false;

			TListTypeTraits<ItemType>::ResetPtr(ItemToNotifyWhenInView);
		}
		else if (!TListTypeTraits<ItemType>::IsPtrValid(ItemToScrollIntoView))
		{
			// No item to notify and no pending scroll request - bNavigateOnScrollIntoView
			// is orphaned (e.g. culling guard skipped the scroll block). Clear it to prevent
			// NotifyItemScrolledIntoView from firing indefinitely.
			bNavigateOnScrollIntoView = false;
		}
	}

	virtual void NotifyFinishedScrolling() override
	{
		OnFinishedScrolling.ExecuteIfBound();
	}

	virtual void NotifyTouchStart() override
	{
		OnTouchStart.ExecuteIfBound();
	}

	virtual void NotifyTouchMove() override
	{
		OnTouchMove.ExecuteIfBound();
	}

	virtual void NotifyTouchEnd() override
	{
		OnTouchEnd.ExecuteIfBound();
	}

	virtual float ScrollBy( const FGeometry& MyGeometry, float ScrollByAmountInSlateUnits, EAllowOverscroll InAllowOverscroll ) override
	{
		auto DoubleFractional = [](double Value) -> double
		{
			return Value - FMath::TruncToDouble(Value);
		};

		if (InAllowOverscroll == EAllowOverscroll::No)
		{
			//check if we are on the top of the list and want to scroll up
			if (DesiredScrollOffset < UE_KINDA_SMALL_NUMBER && ScrollByAmountInSlateUnits < 0)
			{
				return 0.0f;
			}

			//check if we are on the bottom of the list and want to scroll down
			if (bWasAtEndOfList && ScrollByAmountInSlateUnits > 0)
			{
				return 0.0f;
			}
		}

		float AbsScrollByAmount = FMath::Abs( ScrollByAmountInSlateUnits );
		int32 StartingItemIndex = (int32)CurrentScrollOffset;
		double NewScrollOffset = DesiredScrollOffset;

		const bool bWholeListVisible = DesiredScrollOffset == 0.0 && bWasAtEndOfList;
		if ( InAllowOverscroll == EAllowOverscroll::Yes && Overscroll.ShouldApplyOverscroll(DesiredScrollOffset == 0.0, bWasAtEndOfList, ScrollByAmountInSlateUnits ) )
		{
			const float UnclampedScrollDelta = FMath::Sign(ScrollByAmountInSlateUnits) * AbsScrollByAmount;				
			const float ActuallyScrolledBy = Overscroll.ScrollBy(MyGeometry, UnclampedScrollDelta);
			if (ActuallyScrolledBy != 0.0f)
			{
				this->RequestLayoutRefresh();
			}
			return ActuallyScrolledBy;
		}
		else if (!bWholeListVisible)
		{
			// We know how far we want to scroll in SlateUnits, but we store scroll offset in "number of widgets".
			// Challenge: each widget can be a different height/width.
			// Strategy:
			//           Scroll "one widget's length" at a time until we've scrolled as far as the user asked us to.
			//           Generate widgets on demand so we can figure out how big they are.

			const TArrayView<const ItemType> Items = GetItems();
			if (Items.Num() > 0)
			{
				int32 ItemIndex = StartingItemIndex;
				const float LayoutScaleMultiplier = MyGeometry.GetAccumulatedLayoutTransform().GetScale();
				while( AbsScrollByAmount != 0 && ItemIndex < Items.Num() && ItemIndex >= 0 )
				{
					const ItemType& CurItem = Items[ItemIndex];
					if (!TListTypeTraits<ItemType>::IsPtrValid(CurItem))
					{
						// If the CurItem is not valid, we do not generate a new widget for it, we skip it.
						++ItemIndex;
						continue;
					}

					TSharedPtr<ITableRow> RowWidget = WidgetGenerator.GetWidgetForItem( CurItem );
					if (!RowWidget.IsValid())
					{
						// We couldn't find an existing widgets, meaning that this data item was not visible before.
						// Make a new widget for it.
						RowWidget = this->GenerateNewWidget( CurItem );

						// It is useful to know the item's index that the widget was generated from.
						// Helps with even/odd coloring
						RowWidget->SetIndexInList(ItemIndex);

						// Let the item generator know that we encountered the current Item and associated Widget.
						WidgetGenerator.OnItemSeen( CurItem, RowWidget.ToSharedRef() );

						RowWidget->AsWidget()->SlatePrepass(LayoutScaleMultiplier);
					}

					const FTableViewDimensions WidgetDimensions(this->Orientation, RowWidget->AsWidget()->GetDesiredSize());
					if (ScrollByAmountInSlateUnits > 0)
					{
						const float RemainingDistance = WidgetDimensions.ScrollAxis * (float)(1.0 - DoubleFractional(NewScrollOffset));

						if (AbsScrollByAmount > RemainingDistance)
						{
							if (ItemIndex != Items.Num())
							{
								AbsScrollByAmount -= RemainingDistance;
								NewScrollOffset = 1.0 + (int32)NewScrollOffset;
								++ItemIndex;
							}
							else
							{
								NewScrollOffset = Items.Num();
								break;
							}
						} 
						else if ( AbsScrollByAmount == RemainingDistance)
						{
							NewScrollOffset = 1.0 + (int32)NewScrollOffset;
							break;
						}
						else
						{
							NewScrollOffset = (int32)NewScrollOffset + (1.0 - ((RemainingDistance - AbsScrollByAmount) / WidgetDimensions.ScrollAxis));
							break;
						}
					}
					else
					{
						float Fractional = FMath::Fractional( (float)NewScrollOffset );
						if ( FMath::IsNearlyEqual(Fractional, 0.f) )
						{
							Fractional = 1.0f;
							--NewScrollOffset;
						}

						const float PrecedingDistance = WidgetDimensions.ScrollAxis * Fractional;

						if ( AbsScrollByAmount > PrecedingDistance)
						{
							if ( ItemIndex != 0 )
							{
								AbsScrollByAmount -= PrecedingDistance;
								NewScrollOffset -= DoubleFractional( NewScrollOffset );
								--ItemIndex;
							}
							else
							{
								NewScrollOffset = 0.0;
								break;
							}
						} 
						else if ( AbsScrollByAmount == PrecedingDistance)
						{
							NewScrollOffset -= DoubleFractional( NewScrollOffset );
							break;
						}
						else
						{
							NewScrollOffset = float(FMath::TruncToInt32(NewScrollOffset)) + ((PrecedingDistance - AbsScrollByAmount) / WidgetDimensions.ScrollAxis);
							break;
						}
					}
				}
			}


			return ScrollTo( (float)NewScrollOffset );
		}

		return 0;
	}

protected:

	TOptional<ItemType> Private_FindNextSelectableOrNavigableWithIndexAndDirection(const ItemType& InItemToSelect, int32 SelectionIdx, bool bSelectForward)
	{
		ItemType ItemToSelect = InItemToSelect;

		if (OnIsSelectableOrNavigable.IsBound())
		{
			// Walk through the list until we either find a navigable item or run out of entries.
			const TArrayView<const ItemType> Items = GetItems();
			while (!OnIsSelectableOrNavigable.Execute(ItemToSelect))
			{
				SelectionIdx += (bSelectForward ? 1 : -1);
				if (Items.IsValidIndex(SelectionIdx))
				{
					ItemToSelect = Items[SelectionIdx];
				}
				else
				{
					// Failed to find a valid item to select
					return TOptional<ItemType>();
				}
			}
		}

		return TOptional<ItemType>(ItemToSelect);
	}

	TOptional<ItemType> Private_FindNextSelectableOrNavigable(const ItemType& InItemToSelect)
	{
		ItemType ItemToSelect = InItemToSelect;

		if (OnIsSelectableOrNavigable.IsBound())
		{
			if (!OnIsSelectableOrNavigable.Execute(ItemToSelect))
			{
				const TArrayView<const ItemType> Items = GetItems();
				int32 NewSelectionIdx = Items.Find(ItemToSelect);

				// By default, we walk forward
				bool bSelectNextItem = true;
				if (SelectedItems.Num() == 1)
				{
					// If the last selected item is after the item to select, we'll want to walk backwards
					NullableItemType LastSelectedItem = *SelectedItems.CreateIterator();
					if (TListTypeTraits<ItemType>::IsPtrValid(LastSelectedItem))
					{
						ItemType NonNullLastSelectedItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(LastSelectedItem);
						const int32 LastSelectedItemIdx = Items.Find(NonNullLastSelectedItem);

						bSelectNextItem = LastSelectedItemIdx < NewSelectionIdx;
					}
				}

				// Walk through the list until we either find a navigable item or run out of entries.
				do
				{
					NewSelectionIdx += (bSelectNextItem ? 1 : -1);
					if (Items.IsValidIndex(NewSelectionIdx))
					{
						ItemToSelect = Items[NewSelectionIdx];
					}
					else
					{
						// Failed to find a valid item to select
						return TOptional<ItemType>();
					}
				} while (!OnIsSelectableOrNavigable.Execute(ItemToSelect));
			}
		}

		return TOptional<ItemType>(ItemToSelect);
	}

	/**
	 * Selects the specified item and scrolls it into view. If shift is held, it will be a range select.
	 * If ctrl is held and multi selection is allowed, the item will be added to the list of currently selected items
	 * 
	 * @param ItemToSelect		The item that was selected by a keystroke
	 * @param InInputEvent	The key event that caused this selection
	 */
	virtual void NavigationSelect(const ItemType& InItemToSelect, const FInputEvent& InInputEvent)
	{
		TOptional<ItemType> ItemToSelect = Private_FindNextSelectableOrNavigable(InItemToSelect);
		if (!ItemToSelect.IsSet())
		{
			return;
		}

		const ESelectionMode::Type CurrentSelectionMode = GetSelectionMode();

		if (CurrentSelectionMode != ESelectionMode::None)
		{
			// Must be set before signaling selection changes because sometimes new items will be selected that need to stomp this value
			SelectorItem = ItemToSelect.GetValue();

			// Always request scroll into view, otherwise partially visible items will be selected - also do this before signaling selection for similar stomp-allowing reasons
			Private_RequestNavigateToItem(ItemToSelect.GetValue(), InInputEvent.GetUserIndex());

			if (bSelectItemOnNavigation)
			{
				if (CurrentSelectionMode == ESelectionMode::Multi && (InInputEvent.IsShiftDown() || InInputEvent.IsControlDown()))
				{
					// Range select.
					if (InInputEvent.IsShiftDown())
					{
						// Holding control makes the range select bidirectional, where as it is normally unidirectional.
						if (!(InInputEvent.IsControlDown()))
						{
							this->Private_ClearSelection();
						}

						this->Private_SelectRangeFromCurrentTo(ItemToSelect.GetValue());
					}
					// For ctrl select, simply add the item to the list of currently selected items
					else
					{
						this->Private_SetItemSelection(ItemToSelect.GetValue(), true, true);
					}
				
					this->Private_SignalSelectionChanged(ESelectInfo::OnNavigation);
				}
				else
				{
					// Single select.
					this->Private_SetSelection(ItemToSelect.GetValue(), ESelectInfo::OnNavigation);
				}
			}
		}
	}

	/**
	* Special handling for navigation events
	*/
	virtual void NavigationSelect(const ItemType& InItemToSelect, const FNavigationEvent& InNavigationEvent)
	{
		NavigationSelect(InItemToSelect, StaticCast<FInputEvent>(InNavigationEvent));

		// If the navigation requested a scroll into view, then set navigation type
		NavigationTypeOnScrollIntoView = bNavigateOnScrollIntoView ? InNavigationEvent.GetNavigationType() : EUINavigation::Invalid;
	}

protected:
	int32 GetMaxPinnedItems() const
	{
		return MaxPinnedItemsAttribute.Get();
	}
	void SetMaxPinnedItems(TAttribute<int32> InMaxPinnedItems)
	{
		MaxPinnedItemsAttribute.Assign(*this, MoveTemp(InMaxPinnedItems));
	}
	TSlateAttributeRef<int32> GetMaxPinnedItemsAttribute() const
	{
		return TSlateAttributeRef<int32>{SharedThis(this), MaxPinnedItemsAttribute};
	}

	const FTableViewStyle* GetStyle() const { return Style;}

	/**
	 * Selects the boundary item that is spatially nearest to the source widget stored in the
	 * navigation context on the cross axis.
	 *
	 * Pass 1 - rendered boundary row: iterates the boundary line items, measures each rendered
	 * widget's cross-axis center and returns the closest one.
	 * Pass 2 - off-screen fallback: divides the list's cross-axis extent by NumItemsPerLine and
	 * maps the source center to a column index.  Skipped for single-column lists.
	 *
	 * @param SourceWidget      Widget that held focus before navigation (used for cross-axis proximity measurement).
	 * @param InNavigationType  The navigation direction (Down, Up, Left, Right) used to determine the comparison axis.
	 * @param ItemsSourceRef   View of the full item array.
	 * @param NumItems         Cached count of items.
	 * @param NumItemsPerLine  Number of items per row/column.
	 * @param bNavForward      True when navigating Down/Right (boundary = first row), false for Up/Left (boundary = last row).
	 * @return Best boundary index, or BoundaryLineStart on failure.
	 */
	int32 FindProximateBoundaryItem(TSharedPtr<SWidget> SourceWidget, EUINavigation InNavigationType, const TArrayView<const ItemType>& ItemsSourceRef, int32 NumItems, int32 NumItemsPerLine, bool bNavForward) const
	{
		int32 EffectiveNumItems = NumItems;
		if (!bNavForward)
		{
			for (int32 i = NumItems - 1; i >= 0; --i)
			{
				if (this->WidgetFromItem(ItemsSourceRef[i]).IsValid())
				{
					EffectiveNumItems = i + 1;
					break;
				}
			}
		}

		if (!SourceWidget.IsValid())
		{
			return bNavForward ? 0 : ((EffectiveNumItems - 1) / NumItemsPerLine) * NumItemsPerLine;
		}

		// The navigation direction determines which axis we compare for proximity - perpendicular
		// to the direction of travel. Down/Up -> preserve X (horizontal position).
		// Left/Right -> preserve Y (vertical position).
		const EUINavigation NavType = InNavigationType;
		const bool bCompareOnXAxis  = (NavType == EUINavigation::Down || NavType == EUINavigation::Up);

		// For cross-axis navigation (e.g. Down into a horizontal list), all items share the same
		// position on the nav axis, so all items are candidates. For primary-axis navigation, only
		// the boundary row/column is searched.
		const bool bNavIsCrossAxis =
			(Orientation == Orient_Vertical   && (NavType == EUINavigation::Left || NavType == EUINavigation::Right)) ||
			(Orientation == Orient_Horizontal && (NavType == EUINavigation::Up   || NavType == EUINavigation::Down));

		const int32 BoundaryLineStart = bNavIsCrossAxis ? 0
		                              : bNavForward     ? 0
		                              :                   ((EffectiveNumItems - 1) / NumItemsPerLine) * NumItemsPerLine;
		const int32 BoundaryLineEnd   = bNavIsCrossAxis ? EffectiveNumItems - 1
		                              : bNavForward     ? FMath::Min(NumItemsPerLine, EffectiveNumItems) - 1
		                              :                   EffectiveNumItems - 1;

		const FSlateRect SourceBounds     = SourceWidget->GetCachedGeometry().GetRenderBoundingRect();
		const float      SourceCrossCenter = bCompareOnXAxis
			? (SourceBounds.Left + SourceBounds.Right) * 0.5f
			: (SourceBounds.Top  + SourceBounds.Bottom) * 0.5f;

		// Pass 1: search rendered boundary items
		int32 BestIndex = BoundaryLineStart;
		float BestDist  = TNumericLimits<float>::Max();

		for (int32 Idx = BoundaryLineStart; Idx <= BoundaryLineEnd; ++Idx)
		{
			const TSharedPtr<ITableRow> Row = this->WidgetFromItem(ItemsSourceRef[Idx]);
			if (!Row.IsValid())
			{
				continue;
			}
			const FSlateRect RowBounds      = Row->AsWidget()->GetCachedGeometry().GetRenderBoundingRect();
			const float      ItemCrossCenter = bCompareOnXAxis
				? (RowBounds.Left + RowBounds.Right) * 0.5f
				: (RowBounds.Top  + RowBounds.Bottom) * 0.5f;
			const float Dist = FMath::Abs(ItemCrossCenter - SourceCrossCenter);
			if (Dist < BestDist)
			{
				BestDist  = Dist;
				BestIndex = Idx;
			}
		}

		if (BestDist < TNumericLimits<float>::Max())
		{
			return BestIndex;
		}

		// Pass 2: off-screen fallback - map source position to a slot index
		if (NumItemsPerLine > 1)
		{
			const FSlateRect ListBounds    = this->GetCachedGeometry().GetRenderBoundingRect();
			const float      ListExtent    = bCompareOnXAxis
				? FMath::Max(ListBounds.Right  - ListBounds.Left, 1.0f)
				: FMath::Max(ListBounds.Bottom - ListBounds.Top,  1.0f);
			const float      ListOrigin    = bCompareOnXAxis ? ListBounds.Left : ListBounds.Top;
			const float      NormalizedPos = FMath::Clamp((SourceCrossCenter - ListOrigin) / ListExtent, 0.0f, 1.0f);
			const int32      ColumnIdx     = FMath::Clamp(FMath::FloorToInt(NormalizedPos * static_cast<float>(NumItemsPerLine)), 0, NumItemsPerLine - 1);
			const int32      CandidateIdx  = BoundaryLineStart + ColumnIdx;
			if (ItemsSourceRef.IsValidIndex(CandidateIdx))
			{
				return CandidateIdx;
			}
		}

		return BoundaryLineStart;
	}

	/**
	 * Resolves ItemNavigatedTo from the pending navigation context when bEnableProximateEntryNavigation is set.
	 * If ItemNavigatedTo is already valid (e.g. a higher-priority rule already set it), the nav context
	 * is skipped. Falls back to the previously selected item, then to item 0, when no proximate
	 * candidate is found.
	 * @param ItemNavigatedTo  In/out - may be pre-set by the caller; only written if still null.
	 * @param UserIndex        Slate user index used to look up and clear the pending context.
	 */
	void TryResolveProximateFocusEntry(NullableItemType& ItemNavigatedTo, const FFocusEvent& InFocusEvent) const
	{
		if (!bEnableProximateEntryNavigation)
		{
			return;
		}

		if (!TListTypeTraits<ItemType>::IsPtrValid(ItemNavigatedTo))
		{
			if (const TSharedPtr<UE::Slate::Private::FListViewNavigationContext> NavContext =
				FSlateApplication::Get().GetPendingNavigationContext<UE::Slate::Private::FListViewNavigationContext>(InFocusEvent.GetUser()))
			{
				const TArrayView<const ItemType> Items     = this->GetItems();
				const int32                      NumItems   = Items.Num();
				const int32                      NumPerLine = this->GetNumItemsPerLine();
				if (NumItems > 0 && NumPerLine > 0)
				{
					const bool  bNavForward  = (NavContext->NavigationType == EUINavigation::Down || NavContext->NavigationType == EUINavigation::Right);
					const int32 ProximateIdx = FindProximateBoundaryItem(NavContext->SourceWidget.Pin(), NavContext->NavigationType, Items, NumItems, NumPerLine, bNavForward);
					if (Items.IsValidIndex(ProximateIdx))
					{
						ItemNavigatedTo = Items[ProximateIdx];
					}
				}
				FSlateApplication::Get().ClearPendingNavigationContext(InFocusEvent.GetUser());
			}
		}

		// Proximate did not resolve: fall back to the last selected item, then to item 0.
		if (!TListTypeTraits<ItemType>::IsPtrValid(ItemNavigatedTo) && TListTypeTraits<ItemType>::IsPtrValid(SelectorItem) && InFocusEvent.GetCause() == EFocusCause::SetDirectly)
		{
			ItemNavigatedTo = SelectorItem;
		}

		if (!TListTypeTraits<ItemType>::IsPtrValid(ItemNavigatedTo) && this->GetItems().Num() > 0)
		{
			ItemNavigatedTo = this->GetItems()[0];
		}
	}

/**
 * Keeps SelectorItem in sync with the currently focused entry by walking
 * the widget path below this list. Always updates SelectorItem to match
 * the focused entry. Only requests a scroll when SelectorItem was
 * previously unset and the matched row is clipped by this list's viewport.
 */
void SyncSelectorItemFromFocusPath(const FWidgetPath& InWidgetPath, uint32 UserIndex)
{
	bool bWasUnset = !TListTypeTraits<ItemType>::IsPtrValid(SelectorItem);

	bool bFoundSelf = false;
	for (int32 i = 0; i < InWidgetPath.Widgets.Num(); ++i)
	{
		if (!bFoundSelf)
		{
			if (&InWidgetPath.Widgets[i].Widget.Get() == this)
			{
				bFoundSelf = true;
			}
			continue;
		}

		const TSharedRef<SWidget>& Candidate = InWidgetPath.Widgets[i].Widget;
		for (const ItemType& Item : this->GetItems())
		{
			if (const TSharedPtr<ITableRow> Row = this->WidgetFromItem(Item))
			{
				if (Row->AsWidget() == Candidate)
				{
					SelectorItem = Item;
					if (bWasUnset)
					{
						const FGeometry& RowGeo = Row->AsWidget()->GetCachedGeometry();
						const FGeometry& ListGeo = this->GetCachedGeometry();
						FTableViewDimensions RowStart(this->Orientation, RowGeo.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector));
						FTableViewDimensions RowEnd(this->Orientation, RowGeo.GetAbsolutePositionAtCoordinates(FVector2D(1.f, 1.f)));
						FTableViewDimensions ListStart(this->Orientation, ListGeo.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector));
						FTableViewDimensions ListEnd(this->Orientation, ListGeo.GetAbsolutePositionAtCoordinates(FVector2D(1.f, 1.f)));
						bool bRowClipped = (RowStart.ScrollAxis < ListStart.ScrollAxis) || (RowEnd.ScrollAxis > ListEnd.ScrollAxis);
						if (bRowClipped)
						{
							RequestScrollIntoView(Item, UserIndex);
						}
					}
					return;
				}
			}
		}
	}
}

/** Pixel scroll delta in list-local space required to honor ScrollIntoViewAlignment. */
float ComputeScrollDeltaForAlignment(
	const FGeometry& ListGeo,
	const FGeometry& WidgetGeo) const
{
	const FVector2f WidgetStartLocal = ListGeo.AbsoluteToLocal(WidgetGeo.GetAbsolutePositionAtCoordinates(FVector2f(0.f, 0.f)));
	const FVector2f WidgetEndLocal = ListGeo.AbsoluteToLocal(WidgetGeo.GetAbsolutePositionAtCoordinates(FVector2f(1.f, 1.f)));
	const float ListSizeOnScrollAxis = (Orientation == Orient_Vertical) ? ListGeo.GetLocalSize().Y : ListGeo.GetLocalSize().X;
	const float WidgetStart = (Orientation == Orient_Vertical) ? WidgetStartLocal.Y : WidgetStartLocal.X;
	const float WidgetEnd = (Orientation == Orient_Vertical) ? WidgetEndLocal.Y : WidgetEndLocal.X;
	const float ListStart = 0.f;
	const float ListEnd = ListSizeOnScrollAxis;
	const float TopAlignDelta = WidgetStart - ListStart;

	float ScrollDelta = 0.f;
	switch (ScrollIntoViewAlignment)
	{
	case EScrollIntoViewAlignment::IntoView:
		if (WidgetEnd > ListEnd)
		{
			ScrollDelta = WidgetEnd - ListEnd;
		}
		else if (WidgetStart < ListStart)
		{
			ScrollDelta = WidgetStart - ListStart;
		}
		if (ScrollDelta > 0.f)
		{
			ScrollDelta = FMath::Min(ScrollDelta, FMath::Max(0.f, WidgetStart - ListStart));
		}
		else if (ScrollDelta < 0.f)
		{
			ScrollDelta = FMath::Max(ScrollDelta, FMath::Min(0.f, WidgetEnd - ListEnd));
		}
		break;
	case EScrollIntoViewAlignment::CenterAligned:
		if ((WidgetEnd - WidgetStart) > ListSizeOnScrollAxis)
		{
			ScrollDelta = TopAlignDelta;
		}
		else
		{
			ScrollDelta = ((WidgetStart + WidgetEnd) * 0.5f) - (ListSizeOnScrollAxis * 0.5f);
		}
		break;
	case EScrollIntoViewAlignment::TopOrLeft:
		ScrollDelta = TopAlignDelta;
		break;
	case EScrollIntoViewAlignment::BottomOrRight:
		ScrollDelta = WidgetEnd - ListEnd;
		break;
	default:
		break;
	}

	if ((WidgetStart - ScrollDelta) < ListStart)
	{
		ScrollDelta = TopAlignDelta;
	}
	return ScrollDelta;
}

/**
 * Requests a deferred pixel-based scroll to fully reveal a nested descendant
 * widget that is clipped by this list's viewport. Intended for ancestor/root
 * lists only - lists actively handling their own navigation scroll
 * (bNavigateOnScrollIntoView == true) skip this to avoid double-scrolling.
 */
void ScrollNestedClippedWidgetIntoView(const SWidget& InWidget)
{
	if (bNavigateOnScrollIntoView) return;

	const FGeometry& ListGeo = this->GetCachedGeometry();
	const FGeometry& WidgetGeo = InWidget.GetCachedGeometry();
	FTableViewDimensions WidgetStart(this->Orientation, WidgetGeo.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector));
	FTableViewDimensions WidgetEnd(this->Orientation, WidgetGeo.GetAbsolutePositionAtCoordinates(FVector2D(1.f, 1.f)));
	FTableViewDimensions ListStart(this->Orientation, ListGeo.GetAbsolutePositionAtCoordinates(FVector2D::ZeroVector));
	FTableViewDimensions ListEnd(this->Orientation, ListGeo.GetAbsolutePositionAtCoordinates(FVector2D(1.f, 1.f)));

	const bool bClipped = (WidgetEnd.ScrollAxis > ListEnd.ScrollAxis) ||
		(WidgetStart.ScrollAxis < ListStart.ScrollAxis);

	if (bClipped)
	{
		// Consumed by ScrollIntoView when PendingProximateNavType == Invalid
		PendingScrollContextWidget = ConstCastSharedRef<SWidget>(InWidget.AsShared());
		RequestLayoutRefresh();
	}
}

protected:
	/** Widget cached for deferred scroll/navigation processing. Used for two mutually exclusive cases:
	 * (1) Cross-entry proximate focus: the navigation source widget, consumed by NotifyItemScrolledIntoView
	 *     in SCommonListView/SCommonTileView when PendingProximateNavType != Invalid.
	 * (2) Clipped-widget scroll: the focused descendant clipped by this list's viewport, consumed by
	 *     ScrollIntoView when PendingProximateNavType == Invalid.
	 */
	TWeakPtr<SWidget> PendingScrollContextWidget;
	EUINavigation     PendingProximateNavType = EUINavigation::Invalid;

	/** A widget generator component */
	FWidgetGenerator WidgetGenerator;

	/** A widget generator component used for pinned items in the list */
	FWidgetGenerator PinnedWidgetGenerator;

	/** Invoked after initializing an entry being generated, before it may be added to the actual widget hierarchy. */
	FOnEntryInitialized OnEntryInitialized;

	/** Delegate to be invoked when the list needs to generate a new widget from a data item. */
	FOnGenerateRow OnGenerateRow;

	/** Delegate to be invoked when the list needs to generate a new pinned widget from a data item. */
	FOnGenerateRow OnGeneratePinnedRow;

	/** Delegate to be invoked when the list needs to update an existing widget for an item. */
	FOnRefreshRow OnRefreshRow;

	/** Assign this to get more diagnostics from the list view. */
	FOnItemToString_Debug OnItemToString_Debug;

	/** Invoked when the tree enters a bad state. */
	FOnTableViewBadState OnEnteredBadState;

	/**/
	FOnWidgetToBeRemoved OnRowReleased;

	/** Delegate to be invoked when an item has come into view after it was requested to come into view. */
	FOnItemScrolledIntoView OnItemScrolledIntoView;

	/** Delegate to be invoked when TargetScrollOffset is reached at the end of a ::Tick. */
	FOnFinishedScrolling OnFinishedScrolling;

	/** Delegate to be invoked when the touch has start. */
	FOnTouchStart OnTouchStart;

	/** Delegate to be invoked when the user has scrolled the list some amount. */
	FOnTouchMove OnTouchMove;

	/** Delegate to be invoked when the user has lifted its finger after scrolling. */
	FOnTouchEnd OnTouchEnd;

	/** A set of selected data items */
	TItemSet SelectedItems;

	/** The item to manipulate selection for */
	NullableItemType SelectorItem;

	/** The item which was last manipulated; used as a start for shift-click selection */
	NullableItemType RangeSelectionStart;

	/** A set of which items should be highlighted */
	TItemSet HighlightedItems;

	UE_DEPRECATED(5.2, "Protected access to ItemsSource is deprecated. Please use GetItems, SetItemsSource or HasValidItemsSource.")
	/** Pointer to the array of data items that we are observing */
	const TArray<ItemType>* ItemsSource;

	/** When not null, the list will try to scroll to this item on tick. */
	NullableItemType ItemToScrollIntoView;

	/** The user index requesting the item to be scrolled into view. */
	uint32 UserRequestingScrollIntoView;

	/** When set, the list will notify this item when it has been scrolled into view */
	NullableItemType ItemToNotifyWhenInView;

	/** How to scroll an item into view */
	EScrollIntoViewAlignment ScrollIntoViewAlignment;

	/** Delegate to invoke when selection changes. */
	FOnSelectionChanged OnSelectionChanged;

	/** Delegate to invoke to see if we can navigate or select item. */
	FIsSelectableOrNavigable OnIsSelectableOrNavigable;

	/** Called when the user clicks on an element in the list view with the left mouse button */
	FOnMouseButtonClick OnClick;

	/** Called when the user double-clicks on an element in the list view with the left mouse button */
	FOnMouseButtonDoubleClick OnDoubleClick;
	
	/** Called when the user presses a keyboard key */
	FOnKeyDown OnKeyDownHandler;

	/** True when the list view supports keyboard focus */
	TAttribute<bool> IsFocusable;

	/** The additional scroll offset (in items) to show when navigating to rows at the edge of the visible area (i.e. how much of the following item(s) to show) */
	float NavigationScrollOffset = 0.5f;

	/** If true, the selection will be cleared if the user clicks in empty space (not on an item) */
	bool bClearSelectionOnClick;

	/** Should gamepad nav be supported */
	bool bHandleGamepadEvents;

	/** Should directional nav be supported */
	bool bHandleDirectionalNavigation;

	/** Should space bar based selection be supported */
	bool bHandleSpacebarSelection = false;

	/** If true, the focus will be returned to the last selected object in a list when navigated to. */
	bool bReturnFocusToSelection;

	/**
	 * When true, enables spatially proximate focus routing for cross-list navigation
	 * and intra-entry navigation within multi-focus entries. Defaults to false.
	 */
	bool bEnableProximateEntryNavigation = false;

	/** True to force the scroll velocity to reset to 0 when an item of the list is selected */
	bool bClearScrollVelocityOnSelection = true;

	/** If true, the item currently slated to be scrolled into view will also be navigated to after being scrolled in */
	bool bNavigateOnScrollIntoView = false;

	/** Cached navigation direction to forward to navigate widget call */
	EUINavigation NavigationTypeOnScrollIntoView = EUINavigation::Invalid;


#if WITH_EDITORONLY_DATA
	/** The maximum number of pinned items allowed */
	UE_DEPRECATED(5.8, "Use SetMaxPinnedItems / GetMaxPinnedItems / GetMaxPinnedItemsAttribute")
	TSlateDeprecatedTAttribute<int32> MaxPinnedItems;
#endif

	/** The initial value of MaxPinnedItems (used to restore it back if overriden) */
	TAttribute<int32> DefaultMaxPinnedItems;
	
	/** If true, number of pinned items > MaxPinnedItems so some items are collapsed in the hierarchy */
	bool bIsHierarchyCollapsed = false;

private:
	/** Style resource for the list */
	const FTableViewStyle* Style;

	/** Pointer to the source data that we are observing */
	TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>> ViewSource;

	/** The maximum number of pinned items allowed */
	TSlateAttribute<int32> MaxPinnedItemsAttribute;

private:
	struct FGenerationPassGuard
	{
		FWidgetGenerator& Generator;
		FGenerationPassGuard( FWidgetGenerator& InGenerator )
			: Generator(InGenerator)
		{
			// Let the WidgetGenerator that we are starting a pass so that it can keep track of data items and widgets.
			Generator.OnBeginGenerationPass();
		}

		~FGenerationPassGuard()
		{
			// We have completed the generation pass. The WidgetGenerator will clean up unused Widgets when it goes out of scope.
			Generator.OnEndGenerationPass();
		}
	};
};

template<typename ItemType>
inline void SListView<ItemType>::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	struct FInvalidation
	{
		static void RequestLayoutRefresh(SWidget& OwningWidget)
		{
			static_cast<SListView<ItemType>&>(OwningWidget).RequestLayoutRefresh();
		}
	};
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxPinnedItemsAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::RequestLayoutRefresh));
}