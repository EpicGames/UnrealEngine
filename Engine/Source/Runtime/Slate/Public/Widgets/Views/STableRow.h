// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "ITableRow.h"
#include "Framework/Views/ITypedTableView.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#if WITH_ACCESSIBILITY
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleWidgetCache.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

template <typename ItemType> class SListView;

/**
 * When the table row should signal the owner widget that the selection changed.
 * This only affect the selection with the left mouse button!
 */
enum class ETableRowSignalSelectionMode
{
	/**
	 * The selection will be updated on the left mouse button down, but the owner table will only get signaled when the mouse button is released or if a drag is detected.
	 */
	Deferred,
	/**
	 * Each time the selection of the owner table is changed the table get signaled.
	 */
	Instantaneous
};

/**
 * Where we are going to drop relative to the target item.
 */
enum class EItemDropZone
{
	AboveItem,
	OntoItem,
	BelowItem
};

template <typename ItemType> class SListView;

DECLARE_DELEGATE_OneParam(FOnTableRowDragEnter, FDragDropEvent const&);
DECLARE_DELEGATE_OneParam(FOnTableRowDragLeave, FDragDropEvent const&);
DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnTableRowDrop, FDragDropEvent const&);

template<typename ItemType>
class STableRow;

template <typename ItemType>
struct TWidgetTypeTraits<STableRow<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * The ListView is populated by Selectable widgets.
 * A Selectable widget is a way of the ListView containing it (OwnerTable) and holds arbitrary Content (Content).
 * A Selectable works with its corresponding ListView to provide selection functionality.
 */
template<typename ItemType>
class STableRow : public ITableRow, public SBorder
{
	static_assert(TIsValidListItem<ItemType>::Value, "Item type T must be UObjectBase*, TObjectPtr<>, TWeakObjectPtr<>, TSharedRef<>, or TSharedPtr<>.");

	SLATE_DECLARE_WIDGET(STableRow, SBorder)
	static inline SLATE_IMPLEMENT_WIDGET(STableRow)
public:
	/** Delegate signature for querying whether this FDragDropEvent will be handled by the drop target of type ItemType. */
	DECLARE_DELEGATE_RetVal_ThreeParams(TOptional<EItemDropZone>, FOnCanAcceptDrop, const FDragDropEvent&, EItemDropZone, ItemType);
	/** Delegate signature for handling the drop of FDragDropEvent onto target of type ItemType */
	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FOnAcceptDrop, const FDragDropEvent&, EItemDropZone, ItemType);
	/** Delegate signature for painting drop indicators. */
	DECLARE_DELEGATE_RetVal_EightParams(int32, FOnPaintDropIndicator, EItemDropZone, const FPaintArgs&, const FGeometry&, const FSlateRect&, FSlateWindowElementList&, int32, const FWidgetStyle&, bool);
public:

	SLATE_BEGIN_ARGS( STableRow< ItemType > )
		: _Style( &FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row") )
		, _ExpanderStyleSet( &FCoreStyle::Get() )
		, _Padding( FMargin(0) )
		, _ShowSelection( true )
		, _ShowWires( false )
		, _bAllowPreselectedItemActivation(false)
		, _SignalSelectionMode( ETableRowSignalSelectionMode::Deferred )
		, _Content()
		{}
	
		SLATE_STYLE_ARGUMENT( FTableRowStyle, Style )
		SLATE_ARGUMENT(const ISlateStyle*, ExpanderStyleSet)

		// High Level DragAndDrop

		/**
		 * Handle this event to determine whether a drag and drop operation can be executed on top of the target row widget.
		 * Most commonly, this is used for previewing re-ordering and re-organization operations in lists or trees.
		 * e.g. A user is dragging one item into a different spot in the list or tree.
		 *      This delegate will be called to figure out if we should give visual feedback on whether an item will 
		 *      successfully drop into the list.
		 */
		SLATE_EVENT( FOnCanAcceptDrop, OnCanAcceptDrop )

		/**
		 * Perform a drop operation onto the target row widget
		 * Most commonly used for executing a re-ordering and re-organization operations in lists or trees.
		 * e.g. A user was dragging one item into a different spot in the list; they just dropped it.
		 *      This is our chance to handle the drop by reordering items and calling for a list refresh.
		 */
		SLATE_EVENT( FOnAcceptDrop,    OnAcceptDrop )

		/**
		 * Used for painting drop indicators
		 */
		SLATE_EVENT( FOnPaintDropIndicator, OnPaintDropIndicator )

		// Low level DragAndDrop
		SLATE_EVENT( FOnDragDetected,      OnDragDetected )
		SLATE_EVENT( FOnTableRowDragEnter, OnDragEnter )
		SLATE_EVENT( FOnTableRowDragLeave, OnDragLeave )
		SLATE_EVENT( FOnTableRowDrop,      OnDrop )

		SLATE_ATTRIBUTE( FMargin, Padding )
	
		SLATE_ARGUMENT( bool, ShowSelection )
		SLATE_ARGUMENT( bool, ShowWires)
		SLATE_ARGUMENT( bool, bAllowPreselectedItemActivation)

		/**
		 * The Signal Selection mode affect when the owner table gets notified that the selection has changed.
		 * This only affect the selection with the left mouse button!
		 * When Deferred, the owner table will get notified when the button is released or when a drag started.
		 * When Instantaneous, the owner table is notified as soon as the selection changed.
		 */
		SLATE_ARGUMENT( ETableRowSignalSelectionMode , SignalSelectionMode)

		SLATE_DEFAULT_SLOT( typename STableRow<ItemType>::FArguments, Content )

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const typename STableRow<ItemType>::FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		/** Note: Please initialize any state in ConstructInternal, not here. This is because STableRow derivatives call ConstructInternal directly to avoid constructing children. **/

		ConstructInternal(InArgs, InOwnerTableView);

		ConstructChildren(
			InOwnerTableView->TableViewMode,
			InArgs._Padding,
			InArgs._Content.Widget
		);
	}

	virtual void ConstructChildren( ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent )
	{
		this->Content = InContent;
		InnerContentSlot = nullptr;

		if ( InOwnerTableMode == ETableViewMode::List || InOwnerTableMode == ETableViewMode::Tile )
		{
			// We just need to hold on to this row's content.
			this->ChildSlot
			.Padding( InPadding )
			[
				InContent
			];

			InnerContentSlot = &ChildSlot.AsSlot();
		}
		else
		{
			// -- Row is for TreeView --
			SHorizontalBox::FSlot* InnerContentSlotNativePtr = nullptr;

			// Rows in a TreeView need an expander button and some indentation
			this->ChildSlot
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this) )
					.StyleSet(ExpanderStyleSet)
					.ShouldDrawWires(bShowWires)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.Expose( InnerContentSlotNativePtr )
				.Padding( InPadding )
				[
					InContent
				]
			];

			InnerContentSlot = InnerContentSlotNativePtr;
		}
	}

#if WITH_ACCESSIBILITY
	protected:
	friend class FSlateAccessibleTableRow;
	/**
	* An accessible implementation of STableRow exposed to platform accessibility APIs.
	* For subclasses of STableRow, inherit from this class and override any functions
	* to give the desired behavior.
	*/
	class FSlateAccessibleTableRow
		: public FSlateAccessibleWidget
		, public IAccessibleTableRow
	{
	public:
		FSlateAccessibleTableRow(TWeakPtr<SWidget> InWidget, EAccessibleWidgetType InWidgetType)
			: FSlateAccessibleWidget(InWidget, InWidgetType)
		{}

		// IAccessibleWidget
		virtual IAccessibleTableRow* AsTableRow() 
		{ 
			return this; 
		}
		// ~
		// IAccessibleTableRow
		virtual void Select() override
		{
			if (Widget.IsValid())
			{
				TSharedPtr<STableRow<ItemType>> TableRow = StaticCastSharedPtr<STableRow<ItemType>>(Widget.Pin());
				if(TableRow->OwnerTablePtr.IsValid())
				{
					TSharedRef< ITypedTableView<ItemType> > OwnerTable = TableRow->OwnerTablePtr.Pin().ToSharedRef();
					const bool bIsActive = OwnerTable->AsWidget()->HasKeyboardFocus();

					if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = TableRow->GetItemForThis(OwnerTable))
					{
						const ItemType& MyItem = *MyItemPtr;
						const bool bIsSelected = OwnerTable->Private_IsItemSelected(MyItem);
						OwnerTable->Private_ClearSelection();
						OwnerTable->Private_SetItemSelection(MyItem, true, true);
						// @TODOAccessibility: Not sure if  irnoring  the signal selection mode will affect anything 
						OwnerTable->Private_SignalSelectionChanged(ESelectInfo::Direct);
					}
				}
			}
		}

		virtual void AddToSelection() override
		{
			// @TODOAccessibility: When multiselection is supported 
		}

		virtual void RemoveFromSelection() override
		{
			// @TODOAccessibility: When multiselection is supported 
		}

		virtual bool IsSelected() const override
		{
			if (Widget.IsValid())
			{
				TSharedPtr<STableRow<ItemType>> TableRow = StaticCastSharedPtr<STableRow<ItemType>>(Widget.Pin());
				return TableRow->IsItemSelected();
			}
			return false; 
		}

		virtual TSharedPtr<IAccessibleWidget> GetOwningTable() const override
		{
			if (Widget.IsValid())
			{
				TSharedPtr<STableRow<ItemType>> TableRow = StaticCastSharedPtr<STableRow<ItemType>>(Widget.Pin());
				if (TableRow->OwnerTablePtr.IsValid())
				{
					TSharedRef<SWidget> OwningTableWidget = TableRow->OwnerTablePtr.Pin()->AsWidget();
					return FSlateAccessibleWidgetCache::GetAccessibleWidgetChecked(OwningTableWidget);
				}
			}
			return nullptr;
		}
		// ~
	};
	public: 
	virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override
	{
		// @TODOAccessibility: Add support for tile table rows and tree table rows etc 
		// The widget type passed in should be based on the table type of the owning tabel
		EAccessibleWidgetType WidgetType = EAccessibleWidgetType::ListItem;
		return MakeShareable<FSlateAccessibleWidget>(new STableRow<ItemType>::FSlateAccessibleTableRow(SharedThis(this), WidgetType));
	}
#endif

	/** Retrieves a brush for rendering a drop indicator for the specified drop zone */
	const FSlateBrush* GetDropIndicatorBrush(EItemDropZone InItemDropZone) const
	{
		switch (InItemDropZone)
		{
			case EItemDropZone::AboveItem: return &Style->DropIndicator_Above; break;
			default:
			case EItemDropZone::OntoItem: return &Style->DropIndicator_Onto; break;
			case EItemDropZone::BelowItem: return &Style->DropIndicator_Below; break;
		};
	}

	int32 PaintSelection( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
		const bool bIsActive = OwnerTable->AsWidget()->HasKeyboardFocus();

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			if (bIsActive && OwnerTable->Private_UsesSelectorFocus() && OwnerTable->Private_HasSelectorFocus(*MyItemPtr))
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId++,
					AllottedGeometry.ToPaintGeometry(),
					&Style->SelectorFocusedBrush,
					ESlateDrawEffect::None,
					Style->SelectorFocusedBrush.GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
				);
			}
		}
		return LayerId;
	}
	int32 PaintBorder( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
	{
		return SBorder::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
	int32 PaintDropIndicator( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
	{
		if (GetItemDropZone().IsSet())
		{
			if (PaintDropIndicatorEvent.IsBound())
			{
				return PaintDropIndicatorEvent.Execute(GetItemDropZone().GetValue(), Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
			}
			else
			{
				return OnPaintDropIndicator(GetItemDropZone().GetValue(), Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
			}
		}

		return LayerId;
	}


	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		LayerId = PaintSelection(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		LayerId = PaintBorder(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		LayerId = PaintDropIndicator(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		return LayerId;
	}

	virtual int32 OnPaintDropIndicator( EItemDropZone InItemDropZone, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		// Draw feedback for user dropping an item above, below, or onto a row.
		const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(InItemDropZone);

		if (OwnerTable->Private_GetOrientation() == Orient_Vertical)
		{
			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(),
				DropIndicatorBrush,
				ESlateDrawEffect::None,
				DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}
		else
		{
			// Reuse the drop indicator asset for horizontal, by rotating the drawn box 90 degrees.
			const FVector2f LocalSize(AllottedGeometry.GetLocalSize());
			const FVector2f Pivot(LocalSize * 0.5f);
			const FVector2f RotatedLocalSize(LocalSize.Y, LocalSize.X);
			FSlateLayoutTransform RotatedTransform(Pivot - RotatedLocalSize * 0.5f);	// Make the box centered to the alloted geometry, so that it can be rotated around the center.

			FSlateDrawElement::MakeRotatedBox(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(RotatedLocalSize, RotatedTransform),
				DropIndicatorBrush,
				ESlateDrawEffect::None,
				-UE_HALF_PI,	// 90 deg CCW
				RotatedLocalSize * 0.5f,	// Relative center to the flipped
				FSlateDrawElement::RelativeToElement,
				DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}

		return LayerId;
	}

	/**
	 * Called when a mouse button is double clicked.  Override this in derived classes.
	 *
	 * @param  InMyGeometry  Widget geometry.
	 * @param  InMouseEvent  Mouse button event.
	 * @return  Returns whether the event was handled, along with other possible actions.
	 */
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

			// Only one item can be double-clicked
			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
			{
				// If we're configured to route double-click messages to the owner of the table, then
				// do that here.  Otherwise, we'll toggle expansion.
				const bool bWasHandled = OwnerTable->Private_OnItemDoubleClicked(*MyItemPtr);
				if (!bWasHandled)
				{
					ToggleExpansion();
				}

				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}

	/**
	 * See SWidget::OnMouseButtonDown
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event.
	 * @param MouseEvent Information about the input event.
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */	
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
		bChangedSelectionOnMouseDown = false;
		bDragWasDetected = false;

		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			const ESelectionMode::Type SelectionMode = GetSelectionMode();
			if (SelectionMode != ESelectionMode::None)
			{
				if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
				{
					const ItemType& MyItem = *MyItemPtr;
					const bool bIsSelected = OwnerTable->Private_IsItemSelected(MyItem);

					if (SelectionMode == ESelectionMode::Multi)
					{
						if (MouseEvent.IsShiftDown())
						{
							OwnerTable->Private_SelectRangeFromCurrentTo(MyItem);
							bChangedSelectionOnMouseDown = true;
							if (SignalSelectionMode == ETableRowSignalSelectionMode::Instantaneous)
							{
								OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
							}
						}
						else if (MouseEvent.IsControlDown())
						{
							OwnerTable->Private_SetItemSelection(MyItem, !bIsSelected, true);
							bChangedSelectionOnMouseDown = true;
							if (SignalSelectionMode == ETableRowSignalSelectionMode::Instantaneous)
							{
								OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
							}
						}
					}

					if ((bAllowPreselectedItemActivation || !bIsSelected) && !bChangedSelectionOnMouseDown)
					{
						OwnerTable->Private_ClearSelection();
						OwnerTable->Private_SetItemSelection(MyItem, true, true);
						bChangedSelectionOnMouseDown = true;
						if (SignalSelectionMode == ETableRowSignalSelectionMode::Instantaneous)
						{
							OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
						}
					}

					return FReply::Handled()
						.DetectDrag(SharedThis(this), EKeys::LeftMouseButton)
						.SetUserFocus(OwnerTable->AsWidget(), EFocusCause::Mouse)
						.CaptureMouse(SharedThis(this));
				}
			}
		}

		return FReply::Unhandled();
	}
	
	/**
	 * See SWidget::OnMouseButtonUp
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event.
	 * @param MouseEvent Information about the input event.
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		// Requires #include "Widgets/Views/SListView.h" in your header (not done in STableRow.h to avoid circular reference).
		TSharedRef< STableViewBase > OwnerTableViewBase = StaticCastSharedRef< SListView<ItemType> >(OwnerTable);

		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			FReply Reply = FReply::Unhandled().ReleaseMouseCapture();

			if ( bChangedSelectionOnMouseDown )
			{
				Reply = FReply::Handled().ReleaseMouseCapture();
			}

			const bool bIsUnderMouse = MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());
			if ( HasMouseCapture() )
			{
				if ( bIsUnderMouse && !bDragWasDetected )
				{
					switch( GetSelectionMode() )
					{
					case ESelectionMode::SingleToggle:
						{
							if ( !bChangedSelectionOnMouseDown )
							{
								OwnerTable->Private_ClearSelection();
								OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
							}

							Reply = FReply::Handled().ReleaseMouseCapture();
						}
						break;

					case ESelectionMode::Multi:
						{
							if ( !bChangedSelectionOnMouseDown && !MouseEvent.IsControlDown() && !MouseEvent.IsShiftDown() )
							{
								if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
								{
									const bool bIsSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);
									if (bIsSelected && OwnerTable->Private_GetNumSelectedItems() > 1)
									{
										// We are mousing up on a previous selected item;
										// deselect everything but this item.

										OwnerTable->Private_ClearSelection();
										OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
										OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);

										Reply = FReply::Handled().ReleaseMouseCapture();
									}
								}
							}
						}
						break;
					}
				}

				if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
				{
					if (OwnerTable->Private_OnItemClicked(*MyItemPtr))
					{
						Reply = FReply::Handled().ReleaseMouseCapture();
					}
				}

				if (bChangedSelectionOnMouseDown && !bDragWasDetected && (SignalSelectionMode == ETableRowSignalSelectionMode::Deferred))
				{
					OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
				}

				return Reply;
			}
		}
		else if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && !OwnerTableViewBase->IsRightClickScrolling() )
		{
			// Handle selection of items when releasing the right mouse button, but only if the user isn't actively
			// scrolling the view by holding down the right mouse button.

			switch( GetSelectionMode() )
			{
			case ESelectionMode::Single:
			case ESelectionMode::SingleToggle:
			case ESelectionMode::Multi:
				{
					// Only one item can be selected at a time
					if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
					{
						const bool bIsSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);

						// Select the item under the cursor
						if (!bIsSelected)
						{
							OwnerTable->Private_ClearSelection();
							OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
							OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
						}

						OwnerTable->Private_OnItemRightClicked(*MyItemPtr, MouseEvent);

						return FReply::Handled();
					}
				}
				break;
			}
		}

		return FReply::Unhandled();
	}

	virtual FReply OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override
	{
		bProcessingSelectionTouch = true;

		return
			FReply::Handled()
			// Drag detect because if this tap turns into a drag, we stop processing
			// the selection touch.
			.DetectDrag( SharedThis(this), EKeys::LeftMouseButton );
	}

	virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override
	{
		FReply Reply = FReply::Unhandled();

		if (bProcessingSelectionTouch)
		{
			bProcessingSelectionTouch = false;
			const TSharedRef<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
			{
				ESelectionMode::Type SelectionMode = GetSelectionMode();
				if (SelectionMode != ESelectionMode::None)
				{
					const bool bIsSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);
					if (!bIsSelected)
					{
						if (SelectionMode != ESelectionMode::Multi)
						{
							OwnerTable->Private_ClearSelection();
						}
						OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
						OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);

						Reply = FReply::Handled();
					}
					else if (SelectionMode == ESelectionMode::SingleToggle || SelectionMode == ESelectionMode::Multi)
					{
						OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
						OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);

						Reply = FReply::Handled();
					}
				}

				if (OwnerTable->Private_OnItemClicked(*MyItemPtr))
				{
					Reply = FReply::Handled();
				}
			}
		}

		return Reply;
	}

	virtual FReply OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if (bProcessingSelectionTouch)
		{
			// With touch input, dragging scrolls the list while selection requires a tap.
			// If we are processing a touch and it turned into a drag; pass it on to the 
			bProcessingSelectionTouch = false;
			return FReply::Handled().CaptureMouse( OwnerTablePtr.Pin()->AsWidget() );
		}
		else if ( HasMouseCapture() )
		{
			// Avoid changing the selection on the mouse up if there was a drag
			bDragWasDetected = true;

			if ( bChangedSelectionOnMouseDown && SignalSelectionMode == ETableRowSignalSelectionMode::Deferred )
			{
				TSharedPtr< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin();
				OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
			}
		}

		if (OnDragDetected_Handler.IsBound())
		{
			return OnDragDetected_Handler.Execute( MyGeometry, MouseEvent );
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override
	{
		if (OnDragEnter_Handler.IsBound())
		{
			OnDragEnter_Handler.Execute(DragDropEvent);
		}
	}

	virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override
	{
		SetItemDropZone(TOptional<EItemDropZone>());

		if (OnDragLeave_Handler.IsBound())
		{
			OnDragLeave_Handler.Execute(DragDropEvent);
		}
	}

	/** @return the zone (above, onto, below) based on where the user is hovering over within the row */
	EItemDropZone ZoneFromPointerPosition(UE::Slate::FDeprecateVector2DParameter LocalPointerPos, UE::Slate::FDeprecateVector2DParameter LocalSize, EOrientation Orientation)
	{
		const float PointerPos = Orientation == EOrientation::Orient_Horizontal ? LocalPointerPos.X : LocalPointerPos.Y;
		const float Size = Orientation == EOrientation::Orient_Horizontal ? LocalSize.X : LocalSize.Y;

		const float ZoneBoundarySu = FMath::Clamp(Size * 0.25f, 3.0f, 10.0f);
		if (PointerPos < ZoneBoundarySu)
		{
			return EItemDropZone::AboveItem;
		}
		else if (PointerPos > Size - ZoneBoundarySu)
		{
			return EItemDropZone::BelowItem;
		}
		else
		{
			return EItemDropZone::OntoItem;
		}
	}

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		if ( OnCanAcceptDrop.IsBound() )
		{
			const TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
			const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			const EItemDropZone ItemHoverZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize(), OwnerTable->Private_GetOrientation());

			SetItemDropZone([ItemHoverZone, DragDropEvent, this]()
			{
				TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
				if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
				{
					return OnCanAcceptDrop.Execute(DragDropEvent, ItemHoverZone, *MyItemPtr);
				}

				return TOptional<EItemDropZone>();
			}());

			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}

	}

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		const FReply Reply = [&]()
		{
			if (OnAcceptDrop.IsBound())
			{
				const TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

				// A drop finishes the drag/drop operation, so we are no longer providing any feedback.
				SetItemDropZone(TOptional<EItemDropZone>());

				// Find item associated with this widget.
				if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
				{
					// Which physical drop zone is the drop about to be performed onto?
					const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
					const EItemDropZone HoveredZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize(), OwnerTable->Private_GetOrientation());

					// The row gets final say over which zone to drop onto regardless of physical location.
					const TOptional<EItemDropZone> ReportedZone = OnCanAcceptDrop.IsBound()
						? OnCanAcceptDrop.Execute(DragDropEvent, HoveredZone, *MyItemPtr)
						: HoveredZone;

					if (ReportedZone.IsSet())
					{
						FReply DropReply = OnAcceptDrop.Execute(DragDropEvent, ReportedZone.GetValue(), *MyItemPtr);
						if (DropReply.IsEventHandled() && ReportedZone.GetValue() == EItemDropZone::OntoItem)
						{
							// Expand the drop target just in case, so that what we dropped is visible.
							OwnerTable->Private_SetItemExpansion(*MyItemPtr, true);
						}

						return DropReply;
					}
				}
			}

			return FReply::Unhandled();
		}();

		// @todo slate : Made obsolete by OnAcceptDrop. Get rid of this.
		if ( !Reply.IsEventHandled() && OnDrop_Handler.IsBound() )
		{
			return OnDrop_Handler.Execute(DragDropEvent);
		}

		return Reply;
	}

	virtual void InitializeRow() override {}
	virtual void ResetRow() override {}

	virtual void SetIndexInList( int32 InIndexInList ) override
	{
		IndexInList = InIndexInList;
	}

	virtual int32 GetIndexInList() override
	{
		return IndexInList;
	}

	virtual bool IsItemExpanded() const override
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			return OwnerTable->Private_IsItemExpanded(*MyItemPtr);
		}
		
		return false;
	}

	virtual void ToggleExpansion() override
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		const bool bItemHasChildren = OwnerTable->Private_DoesItemHaveChildren( IndexInList );
		// Nothing to expand if row being clicked on doesn't have children
		if( bItemHasChildren )
		{
			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
			{
				const bool bIsItemExpanded = bItemHasChildren && OwnerTable->Private_IsItemExpanded(*MyItemPtr);
				OwnerTable->Private_SetItemExpansion(*MyItemPtr, !bIsItemExpanded);
			}
		}
	}

	virtual bool IsItemSelected() const override
	{
		TSharedRef<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			return OwnerTable->Private_IsItemSelected(*MyItemPtr);
		}

		return false;
	}

	virtual int32 GetIndentLevel() const override
	{
		return OwnerTablePtr.Pin()->Private_GetNestingDepth( IndexInList );
	}

	virtual int32 DoesItemHaveChildren() const override
	{
		return OwnerTablePtr.Pin()->Private_DoesItemHaveChildren( IndexInList );
	}

	virtual TBitArray<> GetWiresNeededByDepth() const override
	{
		return OwnerTablePtr.Pin()->Private_GetWiresNeededByDepth(IndexInList);
	}

	virtual bool IsLastChild() const override
	{
		return OwnerTablePtr.Pin()->Private_IsLastChild(IndexInList);
	}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		return SharedThis(this);
	}

	/** Set the entire content of this row, replacing any extra UI (such as the expander arrows for tree views) that was added by ConstructChildren */
	virtual void SetRowContent(TSharedRef< SWidget > InContent)
	{
		this->Content = InContent;
		InnerContentSlot = nullptr;
		SBorder::SetContent(InContent);
	}

	/** Set the inner content of this row, preserving any extra UI (such as the expander arrows for tree views) that was added by ConstructChildren */
	virtual void SetContent(TSharedRef< SWidget > InContent) override
	{
		this->Content = InContent;

		if (InnerContentSlot)
		{
			InnerContentSlot->AttachWidget(InContent);
		}
		else
		{
			SBorder::SetContent(InContent);
		}
	}

	/** Get the inner content of this row */
	virtual TSharedPtr<SWidget> GetContent() override
	{
		if ( this->Content.IsValid() )
		{
			return this->Content.Pin();
		}
		else
		{
			return TSharedPtr<SWidget>();
		}
	}

	virtual void Private_OnExpanderArrowShiftClicked() override
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		const bool bItemHasChildren = OwnerTable->Private_DoesItemHaveChildren( IndexInList );
		// Nothing to expand if row being clicked on doesn't have children
		if( bItemHasChildren )
		{
			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
			{
				const bool IsItemExpanded = bItemHasChildren && OwnerTable->Private_IsItemExpanded(*MyItemPtr);
				OwnerTable->Private_OnExpanderArrowShiftClicked(*MyItemPtr, !IsItemExpanded);
			}
		}
	}

	/** @return The border to be drawn around this list item */
	virtual const FSlateBrush* GetBorder() const 
	{
		TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		const bool bIsActive = OwnerTable->AsWidget()->HasKeyboardFocus();

		const bool bItemHasChildren = OwnerTable->Private_DoesItemHaveChildren( IndexInList );

		static FName GenericWhiteBoxBrush("GenericWhiteBox");

		// @todo: Slate Style - make this part of the widget style
		const FSlateBrush* WhiteBox = FCoreStyle::Get().GetBrush(GenericWhiteBoxBrush);

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			const bool bIsSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);
			const bool bIsHighlighted = OwnerTable->Private_IsItemHighlighted(*MyItemPtr);

			const bool bAllowSelection = GetSelectionMode() != ESelectionMode::None;
			const bool bEvenEntryIndex = (IndexInList % 2 == 0);

			if (bIsSelected && bShowSelection)
			{
				if (bIsActive)
				{
					return IsHovered()
						? &Style->ActiveHoveredBrush
						: &Style->ActiveBrush;
				}
				else
				{
					return IsHovered()
						? &Style->InactiveHoveredBrush
						: &Style->InactiveBrush;
				}
			}
			else if (!bIsSelected && bIsHighlighted)
			{
				if (bIsActive)
				{
					return IsHovered()
						? (bEvenEntryIndex ? &Style->EvenRowBackgroundHoveredBrush : &Style->OddRowBackgroundHoveredBrush)
						: &Style->ActiveHighlightedBrush;
				}
				else
				{
					return IsHovered()
						? (bEvenEntryIndex ? &Style->EvenRowBackgroundHoveredBrush : &Style->OddRowBackgroundHoveredBrush)
						: &Style->InactiveHighlightedBrush;
				}
			}
			else if (bItemHasChildren && Style->bUseParentRowBrush && GetIndentLevel() == 0)
			{
				return IsHovered() 
				? &Style->ParentRowBackgroundHoveredBrush	
				: &Style->ParentRowBackgroundBrush;	
			}
			else
			{
				// Add a slightly lighter background for even rows
				if (bEvenEntryIndex)
				{
					return (IsHovered() && bAllowSelection)
						? &Style->EvenRowBackgroundHoveredBrush
						: &Style->EvenRowBackgroundBrush;

				}
				else
				{
					return (IsHovered() && bAllowSelection)
						? &Style->OddRowBackgroundHoveredBrush
						: &Style->OddRowBackgroundBrush;

				}
			}
		}

		return nullptr;
	}

	/** 
	 * Callback to determine if the row is selected singularly and has keyboard focus or not
	 *
	 * @return		true if selected by owning widget.
	 */
	bool IsSelectedExclusively() const
	{
		TSharedRef< ITypedTableView< ItemType > > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		if (!OwnerTable->AsWidget()->HasKeyboardFocus() || OwnerTable->Private_GetNumSelectedItems() > 1)
		{
			return false;
		}

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			return OwnerTable->Private_IsItemSelected(*MyItemPtr);
		}

		return false;
	}

	/**
	 * Callback to determine if the row is selected or not
	 *
	 * @return		true if selected by owning widget.
	 */
	bool IsSelected() const
	{
		TSharedRef< ITypedTableView< ItemType > > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			return OwnerTable->Private_IsItemSelected(*MyItemPtr);
		}

		return false;
	}

	/**
	 * Callback to determine if the row is highlighted or not
	 *
	 * @return		true if highlighted by owning widget.
	 */
	bool IsHighlighted() const
	{
		TSharedRef< ITypedTableView< ItemType > > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
		{
			return OwnerTable->Private_IsItemHighlighted(*MyItemPtr);
		}

		return false;
	}

	/** By default, this function does nothing, it should be implemented by derived class */
	virtual FVector2D GetRowSizeForColumn(const FName& InColumnName) const override
	{
		return FVector2D::ZeroVector;
	}

	void SetExpanderArrowVisibility(const EVisibility InExpanderArrowVisibility)
	{
		if(ExpanderArrowWidget)
		{
			ExpanderArrowWidget->SetVisibility(InExpanderArrowVisibility);
		}
	}

	/** Protected constructor; SWidgets should only be instantiated via declarative syntax. */
	STableRow()
		: IndexInList(0)
		, bShowSelection(true)
		, SignalSelectionMode( ETableRowSignalSelectionMode::Deferred )
		, ItemDropZoneAttribute(*this, TOptional<EItemDropZone>())
	{
#if WITH_ACCESSIBILITY
		// As the contents of table rows could be anything,
		// Ideally, somebody would assign a custom label to each table row with non-accessible content.
		// However, that's not always feasible so we want the screen reader to read out the concatenated contents of children.
		// E.g If ItemType == FString, then the screen reader can just read out the contents of the text box.
		AccessibleBehavior = EAccessibleBehavior::Summary;
		bCanChildrenBeAccessible = true;
#endif
	}

protected:

	/**
	 * An internal method to construct and setup this row widget (purposely avoids child construction). 
	 * Split out from Construct() so that sub-classes can invoke super construction without invoking 
	 * ConstructChildren() (sub-classes may want to constuct their own children in their own special way).
	 * 
	 * @param  InArgs			Declaration data for this widget.
	 * @param  InOwnerTableView	The table that this row belongs to.
	 */
	void ConstructInternal(FArguments const& InArgs, TSharedRef<STableViewBase> const& InOwnerTableView)
	{
		bProcessingSelectionTouch = false;

		check(InArgs._Style);
		Style = InArgs._Style;

		check(InArgs._ExpanderStyleSet);
		ExpanderStyleSet = InArgs._ExpanderStyleSet;

		SetBorderImage(TAttribute<const FSlateBrush*>(this, &STableRow::GetBorder));

		this->SetForegroundColor(TAttribute<FSlateColor>( this, &STableRow::GetForegroundBasedOnSelection ));

		this->OnCanAcceptDrop = InArgs._OnCanAcceptDrop;
		this->OnAcceptDrop = InArgs._OnAcceptDrop;

		this->OnDragDetected_Handler = InArgs._OnDragDetected;
		this->OnDragEnter_Handler = InArgs._OnDragEnter;
		this->OnDragLeave_Handler = InArgs._OnDragLeave;
		this->OnDrop_Handler = InArgs._OnDrop;
		
		this->SetOwnerTableView( InOwnerTableView );

		this->bShowSelection = InArgs._ShowSelection;

		this->SignalSelectionMode = InArgs._SignalSelectionMode;

		this->bShowWires = InArgs._ShowWires;

		this->bAllowPreselectedItemActivation = InArgs._bAllowPreselectedItemActivation;
	}

	void SetOwnerTableView( TSharedPtr<STableViewBase> OwnerTableView )
	{
		// We want to cast to a ITypedTableView.
		// We cast to a SListView<ItemType> because C++ doesn't know that
		// being a STableView implies being a ITypedTableView.
		// See SListView.
		this->OwnerTablePtr = StaticCastSharedPtr< SListView<ItemType> >(OwnerTableView);
	}

	FSlateColor GetForegroundBasedOnSelection() const
	{
		const TSharedPtr< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin();
		const FSlateColor& NonSelectedForeground = Style->TextColor; 
		const FSlateColor& SelectedForeground = Style->SelectedTextColor;

		if ( !bShowSelection || !OwnerTable.IsValid() )
		{
			return NonSelectedForeground;
		}

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable.ToSharedRef()))
		{
			const bool bIsSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);

			return bIsSelected
				? SelectedForeground
				: NonSelectedForeground;
		}

		return NonSelectedForeground;
	}

	virtual ESelectionMode::Type GetSelectionMode() const override
	{
		const TSharedPtr< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin();
		return OwnerTable->Private_GetSelectionMode();
	}

	const TObjectPtrWrapTypeOf<ItemType>* GetItemForThis(const TSharedRef<ITypedTableView<ItemType>>& OwnerTable) const
	{
		const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable->Private_ItemFromWidget(this);
		if (MyItemPtr)
		{
			return MyItemPtr;
		}
		else
		{
			checkf(OwnerTable->Private_IsPendingRefresh(), TEXT("We were unable to find the item for this widget.  If it was removed from the source collection, the list should be pending a refresh."));
		}

		return nullptr;
	}

protected:
	const TOptional<EItemDropZone>& GetItemDropZone() const { return ItemDropZoneAttribute.Get(); }
	void SetItemDropZone(TOptional<EItemDropZone> InItemDropZone) { ItemDropZoneAttribute.Assign(*this, MoveTemp(InItemDropZone)); }
	TSlateAttributeRef<TOptional<EItemDropZone>> GetItemDropZoneAttribute() const
	{
		return TSlateAttributeRef<TOptional<EItemDropZone>>{SharedThis(this), ItemDropZoneAttribute};
	}

protected:

	/** The list that owns this Selectable */
	TWeakPtr< ITypedTableView<ItemType> > OwnerTablePtr;

	/** Index of the corresponding data item in the list */
	int32 IndexInList;

	/** Whether or not to visually show that this row is selected */
	bool bShowSelection;

	/** When should we signal that selection changed for a left click */
	ETableRowSignalSelectionMode SignalSelectionMode;

	/** Style used to draw this table row */
	const FTableRowStyle* Style;

	/** The slate style to use with the expander */
	const ISlateStyle* ExpanderStyleSet;

	/** A pointer to the expander arrow on the row (if it exists) */
	TSharedPtr<SExpanderArrow> ExpanderArrowWidget;

	/** @see STableRow's OnCanAcceptDrop event */
	FOnCanAcceptDrop OnCanAcceptDrop;

	/** @see STableRow's OnAcceptDrop event */
	FOnAcceptDrop OnAcceptDrop;

	/** Optional delegate for painting drop indicators */
	FOnPaintDropIndicator PaintDropIndicatorEvent;

#if WITH_EDITORONLY_DATA
	/** Are we currently dragging/dropping over this item? */
	UE_DEPRECATED(5.8, "Use GetItemDropZone / SetItemDropZone / GetItemDropZoneAttribute.")
	TOptional<EItemDropZone> ItemDropZone;
#endif

	/** Delegate triggered when a user starts to drag a list item */
	FOnDragDetected OnDragDetected_Handler;

	/** Delegate triggered when a user's drag enters the bounds of this list item */
	FOnTableRowDragEnter OnDragEnter_Handler;

	/** Delegate triggered when a user's drag leaves the bounds of this list item */
	FOnTableRowDragLeave OnDragLeave_Handler;

	/** Delegate triggered when a user's drag is dropped in the bounds of this list item */
	FOnTableRowDrop OnDrop_Handler;

	/** The slot that contains the inner content for this row. If this is set, SetContent populates this slot with the new content rather than replace the content wholesale */
	FSlotBase* InnerContentSlot;

	/** The widget in the content slot for this row */
	TWeakPtr<SWidget> Content;

	bool bChangedSelectionOnMouseDown;

	bool bDragWasDetected;

	/** Did the current a touch interaction start in this item?*/
	bool bProcessingSelectionTouch;

	/** When activating an item via mouse button, we generally don't allow pre-selected items to be activated */
	bool bAllowPreselectedItemActivation;

private:
	bool bShowWires;

	/** Are we currently dragging/dropping over this item? */
	TSlateAttribute<TOptional<EItemDropZone>> ItemDropZoneAttribute;
};

template<typename ItemType>
inline void STableRow<ItemType>::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ItemDropZoneAttribute, EInvalidateWidgetReason::Paint);
}


template<typename ItemType>
class SMultiColumnTableRow;

template <typename ItemType>
struct TWidgetTypeTraits<SMultiColumnTableRow<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

template<typename ItemType>
class SMultiColumnTableRow : public STableRow<ItemType>
{
public:

	/**
	 * Users of SMultiColumnTableRow would usually some piece of data associated with it.
	 * The type of this data is ItemType; it's the stuff that your TableView (i.e. List or Tree) is visualizing.
	 * The ColumnName tells you which column of the TableView we need to make a widget for.
	 * Make a widget and return it.
	 *
	 * @param ColumnName    A unique ID for a column in this TableView; see SHeaderRow::FColumn for more info.
	 * @return a widget to represent the contents of a cell in this row of a TableView. 
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& InColumnName ) = 0;

	/** Use this to construct the superclass; e.g. FSuperRowType::Construct( FTableRowArgs(), OwnerTableView ) */
	typedef SMultiColumnTableRow< ItemType > FSuperRowType;

	/** Use this to construct the superclass; e.g. FSuperRowType::Construct( FTableRowArgs(), OwnerTableView ) */
	typedef typename STableRow<ItemType>::FArguments FTableRowArgs;

protected:
	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		STableRow<ItemType>::Construct(
			FTableRowArgs()
			.Style(InArgs._Style)
			.ExpanderStyleSet(InArgs._ExpanderStyleSet)
			.Padding(InArgs._Padding)
			.ShowSelection(InArgs._ShowSelection)
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(InArgs._OnDragDetected)
			.OnDragEnter(InArgs._OnDragEnter)
			.OnDragLeave(InArgs._OnDragLeave)
			.OnDrop(InArgs._OnDrop)
			.Content()
			[
				SAssignNew( Box, SHorizontalBox )
			]
		
			, OwnerTableView );

		// Sign up for notifications about changes to the HeaderRow
		TSharedPtr< SHeaderRow > HeaderRow = OwnerTableView->GetHeaderRow();
		check( HeaderRow.IsValid() );
		HeaderRow->OnColumnsChanged()->AddSP( this, &SMultiColumnTableRow<ItemType>::GenerateColumns );

		// Populate the row with user-generated content
		this->GenerateColumns( HeaderRow.ToSharedRef() );
	}

	virtual void ConstructChildren( ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent ) override
	{
		STableRow<ItemType>::Content = InContent;

		// MultiColumnRows let the user decide which column should contain the expander/indenter item.
		this->ChildSlot
		.Padding( InPadding )
		[
			InContent
		];
	}

	void GenerateColumns( const TSharedRef<SHeaderRow>& InColumnHeaders )
	{
		Box->ClearChildren();
		const TIndirectArray<SHeaderRow::FColumn>& Columns = InColumnHeaders->GetColumns();
		const int32 NumColumns = Columns.Num();
		TMap< FName, TSharedRef< SWidget > > NewColumnIdToSlotContents;

		for( int32 ColumnIndex = 0; ColumnIndex < NumColumns; ++ColumnIndex )
		{
			const SHeaderRow::FColumn& Column = Columns[ColumnIndex];
			if ( InColumnHeaders->ShouldGeneratedColumn(Column.ColumnId) )
			{
				TSharedRef< SWidget >* ExistingWidget = ColumnIdToSlotContents.Find(Column.ColumnId);
				TSharedRef< SWidget > CellContents = SNullWidget::NullWidget;
				if (ExistingWidget != nullptr)
				{
					CellContents = *ExistingWidget;
				}
				else
				{
					CellContents = GenerateWidgetForColumn(Column.ColumnId);
				}

				bool bAddAlignedColumnCell = true;
				if ( CellContents != SNullWidget::NullWidget )
				{
					CellContents->SetClipping(EWidgetClipping::OnDemand);
				}
				else if ( !InColumnHeaders->ShouldGenerateEmptyColumnForSpacing(Column.ColumnId) )
				{
					bAddAlignedColumnCell = false;
				}

				if (bAddAlignedColumnCell)
				{
					switch (Column.SizeRule)
					{
					case EColumnSizeMode::Fill:
					{
						TAttribute<float> WidthBinding;
						WidthBinding.BindRaw(&Column, &SHeaderRow::FColumn::GetWidth);

						Box->AddSlot()
						.HAlign(Column.CellHAlignment)
						.VAlign(Column.CellVAlignment)
						.FillWidth(WidthBinding)
						[
							CellContents
						];
					}
					break;

					case EColumnSizeMode::Fixed:
					{
						Box->AddSlot()
						.AutoWidth()
						[
							SNew(SBox)
							.WidthOverride(Column.Width.Get())
							.HAlign(Column.CellHAlignment)
							.VAlign(Column.CellVAlignment)
							.Clipping(EWidgetClipping::OnDemand)
							[
								CellContents
							]
						];
					}
					break;

					case EColumnSizeMode::Manual:
					case EColumnSizeMode::FillSized:
					{
						auto GetColumnWidthAsOptionalSize = [&Column]() -> FOptionalSize
						{
							const float DesiredWidth = Column.GetWidth();
							return FOptionalSize(DesiredWidth);
						};

						TAttribute<FOptionalSize> WidthBinding;
						WidthBinding.Bind(TAttribute<FOptionalSize>::FGetter::CreateLambda(GetColumnWidthAsOptionalSize));

						Box->AddSlot()
						.AutoWidth()
						[
							SNew(SBox)
							.WidthOverride(WidthBinding)
							.HAlign(Column.CellHAlignment)
							.VAlign(Column.CellVAlignment)
							.Clipping(EWidgetClipping::OnDemand)
							[
								CellContents
							]
						];
					}
					break;

					default:
						ensure(false);
						break;
					}
				}

				NewColumnIdToSlotContents.Add(Column.ColumnId, CellContents);
			}
		}

		ColumnIdToSlotContents = NewColumnIdToSlotContents;
	}

	void ClearCellCache()
	{
		ColumnIdToSlotContents.Empty();
	}

	const TSharedRef<SWidget>* GetWidgetFromColumnId(const FName& ColumnId) const
	{
		return ColumnIdToSlotContents.Find(ColumnId);
	}

private:
	
	TSharedPtr<SHorizontalBox> Box;
	TMap< FName, TSharedRef< SWidget > > ColumnIdToSlotContents;
};
