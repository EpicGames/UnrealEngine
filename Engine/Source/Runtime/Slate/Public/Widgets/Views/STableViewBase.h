// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Geometry.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Framework/Views/ITypedTableView.h"
#include "Framework/Layout/InertialScrollManager.h"
#include "Framework/Layout/Overscroll.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"

#include "STableViewBase.generated.h"

class FPaintArgs;
class FSlateWindowElementList;
class ITableRow;
class SHeaderRow;
class SListPanel;
class SScrollBar;
enum class EConsumeMouseWheel : uint8;
enum class ESlateVisibility : uint8;
struct FScrollBarStyle;

/** If the list panel is arranging items as tiles, this enum dictates how the items should be aligned (basically, where any extra space is placed) */
UENUM(BlueprintType)
enum class EListItemAlignment : uint8
{
	/** Items are distributed evenly along the line (any extra space is added as padding between the items) */
	EvenlyDistributed UMETA(DisplayName = "Evenly (Padding)"),

	/** Items are distributed evenly along the line (any extra space is used to scale up the size of the item proportionally.) */
	EvenlySize UMETA(DisplayName = "Evenly (Size)"),

	/** Items are distributed evenly along the line, any extra space is used to scale up width of the items proportionally.) */
	EvenlyWide UMETA(DisplayName = "Evenly (Wide)"),

	/** Items are left aligned on the line (any extra space is added to the right of the items) */
	LeftAligned,

	/** Items are right aligned on the line (any extra space is added to the left of the items) */
	RightAligned,

	/** Items are center aligned on the line (any extra space is halved and added to the left of the items) */
	CenterAligned,

	/** Items are evenly stretched to distribute any extra space on the line */
	Fill,
};

/** How the list panel scrolls an offscreen item into view */
UENUM(BlueprintType)
enum class EScrollIntoViewAlignment : uint8
{
	/** Item will only be scrolled enough to be brought into view */
	IntoView,

	/** Always scroll the widget so it appears at the top/Left of the scrollable area. */
	TopOrLeft,

	/** Item will be aligned to the center of the view */
	CenterAligned,

	/** Always scroll the widget so it appears at the bottom/Right of the scrollable area. */
	BottomOrRight
};

DECLARE_DELEGATE_OneParam(
	FOnTableViewScrolled,
	double );	/** Scroll offset from the beginning of the list in items */


/** Abstracts away the need to distinguish between X or Y when calculating table layout elements */
struct FTableViewDimensions
{
	SLATE_API FTableViewDimensions(EOrientation InOrientation);
	SLATE_API FTableViewDimensions(EOrientation InOrientation, float X, float Y);
	SLATE_API FTableViewDimensions(EOrientation InOrientation, const UE::Slate::FDeprecateVector2DParameter& Size);

	UE::Slate::FDeprecateVector2DResult ToVector2D() const
	{
		return Orientation == Orient_Vertical ? FVector2f(LineAxis, ScrollAxis) : FVector2f(ScrollAxis, LineAxis);
	}

	FTableViewDimensions operator+(const FTableViewDimensions& Other) const
	{
		return FTableViewDimensions(Orientation, ToVector2D() + Other.ToVector2D());
	}

	EOrientation Orientation = Orient_Vertical;

	/** The dimension along the scrolling axis of the table view (Y when oriented vertically, X when horizontal) */
	float ScrollAxis = 0.f;
	
	/** The dimension orthogonal to the scroll axis, along which lines of items are created. Only really relevant for tile views. */
	float LineAxis = 0.f;
};


template <>
struct TWidgetTypeTraits<class STableViewBase>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Contains ListView functionality that does not depend on the type of data being observed by the ListView.
 */
class STableViewBase
	: public SCompoundWidget
	, public IScrollableWidget
{
	SLATE_DECLARE_WIDGET_API(STableViewBase, SCompoundWidget, SLATE_API)
public:

	virtual bool IsTableView() const { return true; }

	/** Create the child widgets that comprise the list */
	SLATE_API void ConstructChildren( const TAttribute<float>& InItemWidth, const TAttribute<float>& InItemHeight, const TAttribute<EListItemAlignment>& InItemAlignment, const TSharedPtr<SHeaderRow>& InHeaderRow, const TSharedPtr<SScrollBar>& InScrollBar, EOrientation InScrollOrientation, const FOnTableViewScrolled& InOnTableViewScrolled, const FScrollBarStyle* InScrollBarStyle = nullptr, const bool bInPreventThrottling = false );

	/** Sets the item height */
	SLATE_API void SetItemHeight(TAttribute<float> Height);

	/** Sets the item width */
	SLATE_API void SetItemWidth(TAttribute<float> Width);

	/**
	 * Invoked by the scrollbar when the user scrolls.
	 *
	 * @param InScrollOffsetFraction  The location to which the user scrolled as a fraction (between 0 and 1) of total height of the content.
	 */
	SLATE_API void ScrollBar_OnUserScrolled( float InScrollOffsetFraction );

	/** @return The number of Widgets we currently have generated. */
	SLATE_API int32 GetNumGeneratedChildren() const;

	/** @return The generated row widget at the specified index if it exists (NULL otherwise). */
	SLATE_API TSharedPtr<SWidget> GetGeneratedChildAt(int32 Index) const;

	SLATE_API TSharedPtr<SHeaderRow> GetHeaderRow() const;

	/** @return Returns true if the user is currently interactively scrolling the view by holding
		        the right mouse button and dragging. */
	SLATE_API bool IsRightClickScrolling() const;

	/** @return Returns true if the user is currently interactively scrolling the view by holding
		        either mouse button and dragging. */
	SLATE_API bool IsUserScrolling() const;

	/**
	 * Mark the list as dirty so that it will refresh its widgets on next tick.
	 * Note that refreshing will only generate/release widgets as needed from any deltas in the list items source.
	 */
	SLATE_API virtual void RequestListRefresh();

	/** Completely wipe existing widgets and fully regenerate them on next tick. */
	virtual void RebuildList() = 0;

	/** Return true if there is currently a refresh pending, false otherwise */
	SLATE_API bool IsPendingRefresh() const;

	/** Is this list backing a tree or just a standalone list */
	const ETableViewMode::Type TableViewMode;

	/** Scrolls the view to the top */
	SLATE_API void ScrollToTop();

	/** Scrolls the view to the bottom */
	SLATE_API void ScrollToBottom();

	/** Returns whether the attached scrollbar is scrolling */
	SLATE_API bool IsScrolling() const;

	/** Gets the scroll offset of this view (in items) */
	SLATE_API float GetScrollOffset() const;

	/** Set the scroll offset of this view (in items) */
	SLATE_API void SetScrollOffset( const float InScrollOffset );

	/** Reset the inertial scroll velocity accumulated in the InertialScrollManager
	 * @param bInShouldStopScrollNow true implies the scroll will stop instantly, else the list will scroll until any accumulated scroll offset is cleared. */
	SLATE_API void EndInertialScrolling(const bool bInShouldStopScrollNow = false);

	/** Add the scroll offset of this view (in items) */
	SLATE_API void AddScrollOffset(const float InScrollOffsetDelta, bool RefreshList = false);

	SLATE_API EVisibility GetScrollbarVisibility() const;

	SLATE_API void SetScrollbarVisibility(const EVisibility InVisibility);

	SLATE_API void SetScrollbarPadding(const FMargin& InScrollbarPadding);

	/** Returns true if scrolling is possible; false if the view is big enough to fit all the content. */
	SLATE_API bool IsScrollbarNeeded() const;

	/** Sets the fixed offset in items to always apply to the top/left (depending on orientation) of the list. */
	SLATE_API void SetFixedLineScrollOffset(TOptional<double> InFixedLineScrollOffset);

	/** Sets whether the list should lerp between scroll offsets or jump instantly between them. */
	SLATE_API void SetIsScrollAnimationEnabled(bool bInEnableScrollAnimation);

	/** Sets the speed of lerp between scroll offsets. */
	SLATE_API void SetScrollingAnimationInterpolationSpeed(float InScrollingAnimationInterpolationSpeed);

	/** Sets whether the list should lerp between scroll offsets or jump instantly between them with touch. */
	SLATE_API void SetEnableTouchAnimatedScrolling(bool bInEnableTouchAnimatedScrolling);

	/** Sets whether to permit overscroll on this list view */
	SLATE_API void SetAllowOverscroll(EAllowOverscroll InAllowOverscroll);

	/** Enables/disables being able to scroll with the right mouse button. */
	SLATE_API void SetIsRightClickScrollingEnabled(const bool bInEnableRightClickScrolling);

	/** Enables/disables being able to scroll using touch input. */
	SLATE_API void SetIsTouchScrollingEnabled(const bool bInEnableTouchScrolling);

	/** Sets whether items will be selected and focused or only focused when navigating to them. */
	SLATE_API void SetSelectItemOnNavigation(const bool bInSelectItemOnNavigation);

	/** Sets the multiplier applied when wheel scrolling. Higher numbers will cover more distance per click of the wheel. */
	SLATE_API void SetWheelScrollMultiplier(float NewWheelScrollMultiplier);

	/** Enables/disables being able to scroll via pointer. This should be use as a temporary mean to disable scrolling. */
	SLATE_API void SetIsPointerScrollingEnabled(bool bInIsPointerScrollingEnabled);

	/** Enables/disables being able to scroll via gamepad. */
	SLATE_API void SetIsGamepadScrollingEnabled(bool bInIsGamepadScrollingEnabled);

	/** Sets the Background Brush */
	SLATE_API void SetBackgroundBrush(TAttribute<const FSlateBrush*> InBackgroundBrush);

public:

	// SWidget interface

	SLATE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	SLATE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	SLATE_API virtual bool SupportsKeyboardFocus() const override;
	SLATE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	SLATE_API virtual FReply OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	SLATE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	SLATE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	SLATE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	SLATE_API virtual FReply OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	SLATE_API virtual FReply OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	SLATE_API virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;

public:

	// IScrollableWidget interface

	SLATE_API virtual FVector2D GetScrollDistance() override;
	SLATE_API virtual FVector2D GetScrollDistanceRemaining() override;
	SLATE_API virtual TSharedRef<class SWidget> GetScrollWidget() override;

	SLATE_API void SetShadowBoxStyle(const FScrollBoxStyle* InShadowBoxStyle);
	const FScrollBoxStyle* GetShadowBoxStyle() const 
	{
		return ShadowBoxStyle;
	}

protected:

	SLATE_API STableViewBase( ETableViewMode::Type InTableViewMode );
	SLATE_API virtual ~STableViewBase();

	/** Returns the "true" scroll offset where the list will ultimately settle (and may already be). */
	SLATE_API double GetTargetScrollOffset() const;

	/**
	 * Scroll the list view by some number of screen units.
	 *
	 * @param MyGeometry      The geometry of the ListView at the time
	 * @param ScrollByAmount  The amount to scroll by in Slate Screen Units.
	 * @param AllowOverscroll Should we allow scrolling past the beginning/end of the list?
	 *
	 * @return The amount actually scrolled in items
	 */
	SLATE_API virtual float ScrollBy( const FGeometry& MyGeometry, float ScrollByAmount, EAllowOverscroll InAllowOverscroll );

	/**
	 * Scroll the view to an offset and resets the inertial scroll velocity 
	 *
	 * @param InScrollOffset       Offset into the total list length to scroll down.
	 *
	 * @return The amount actually scrolled
	 */
	SLATE_API virtual float ScrollTo( float InScrollOffset);

	/** Insert WidgetToInsert at the top of the view. */
	SLATE_API void InsertWidget( const TSharedRef<ITableRow> & WidgetToInset );

	/** Add a WidgetToAppend to the bottom of the view. */
	SLATE_API void AppendWidget( const TSharedRef<ITableRow>& WidgetToAppend );

	SLATE_API const FChildren* GetConstructedTableItems() const;

	/**
	 * Remove all the widgets from the view.
	 */
	SLATE_API void ClearWidgets();

	/** Insert WidgetToInsert at the top of the pinned view. */
	SLATE_API void InsertPinnedWidget(const TSharedRef<SWidget>& WidgetToInset);

	/** Add a WidgetToAppend to the bottom of the pinned view. */
	SLATE_API void AppendPinnedWidget(const TSharedRef<SWidget>& WidgetToAppend);

	/**
	 * Remove all the pinned widgets from the view.
	 */
	SLATE_API void ClearPinnedWidgets();

	/**
	 * Get the uniform item width.
	 */
	SLATE_API float GetItemWidth() const;

	/**
	 * Get the uniform item height that is enforced by ListViews.
	 */
	SLATE_API float GetItemHeight() const;

	/**
	* Get the uniform item
	*/
	SLATE_API UE::Slate::FDeprecateVector2DResult GetItemSize() const;

	/** @return the number of items that can fit on the screen */
	SLATE_API virtual float GetNumLiveWidgets() const;

	/**
	 * Get the number of items that can fit in the view along the line axis (orthogonal to the scroll axis) before creating a new line.
	 * Default is 1, but may be more in subclasses (like STileView)
	 */
	SLATE_API virtual int32 GetNumItemsPerLine() const;

	/**
	 * Get the offset of the first list item.
	 */
	SLATE_API virtual float GetFirstLineScrollOffset() const;

	/*
	 * Right click down
	 */
	virtual void OnRightMouseButtonDown(const FPointerEvent& MouseEvent) {}

	/**
	 * Opens a context menu as the result of a right click if OnContextMenuOpening is bound and we are not right click scrolling.
	 */
	SLATE_API virtual void OnRightMouseButtonUp(const FPointerEvent& MouseEvent);
	
	/**
	 * Get the scroll rate in items that best approximates a constant physical scroll rate.
	 */
	SLATE_API float GetScrollRateInItems() const;

	/**
	 * Remove any items that are no longer in the list from the selection set.
	 */
	virtual void UpdateSelectionSet() = 0;

	/**
	 * Automatically called when the Selection Mode has changed
	 */
	virtual void OnSelectionModeChanged(ESelectionMode::Type OldSelectionMode, ESelectionMode::Type NewSelectionMode) {}

	/** Internal request for a layout update on the next tick (i.e. a refresh without implication that the source items have changed) */
	SLATE_API void RequestLayoutRefresh();

	/** Information about the outcome of the WidgetRegeneratePass */
	struct FReGenerateResults
	{
		FReGenerateResults(double InNewScrollOffset, double InLengthGenerated, double InItemsOnScreen, bool AtEndOfList)
			: NewScrollOffset(InNewScrollOffset)
			, LengthOfGeneratedItems(InLengthGenerated)
			, ExactNumLinesOnScreen(InItemsOnScreen)
			, bGeneratedPastLastItem(AtEndOfList)
		{}

		/** The scroll offset that we actually use might not be what the user asked for */
		double NewScrollOffset = 0.;

		/** The total length along the scroll axis of the widgets that we have generated to represent the visible subset of the items*/
		double LengthOfGeneratedItems = 0.;

		/** How many lines are fitting on the screen, including fractions */
		double ExactNumLinesOnScreen = 0.;

		/** True when we have generated  */
		bool bGeneratedPastLastItem = false;
	};
	/**
	 * Update generate Widgets for Items as needed and clean up any Widgets that are no longer needed.
	 * Re-arrange the visible widget order as necessary.
	 */
	virtual FReGenerateResults ReGenerateItems( const FGeometry& MyGeometry ) = 0;

	/** @return how many items there are in the TArray being observed */
	virtual int32 GetNumItemsBeingObserved() const = 0;

	/** @return how many pinned items are in the table */
	SLATE_API int32 GetNumPinnedItems() const;

	UE_DEPRECATED(5.5, "The Pinned items visibility is no more an attribute.")
	SLATE_API EVisibility GetPinnedItemsVisiblity() const;

	enum class EScrollIntoViewResult
	{
		/** The function scrolled an item (if set) into view (or the item was already in view) */
		Success,
		/** The function did not have enough data to scroll the given item into view, so it should be deferred until the next Tick */
		Deferred,
		/** The function failed to scroll to the specified item.*/
		Failure
	};

	/**
	 * If there is a pending request to scroll an item into view, do so.
	 * 
	 * @param ListViewGeometry  The geometry of the listView; can be useful for centering the item.
	 */
	virtual EScrollIntoViewResult ScrollIntoView(const FGeometry& ListViewGeometry) = 0;

	/**
	 * Called when an item has entered the visible geometry to check to see if the ItemScrolledIntoView delegate should be fired.
	 */
	virtual void NotifyItemScrolledIntoView() = 0;

	/**
	 * Called at the end of a ::Tick when
	 * - CurrentScrollOffset == TargetScrollOffset 
	 * - the user is not scrolling
	 * - the table view is not scrolling from an overscroll position
	 */
	virtual void NotifyFinishedScrolling() = 0;

	/** Called when the user has started to touch. */
	virtual void NotifyTouchStart() = 0;

	/** Called when the user has scrolled the list some amount. */
	virtual void NotifyTouchMove() = 0;

	/** Called when the user has lifted its finger after scrolling. */
	virtual void NotifyTouchEnd() = 0;

	/** Util Function so templates classes don't need to include SlateApplication */
	SLATE_API void NavigateToWidget(const uint32 UserIndex, const TSharedPtr<SWidget>& NavigationDestination, ENavigationSource NavigationSource = ENavigationSource::FocusedWidget, EUINavigation NavigationType = EUINavigation::Invalid) const;

	/** Util function to find the child index under the given position. */
	SLATE_API int32 FindChildUnderPosition(FArrangedChildren& ArrangedChildren, const FVector2D& ArrangedSpacePosition) const;

protected:
	ESelectionMode::Type GetSelectionMode() const
	{
		return SelectionModeAttribute.Get();
	}
	void Private_SetSelectionMode(TAttribute<ESelectionMode::Type> InSelectionMode)
	{
		SelectionModeAttribute.Assign(*this, MoveTemp(InSelectionMode));
	}
	TSlateAttributeRef<ESelectionMode::Type> GetSelectionModeAttribute() const
	{
		return TSlateAttributeRef<ESelectionMode::Type>{SharedThis(this), SelectionModeAttribute};
	}

	const FSlateBrush* GetBackgroundBrush() const
	{
		return BackgroundBrushAttribute.Get().GetImage();
	}
	TSlateAttributeRef<FSlateBrushCached> GetBackgroundBrushAttribute() const
	{
		return TSlateAttributeRef<FSlateBrushCached>{SharedThis(this), BackgroundBrushAttribute};
	}

protected:
	/** The panel which holds the visible widgets in this list */
	TSharedPtr< SListPanel > ItemsPanel;

	/** The panel which holds the pinned widgets in this list */
	TSharedPtr< SListPanel > PinnedItemsPanel;

	/** The scroll bar widget */
	TSharedPtr< SScrollBar > ScrollBar;

	/** The ShadowImage widget for either the Top or the Left shadow depending on the Orientation */
	TSharedPtr< class SImage > TopLeftShadowImage;
	/** The ShadowImage widget for either the Bottom or the Right shadow depending on the Orientation */
	TSharedPtr< class SImage > BottomRightShadowImage;

	/** Padding to the scrollbox */
	FMargin ScrollBarSlotPadding;

	/** Delegate to call when the table view is scrolled */
	FOnTableViewScrolled OnTableViewScrolled;

	/** Gets the component of a vector in the direction of scrolling based on the Orientation property. */
	inline float GetScrollComponentFromVector(FVector2f Vector) const
	{
		return float(Orientation == Orient_Vertical ? Vector.Y : Vector.X);
	}

	/** 
	 * The fixed offset in lines to always apply to the top/left (depending on orientation) of the list.
	 *
	 * If provided:
	 *
	 * - all non-inertial means of scrolling will settle with exactly this offset of the topmost entry.
	 *   Ex: A value of 0.25 would cause the topmost full entry to be offset down by a quarter length of the preceding entry.
	 *   
	 * - all inertial means of scrolling will, at the end of the scroll, settle with exactly this offset of the entry most in view.
	 *	 Ex: if at the end of the scroll we see 0.25% of a line and 0.75% of next line at the top/left, the list will scroll to show the second line minus the offset.
	 */
	TOptional<double> FixedLineScrollOffset;

	/** True to lerp smoothly between offsets when the desired scroll offset changes. */
	bool bEnableAnimatedScrolling = false;

	/** The speed of interpolation for the scrolling animation */
	float ScrollingAnimationInterpolationSpeed = 12.f;

	/** True to lerp smoothly between offsets when the desired scroll offset changes with touch. */
	bool bEnableTouchAnimatedScrolling = false;

	/** True to allow right click drag scrolling. */
	bool bEnableRightClickScrolling = true;

	/** True to allow scrolling by using touch input. */
	bool bEnableTouchScrolling = true;

	/** If true, items will be "selected" (in addition to focused) when navigating to them. If false, they will only be focused. */
	bool bSelectItemOnNavigation = true;

	/** The currently displayed scroll offset from the beginning of the list in items. */
	double CurrentScrollOffset = 0.;

	/** 
	 * The raw desired scroll offset from the beginning of the list in items.
	 * Does not incorporate the FixedLineScrollOffset. Use GetTargetScrollOffset() to know the final target offset to display.
	 * Note: If scroll animation is disabled and there is no FixedLineScrollOffset, this is identical to both the CurrentScrollOffset and the target offset.
	 */
	double DesiredScrollOffset = 0.;

	/** Did the user start an interaction in this list? */
	bool bStartedTouchInteraction;

	/** How much we scrolled while the rmb has been held */
	float AmountScrolledWhileRightMouseDown;

	/** The location in screenspace the view was pressed */
	FVector2f PressedScreenSpacePosition;

	/** The amount we have scrolled this tick cycle */
	float TickScrollDelta;

	/** Information about the widgets we generated during the last regenerate pass */
	FReGenerateResults LastGenerateResults;

	/** Last time we scrolled, did we end up at the end of the list. */
	bool bWasAtEndOfList;

	/** What the list's geometry was the last time a refresh occurred. */
	FGeometry PanelGeometryLastTick;

	/** Delegate to invoke when the context menu should be opening. If it is nullptr, a context menu will not be summoned. */
	FOnContextMenuOpening OnContextMenuOpening;

	/** Delegate to invoke after the children have been regenerated. */
	FSimpleDelegate OnItemsRebuilt;

#if WITH_EDITORONLY_DATA
	/** The selection mode that this tree/list is in. Note that it is up to the generated ITableRows to respect this setting. */
	UE_DEPRECATED(5.8, "Use Private_SetSelectionMode / GetSelectionMode / GetSelectionModeAttribute")
	TSlateDeprecatedTAttribute<ESelectionMode::Type> SelectionMode;
#endif

	/** Column headers that describe which columns this list shows */
	TSharedPtr< SHeaderRow > HeaderRow;

	/** Helper object to manage inertial scrolling */
	FInertialScrollManager InertialScrollManager;

	/**	The current position of the software cursor */
	FVector2f SoftwareCursorPosition;

	/**	Whether the software cursor should be drawn in the viewport */
	bool bShowSoftwareCursor;

	/** How much to scroll when using mouse wheel */
	float WheelScrollMultiplier;

	/** Whether the list is allowed to scroll via cursor. */
	bool bIsPointerScrollingEnabled = true;

	/** Whether the list is allowed to scroll via gamepad. */
	bool bIsGamepadScrollingEnabled = true;

	/** The layout and scroll orientation of the list */
	EOrientation Orientation = Orient_Vertical;

	/** Passing over the clipping to SListPanel */
	SLATE_API virtual void OnClippingChanged() override;

	/** Should Use ShdadowBox Styling */
	bool bShouldUseShadowBoxStyle = false;

#if WITH_EDITORONLY_DATA
	/** Brush resource representing the background area of the view */
	UE_DEPRECATED(5.8, "Use SetBackgroundBrush / GetBackgroundBrush / GetBackgroundBrushAttribute")
	FDeprecatedInvalidatableBrushAttribute BackgroundBrush;
#endif

	/** Does the user need a hint that they can scroll to the start of the list? */
	FSlateColor GetStartShadowOpacity() const;
	
	/** Does the user need a hint that they can scroll to the end of the list? */
	FSlateColor GetEndShadowOpacity() const;

protected:

	/** Check whether the current state of the table warrants inertial scroll by the specified amount */
	SLATE_API bool CanUseInertialScroll( float ScrollAmount ) const;

	/** Active timer to update the inertial scroll */
	SLATE_API EActiveTimerReturnType UpdateInertialScroll(double InCurrentTime, float InDeltaTime);

	/** Whether the active timer to update the inertial scrolling is currently registered */
	bool bIsScrollingActiveTimerRegistered;

protected:

	FOverscroll Overscroll;

	/** Whether to permit overscroll on this list view */
	EAllowOverscroll AllowOverscroll;

	/** How we should handle scrolling with the mouse wheel */
	EConsumeMouseWheel ConsumeMouseWheel;

	float GetSmoothDeltaTime(float InDeltaTime);
	void UpdateSmoothDeltaTime(float InDeltaTime);

public:

	float GetOverscroll() const { return Overscroll.GetOverscroll(GetTickSpaceGeometry()); }

private:
	/** When true, a refresh should occur the next tick */
	bool bItemsNeedRefresh = false;

	/** Whether or not the view should scroll to a fixed line offset if the scroll velocity is 0 */
	bool ShouldScrollToFixedLineOffsetAtZeroVelocity() const;

	/** Whether or not the current scroll is inertial */
	bool bIsInertialScroll = false;
	
	/** Construct the vertical shadow box when enabled*/
	TSharedRef<SWidget> ConstructVerticalShadowBox(TSharedRef<SWidget> Content);
	
	/** Construct the horizontal shadow box when enabled */
	TSharedRef<SWidget> ConstructHorizontalShadowBox(TSharedRef<SWidget> Content);

	const float ShadowFadeDistance = 32.0f;

	/** Style resource for the Shadow Box when scrolling */
	const FScrollBoxStyle* ShadowBoxStyle;

	TArray<float> DeltaTimeQueue;
	int32 DeltaTimeCircularIndex = 0;
	TArray<float> SortedDeltaTimes;

	// Initialized at -1.0f to detect if the variable has been set and is usable since DeltaTimes cannot be negative.
	float SmoothDeltaTime = -1.0f;

	TSlateAttribute<ESelectionMode::Type> SelectionModeAttribute;
	ESelectionMode::Type PreviousSelectionMode;

	TSlateAttribute<FSlateBrushCached> BackgroundBrushAttribute;

	union
	{
		// vertical scroll bar is stored in horizontal box and vice versa
		SHorizontalBox::FSlot* VerticalScrollBarSlot; // valid when Orientation == Orient_Vertical
		SVerticalBox::FSlot* HorizontalScrollBarSlot; // valid when Orientation == Orient_Horizontal
	};
};


namespace TableViewHelpers
{
	/**
	 * Helper for implementing an efficient version of an item that is not in the tree
	 * view, and therefore does not to know about parent wires.
	 */
	SLATE_API const TBitArray<>& GetEmptyBitArray();
}
