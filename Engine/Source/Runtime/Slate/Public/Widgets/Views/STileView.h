// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/SListView.h"


template <typename ItemType>
class STileView;

template <typename ItemType>
struct TWidgetTypeTraits<STileView<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A TileView widget is a list which arranges its items horizontally until there is no more space then creates a new row.
 * Items are spaced evenly horizontally.
 */
template <typename ItemType>
class STileView : public SListView<ItemType>
{
public:
	typedef typename TListTypeTraits< ItemType >::NullableType NullableItemType;

	typedef typename TSlateDelegates< ItemType >::FOnGenerateRow FOnGenerateRow;
	typedef typename TSlateDelegates< ItemType >::FOnRefreshRow FOnRefreshRow;
	typedef typename TSlateDelegates< ItemType >::FOnItemScrolledIntoView FOnItemScrolledIntoView;
	typedef typename TSlateDelegates< ItemType >::FOnMouseButtonClick FOnMouseButtonClick;
	typedef typename TSlateDelegates< ItemType >::FOnMouseButtonDoubleClick FOnMouseButtonDoubleClick;
	typedef typename TSlateDelegates< NullableItemType >::FOnSelectionChanged FOnSelectionChanged;
	typedef typename TSlateDelegates< ItemType >::FIsSelectableOrNavigable FIsSelectableOrNavigable;

	typedef typename TSlateDelegates< ItemType >::FOnItemToString_Debug FOnItemToString_Debug; 

	using FOnWidgetToBeRemoved = typename SListView<ItemType>::FOnWidgetToBeRemoved;

public:

	SLATE_BEGIN_ARGS(STileView<ItemType>)
		: _OnGenerateTile()
		, _OnRefreshTile()
		, _OnTileReleased()
		, _ItemHeight(128)
		, _ItemWidth(128)
		, _ItemAlignment(EListItemAlignment::EvenlyDistributed)
		, _OnContextMenuOpening()
		, _OnItemsRebuilt()
		, _OnMouseButtonClick()
		, _OnMouseButtonDoubleClick()
		, _OnSelectionChanged()
		, _OnIsSelectableOrNavigable()
		, _SelectionMode(ESelectionMode::Multi)
		, _ClearSelectionOnClick(true)
		, _ExternalScrollbar()
		, _Orientation(Orient_Vertical)
		, _EnableAnimatedScrolling(false)
		, _ScrollbarVisibility(EVisibility::Visible)
		, _ScrollbarDragFocusCause(EFocusCause::Mouse)
		, _AllowOverscroll(EAllowOverscroll::Yes)
		, _ScrollBarStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
		, _ScrollbarDisabledVisibility(EVisibility::Collapsed)
		, _ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
		, _WheelScrollMultiplier(GetGlobalScrollAmount())
		, _HandleGamepadEvents(true)
		, _HandleDirectionalNavigation(true)
		, _IsFocusable(true)
		, _OnItemToString_Debug()
		, _OnEnteredBadState()
		, _WrapHorizontalNavigation(true)
		{
			this->_Clipping = EWidgetClipping::ClipToBounds;
		}

		SLATE_EVENT( FOnGenerateRow, OnGenerateTile )

		SLATE_EVENT( FOnRefreshRow, OnRefreshTile )

		SLATE_EVENT( FOnWidgetToBeRemoved, OnTileReleased )

		SLATE_EVENT( FOnTableViewScrolled, OnTileViewScrolled )
		
		SLATE_EVENT( FOnFinishedScrolling, OnFinishedScrolling )

		SLATE_EVENT( FOnTouchStart, OnTouchStart )

		SLATE_EVENT( FOnTouchMove, OnTouchMove )

		SLATE_EVENT( FOnTouchEnd, OnTouchEnd )

		SLATE_EVENT( FOnItemScrolledIntoView, OnItemScrolledIntoView )

		SLATE_ITEMS_SOURCE_ARGUMENT( ItemType, ListItemsSource )

		SLATE_ATTRIBUTE( float, ItemHeight )

		SLATE_ATTRIBUTE( float, ItemWidth )

		SLATE_ATTRIBUTE( EListItemAlignment, ItemAlignment )

		SLATE_EVENT( FOnContextMenuOpening, OnContextMenuOpening )

		SLATE_EVENT( FSimpleDelegate, OnItemsRebuilt )

		SLATE_EVENT( FOnMouseButtonClick, OnMouseButtonClick )

		SLATE_EVENT( FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick )

		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )

		SLATE_EVENT( FIsSelectableOrNavigable, OnIsSelectableOrNavigable)

		SLATE_ATTRIBUTE( ESelectionMode::Type, SelectionMode )

		SLATE_ARGUMENT ( bool, ClearSelectionOnClick )

		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar )

		SLATE_ARGUMENT(EOrientation, Orientation)

		SLATE_ARGUMENT( bool, EnableAnimatedScrolling)

		SLATE_ARGUMENT( TOptional<double>, FixedLineScrollOffset )

		SLATE_ATTRIBUTE(EVisibility, ScrollbarVisibility)

		SLATE_ARGUMENT(EFocusCause, ScrollbarDragFocusCause)

		SLATE_ARGUMENT( EAllowOverscroll, AllowOverscroll );

		SLATE_STYLE_ARGUMENT( FScrollBarStyle, ScrollBarStyle );

		SLATE_ARGUMENT( EVisibility, ScrollbarDisabledVisibility );

		SLATE_ARGUMENT( EConsumeMouseWheel, ConsumeMouseWheel );

		SLATE_ARGUMENT( float, WheelScrollMultiplier );

		SLATE_ARGUMENT( bool, HandleGamepadEvents );

		SLATE_ARGUMENT( bool, HandleDirectionalNavigation );

		SLATE_ATTRIBUTE(bool, IsFocusable)

		/** Assign this to get more diagnostics from the list view. */
		SLATE_EVENT(FOnItemToString_Debug, OnItemToString_Debug)

		SLATE_EVENT(FOnTableViewBadState, OnEnteredBadState);
		
		SLATE_ARGUMENT(bool, WrapHorizontalNavigation);

		/** When true, enables spatially proximate focus routing for cross-list navigation
		 * and intra-entry navigation within multi-focus entries. Defaults to false.
		 */
		SLATE_ARGUMENT(bool, EnableProximateEntryNavigation)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const typename STileView<ItemType>::FArguments& InArgs )
	{
		this->Clipping = InArgs._Clipping;

		this->OnGenerateRow = InArgs._OnGenerateTile;
		this->OnRefreshRow = InArgs._OnRefreshTile;
		this->OnRowReleased = InArgs._OnTileReleased;
		this->OnItemScrolledIntoView = InArgs._OnItemScrolledIntoView;
		
		this->SetItemsSource(InArgs.MakeListItemsSource(this->SharedThis(this)));
		this->OnContextMenuOpening = InArgs._OnContextMenuOpening;
		this->OnItemsRebuilt = InArgs._OnItemsRebuilt;
		this->OnClick = InArgs._OnMouseButtonClick;
		this->OnDoubleClick = InArgs._OnMouseButtonDoubleClick;
		this->OnSelectionChanged = InArgs._OnSelectionChanged;
		this->OnIsSelectableOrNavigable = InArgs._OnIsSelectableOrNavigable;
		this->SetSelectionMode(InArgs._SelectionMode);

		this->bClearSelectionOnClick = InArgs._ClearSelectionOnClick;

		this->AllowOverscroll = InArgs._AllowOverscroll;
		this->ConsumeMouseWheel = InArgs._ConsumeMouseWheel;
		this->WheelScrollMultiplier = InArgs._WheelScrollMultiplier;

		this->bHandleGamepadEvents = InArgs._HandleGamepadEvents;
		this->bHandleDirectionalNavigation = InArgs._HandleDirectionalNavigation;
		this->IsFocusable = InArgs._IsFocusable;

		this->bEnableAnimatedScrolling = InArgs._EnableAnimatedScrolling;
		this->FixedLineScrollOffset = InArgs._FixedLineScrollOffset;

		this->OnItemToString_Debug = InArgs._OnItemToString_Debug.IsBound()
			? InArgs._OnItemToString_Debug
			: SListView< ItemType >::GetDefaultDebugDelegate();
		this->OnEnteredBadState = InArgs._OnEnteredBadState;

		this->bWrapHorizontalNavigation = InArgs._WrapHorizontalNavigation;
		this->bEnableProximateEntryNavigation = InArgs._EnableProximateEntryNavigation;

		// Check for any parameters that the coder forgot to specify.
		FString ErrorString;
		{
			if ( !this->OnGenerateRow.IsBound() )
			{
				ErrorString += TEXT("Please specify an OnGenerateTile. \n");
			}

			if ( !this->HasValidItemsSource() )
			{
				ErrorString += TEXT("Please specify a ListItemsSource. \n");
			}
		}

		if (ErrorString.Len() > 0)
		{
			// Let the coder know what they forgot
			this->ChildSlot
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(ErrorString))
				];
		}
		else
		{
			// Make the TableView
			this->ConstructChildren(InArgs._ItemWidth, InArgs._ItemHeight, InArgs._ItemAlignment, TSharedPtr<SHeaderRow>(), InArgs._ExternalScrollbar, InArgs._Orientation, InArgs._OnTileViewScrolled, InArgs._ScrollBarStyle);
			if (this->ScrollBar.IsValid())
			{
				this->ScrollBar->SetDragFocusCause(InArgs._ScrollbarDragFocusCause);
				this->ScrollBar->SetUserVisibility(InArgs._ScrollbarVisibility);
				this->ScrollBar->SetScrollbarDisabledVisibility(InArgs._ScrollbarDisabledVisibility);
			}
			this->AddMetadata(MakeShared<TTableViewMetadata<ItemType>>(this->SharedThis(this)));
		}
	}

	STileView( ETableViewMode::Type InListMode = ETableViewMode::Tile )
	: SListView<ItemType>( InListMode )
	{
	}

public:

	// SWidget overrides

	virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override
	{
		if (this->HasValidItemsSource() && this->bHandleDirectionalNavigation && (this->bHandleGamepadEvents || InNavigationEvent.GetNavigationGenesis() != ENavigationGenesis::Controller))
		{
			const TArrayView<const ItemType>& ItemsSourceRef = this->GetItems();

			const int32 NumItemsPerLine = GetNumItemsPerLine();
			const int32 CurSelectionIndex = (!TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem)) ? 0 : ItemsSourceRef.Find(TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(this->SelectorItem));
			int32 AttemptSelectIndex = -1;

			const EUINavigation NavType = InNavigationEvent.GetNavigationType();
			if ((this->Orientation == Orient_Vertical && NavType == EUINavigation::Left) ||
				(this->Orientation == Orient_Horizontal && NavType == EUINavigation::Up))
			{
				if (bWrapHorizontalNavigation || (CurSelectionIndex % NumItemsPerLine) > 0)
				{
					AttemptSelectIndex = CurSelectionIndex - 1;
				}
			}
			else if ((this->Orientation == Orient_Vertical && NavType == EUINavigation::Right) ||
				(this->Orientation == Orient_Horizontal && NavType == EUINavigation::Down))
			{
				if (bWrapHorizontalNavigation || (CurSelectionIndex % NumItemsPerLine) < (NumItemsPerLine - 1))
				{
					AttemptSelectIndex = CurSelectionIndex + 1;
				}
			}

			// If it's valid we'll scroll it into view and return an explicit widget in the FNavigationReply
			if (ItemsSourceRef.IsValidIndex(AttemptSelectIndex) && this->bIsGamepadScrollingEnabled)
			{
				this->NavigationSelect(ItemsSourceRef[AttemptSelectIndex], InNavigationEvent);
				return FNavigationReply::Explicit(nullptr);
			}
		}
		
		return SListView<ItemType>::OnNavigation(MyGeometry, InNavigationEvent);
	}

public:	

	virtual STableViewBase::FReGenerateResults ReGenerateItems( const FGeometry& MyGeometry ) override
	{
		// Clear all the items from our panel. We will re-add them in the correct order momentarily.
		this->ClearWidgets();
		
		const TArrayView<const ItemType> Items = this->GetItems();
		if (Items.Num() > 0)
		{
			// Item width and height is constant by design.
			FTableViewDimensions TileDimensions = GetTileDimensions();
			FTableViewDimensions AllottedDimensions(this->Orientation, MyGeometry.GetLocalSize());

			const int32 NumItems = Items.Num();
			const int32 NumItemsPerLine = GetNumItemsPerLine();
			const int32 NumItemsPaddedToFillLastLine = (NumItems % NumItemsPerLine != 0)
				? NumItems + NumItemsPerLine - NumItems % NumItemsPerLine
				: NumItems;

			const double LinesPerScreen = AllottedDimensions.ScrollAxis / TileDimensions.ScrollAxis;
			const double EndOfListOffset = NumItemsPaddedToFillLastLine - NumItemsPerLine * LinesPerScreen;
			const double ClampedScrollOffset = FMath::Clamp(this->CurrentScrollOffset, 0.0, EndOfListOffset);
			const float LayoutScaleMultiplier = MyGeometry.GetAccumulatedLayoutTransform().GetScale();
			
			// Once we run out of vertical and horizontal space, we stop generating widgets.
			FTableViewDimensions DimensionsUsedSoFar(this->Orientation);
			
			// Index of the item at which we start generating based on how far scrolled down we are
			int32 StartIndex = FMath::Max( 0, FMath::FloorToInt32(ClampedScrollOffset / NumItemsPerLine) * NumItemsPerLine);

			// Let the WidgetGenerator know that we are starting a pass so that it can keep track of data items and widgets.
			this->WidgetGenerator.OnBeginGenerationPass();

			// Actually generate the widgets.
			bool bIsAtEndOfList = false;
			bool bHasFilledAvailableArea = false;
			bool bNewLine = true;
			bool bFirstLine = true;
			double NumLinesShownOnScreen = 0;
			for( int32 ItemIndex = StartIndex; !bHasFilledAvailableArea && ItemIndex < NumItems; ++ItemIndex )
			{
				const ItemType& CurItem = Items[ItemIndex];

				if (bNewLine)
				{
					bNewLine = false;
					
					float LineFraction = 1.f;
					if (bFirstLine)
					{
						bFirstLine = false;
						LineFraction -= (float)FMath::Fractional(ClampedScrollOffset / NumItemsPerLine);
					}

					DimensionsUsedSoFar.ScrollAxis += TileDimensions.ScrollAxis * LineFraction;
					
					if (DimensionsUsedSoFar.ScrollAxis > AllottedDimensions.ScrollAxis)
					{
						NumLinesShownOnScreen += FMath::Max(1.0f - ((DimensionsUsedSoFar.ScrollAxis - AllottedDimensions.ScrollAxis) / TileDimensions.ScrollAxis), 0.0f);
					}
					else
					{
						NumLinesShownOnScreen += LineFraction;
					}
				}

				SListView<ItemType>::GenerateWidgetForItem(CurItem, ItemIndex, StartIndex, LayoutScaleMultiplier);

				// The widget used up some of the available space for the current line
				DimensionsUsedSoFar.LineAxis += TileDimensions.LineAxis;

				bIsAtEndOfList = ItemIndex >= NumItems - 1;

				if (DimensionsUsedSoFar.LineAxis + TileDimensions.LineAxis > AllottedDimensions.LineAxis)
				{
					// A new line of widgets was completed - time to start another one
					DimensionsUsedSoFar.LineAxis = 0;
					bNewLine = true;
				}

				if (bIsAtEndOfList || bNewLine)
				{
					// We've filled all the available area when we've finished a line that's partially clipped by the end of the view
					const float FloatPrecisionOffset = 0.001f;
					bHasFilledAvailableArea = DimensionsUsedSoFar.ScrollAxis > AllottedDimensions.ScrollAxis + FloatPrecisionOffset;
				}
			}

			// We have completed the generation pass. The WidgetGenerator will clean up unused Widgets.
			this->WidgetGenerator.OnEndGenerationPass();

			const float TotalGeneratedLineAxisSize = (float)(FMath::CeilToFloat(NumLinesShownOnScreen) * TileDimensions.ScrollAxis);
			return STableViewBase::FReGenerateResults(ClampedScrollOffset, TotalGeneratedLineAxisSize, NumLinesShownOnScreen, bIsAtEndOfList && !bHasFilledAvailableArea);
		}

		return STableViewBase::FReGenerateResults(0, 0, 0, false);

	}

	virtual int32 GetNumItemsBeingObserved() const override
	{
		const int32 NumItemsBeingObserved = this->GetItems().Num();
		const int32 NumItemsPerLine = GetNumItemsPerLine();
		
		int32 NumEmptySpacesAtEnd = 0;
		if ( NumItemsPerLine > 0 )
		{
			NumEmptySpacesAtEnd = NumItemsPerLine - (NumItemsBeingObserved % NumItemsPerLine);
			if ( NumEmptySpacesAtEnd >= NumItemsPerLine )
			{
				NumEmptySpacesAtEnd = 0;
			}
		}

		return NumItemsBeingObserved + NumEmptySpacesAtEnd;
	}

protected:

	FTableViewDimensions GetTileDimensions() const
	{
		return FTableViewDimensions(this->Orientation, this->GetItemWidth(), this->GetItemHeight());
	}

	virtual float ScrollBy( const FGeometry& MyGeometry, float ScrollByAmountInSlateUnits, EAllowOverscroll InAllowOverscroll ) override
	{
		const bool bWholeListVisible = this->DesiredScrollOffset == 0 && this->bWasAtEndOfList;
		if (InAllowOverscroll == EAllowOverscroll::Yes && this->Overscroll.ShouldApplyOverscroll(this->DesiredScrollOffset == 0, this->bWasAtEndOfList, ScrollByAmountInSlateUnits))
		{
			const float UnclampedScrollDelta = ScrollByAmountInSlateUnits / (float)GetNumItemsPerLine();
			const float ActuallyScrolledBy = this->Overscroll.ScrollBy(MyGeometry, UnclampedScrollDelta);
			if (ActuallyScrolledBy != 0.0f)
			{
				this->RequestListRefresh();
			}
			return ActuallyScrolledBy;
		}
		else if (!bWholeListVisible)
		{
			const double NewScrollOffset = this->DesiredScrollOffset + ((ScrollByAmountInSlateUnits * (float)GetNumItemsPerLine()) / GetTileDimensions().ScrollAxis);

			return this->ScrollTo( (float)NewScrollOffset );
		}

		return 0.f;
	}

	virtual int32 GetNumItemsPerLine() const override
	{
		FTableViewDimensions PanelDimensions(this->Orientation, this->PanelGeometryLastTick.GetLocalSize());
		FTableViewDimensions TileDimensions = GetTileDimensions();

		const int32 NumItemsPerLine = TileDimensions.LineAxis > 0 ? FMath::FloorToInt(PanelDimensions.LineAxis / TileDimensions.LineAxis) : 1;
		return FMath::Max(1, NumItemsPerLine);
	}

	/**
	 * If there is a pending request to scroll an item into view, do so.
	 *
	 * @param ListViewGeometry  The geometry of the listView; can be useful for centering the item.
	 */
	virtual typename SListView<ItemType>::EScrollIntoViewResult ScrollIntoView(const FGeometry& ListViewGeometry) override
	{
		if (TListTypeTraits<ItemType>::IsPtrValid(this->ItemToScrollIntoView) && this->HasValidItemsSource())
		{
			const int32 IndexOfItem = this->GetItems().Find(TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(this->ItemToScrollIntoView));
			if (IndexOfItem != INDEX_NONE)
			{
				const float NumLinesInView = FTableViewDimensions(this->Orientation, ListViewGeometry.GetLocalSize()).ScrollAxis / GetTileDimensions().ScrollAxis;

				double NumLiveWidgets = this->GetNumLiveWidgets();
				if (NumLiveWidgets == 0 && this->IsPendingRefresh())
				{
					// Use the last number of widgets on screen to estimate if we actually need to scroll.
					NumLiveWidgets = this->LastGenerateResults.ExactNumLinesOnScreen;

					// If we still don't have any widgets, we're not in a situation where we can scroll an item into view
					// (probably as nothing has been generated yet), so we'll defer this again until the next frame
					if (NumLiveWidgets == 0)
					{
						return SListView<ItemType>::EScrollIntoViewResult::Deferred;
					}
				}

				this->EndInertialScrolling();

				// Only scroll the item into view if it's not already in the visible range
				const int32 NumItemsPerLine = GetNumItemsPerLine();
				const double ScrollLineOffset = this->GetTargetScrollOffset() / NumItemsPerLine;
				const int32 LineOfItem = FMath::FloorToInt((float)IndexOfItem / (float)NumItemsPerLine);
				const int32 NumFullLinesInView = FMath::FloorToInt32(ScrollLineOffset + NumLinesInView) - FMath::CeilToInt32(ScrollLineOffset);
				
				const double MinDisplayedLine = this->bNavigateOnScrollIntoView ? FMath::FloorToDouble(ScrollLineOffset) : FMath::CeilToDouble(ScrollLineOffset);
				const double MaxDisplayedLine = this->bNavigateOnScrollIntoView ? FMath::CeilToDouble(ScrollLineOffset + NumFullLinesInView) : FMath::FloorToDouble(ScrollLineOffset + NumFullLinesInView);

				if (LineOfItem < MinDisplayedLine || LineOfItem > MaxDisplayedLine)
				{
					float NewLineOffset = 0.0f;
					if (LineOfItem < MinDisplayedLine - 1 || LineOfItem > MaxDisplayedLine + 1)
					{
						// The item was completely hidden so we scroll to center it.
						
						// Set the line with the item at the beginning of the view area
						NewLineOffset = (float)LineOfItem;
						// Center the line in the view area
						NewLineOffset -= NumLinesInView * 0.5f;
						// Center the tile itself
						NewLineOffset += 0.5f;
					}
					else if (LineOfItem == MinDisplayedLine - 1)
					{
						// The item was just on the line above, set the line with the item at the beginning of the view area
						// This prevent issues when navigating the list element by element triggers this code since we don't want to suddenly center an element
						NewLineOffset = (float)LineOfItem;
					}
					else // LineOfItem == MaxDisplayedLine + 1
					{
						// The item was just on the next line, scroll it into view
						// This prevent issues when navigating the list element by element triggers this code since we don't want to suddenly center an element
						if (this->FixedLineScrollOffset.IsSet())
						{
							// We should scroll to align a line with the top of the view while putting the item at the bottom
							const int32 FullLinesInView = FMath::FloorToInt32(NumLinesInView);
							NewLineOffset = (float)LineOfItem - ((float)FullLinesInView - 1.f);
						}
						else
						{
							// We should scroll just enough to show the item on the last line
							NewLineOffset = (float)LineOfItem - NumLinesInView + 1.f + this->NavigationScrollOffset;					
						}
					}
					
					// Convert the line offset into an item offset
					double NewScrollOffset = NewLineOffset * (double)NumItemsPerLine;
					// And clamp the scroll offset within the allowed limits
					NewScrollOffset = FMath::Clamp(NewScrollOffset, 0., (double)GetNumItemsBeingObserved() - (double)NumItemsPerLine * NumLinesInView);

					this->SetScrollOffset((float)NewScrollOffset);
				}
				else
				{
					// Make sure the line containing the existing entry for this item is fully in view
					if (LineOfItem == MinDisplayedLine)
					{
						// This line is clipped at the top/left, so set it as the new offset
						this->SetScrollOffset((float)(LineOfItem * NumItemsPerLine) - (this->FixedLineScrollOffset.IsSet() && LineOfItem > 0 ? 0.f : this->NavigationScrollOffset));
					}
					else if (LineOfItem == MaxDisplayedLine)
					{
						if (this->FixedLineScrollOffset.IsSet())
						{
							// This line is clipped at the end but since FixedLineScrollOffset is set we must align a line to the top of the list,
							// therefore we just scroll to the next line
							const float HiddenLinesCount = (float)LineOfItem - FMath::FloorToFloat(NumLinesInView); 
							this->SetScrollOffset((1.0f + HiddenLinesCount) * (float)NumItemsPerLine);							
						}
						else
						{
							// This line is clipped at the end, so we need to advance just enough to bring it fully into view
							// Since all tiles are required to be of the same size, this is straightforward
							const float NewLineOffset = (float)LineOfItem - NumLinesInView + 1.f + this->NavigationScrollOffset;
							this->SetScrollOffset(NewLineOffset * (float)NumItemsPerLine);
						}
					}
				}

				this->RequestListRefresh();

				this->ItemToNotifyWhenInView = this->ItemToScrollIntoView;
			}

			TListTypeTraits<ItemType>::ResetPtr(this->ItemToScrollIntoView);
		}

		if (TListTypeTraits<ItemType>::IsPtrValid(this->ItemToNotifyWhenInView))
		{
			if (this->bEnableAnimatedScrolling)
			{
				// When we have a target item we're shooting for, we haven't succeeded with the scroll until a widget for it exists
				const bool bHasWidgetForItem = this->WidgetFromItem(TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(this->ItemToNotifyWhenInView)).IsValid();
				return bHasWidgetForItem ? SListView<ItemType>::EScrollIntoViewResult::Success : SListView<ItemType>::EScrollIntoViewResult::Deferred;
			}
		}

		return SListView<ItemType>::EScrollIntoViewResult::Success;
	}

	/** Should the left and right navigations be handled as a wrap when hitting the bounds. (you'll move to the previous / next row when appropriate) */
	bool bWrapHorizontalNavigation = true;
};
