// Copyright Epic Games, Inc. All Rights Reserved
 
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/SlateTypes.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/SListView.h"
#include "Algo/Reverse.h"
#include "Math/NumericLimits.h"
#include "Widgets/SWidget.h"

/** Info needed by a (relatively) small fraction of the tree items; some of them may not be visible. */
struct FSparseItemInfo
{
	/**
	 * Construct a new FTreeItem.
	 *
	 * @param InItemToVisualize   The DateItem pointer being wrapped by this FTreeItem
	 * @param InHasChildren       Does this item have children? True if yes.
	 */
	FSparseItemInfo( bool InIsExpanded, bool InHasExpandedChildren )
	: bIsExpanded( InIsExpanded )
	, bHasExpandedChildren( InHasExpandedChildren )
	{	
	}

	/** Is this tree item expanded? */
	bool bIsExpanded;

	/** Does this tree item have any expanded children? */
	bool bHasExpandedChildren;
};


/** Info needed by every visible item in the tree */
struct FItemInfo
{
	FItemInfo()
	{		
	}

	FItemInfo(TBitArray<> InNeedsVerticalWire, bool InHasChildren, bool InIsLastChild, int32 InParentIndex )
	: NeedsVerticalWire(InNeedsVerticalWire)
	, bHasChildren( InHasChildren )
	, bIsLastChild( InIsLastChild )
	, ParentIndex( InParentIndex )
	{
	}

	/**
	 * Flags for whether we need a wire drawn for this level of the tree.
	 *
	 * NeedsVerticalWrite.Num() is the nesting level within the tree. e.g. 0 is root-level, 1 is children of root, etc.
	 */
	TBitArray<> NeedsVerticalWire;

	int32 GetNestingLevel() const { return NeedsVerticalWire.Num()-1; }

	/** Does this tree item have children? */
	uint32 bHasChildren : 1;

	/** Is this the last child of its parent? If so, it gets a special kind of wire/connector. */
	uint32 bIsLastChild : 1;

	/** Index into the linearized tree of the parent for this item, if any, otherwise INDEX_NONE. */
	int32 ParentIndex;
};


template <typename ItemType>
class STreeView;

template <typename ItemType>
struct TWidgetTypeTraits<STreeView<ItemType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * This assumes you are familiar with SListView; see SListView.
 *
 * TreeView setup is virtually identical to that of ListView.
 * Additionally, TreeView introduces a new delegate: OnGetChildren().
 * OnGetChildren() takes some DataItem being observed by the tree
 * and returns that item's children. Like ListView, TreeView operates
 * exclusively with pointers to DataItems.
 * 
 */
template<typename ItemType>
class STreeView : public SListView< ItemType >
{
	SLATE_DECLARE_WIDGET(STreeView, SListView<ItemType>)
	static inline SLATE_IMPLEMENT_WIDGET(STreeView)
public:

	using NullableItemType  = typename TListTypeTraits< ItemType >::NullableType;
	using MapKeyFuncs       = typename TListTypeTraits< ItemType >::MapKeyFuncs;
	using MapKeyFuncsSparse = typename TListTypeTraits< ItemType >::MapKeyFuncsSparse;

	using TSparseItemMap    = TMap< ItemType, FSparseItemInfo, FDefaultSetAllocator, MapKeyFuncsSparse >;
	using TItemSet          = TSet< TObjectPtrWrapTypeOf<ItemType>, typename TListTypeTraits< TObjectPtrWrapTypeOf<ItemType> >::SetKeyFuncs >;

	using FOnGetChildren            = typename TSlateDelegates< ItemType >::FOnGetChildren;
	using FOnGenerateRow            = typename TSlateDelegates< ItemType >::FOnGenerateRow;
	using FOnRefreshRow				= typename TSlateDelegates< ItemType >::FOnRefreshRow;
	using FOnSetExpansionRecursive  = typename TSlateDelegates< ItemType >::FOnSetExpansionRecursive;
	using FOnItemScrolledIntoView   = typename TSlateDelegates< ItemType >::FOnItemScrolledIntoView;
	using FOnSelectionChanged       = typename TSlateDelegates< NullableItemType >::FOnSelectionChanged;
	using FIsSelectableOrNavigable	= typename TSlateDelegates< ItemType >::FIsSelectableOrNavigable;
	using FOnMouseButtonClick       = typename TSlateDelegates< ItemType >::FOnMouseButtonClick;
	using FOnMouseButtonDoubleClick = typename TSlateDelegates< ItemType >::FOnMouseButtonDoubleClick;
	using FOnExpansionChanged       = typename TSlateDelegates< ItemType >::FOnExpansionChanged;

	using FOnItemToString_Debug     = typename TSlateDelegates< ItemType >::FOnItemToString_Debug; 

	using FOnWidgetToBeRemoved      = typename SListView< ItemType >::FOnWidgetToBeRemoved;

public:
	
	SLATE_BEGIN_ARGS( STreeView<ItemType> )
		: _TreeViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView"))
		, _OnGenerateRow()
		, _OnGeneratePinnedRow()
		, _OnRefreshRow()
		, _OnGetChildren()
		, _OnSetExpansionRecursive()
		, _MaxPinnedItems(6) // Having more than the max amount of items leads to the extra items in the middle being collapsed into ellipses, and the last item is fully shown
		, _OnContextMenuOpening()
		, _OnItemsRebuilt()
		, _OnMouseButtonClick()
		, _OnMouseButtonDoubleClick()
		, _OnSelectionChanged()
		, _OnExpansionChanged()
		, _OnIsSelectableOrNavigable()
		, _SelectionMode(ESelectionMode::Multi)
		, _ClearSelectionOnClick(true)
		, _ExternalScrollbar()
		, _EnableAnimatedScrolling(false)
		, _ScrollbarDragFocusCause(EFocusCause::Mouse)
		, _ConsumeMouseWheel( EConsumeMouseWheel::WhenScrollingPossible )
		, _AllowOverscroll(EAllowOverscroll::Yes)
		, _ScrollBarStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
		, _PreventThrottling(false)
		, _WheelScrollMultiplier(GetGlobalScrollAmount())
		, _ShadowBoxStyle()
		, _bEnableShadowBoxStyle(false)
		, _OnItemToString_Debug()
		, _OnEnteredBadState()
		, _HandleGamepadEvents(true)
		, _HandleDirectionalNavigation(true)
		, _AllowInvisibleItemSelection(false)
		, _HighlightParentNodesForSelection(false)
		, _ReturnFocusToSelection()
		, _ShouldStackHierarchyHeaders(false)
		{
			this->_Clipping = EWidgetClipping::ClipToBounds;
		}

		SLATE_STYLE_ARGUMENT( FTableViewStyle, TreeViewStyle )

		SLATE_EVENT( FOnGenerateRow, OnGenerateRow )

		SLATE_EVENT( FOnGenerateRow, OnGeneratePinnedRow )

		SLATE_EVENT( FOnRefreshRow, OnRefreshRow )

		SLATE_EVENT( FOnWidgetToBeRemoved, OnRowReleased )

		SLATE_EVENT( FOnTableViewScrolled, OnTreeViewScrolled )

		SLATE_EVENT( FOnFinishedScrolling, OnFinishedScrolling )

		SLATE_EVENT( FOnTouchStart, OnTouchStart )

		SLATE_EVENT( FOnTouchMove, OnTouchMove )

		SLATE_EVENT( FOnTouchEnd, OnTouchEnd )

		SLATE_EVENT( FOnItemScrolledIntoView, OnItemScrolledIntoView )

		SLATE_EVENT( FOnGetChildren, OnGetChildren )

		SLATE_EVENT( FOnSetExpansionRecursive, OnSetExpansionRecursive )

		SLATE_ITEMS_SOURCE_ARGUMENT( ItemType, TreeItemsSource )

		SLATE_ATTRIBUTE_DEPRECATED(float, ItemHeight, 5.5, "The ItemHeight is only used for Tile. See ShouldArrangeAsTiles")

		SLATE_ATTRIBUTE( int32, MaxPinnedItems );

		SLATE_EVENT( FOnContextMenuOpening, OnContextMenuOpening )

		SLATE_EVENT( FSimpleDelegate, OnItemsRebuilt )

		SLATE_EVENT( FOnMouseButtonClick, OnMouseButtonClick)

		SLATE_EVENT( FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick )

		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )

		SLATE_EVENT( FOnExpansionChanged, OnExpansionChanged )

		SLATE_EVENT( FIsSelectableOrNavigable, OnIsSelectableOrNavigable)

		SLATE_ATTRIBUTE( ESelectionMode::Type, SelectionMode )

		SLATE_ARGUMENT( TSharedPtr<SHeaderRow>, HeaderRow )

		SLATE_ARGUMENT ( bool, ClearSelectionOnClick )

		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar )

		SLATE_ARGUMENT( bool, EnableAnimatedScrolling)

		SLATE_ARGUMENT( TOptional<double>, FixedLineScrollOffset )

		SLATE_ARGUMENT( EFocusCause, ScrollbarDragFocusCause )

		SLATE_ARGUMENT( EConsumeMouseWheel, ConsumeMouseWheel );
		
		SLATE_ARGUMENT( EAllowOverscroll, AllowOverscroll );
		
		SLATE_STYLE_ARGUMENT( FScrollBarStyle, ScrollBarStyle );

		SLATE_ARGUMENT( bool, PreventThrottling )

		SLATE_ARGUMENT( float, WheelScrollMultiplier );

		SLATE_STYLE_ARGUMENT(FScrollBoxStyle, ShadowBoxStyle)

		SLATE_ARGUMENT(bool, bEnableShadowBoxStyle)

		/** Assign this to get more diagnostics from the list view. */
		SLATE_EVENT(FOnItemToString_Debug, OnItemToString_Debug)

		SLATE_EVENT(FOnTableViewBadState, OnEnteredBadState);

		SLATE_ARGUMENT(bool, HandleGamepadEvents);

		SLATE_ARGUMENT(bool, HandleDirectionalNavigation);

		SLATE_ARGUMENT(bool, AllowInvisibleItemSelection);

		SLATE_ARGUMENT(bool, HighlightParentNodesForSelection);

		SLATE_ARGUMENT(bool, ReturnFocusToSelection)

		/** If true, Show the current hierarchy of items pinned at the top of the Tree View */
		SLATE_ATTRIBUTE(bool, ShouldStackHierarchyHeaders)
	
		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

	SLATE_END_ARGS()

		
	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget.
	 */
	void Construct( const FArguments& InArgs )
	{
		this->Clipping = InArgs._Clipping;

		this->OnGenerateRow = InArgs._OnGenerateRow;
		this->OnGeneratePinnedRow = InArgs._OnGeneratePinnedRow;
		this->OnRefreshRow = InArgs._OnRefreshRow;
		this->OnRowReleased = InArgs._OnRowReleased;
		this->OnItemScrolledIntoView = InArgs._OnItemScrolledIntoView;
		this->OnGetChildren = InArgs._OnGetChildren;
		this->OnSetExpansionRecursive = InArgs._OnSetExpansionRecursive;

		this->SetRootItemsSource(InArgs.MakeTreeItemsSource(this->SharedThis(this)));

		this->OnKeyDownHandler = InArgs._OnKeyDownHandler;
		this->OnContextMenuOpening = InArgs._OnContextMenuOpening;
		this->OnItemsRebuilt = InArgs._OnItemsRebuilt;
		this->OnClick = InArgs._OnMouseButtonClick;
		this->OnDoubleClick = InArgs._OnMouseButtonDoubleClick;
		this->OnSelectionChanged = InArgs._OnSelectionChanged;
		this->OnExpansionChanged = InArgs._OnExpansionChanged;
		this->OnIsSelectableOrNavigable = InArgs._OnIsSelectableOrNavigable;
		this->SetSelectionMode(InArgs._SelectionMode);

		this->bClearSelectionOnClick = InArgs._ClearSelectionOnClick;
		this->ConsumeMouseWheel = InArgs._ConsumeMouseWheel;
		this->AllowOverscroll = InArgs._AllowOverscroll;

		this->WheelScrollMultiplier = InArgs._WheelScrollMultiplier;

		this->bEnableAnimatedScrolling = InArgs._EnableAnimatedScrolling;
		this->FixedLineScrollOffset = InArgs._FixedLineScrollOffset;

		this->OnItemToString_Debug = InArgs._OnItemToString_Debug.IsBound()
			? InArgs._OnItemToString_Debug
			: SListView< ItemType >::GetDefaultDebugDelegate();
		this->OnEnteredBadState = InArgs._OnEnteredBadState;

		this->bHandleGamepadEvents = InArgs._HandleGamepadEvents;
		this->bHandleDirectionalNavigation = InArgs._HandleDirectionalNavigation;
		this->bAllowInvisibleItemSelection = InArgs._AllowInvisibleItemSelection;
		this->bHighlightParentNodesForSelection = InArgs._HighlightParentNodesForSelection;

		this->bReturnFocusToSelection = InArgs._ReturnFocusToSelection;

		this->bShouldStackHierarchyHeaders.Assign(*this, InArgs._ShouldStackHierarchyHeaders);

		this->SetStyle(InArgs._TreeViewStyle);

		this->SetShadowBoxStyle(InArgs._ShadowBoxStyle);
		this->bShouldUseShadowBoxStyle = InArgs._bEnableShadowBoxStyle;

		this->SetMaxPinnedItems(InArgs._MaxPinnedItems);
		this->DefaultMaxPinnedItems = InArgs._MaxPinnedItems;

		// Check for any parameters that the coder forgot to specify.
		FString ErrorString;
		{
			if ( !this->OnGenerateRow.IsBound() )
			{
				ErrorString += TEXT("Please specify an OnGenerateRow. \n");
			}

			if (!this->HasValidRootItemsSource())
			{
				ErrorString += TEXT("Please specify a TreeItemsSource. \n");
			}

			if ( !this->OnGetChildren.IsBound() )
			{
				ErrorString += TEXT("Please specify an OnGetChildren. \n");
			}
		}

		if (ErrorString.Len() > 0)
		{
			// Let the coder know what they forgot
			this->ChildSlot
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ErrorString))
			];
		}
		else
		{
			// Make the TableView
			this->ConstructChildren( 0.0f, 0.0f, EListItemAlignment::LeftAligned, InArgs._HeaderRow, InArgs._ExternalScrollbar, Orient_Vertical, InArgs._OnTreeViewScrolled, InArgs._ScrollBarStyle, InArgs._PreventThrottling );
			if (this->ScrollBar.IsValid())
			{
				this->ScrollBar->SetDragFocusCause(InArgs._ScrollbarDragFocusCause);
			}
			this->AddMetadata(MakeShared<TTableViewMetadata<ItemType>>(this->SharedThis(this)));
		}
	}

	/** Default constructor. */
	STreeView()
		: SListView< ItemType >( ETableViewMode::Tree )
		, bTreeItemsAreDirty( true )
		, bShouldStackHierarchyHeaders( *this, false )
	{
		SListView<ItemType>::SetItemsSource(&LinearizedItems);
	}

public:

	//~ SWidget overrides

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if (this->OnKeyDownHandler.IsBound())
		{
			FReply Reply = this->OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
		
		// Check for selection/expansion toggling keys (Left, Right)
		// SelectorItem represents the keyboard selection. If it isn't valid then we don't know what to expand.
		// Don't respond to key-presses containing "Alt" as a modifier
		if ( TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem) && !InKeyEvent.IsAltDown() )
		{
			if ( InKeyEvent.GetKey() == EKeys::Left )
			{
				if( TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem) )
				{
					ItemType RangeSelectionEndItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( this->SelectorItem );
					int32 SelectionIndex = this->LinearizedItems.Find( RangeSelectionEndItem );

					const bool bDoesItemHaveChildren = Private_DoesItemHaveChildren(SelectionIndex);
					if( bDoesItemHaveChildren && InKeyEvent.GetModifierKeys().IsShiftDown() )
					{
						// Collapse the selected item and all of it's children
						Private_OnExpanderArrowShiftClicked(RangeSelectionEndItem, false);
					}
					else if ( bDoesItemHaveChildren && Private_IsItemExpanded( RangeSelectionEndItem ) )
					{
						// Collapse the selected item
						Private_SetItemExpansion(RangeSelectionEndItem, false);
					}
					else
					{
						// Select the parent, who should be a previous item in the list whose nesting level is less than the selected one
						int32 SelectedNestingDepth = Private_GetNestingDepth(SelectionIndex);
						for (SelectionIndex--; SelectionIndex >= 0; --SelectionIndex)
						{
							if ( Private_GetNestingDepth(SelectionIndex) < SelectedNestingDepth )
							{
								// Found the parent
								this->NavigationSelect(this->LinearizedItems[SelectionIndex], InKeyEvent);
								break;
							}
						}
					}
				}

				return FReply::Handled();
			}
			else if ( InKeyEvent.GetKey() == EKeys::Right )
			{
				if( TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem) )
				{
					ItemType RangeSelectionEndItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( this->SelectorItem );
					int32 SelectionIndex = this->LinearizedItems.Find( RangeSelectionEndItem );

					// Right only applies to items with children
					if ( Private_DoesItemHaveChildren(SelectionIndex) )
					{
						if( InKeyEvent.GetModifierKeys().IsShiftDown() )
						{
							// Expand the selected item and all of it's children
							Private_OnExpanderArrowShiftClicked(RangeSelectionEndItem, true);
						}
						else if ( !Private_IsItemExpanded(RangeSelectionEndItem) )
						{
							// Expand the selected item
							Private_SetItemExpansion(RangeSelectionEndItem, true);
						}
						else
						{
							// Select the first child, who should be the next item in the list						
							// Make sure we aren't the last item on the list
							if ( SelectionIndex < this->LinearizedItems.Num() - 1 )
							{
								this->NavigationSelect(this->LinearizedItems[SelectionIndex + 1], InKeyEvent);
							}
						}
					}
				}

				return FReply::Handled();
			}
		}

		return SListView<ItemType>::OnKeyDown_Internal(MyGeometry, InKeyEvent);
	}
	
private:

	//~ Tree View adds the ability to expand/collapse items.
	//~ All the selection functionality is inherited from ListView.

	virtual bool Private_IsItemExpanded( const ItemType& TheItem ) const override
	{
		const FSparseItemInfo* ItemInfo = SparseItemInfos.Find(TheItem);
		return ItemInfo != nullptr && ItemInfo->bIsExpanded;
	}

	virtual void Private_SetItemExpansion( ItemType TheItem, bool bShouldBeExpanded ) override
	{
		const FSparseItemInfo* const SparseItemInfo = SparseItemInfos.Find(TheItem);
		bool bWasExpanded = false;

		if(SparseItemInfo)
		{
			bWasExpanded = SparseItemInfo->bIsExpanded;
			SparseItemInfos.Add(TheItem, FSparseItemInfo(bShouldBeExpanded, SparseItemInfo->bHasExpandedChildren));
		}
		else if(bShouldBeExpanded)
		{
			SparseItemInfos.Add(TheItem, FSparseItemInfo(bShouldBeExpanded, false));
		}
		
		if(bWasExpanded != bShouldBeExpanded)
		{
			OnExpansionChanged.ExecuteIfBound(TheItem, bShouldBeExpanded);

			// We must rebuild the linearized version of the tree because
			// either some children became visible or some got removed.
			RequestTreeRefresh();
		}
	}

	virtual void Private_OnExpanderArrowShiftClicked( ItemType TheItem, bool bShouldBeExpanded ) override
	{
		if(OnSetExpansionRecursive.IsBound())
		{
			OnSetExpansionRecursive.Execute(TheItem, bShouldBeExpanded);

			// We must rebuild the linearized version of the tree because
			// either some children became visible or some got removed.
			RequestTreeRefresh();
		}
	}

	virtual bool Private_DoesItemHaveChildren( int32 ItemIndexInList ) const override
	{
		bool bHasChildren = false;

		if (DenseItemInfos.IsValidIndex(ItemIndexInList))
		{
			bHasChildren = DenseItemInfos[ItemIndexInList].bHasChildren;
		}
		return bHasChildren;
	}

	virtual int32 Private_GetNestingDepth( int32 ItemIndexInList ) const override
	{
		int32 NestingLevel = 0;
		if (DenseItemInfos.IsValidIndex(ItemIndexInList))
		{
			NestingLevel = DenseItemInfos[ItemIndexInList].GetNestingLevel();
		}
		return NestingLevel;
	}

	virtual const TBitArray<>& Private_GetWiresNeededByDepth(int32 ItemIndexInList) const override
	{
		return (DenseItemInfos.IsValidIndex(ItemIndexInList))
			? DenseItemInfos[ItemIndexInList].NeedsVerticalWire
			: TableViewHelpers::GetEmptyBitArray();
	}

	virtual bool Private_IsLastChild(int32 ItemIndexInList) const override
	{
		return (DenseItemInfos.IsValidIndex(ItemIndexInList))
			? DenseItemInfos[ItemIndexInList].bIsLastChild
			: false;
	}

	/**
	 * This clears the highlight state from all nodes and then traverses the parents of each selected node to add it to the highlighted set.
	 */
	virtual void Private_UpdateParentHighlights()
	{
		this->Private_ClearHighlightedItems();
		
		for( typename TItemSet::TConstIterator SelectedItemIt(this->SelectedItems); SelectedItemIt; ++SelectedItemIt )
		{
			// Sometimes selection events can come through before the Linearized List is built, so the item may not exist yet.
			int32 ItemIndex = LinearizedItems.Find(*SelectedItemIt);
			if (ItemIndex == INDEX_NONE)
			{
				continue;
			}

			if(DenseItemInfos.IsValidIndex(ItemIndex))
			{
				const FItemInfo& ItemInfo = DenseItemInfos[ItemIndex];
				int32 ParentIndex = ItemInfo.ParentIndex;
				while (ParentIndex != INDEX_NONE)
				{
					const ItemType& ParentItem = this->LinearizedItems[ParentIndex];
					this->Private_SetItemHighlighted(ParentItem, true);

					const FItemInfo& ParentItemInfo = DenseItemInfos[ParentIndex];
					ParentIndex = ParentItemInfo.ParentIndex;
				}
			}
		}
	}

public:
	virtual void Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo) override
	{
		SListView< ItemType >::Private_SignalSelectionChanged(SelectInfo);
		
		if (bHighlightParentNodesForSelection)
		{
			this->Private_UpdateParentHighlights();
		}
	}



	/**
	 * See SWidget::Tick()
	 *
	 * @param  AllottedGeometry The space allotted for this widget.
	 * @param  InCurrentTime  Current absolute real time.
	 * @param  InDeltaTime  Real time passed since last tick.
	 */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STreeView::Tick);

		if ( bTreeItemsAreDirty )
		{
			// Check that ItemsPanel was made; we never make it if the user failed to specify all the parameters necessary to make the tree work.
			if ( this->ItemsPanel.IsValid() )
			{
				// We are about to repopulate linearized items; the ListView that TreeView is built on top of will also need to refresh.
				bTreeItemsAreDirty = false;

				if ( OnGetChildren.IsBound() && HasValidRootItemsSource() )
				{
					// We make copies of the old expansion and selection sets so that we can remove
					// any items that are no longer seen by the tree.
					TItemSet TempSelectedItemsMap;
					TSparseItemMap TempSparseItemInfo;
					TArray<FItemInfo> TempDenseItemInfos;
					
					// Rebuild the linearized view of the tree data.
					LinearizedItems.Empty();
					PopulateLinearizedItems(GetRootItems(), LinearizedItems, TempDenseItemInfos, TBitArray<>(), TempSelectedItemsMap, TempSparseItemInfo, true, INDEX_NONE);

					if( !bAllowInvisibleItemSelection &&
						(this->SelectedItems.Num() != TempSelectedItemsMap.Num() ||
						this->SelectedItems.Difference(TempSelectedItemsMap).Num() > 0 || 
						TempSelectedItemsMap.Difference(this->SelectedItems).Num() > 0 ))
					{
						this->SelectedItems = TempSelectedItemsMap;

						if ( !TListTypeTraits<ItemType>::IsPtrValid( this->RangeSelectionStart ) ||
								!this->SelectedItems.Contains( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( this->RangeSelectionStart ) ))
						{
								TListTypeTraits< ItemType >::ResetPtr( this->RangeSelectionStart );
								TListTypeTraits< ItemType >::ResetPtr( this->SelectorItem );
						}	
						else if ( !TListTypeTraits<ItemType>::IsPtrValid( this->SelectorItem ) || 
									!this->SelectedItems.Contains( TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType( this->SelectorItem ) ) )
						{
							this->SelectorItem = this->RangeSelectionStart;
						}

						this->Private_SignalSelectionChanged(ESelectInfo::Direct);
					}
						
					// these should come after Private_SignalSelectionChanged(); because through a
					// series of calls, Private_SignalSelectionChanged() could end up in a child
					// that indexes into either of these arrays (the index wouldn't be updated yet,
					// and could be invalid)
					SparseItemInfos = MoveTemp(TempSparseItemInfo);
					DenseItemInfos  = MoveTemp(TempDenseItemInfos);

					// Once the selection changed events have gone through we can update the parent highlight statuses, which are based on your current selection.
					if (bHighlightParentNodesForSelection)
					{
						this->Private_UpdateParentHighlights();
					}
				}
			}
		}
			
		// Tick the TreeView superclass so that it can refresh.
		// This may be due to TreeView requesting a refresh or because new items became visible due to resizing or scrolling.
		SListView< ItemType >::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
		
	/** 
	 * Given: an array of items (ItemsSource) each of which potentially has a child.
	 * Task: populate the LinearizedItems array with a flattened version of the visible data items.
	 *       In the process, remove any items that are not visible while maintaining any collapsed
	 *       items that may have expanded children.
	 *
	 * @param ItemsSource          An array of data items each of which may have 0 or more children.
	 * @param LinearizedItems      Array to populate with items based on expanded/collapsed state.
	 * @param NewDenseItemInfos    Array representing how nested each item in the Linearized items is, and whether it has children.
	 * @param TreeLevel            The current level of indentation.
	 * @param OutNewSelectedItems  Selected items minus any items that are no longer observed by the list.
	 * @param NewSparseItemInfo    Expanded items and items that have expanded children minus any items that are no longer observed by the list.
	 * @param bAddingItems         Are we adding encountered items to the linearized items list or just testing them for existence.
	 * @param ParentIndex		   The index in the resulting linearized item list of the parent node for the currently processed level.
	 *
	 * @return true if we encountered expanded children; false otherwise.
	 */
	bool PopulateLinearizedItems(
		TArrayView<const ItemType> InItemsSource,
		TArray<ItemType>& InLinearizedItems,
		TArray<FItemInfo>& NewDenseItemInfos,
		TBitArray<> NeedsParentWire,
		TItemSet& OutNewSelectedItems,
		TSparseItemMap& NewSparseItemInfo,
		bool bAddingItems,
		int32 ParentIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STreeView::PopulateLinearizedItems);

		NeedsParentWire.Add(false);
		const int32 NestingDepthIndex = NeedsParentWire.Num()-1;

		bool bSawExpandedItems = false;
		for ( int32 ItemIndex = 0; ItemIndex < InItemsSource.Num(); ++ItemIndex )
		{
			const ItemType& CurItem = InItemsSource[ItemIndex];

			// Find this items children.
			TArray<ItemType> ChildItems;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STreeView_GetChildren);
				OnGetChildren.Execute(InItemsSource[ItemIndex], ChildItems);
			}

			const bool bHasChildren = ChildItems.Num() > 0;

			// Child items will need a parent wire at this depth if the item we are inserting now is
			// not the last item in its immediate parent's list.
			const bool bIsLastChild = (ItemIndex == InItemsSource.Num() - 1);			
			NeedsParentWire[NestingDepthIndex] = !bIsLastChild;
			
			// Is this item expanded, does it have expanded children?
			const FSparseItemInfo* CurItemInfo = SparseItemInfos.Find( CurItem );
			const bool bIsExpanded = (CurItemInfo == nullptr) ? false : CurItemInfo->bIsExpanded;
			bool bHasExpandedChildren = (CurItemInfo == nullptr) ? false : CurItemInfo->bHasExpandedChildren;
								
			// Add this item to the linearized list and update the selection set.
			if (bAddingItems)
			{
				InLinearizedItems.Add( CurItem );

				NewDenseItemInfos.Add( FItemInfo(NeedsParentWire, bHasChildren, bIsLastChild, ParentIndex) );

				const bool bIsSelected = this->IsItemSelected( CurItem );
				if (bIsSelected)
				{
					OutNewSelectedItems.Add( CurItem );
				}
			}

			if ( bIsExpanded || bHasExpandedChildren )
			{ 
				// If this item is expanded, we should process all of its children at the next indentation level.
				// If it is collapsed, process its children but do not add them to the linearized list.
				const bool bAddChildItems = bAddingItems && bIsExpanded;
				bHasExpandedChildren = PopulateLinearizedItems( ChildItems, InLinearizedItems, NewDenseItemInfos, NeedsParentWire, OutNewSelectedItems, NewSparseItemInfo, bAddChildItems, InLinearizedItems.Num() - 1);
			}

			if ( bIsExpanded || bHasExpandedChildren )
			{
				// Update the item info for this tree item.
				NewSparseItemInfo.Add( CurItem, FSparseItemInfo( bIsExpanded, bHasExpandedChildren) );
			}


			// If we encountered any expanded nodes along the way, the parent has expanded children.
			bSawExpandedItems = bSawExpandedItems || bIsExpanded || bHasExpandedChildren;				
		}

		return bSawExpandedItems;
	}

	int32 PopulatePinnedItems(const TArray<ItemType>& InItemsSource, TArray< ItemType >& InPinnedItems, const STableViewBase::FReGenerateResults& Results)
	{
		// The value we return, to signify if we want the hierarchy to be collapsed even if it doesn't reach the max amount
		int32 MaxPinnedItemsOverride = -1;

		if (InItemsSource.IsEmpty())
		{
			return MaxPinnedItemsOverride;
		}

		// Calculate the index of the first item in view
		int32 StartIndex = FMath::Clamp((int32)(FMath::FloorToDouble(Results.NewScrollOffset)), 0, InItemsSource.Num() - 1);
		int32 CurrentItemIndex = StartIndex;

		auto GetNonVisibleParents = [this, &InItemsSource, StartIndex](TArray<ItemType>& OutParents, int32 ItemIndex) {
			if (!DenseItemInfos.IsValidIndex(ItemIndex))
			{
				return;
			}

			int32 ParentIndex = ItemIndex;

			// Walk through the list of parents of the current item until you reach the root
			do
			{
				ParentIndex = DenseItemInfos[ItemIndex].ParentIndex;

				// If the current item has a parent, and the parent is not visible, add the parent to the list of pinned items
				if (InItemsSource.IsValidIndex(ParentIndex) && ParentIndex < StartIndex)
				{
					OutParents.Add(InItemsSource[ParentIndex]);
				}

				ItemIndex = ParentIndex;

			} while (ParentIndex != INDEX_NONE);
		};

		/* Special Case for if we are at the end of the list. When there is no space to scroll down in a list, changing the pinned hierarchy could also change the first visible item
		 * which is used to calculate the pinned hierarchy. This leads to an infinite loop, so we solve this by finding a first visible item that has a hierachy large enough to hide
		 * itself, and then collapse the hierarchy until the item remains the first visible item (so there are no infinite loops since the first visible item doesn't change)
		 *  
		 */ 
		if (Results.bGeneratedPastLastItem && Results.NewScrollOffset > 0)
		{
			int32 LastItem = InItemsSource.Num() - 1;
			int32 CurrentMaxPinnedItems = this->GetMaxPinnedItems();

			// Could be different than reported by STableViewBase if some items are collapsed
			int32 NumPinnedItems = (this->GetNumPinnedItems() < CurrentMaxPinnedItems) ? this->GetNumPinnedItems() : CurrentMaxPinnedItems;

			// This is the first item that would be visible, if there were no pinned rows
			int32 FirstItem = FMath::TruncToInt32(Results.NewScrollOffset - NumPinnedItems);

			// We find items that have a hierarchy big enough to cover themselves, but select the smallest among them
			int32 MinSpaceOccupied = TNumericLimits<int32>::Max();

			// The index of the item we select
			int32 MinIndex = -1;

			for (int32 ItemIndex = FirstItem; ItemIndex <= LastItem; ItemIndex++)
			{
				// Get all parents of the current item that are not visible, to calculate the number of items in its hierarchy
				TArray<ItemType> NonVisibleParents;
				GetNonVisibleParents(NonVisibleParents, ItemIndex);

				int32 NumParents = NonVisibleParents.Num();

				// How many items would be required in the hierarchy to cover the item itself
				int32 IndexOffset = ItemIndex - FirstItem;

				// If the hierarchy is too small, ignore it
				if (NumParents < IndexOffset)
				{
					continue;
				}

				// If hierarchy is the smallest we have found so far, AND the number of pinned items it will require is < the allowed max
				if (NumParents - IndexOffset < MinSpaceOccupied && IndexOffset <= CurrentMaxPinnedItems)
				{
					MinSpaceOccupied = NumParents - IndexOffset;
					MinIndex = ItemIndex;
				}
				
			}

			// If we found no such items, we are in the middle of generating the list so pinned rows are not required
			if (MinIndex == -1)
			{
				return MaxPinnedItemsOverride;
			}

			CurrentItemIndex = MinIndex;
			MaxPinnedItemsOverride = MinIndex - FirstItem;
		}

		// Get all the parents of the item that are not visible, which is the hierarchy to stack
		GetNonVisibleParents(InPinnedItems, CurrentItemIndex);

		// Reverse the list so the root is at the front
		Algo::Reverse(InPinnedItems);

		return MaxPinnedItemsOverride;
	}
		
	/**
	 * Given a TreeItem, create a Widget to represent it in the tree view.
	 *
	 * @param InItem   Item to visualize.
	 * @return A widget that represents this item.
	 */
	virtual TSharedRef<ITableRow> GenerateNewWidget( ItemType InItem ) override
	{
		if ( this->OnGenerateRow.IsBound() )
		{
			return this->OnGenerateRow.Execute( InItem, this->SharedThis(this) );
		}
		else
		{
			TSharedRef< STreeView<ItemType> > This = StaticCastSharedRef< STreeView<ItemType> >(this->AsShared());

			// The programmer did not provide an OnGenerateRow() handler; let them know.
			TSharedRef< STableRow<ItemType> > NewTreeItemWidget = 
				SNew( STableRow<ItemType>, This )
				.Content()
				[
					SNew(STextBlock) .Text( NSLOCTEXT("STreeView", "BrokenSetupMessage", "OnGenerateWidget() not assigned.") )
				];

			return NewTreeItemWidget;
		}
	}
		
	/** Queue up a regeneration of the linearized items on the next tick. */
	virtual void RequestListRefresh() override
	{
		if (!bTreeItemsAreDirty)
		{
			bTreeItemsAreDirty = true;
			SListView<ItemType>::RequestListRefresh();
		}
	}

	void RequestTreeRefresh()
	{
		RequestListRefresh();
	}

	virtual void RebuildList() override
	{
		LinearizedItems.Empty();
		SListView<ItemType>::RebuildList();
	}

	/**
	 * Set whether some data item is expanded or not.
	 * 
	 * @param InItem         The item whose expansion state to control.
	 * @param InExpandItem   If true the item should be expanded; otherwise collapsed.
	 */
	void SetItemExpansion( const ItemType& InItem, bool InShouldExpandItem )
	{
		Private_SetItemExpansion(InItem, InShouldExpandItem);
	}

	/** Collapse all the items in the tree and expand InItem */
	void SetSingleExpandedItem( const ItemType& InItem )
	{
		// Will we have to do any work?
		const bool bItemAlreadyLoneExpanded = (this->SparseItemInfos.Num() == 1) && this->IsItemExpanded(InItem);

		if (!bItemAlreadyLoneExpanded)
		{
			this->SparseItemInfos.Empty();
			Private_SetItemExpansion(InItem, true);
		}
	}
		
	/** 
	 * @param InItem   The data item whose expansion state to query.
	 *
	 * @return true if the item is expanded; false otherwise.
	 */
	bool IsItemExpanded( const ItemType& InItem ) const
	{
		return Private_IsItemExpanded( InItem );
	}

public:
	//~ Hide the base function from SListView 
	UE_DEPRECATED(5.3, "SetItemsSource is deprecated. You probably want to use SetTreeItemsSource.")
	void SetItemsSource(const TArray<ItemType>* InListItemsSource)
	{
		SListView<ItemType>::SetItemsSource(InListItemsSource);
	}
	UE_DEPRECATED(5.3, "SetItemsSource is deprecated. You probably want to use SetTreeItemsSource.")
	void SetItemsSource(TSharedRef<::UE::Slate::Containers::TObservableArray<ItemType>> InListItemsSource)
	{
		SListView<ItemType>::SetItemsSource(InListItemsSource);
	}
	UE_DEPRECATED(5.3, "SetItemsSource is deprecated. You probably want to use SetTreeItemsSource.")
	void SetItemsSource(TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>> Provider)
	{
		SListView<ItemType>::SetItemsSource(MoveTemp(Provider));
	}
	UE_DEPRECATED(5.3, "ClearItemsSource is deprecated. You probably want to use ClearRootItemsSource.")
	void ClearItemsSource()
	{
		SListView<ItemType>::ClearItemsSource();
	}
	UE_DEPRECATED(5.3, "HasValidItemsSource is deprecated. You probably want to use HasValidRootItemsSource.")
	bool HasValidItemsSource() const
	{
		return SListView<ItemType>::HasValidItemsSource();
	}
	UE_DEPRECATED(5.3, "GetItems is deprecated. You probably want to use GetRootItems.")
	TArrayView<const ItemType> GetItems() const
	{
		return SListView<ItemType>::GetItems();
	}

public:

	/**
	 * Set the TreeItemsSource. The Tree will generate widgets to represent these items.
	 * @param InItemsSource  A pointer to the array of items that should be observed by this TreeView.
	 */
	void SetTreeItemsSource( const TArray<ItemType>* InItemsSource)
	{
		SetRootItemsSource(InItemsSource);
	}

	/**
	 * Set the Root items. The tree will generate widgets to represent these items.
	 * @param InItemsSource  A pointer to the array of items that should be observed by this TreeView.
	 */
	void SetRootItemsSource(const TArray<ItemType>* InItemsSource)
	{
		ensureMsgf(InItemsSource, TEXT("The TreeItemsSource is invalid."));
		if (TreeViewSource == nullptr || !TreeViewSource->IsSame(reinterpret_cast<const void*>(InItemsSource)))
		{
			if (InItemsSource)
			{
				SetRootItemsSource(MakeUnique<UE::Slate::ItemsSource::FArrayPointer<ItemType>>(InItemsSource));
			}
			else
			{
				ClearRootItemsSource();
			}
		}
	}

	/**
	 * Set the RootItemsSource. The tree will generate widgets to represent these items.
	 * @param InItemsSource  A pointer to the array of items that should be observed by this TreeView.
	 */
	void SetRootItemsSource(TSharedRef<UE::Slate::Containers::TObservableArray<ItemType>> InItemsSource)
	{
		if (TreeViewSource == nullptr || !TreeViewSource->IsSame(reinterpret_cast<const void*>(&InItemsSource.Get())))
		{
			SetRootItemsSource(MakeUnique<UE::Slate::ItemsSource::FSharedObservableArray<ItemType>>(this->SharedThis(this), MoveTemp(InItemsSource)));
		}
	}

	/**
	 * Establishes a new list of root items being observed by the list.
	 * Wipes all existing state and requests and will fully rebuild on the next tick.
	 */
	void SetRootItemsSource(TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>> Provider)
	{
		TreeViewSource = MoveTemp(Provider);
		RequestTreeRefresh();
	}

	void ClearRootItemsSource()
	{
		SetRootItemsSource(TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>>());
	}

	bool HasValidRootItemsSource() const
	{
		return TreeViewSource != nullptr;
	}

	TArrayView<const ItemType> GetRootItems() const
	{
		return TreeViewSource ? TreeViewSource->GetItems() : TArrayView<const ItemType>();
	}

	/**
	 * Generates a set of items that are currently expanded.
	 *
	 * @param ExpandedItems	The generated set of expanded items.
	 */
	void GetExpandedItems( TItemSet& ExpandedItems ) const
	{
		for( typename TSparseItemMap::TConstIterator InfoIterator(SparseItemInfos); InfoIterator; ++InfoIterator )
		{
			if ( InfoIterator.Value().bIsExpanded )
			{
				ExpandedItems.Add( InfoIterator.Key() );
			}
		}			
	}

	/** Clears the entire set of expanded items. */
	void ClearExpandedItems()
	{
		SparseItemInfos.Empty();
		RequestTreeRefresh();
	}

	virtual STableViewBase::FReGenerateResults ReGenerateItems(const FGeometry& MyGeometry) override
	{
		// We need to call the parent function first to know if we reached the end of the list
		STableViewBase::FReGenerateResults Results = SListView<ItemType>::ReGenerateItems(MyGeometry);

		if (bShouldStackHierarchyHeaders.Get())
		{
			TArray<ItemType> PinnedItems;

			// If we reached the end of the list and there is space, a special case requires the hierarchy to be collapsed forcefully
			int32 MaxPinnedItemsOverride = PopulatePinnedItems(LinearizedItems, PinnedItems, Results);
			this->ReGeneratePinnedItems(PinnedItems, MyGeometry, MaxPinnedItemsOverride);
		}
		else
		{
			this->ClearPinnedWidgets();
		}

		return Results;
	}

protected:
	
	/** The delegate that is invoked whenever we need to gather an item's children. */
	FOnGetChildren OnGetChildren;

	/** The delegate that is invoked to recursively expand/collapse a tree items children. */
	FOnSetExpansionRecursive OnSetExpansionRecursive;

	UE_DEPRECATED(5.3, "Protected access to TreeItemsSource is deprecated. Please use GetTreeItems, SetTreeItemsSource or HasValidTreeItemsSource.")
	/** A pointer to the items being observed by the tree view. */
	const TArray<ItemType>* TreeItemsSource;		
		
	/** Info needed by a small fraction of tree items; some of these are not visible to the user. */
	TSparseItemMap SparseItemInfos;

	/** Info needed by every item in the linearized version of the tree. */
	TArray<FItemInfo> DenseItemInfos;

	/**
	 * A linearized version of the items being observed by the tree view.
	 * Note that we inherit from a ListView, which we point at this linearized version of the tree.
	 */
	TArray< ItemType > LinearizedItems;

	/** The delegate that is invoked whenever an item in the tree is expanded or collapsed. */
	FOnExpansionChanged OnExpansionChanged;

private:		
	/** Pointer to the source data that we are observing */
	TUniquePtr<UE::Slate::ItemsSource::IItemsSource<ItemType>> TreeViewSource;

	/** true when the LinearizedItems need to be regenerated. */
	bool bTreeItemsAreDirty = false;

	/** true if we allow invisible items to stay selected. */
	bool bAllowInvisibleItemSelection = false;

	/** true if we should highlight all parents for each of the currently selected items */
	bool bHighlightParentNodesForSelection = false;

	/** true if we want to show the hierarchy of items pinned at the top */
	SWidget::TSlateAttribute<bool> bShouldStackHierarchyHeaders;
};

template<typename ItemType>
inline void STreeView<ItemType>::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	struct FInvalidation
	{
		static void RequestLayoutRefresh(SWidget& OwningWidget)
		{
			static_cast<STreeView<ItemType>&>(OwningWidget).RequestLayoutRefresh();
		}
	};
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bShouldStackHierarchyHeaders, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::RequestLayoutRefresh));
}