// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Slate : ModuleRules
{
	public Slate(ReadOnlyTargetRules Target) : base(Target)
	{
		// Direct SDK dependecies were removed in the mac platform, but IOS still depends on it due to IOSPlatformTextField
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.IOS))
		{
			bRequiresPlatformSDK = true;
		}
		
        PrivateDefinitions.Add("SLATE_MODULE=1");
        SharedPCHHeaderFile = "Public/SlateSharedPCH.h";

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Json",
				"SlateCore",
				"ImageWrapper"
			});
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"TraceLog",
			});

		if (Target.bCompileAgainstApplicationCore)
		{
			PublicDependencyModuleNames.Add("ApplicationCore");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "XInput");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");
		}
		
		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			DynamicallyLoadedModuleNames.Add("MacMenu");
		}

		// Add slate runtime dependencies
		if (Target.bUsesSlate)
		{
			RuntimeDependencies.Add("$(EngineDir)/Content/Slate/...*.ttf", StagedFileType.UFS);
			RuntimeDependencies.Add("$(EngineDir)/Content/Slate/...*.png", StagedFileType.UFS);
			RuntimeDependencies.Add("$(EngineDir)/Content/Slate/...*.svg", StagedFileType.UFS);
			RuntimeDependencies.Add("$(EngineDir)/Content/Slate/...*.tps", StagedFileType.UFS);
			RuntimeDependencies.Add("$(EngineDir)/Content/SlateDebug/...", StagedFileType.DebugNonUFS);

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{ 
				RuntimeDependencies.Add("$(EngineDir)/Content/Slate/...*.cur", StagedFileType.NonUFS);
			}

			if (Target.ProjectFile != null)
			{
				RuntimeDependencies.Add("$(ProjectDir)/Content/Slate/...", StagedFileType.UFS);
				RuntimeDependencies.Add("$(ProjectDir)/Content/SlateDebug/...", StagedFileType.DebugNonUFS);
			}
		}

		if (Target.bBuildDeveloperTools)
        {
            DynamicallyLoadedModuleNames.Add("Settings");
        }

		PrivateIncludePaths.Add(System.IO.Path.Combine(EngineDirectory,"Source/ThirdParty/AHEasing/AHEasing-1.3.2"));
	}
}
